[CoreRedirects]
+PropertyRedirects=(OldName="MoviePipelineDeferredPassBase.StencilLayers", NewName="ActorLayers")
+ClassRedirects=(OldName="/Script/MovieRenderPipelineCore.MovieGraphOutputSettingNode",NewName="/Script/MovieRenderPipelineCore.MovieGraphGlobalOutputSettingNode")
+PropertyRedirects=(OldName="/Script/MovieRenderPipelineCore.MovieGraphCommandLineEncoderNode.FileNameFormatOverride",NewName="/Script/MovieRenderPipelineCore.MovieGraphCommandLineEncoderNode.FileNameFormat")
+PropertyRedirects=(OldName="/Script/MovieRenderPipelineCore.MovieGraphCommandLineEncoderNode.bOverride_FileNameFormatOverride",NewName="/Script/MovieRenderPipelineCore.MovieGraphCommandLineEncoderNode.bOverride_FileNameFormat")

+ClassRedirects=(OldName="/Script/MovieRenderPipelineCore.MovieGraphDeferredRenderPassNode",NewName="/Script/MovieRenderPipelineRenderPasses.MovieGraphDeferredRenderPassNode")
+ClassRedirects=(OldName="/Script/MovieRenderPipelineCore.MovieGraphImagePassBaseNode",NewName="/Script/MovieRenderPipelineRenderPasses.MovieGraphImagePassBaseNode")
+ClassRedirects=(OldName="/Script/MovieRenderPipelineCore.MovieGraphPathTracedRenderPassNode",NewName="/Script/MovieRenderPipelineRenderPasses.MovieGraphPathTracerRenderPassNode")
+ClassRedirects=(OldName="/Script/MovieRenderPipelineRenderPasses.MovieGraphPathTracedRenderPassNode",NewName="/Script/MovieRenderPipelineRenderPasses.MovieGraphPathTracerRenderPassNode")
+ClassRedirects=(OldName="/Script/MovieRenderPipelineCore.MoviePipelineCollectionModifier",NewName="/Script/MovieRenderPipelineCore.MovieGraphCollectionModifier")
+ClassRedirects=(OldName="/Script/MovieRenderPipelineCore.MoviePipelineMaterialModifier",NewName="/Script/MovieRenderPipelineCore.MovieGraphMaterialModifier")
+ClassRedirects=(OldName="/Script/MovieRenderPipelineCore.MoviePipelineVisibilityModifier",NewName="/Script/MovieRenderPipelineCore.MovieGraphRenderPropertyModifier")
+ClassRedirects=(OldName="/Script/MovieRenderPipelineCore.MoviePipelineCollection",NewName="/Script/MovieRenderPipelineCore.MovieGraphCollection")
+EnumRedirects=(OldName="/Script/MovieRenderPipelineCore.EMovieGraphQuickRenderPostRenderActionType",NewName="/Script/MovieRenderPipelineCore.EMoviePipelinePostRenderActionType")