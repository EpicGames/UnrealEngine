{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.3",
	"FriendlyName": "Movie Render Queue",
	"Description": "Advanced movie rendering pipeline for use in creating rendered cinematics or other multi-media creation.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"Plugins": [
		{
			"Name": "ActorLayerUtilities",
			"Enabled": true,
			"Optional": false
		},
		{
			"Name": "ConsoleVariables",
			"Enabled": true,
			"Optional": false
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true,
			"Optional": true
		},
		{
			"Name": "OpenColorIO",
			"Enabled": true,
			"Optional": false
		},
		{
			"Name": "SequencerScripting",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAsset",
			"Enabled": true
		},
		{
			"Name": "NamingTokens",
			"Enabled": true,
			"Optional": false
		}
	],
	"Modules": [
		{
			"Name": "MovieRenderPipelineCore",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Mac", "Win64", "Linux" ]
		},
		{
			"Name": "MovieRenderPipelineSettings",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Mac", "Win64", "Linux" ]
		},
		{
			"Name": "MovieRenderPipelineRenderPasses",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Mac", "Win64", "Linux" ]
		},
		{
			"Name": "MovieRenderPipelineEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MovieRenderPipelineMP4Encoder",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": ["Mac", "Win64", "Linux" ]
		},
		{
			"Name": "UEOpenExrRTTI",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Mac", "Win64", "Linux" ]
		}
	]
}