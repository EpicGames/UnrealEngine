// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MovieRenderPipelineCore : ModuleRules
{
	public MovieRenderPipelineCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"RenderCore",
				"RHI",
				"UMG",
				"Landscape", // To flush grass
				"AudioMixer",
				"NonRealtimeAudioRenderer",
				"Sockets", 
				"Networking",
				"HTTP",
				"DeveloperSettings",
				"ClothingSystemRuntimeInterface",
				"Slate",
				"OpenColorIOWrapper",
				"ImageCore",
				"ChaosClothAssetEngine",
				"Renderer",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"SlateCore",
                "MovieScene",
                "MovieSceneTracks",
				"LevelSequence",
				"Engine",
				"ImageWriteQueue", // For debug tile writing
				"OpenColorIO",
				"CinematicCamera",
				"NamingTokens",
			}
		);

		if (Target.bBuildEditor == true)
        {
	        PrivateDependencyModuleNames.AddRange(
		        new string[]
		        {
			        "ClassViewer",
			        "ContentBrowserData",
			        "DataLayerEditor",
			        "EditorWidgets",
			        "Layers",
			        "SceneOutliner",
			        "UnrealEd",
					"SettingsEditor",
					"Slate",
					"SlateCore",
				});
	        
			PublicDependencyModuleNames.Add("MovieSceneTools");
        }
	}
}
