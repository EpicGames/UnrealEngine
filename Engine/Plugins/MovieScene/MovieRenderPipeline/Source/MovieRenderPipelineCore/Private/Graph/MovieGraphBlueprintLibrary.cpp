// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphBlueprintLibrary.h"

#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphProjectSettings.h"
#include "Graph/Nodes/MovieGraphCameraNode.h"
#include "Graph/Nodes/MovieGraphCommandLineEncoderNode.h"
#include "Graph/Nodes/MovieGraphFileOutputNode.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"

#include "CineCameraActor.h"
#include "GlobalNamingTokens.h"
#include "HAL/FileManager.h"
#include "Internationalization/Regex.h"
#include "Misc/Paths.h"
#include "NamingTokensEngineSubsystem.h"
#include "Utils/NamingTokenUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphBlueprintLibrary)

namespace
{
	/**
	 * Similar to UE::NamingTokens::Utils::GetTokenKeysFromString(), but allows for spaces within the namespace or token name.
	 * 
	 * Spaces are not allowed by Naming Tokens, but users may accidentally place them in format strings. We should still detect these
	 * as valid tokens, but keep them un-resolved.
	 */
	TArray<FString> GetTokensFromString(const FString& InFormatString)
	{
    	static const FString PatternString = FString::Printf(TEXT(R"(\{\s*((?:[a-zA-Z0-9_ ]+%s)*[a-zA-Z0-9_ ]+)\s*\})"), *UE::NamingTokens::Utils::GetNamespaceDelimiter());
    	static const FRegexPattern Pattern(PatternString);
    
    	TSet<FString, FLocKeySetFuncs> Tokens;
    	FRegexMatcher Matcher(Pattern, InFormatString);

    	while (Matcher.FindNext())
    	{
    		FString Token = Matcher.GetCaptureGroup(1);
    		Tokens.Add(Token);
    	}
    
    	return Tokens.Array();
	}
	
	/**
	 * From the provided format string, resolve all of the non-MRG {tokens} from the Naming Tokens system. Resolved values are provided in both the
	 * InOutFilenameArguments and InOutFormatArguments maps. The former is exposed to the API, and the latter is just used for FString::Format().
	 *
	 * Note that tokens in the Naming Tokens system are usually namespaced, so the tokens added to InOutFormatArguments will be renamed to be
	 * compatible with FString::Format() (to eliminate the namespacing colon delimiter). Tokens added to InOutFilenameArguments will not have their
	 * names modified. InOutFormatString will be modified in-place to contain the updated token names that InOutFormatArguments uses.
	 */
	void UpdateTokenMapsWithResolvedExternalTokens(
		FString& InOutFormatString, TMap<FString, FString>& InOutFilenameArguments, FStringFormatNamedArguments& InOutFormatArguments)
	{
		// With the given token name and evaluated token data, updates the format string as needed and updates the format and filename arguments with
		// the evaluated token value.
		auto UpdateFormatStringAndArguments = [&InOutFormatString, &InOutFilenameArguments, &InOutFormatArguments](
			const FString& InTokenName, const FNamingTokenValueData& InEvaluatedTokenData)
		{
			// FString::Format() does not work with tokens that contain colons, so generate a temporary replacement token and do a find-and-replace
			// on the source format string to use the new substitute token name. Eg, "tr:slate" -> "__MRGTEMP_tr_slate__".
			const FString TempMrgTokenName = FString::Printf(TEXT("__MRGTEMP_%s_%s__"), *InEvaluatedTokenData.TokenNamespace, *InEvaluatedTokenData.TokenKey);

			// Replace the existing {token} with the temporary token name generated above.
			InOutFormatString.ReplaceInline(
				*FString::Printf(TEXT("{%s}"), *InTokenName),
				*FString::Printf(TEXT("{%s}"), *TempMrgTokenName));

			// Add the modified token to the format arguments, but use the un-modified token name in the resolved arguments (which is visible to API consumers).
			const FString EvaluatedTokenValue = InEvaluatedTokenData.TokenValue.ToString();
			InOutFormatArguments.Add(TempMrgTokenName, EvaluatedTokenValue);
			InOutFilenameArguments.Add(InTokenName, EvaluatedTokenValue);
		};

		// Replaces any invalid characters (like ':') in a token with '_'. Colons in tokens can happen if the token provided is invalid and
		// fails to be resolved.
		auto SanitizeInvalidToken = [&InOutFormatString](const FString& InTokenName)
		{
			InOutFormatString.ReplaceInline(
				*FString::Printf(TEXT("{%s}"), *InTokenName),
				*FString::Printf(TEXT("{%s}"), *FPaths::MakeValidFileName(InTokenName, TEXT('_'))));
		};
		
		// Parse out all of the filename argument {tokens} in the string, and find the ones that do not come from MRG.
		TSet<FString> NonMrgTokenNames;
		for (const FString& FilenameArgument : GetTokensFromString(InOutFormatString))
		{
			if (!InOutFilenameArguments.Contains(FilenameArgument))
			{
				NonMrgTokenNames.Add(FilenameArgument);
			}
		}

		// Can't resolve naming tokens w/o the Naming Tokens plugin loaded. We can't really warn here at this point because even if some non-MRG
		// tokens were found, we can't say definitively that they're defined in the Naming Tokens system (eg, the token could just be misspelled).
		UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
		if (!NamingTokensSubsystem)
		{
			return;
		}

		// Resolve via the Naming Tokens system and inject the values into the argument maps.
		TSet<FString> ValidNamingTokens;
		for (const FNamingTokenValueData& EvaluatedTokenData : NamingTokensSubsystem->EvaluateTokenList(NonMrgTokenNames.Array()))
		{
			const FString NamespacedToken = UE::NamingTokens::Utils::CombineNamespaceAndTokenKey(EvaluatedTokenData.TokenNamespace, EvaluatedTokenData.TokenKey);

			// If a token was returned by EvaluateTokenList(), it was recognized as meeting token naming requirements by the Naming Tokens
			// system. However, that does not guarantee that the token was actually registered or can be resolved.
			ValidNamingTokens.Add(NamespacedToken);
			
			// If the token could not be resolved, skip it
			if (!EvaluatedTokenData.bWasEvaluated)
			{
				// Only namespaced tokens need to be sanitized. The colon delimiter in namespaced tokens is the primary character that
				// needs to be removed.
				SanitizeInvalidToken(NamespacedToken);
				continue;
			}
			
			if (NonMrgTokenNames.Contains(NamespacedToken))
			{
				UpdateFormatStringAndArguments(NamespacedToken, EvaluatedTokenData);
			}

			const FString NonNamespacedToken = EvaluatedTokenData.TokenKey;
			if (NonMrgTokenNames.Contains(NonNamespacedToken))
			{
				UpdateFormatStringAndArguments(NonNamespacedToken, EvaluatedTokenData);
			}
		}

		// For the tokens that weren't recognized as MRG tokens or valid Naming Tokens, keep them as-is in the format string. However, if it's a
		// namespaced Naming Token, replace the colon delimiter with an underscore so the file path remains valid. Tokens that can fall into this
		// code path are typically those that have an invalid space in their name (eg, "Token Namespace:TokenName").
		const TSet<FString> UnhandledTokens = NonMrgTokenNames.Difference(ValidNamingTokens);
		for (const FString& UnhandledToken : UnhandledTokens)
		{
			SanitizeInvalidToken(UnhandledToken);
		}
	}

	/** Removes all "mrg:" namespacing from {tokens} in the format string. For example, replaces {mrg:branch_name} with {branch_name}. */
	void RemoveMrgTokenNamespacing(FString& InOutFormatString)
	{
		static const FString MrgNamespace = FString::Printf(TEXT("%s:"), *UMovieRenderPipelineNamingTokens::TokenNamespace);
		
		for (const FString& IncludedTokenName : GetTokensFromString(InOutFormatString))
		{
			if (IncludedTokenName.StartsWith(MrgNamespace))
			{
				const FString TokenNameWithoutNamespace = IncludedTokenName.Replace(*MrgNamespace, TEXT(""));
				const FString SourceToken = FString::Printf(TEXT("{%s}"), *IncludedTokenName);
				const FString DestToken = FString::Printf(TEXT("{%s}"), *TokenNameWithoutNamespace);
				
				InOutFormatString.ReplaceInline(*SourceToken, *DestToken, ESearchCase::CaseSensitive);
			}
		}
	}

	FString ResolveFormatArgumentsHelper(const FString& InFormatString, const FMovieGraphFilenameResolveParams& InParams, const bool bResolveToPath, FMovieGraphResolveArgs& OutMergedFormatArgs)
	{
		// There are a number of different sources for file/metadata KVPs. 
		// UMovieGraphPipeline - these are handled as resolve params to allow resolving during edit time
		// UMoviePipelineExecutorJob - author, sequence, level, etc.
		// UMovieGraphSettingNode - every node can contribute it's own file/metadata KVP based on their own settings

		// Keep track of the resolve args that are generated JUST for this call to ResolveFilenameFormatArguments(). This generates a record of all tokens
		// that were generated from MRG, and thus allows us to determine which tokens originate from outside MRG. These will be merged with
		// OutMergedFormatArgs before the function returns.
		FMovieGraphResolveArgs GeneratedResolveArgs;
		
		bool bOverwriteExisting = true;

		// Data from the UMovieGraphPipeline comes from the resolve params themselves to allow resolving without a runtime instance.
		{
			// Zero-pad our frame numbers when we format the strings. Some programs struggle when ingesting frames that 
			// go 1,2,3,...,10,11. To work around this issue we allow the user to specify how many zeros they want to
			// pad the numbers with, 0001, 0002, etc. We also allow offsetting the output frame numbers, this is useful
			// when your sequence starts at zero and you use handle frames (which would cause negative output frame 
			// numbers), so we allow the user to add a fixed amount to all output to ensure they are positive.
			int32 FrameNumberOffset = InParams.FrameNumberOffset;
			FString FrameNumber = UE::MoviePipeline::GetPaddingFormatString(InParams.ZeroPadFrameNumberCount, InParams.RootFrameNumber + FrameNumberOffset); // Sequence Frame #
			FString FrameNumberShot = UE::MoviePipeline::GetPaddingFormatString(InParams.ZeroPadFrameNumberCount, InParams.ShotFrameNumber + FrameNumberOffset); // Shot Frame #
			FString FrameNumberRel = UE::MoviePipeline::GetPaddingFormatString(InParams.ZeroPadFrameNumberCount, InParams.RootFrameNumberRel + FrameNumberOffset); // Relative to 0
			FString FrameNumberShotRel = UE::MoviePipeline::GetPaddingFormatString(InParams.ZeroPadFrameNumberCount, InParams.ShotFrameNumberRel + FrameNumberOffset); // Relative to 0 within the shot.

			// If time dilation is in effect, RootFrameNumber and ShotFrameNumber will contain duplicates and the files will overwrite each other, 
			// so we force them into relative mode and then warn users we did that (as their numbers will be out of sync with what they see in the level sequence now).
			if (InFormatString.Contains(TEXT("{frame")) && InParams.bForceRelativeFrameNumbers && !InFormatString.Contains(TEXT("_rel}")))
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "Time Dilation was used but output format does not use relative time, forcing relative numbers. Change {frame_number} to {frame_number_rel} (or the shot version) to remove this message.");
			}

			// Ensure they used relative frame numbers in the output so they get the right number of output frames.
			if (InParams.bForceRelativeFrameNumbers)
			{
				FrameNumber = FrameNumberRel;
				FrameNumberShot = FrameNumberShotRel;
			}

			FString ShotName = InParams.Shot ? InParams.Shot->OuterName : FString();
			ShotName = ShotName.Len() > 0 ? ShotName : TEXT("NoShot");

			// Use the legacy system to add filename and metadata kvp
			MoviePipeline::GetOutputStateFormatArgs(GeneratedResolveArgs.FilenameArguments, GeneratedResolveArgs.FileMetadata, FrameNumber, FrameNumberShot, FrameNumberRel, FrameNumberShotRel, TEXT("DummyCameraToken"), ShotName);

			// Add on Render Data Identifier, overwriting the dummy camera name above.
			GeneratedResolveArgs.FilenameArguments.Add(TEXT("camera_name"), InParams.RenderDataIdentifier.CameraName);
			GeneratedResolveArgs.FilenameArguments.Add(TEXT("renderer_name"), InParams.RenderDataIdentifier.RendererName);
			GeneratedResolveArgs.FilenameArguments.Add(TEXT("renderer_sub_name"), InParams.RenderDataIdentifier.SubResourceName);
			GeneratedResolveArgs.FilenameArguments.Add(TEXT("layer_name"), InParams.RenderDataIdentifier.LayerName);
			GeneratedResolveArgs.FilenameArguments.Add(TEXT("branch_name"), InParams.RenderDataIdentifier.RootBranchName.ToString());

			FString MetadataPath = UE::MoviePipeline::GetMetadataPrefixPath(InParams.RenderDataIdentifier);
			GeneratedResolveArgs.FileMetadata.Add(FString::Printf(TEXT("%s/cameraName"), *MetadataPath), InParams.RenderDataIdentifier.CameraName);
			GeneratedResolveArgs.FileMetadata.Add(FString::Printf(TEXT("%s/rendererName"), *MetadataPath), InParams.RenderDataIdentifier.RendererName);
			GeneratedResolveArgs.FileMetadata.Add(FString::Printf(TEXT("%s/subResourceName"), *MetadataPath), InParams.RenderDataIdentifier.SubResourceName);
			GeneratedResolveArgs.FileMetadata.Add(FString::Printf(TEXT("%s/layerName"), *MetadataPath), InParams.RenderDataIdentifier.LayerName);
			GeneratedResolveArgs.FileMetadata.Add(FString::Printf(TEXT("%s/branchName"), *MetadataPath), InParams.RenderDataIdentifier.RootBranchName.ToString());
		}


		// Add data from the job. ToDo: This probably needs to come from the job itself, so you can plug in different data sources eventually
		{
			// Frame Rate we handle separately, because it needs to be resolved.
			double FrameRate = InParams.DefaultFrameRate.AsDecimal();
			if (InParams.EvaluatedConfig)
			{
				const bool bIncludeCDOs = false;
				UMovieGraphGlobalOutputSettingNode* OutputSettingNode = InParams.EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphSettingNode::GlobalsPinName, bIncludeCDOs);

				if (OutputSettingNode)
				{
					FrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputSettingNode, InParams.DefaultFrameRate).AsDecimal();
					bOverwriteExisting = OutputSettingNode->bOverwriteExistingOutput;
				}
			}

			GeneratedResolveArgs.FilenameArguments.Add(TEXT("frame_rate"), FString::SanitizeFloat(FrameRate));
			GeneratedResolveArgs.FileMetadata.Add(TEXT("unreal/frameRate"), FString::SanitizeFloat(FrameRate));

			FString LevelName;
			FString SequenceName;
			if (InParams.Job)
			{
				LevelName = InParams.Job->Map.GetAssetName();
				SequenceName = InParams.Job->Sequence.GetAssetName();
			}
			
			GeneratedResolveArgs.FilenameArguments.Add(TEXT("level_name"), LevelName);
			GeneratedResolveArgs.FilenameArguments.Add(TEXT("sequence_name"), SequenceName);
			
			GeneratedResolveArgs.FileMetadata.Add(TEXT("unreal/levelName"), LevelName);
			GeneratedResolveArgs.FileMetadata.Add(TEXT("unreal/sequenceName"), SequenceName);
			
			// Add KVP data from the job (date, time, job name, job author, job comment)
			UE::MoviePipeline::GetSharedFormatArguments(GeneratedResolveArgs.FilenameArguments, GeneratedResolveArgs.FileMetadata, InParams.InitializationTime, InParams.Version, InParams.Job, InParams.InitializationTimeOffset);
		}
		

		//  Now get the settings from our config. We need to gather KVP data from all possible nodes, even if not expressed in your configuration. This is because you might want to
		// always use the {ts_count} token even if you don't have a Temporal Sample Count node to add it. We look at all possible class types and then get all instances of them in
		// the evaluated graph. We look at all instances because "Named" nodes can end up having multiple copies in the final graph which may want to provide different data.
		TArray<UClass*> AllSettingsNodeClasses = UE::MovieRenderPipeline::FindMoviePipelineSettingClasses(UMovieGraphSettingNode::StaticClass(), false);

		// ToDo: This loops through class iterators every frame, we should probably initialize a copy of everything into the flattened config, since we could cache the classes
		// once per run there. We don't cache the returned results here because you could potentially add/remove classes (via Blueprints) which would invalidate our cache.
		for (UClass* InClass : AllSettingsNodeClasses)
		{
			TArray<UMovieGraphSettingNode*> SettingInstances;
			if (InParams.EvaluatedConfig)
			{
				const bool bIncludeCDOs = true;
				const bool bExactMatch = true;
				SettingInstances = InParams.EvaluatedConfig->GetSettingsForBranch(InClass, InParams.RenderDataIdentifier.RootBranchName, bIncludeCDOs, bExactMatch);
			}
			else
			{
				// GetFormatResolveArgs is const, but GetSettingsForBranch doesn't return a const pointer array,
				// so we just get a non-const CDO pointer here instead, knowing that the function being called is const.
				SettingInstances.Add(GetMutableDefault<UMovieGraphSettingNode>(InClass));
			}

			for (UMovieGraphSettingNode* Instance : SettingInstances)
			{
				Instance->GetFormatResolveArgs(GeneratedResolveArgs, InParams.RenderDataIdentifier);
			}
		}

		// Copy the metadata in the parameters into the merged output version
		for (const TPair<FString, FString>& FileMetadata : InParams.FileMetadata)
		{
			GeneratedResolveArgs.FileMetadata.Add(FileMetadata.Key, FileMetadata.Value);
		}

		// We expect the incoming string to have a {file_dup} token where they want file duplication numbers to be handled
		// but we don't actually want a value there (unless there's a collision). So we override it with an empty string by default.
		GeneratedResolveArgs.FilenameArguments.Add(TEXT("file_dup"), FString());

		// Format the {version} token so it's of the form "v###"
		GeneratedResolveArgs.FilenameArguments.Add(TEXT("version"), FString::Printf(TEXT("v%0*d"), 3, InParams.Version));

		// Overwrite any {tokens} with the user-supplied overrides if needed. This allows different requesters to share the same variables (ie: filename extension)
		for (const TPair<FString, FString>& KVP : InParams.FileNameFormatOverrides)
		{
			GeneratedResolveArgs.FilenameArguments.Add(KVP.Key, KVP.Value);
		}

		// Convert from our Python/BP exposed <FString, FString> to a named argument array for the formatter.
		FStringFormatNamedArguments NamedArgs;
		for (const TPair<FString, FString>& Argument : GeneratedResolveArgs.FilenameArguments)
		{
			NamedArgs.Add(Argument.Key, Argument.Value);
		}

		// The format string may need to be modified from this point onwards, so make a copy so the source string isn't unintentionally changed.
		FString FormatStringCopy = InFormatString;

		// The "mrg:" namespace is technically allowed. However, for the purposes of resolving, we need to strip off this namespace because the token
		// names discovered above do not include this namespace, and the Naming Tokens system will not know how to resolve these either.
		RemoveMrgTokenNamespacing(FormatStringCopy);

		// Resolve non-MRG tokens via the Naming Tokens system.
		UpdateTokenMapsWithResolvedExternalTokens(FormatStringCopy, GeneratedResolveArgs.FilenameArguments, NamedArgs);

		// Now that all format args have been discovered, add them to the merged format args.
		OutMergedFormatArgs.FileMetadata.Append(GeneratedResolveArgs.FileMetadata);
		OutMergedFormatArgs.FilenameArguments.Append(GeneratedResolveArgs.FilenameArguments);

		// If no format string is provided, there's nothing left to do.
		if (InFormatString.IsEmpty())
		{
			return FString();
		}

		// Apply all of our named args to create a formatted string. This may or may not be a file path.
		FString FormattedString = FString::Format(*FormatStringCopy, NamedArgs);

		// If not generating a file path, we can stop here.
		if (!bResolveToPath)
		{
			return FormattedString;
		}
		
		if (InParams.bEnsureAbsolutePath)
		{
			FormattedString = FPaths::ConvertRelativePathToFull(FormattedString);
		}

		// Fix-up slashes
		{
			FPaths::NormalizeFilename(FormattedString);

			// Remove duplicate slashes. Recognize a double-slash at the beginning to ensure network paths remain intact. Note that this also works on
			// network paths that start with "\\" because NormalizeFilename() will convert "\\" to "//".
			if (FormattedString.StartsWith(TEXT("//")))
			{
				FormattedString.RightChopInline(1, EAllowShrinking::No);
				FPaths::RemoveDuplicateSlashes(FormattedString);
				FormattedString = TEXT("/") + MoveTemp(FormattedString);
			}
			else
			{
				FPaths::RemoveDuplicateSlashes(FormattedString);
			}
		}

		// In the event of multiple dots in the filename, we need to replace them with a single dot to ensure a valid file name
		while (FormattedString.Contains(".."))
		{
			FormattedString.ReplaceInline(TEXT(".."), TEXT("."));
		}

		// If we end with a "." character, remove it. The extension will put it back on. We can end up with this sometimes
		// after resolving file format strings, ie: {sequence_name}.{frame_number} becomes {sequence_name}. for
		// videos (which can't use frame_numbers).
		FormattedString.RemoveFromEnd(TEXT("."));
		
		// If the extension is not resolved, ThisTry will be a path
		const FString ExtToken = TEXT(".{ext}");
		const FString Extension = FString::Format(*ExtToken, NamedArgs);
		const FString ResolvedExtension = (Extension == ExtToken || Extension == TEXT(".")) ? FString() : Extension;
		FString ThisTry = FormattedString + ResolvedExtension;
		
		// Check that it's a valid path that we can write to.
		if (UE::MoviePipeline::CanWriteToFile(*ThisTry, bOverwriteExisting))
		{
			return ThisTry;
		}

		// If we got here it means that there was already a file there with that name and bOverwriteExisting was false.
		// So we start by swapping in _(2) where the file_dup token is, and try again. If that fails, _(3), etc. We start
		// at _(2) because the file they're trying to write (foo.png) is technically "_(1)", so we start the rename process
		// at _(2) so it accurately reflects that this is the second version of that file.
		int32 DuplicateIndex = 2;
		while (true)
		{
			NamedArgs.Add(TEXT("file_dup"), FString::Printf(TEXT("_(%d)"), DuplicateIndex));

			// Re-resolve the format string now that we've reassigned frame_dup to a number.
			ThisTry = FString::Format(*FormatStringCopy, NamedArgs) + ResolvedExtension;

			// If the file doesn't exist, we can use that, else, increment the index and try again
			if (UE::MoviePipeline::CanWriteToFile(*ThisTry, bOverwriteExisting))
			{
				return ThisTry;
			}

			++DuplicateIndex;
		}
	}
}

FString UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(const FString& InFormatString, const FMovieGraphFilenameResolveParams& InParams, FMovieGraphResolveArgs& OutMergedFormatArgs)
{
	constexpr bool bResolveToPath = true;
	return ResolveFormatArgumentsHelper(InFormatString, InParams, bResolveToPath, OutMergedFormatArgs);
}

FString UMovieGraphBlueprintLibrary::ResolveFormatArguments(const FString& InFormatString, const FMovieGraphFilenameResolveParams& InParams, FMovieGraphResolveArgs& OutMergedFormatArgs)
{
	constexpr bool bResolveToPath = false;
	return ResolveFormatArgumentsHelper(InFormatString, InParams, bResolveToPath, OutMergedFormatArgs);
}

FFrameRate UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(UMovieGraphGlobalOutputSettingNode* InNode, const FFrameRate& InDefaultRate)
{
	if (InNode && InNode->bOverride_OutputFrameRate)
	{
		return InNode->OutputFrameRate;
	}

	return InDefaultRate;
}

int32 UMovieGraphBlueprintLibrary::ResolveVersionNumber(FMovieGraphFilenameResolveParams InParams, const bool bGetNextVersion)
{
	// Note: InParams is passed by copy rather than const& because it is modified within this method.
	
	if (!InParams.EvaluatedConfig)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot resolve version number without a valid evaluated graph to pull settings from."), ELogVerbosity::Error);
		return -1;
	}

	bool bIncludeCDOs = true;
	bool bExactMatch = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSettingNode =
		InParams.EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);
	if (!OutputSettingNode->VersioningSettings.bAutoVersioning)
	{
		return OutputSettingNode->VersioningSettings.VersionNumber;
	}

	// Force the Version string to stay as {version} so we can substring based on it later.
	InParams.FileNameFormatOverrides.Add(TEXT("version"), TEXT("{version}"));

	// Get output nodes from the evaluated graph
	bIncludeCDOs = false;
	bExactMatch = false;
	TArray<UMovieGraphSettingNode*> ResultNodes = InParams.EvaluatedConfig->GetSettingsForBranch(
		UMovieGraphCommandLineEncoderNode::StaticClass(), InParams.RenderDataIdentifier.RootBranchName, bIncludeCDOs, bExactMatch);
	ResultNodes.Append(InParams.EvaluatedConfig->GetSettingsForBranch(
		UMovieGraphFileOutputNode::StaticClass(), InParams.RenderDataIdentifier.RootBranchName, bIncludeCDOs, bExactMatch));
	
	int32 HighestVersion = -1;

	auto ExtrapolateHighestVersionFromResultNode = [&InParams, OutputSettingNode, &HighestVersion](const FString& FileNameFormat)
	{
		// Calculate a version number by looking at the output path and then scanning for a version token.
		const FString FileNameFormatString = InParams.FileNameOverride.Len() > 0
			? InParams.FileNameOverride
			: OutputSettingNode->OutputDirectory.Path / FileNameFormat;

		FMovieGraphResolveArgs FinalFormatArgs;
		FString FinalPath = ResolveFilenameFormatArguments(FileNameFormatString, InParams, FinalFormatArgs);
		FinalPath = FPaths::ConvertRelativePathToFull(FinalPath);
		FPaths::NormalizeFilename(FinalPath);

		// Can't resolve a version if it's not clear from the path where the version number will be used.
		if (!FinalPath.Contains(TEXT("{version}")))
		{
			return;
		}

		// FinalPath can have {version} either in a folder name or in a file name. We need to find the 'parent' of either the
		// file or folder that contains it. We can do this by looking for {version} and then finding the last "/" character,
		// which will be the containing folder.
		const int32 VersionStringIndex = FinalPath.Find(TEXT("{version}"), ESearchCase::Type::IgnoreCase, ESearchDir::Type::FromStart);
		if (VersionStringIndex >= 0)
		{
			const int32 LastParentFolder = FinalPath.Find(TEXT("/"), ESearchCase::Type::IgnoreCase, ESearchDir::Type::FromEnd, VersionStringIndex);
			FinalPath.LeftInline(LastParentFolder + 1);

			// Now that we have the parent folder of either the folder with the version token, or the file with the version
			// token, we will look through all immediate children and scan for version tokens so we can find the highest one.
			const FRegexPattern VersionSearchPattern(TEXT("v([0-9]{3})"));
			constexpr bool bFindFiles = true;
			constexpr bool bFindDirectories = true;
			const FString SearchString = FinalPath / TEXT("*.*");
			TArray<FString> FoundFilesAndFoldersInDirectory;
			IFileManager& FileManager = IFileManager::Get();
			FileManager.FindFiles(FoundFilesAndFoldersInDirectory, *SearchString, bFindFiles, bFindDirectories);

			for (const FString& Path : FoundFilesAndFoldersInDirectory)
			{
				FRegexMatcher Regex(VersionSearchPattern, *Path);
				if (Regex.FindNext())
				{
					FString Result = Regex.GetCaptureGroup(0);
					if (Result.Len() > 0)
					{
						// Strip the "v" token off, expected pattern is vXXX
						Result.RightChopInline(1);
					}

					int32 VersionNumber = 0;
					LexFromString(VersionNumber, *Result);
					if (VersionNumber > HighestVersion)
					{
						HighestVersion = VersionNumber;
					}
				}
			}
		}
	};
	
	for (const UMovieGraphSettingNode* ResultNode : ResultNodes)
	{
		FString FileNameFormat = FString();

		if (const UMovieGraphFileOutputNode* FileOutputNode = Cast<UMovieGraphFileOutputNode>(ResultNode))
		{
			FileNameFormat = FileOutputNode->FileNameFormat;
		}
		else if (const UMovieGraphCommandLineEncoderNode* CommandLineEncoderNode = Cast<UMovieGraphCommandLineEncoderNode>(ResultNode))
		{
			FileNameFormat = CommandLineEncoderNode->FileNameFormat;
		}

		ExtrapolateHighestVersionFromResultNode(FileNameFormat);
	}
	
	return HighestVersion + (bGetNextVersion ? 1 : 0);
}

int32 UMovieGraphBlueprintLibrary::GetCurrentVersionNumber(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	// This is effectively identical to UMoviePipelineBlueprintLibrary::GetCurrentVersionNumber (since the
	// version is stored on the shot structure which is shared between the two pipelines) but provided here
	// via a UMovieGraphPipeline specific pointer for consistency in fetching values in the UI to go along
	// with the other functions here get things like focal length, etc.
	if (!InMovieGraphPipeline)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get version number from null pipeline!"), ELogVerbosity::Error);
		return 0;
	}
	
	int32 CurrentShotIndex = InMovieGraphPipeline->GetCurrentShotIndex();
	if(!InMovieGraphPipeline->GetActiveShotList().IsValidIndex(CurrentShotIndex))
	{
		FFrame::KismetExecutionMessage(TEXT("No shot is currently active to get the version number from."), ELogVerbosity::Error);
		return 0;
	}

	return InMovieGraphPipeline->GetActiveShotList()[CurrentShotIndex]->ShotInfo.VersionNumber;
}

FIntPoint UMovieGraphBlueprintLibrary::GetEffectiveOutputResolution(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float DefaultOverscan)
{
	return GetOverscannedResolution(InEvaluatedGraph, DefaultOverscan);
}

FIntPoint UMovieGraphBlueprintLibrary::GetDesiredOutputResolution(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float CameraAspectRatio)
{
	if (!InEvaluatedGraph)
	{
		FFrame::KismetExecutionMessage(TEXT("InEvaluatedGraph cannot be null."), ELogVerbosity::Error);
		return FIntPoint();
	}
	
	constexpr bool bIncludeCDOs = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSetting = InEvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs);
	if (!ensure(OutputSetting))
	{
		return FIntPoint();
	}

	// We need to look at the Project Settings for the latest value for a given profile
	FMovieGraphNamedResolution NamedResolution;
	if (UMovieGraphBlueprintLibrary::IsNamedResolutionValid(OutputSetting->OutputResolution.ProfileName))
	{
		NamedResolution = UMovieGraphBlueprintLibrary::NamedResolutionFromProfile(OutputSetting->OutputResolution.ProfileName);
	}
	else
	{
		// Otherwise if it's not in the output settings as a valid profile, we use our internally stored one.
		NamedResolution = OutputSetting->OutputResolution;
	}

	EMovieGraphAspectRatioAdaptBehavior ResizeBehavior = OutputSetting->AdaptResolution;
	if (ResizeBehavior == EMovieGraphAspectRatioAdaptBehavior::Disabled)
	{
		return NamedResolution.Resolution;
	}

	float TargetAspectRatio = NamedResolution.Resolution.X / (float)NamedResolution.Resolution.Y;

	// For backwards compatibility reasons, if they did not specify an aspect ratio, then we use the aspect ratio of the
	// output image. This should result in no change for previous callers that do not specify the camera aspect ratio.
	if (FMath::IsNearlyZero(CameraAspectRatio))
	{
		CameraAspectRatio = TargetAspectRatio;
	}

	// Automatic works by inspecting the aspect ratio and picking the 'correct' behavior that matches
	// what you would see in the viewport, but without the letterboxing.
	if (ResizeBehavior == EMovieGraphAspectRatioAdaptBehavior::Automatic)
	{
		if (CameraAspectRatio > TargetAspectRatio)
		{
			ResizeBehavior = EMovieGraphAspectRatioAdaptBehavior::ScaleToWidth;
		}
		else
		{
			ResizeBehavior = EMovieGraphAspectRatioAdaptBehavior::ScaleToHeight;
		}
	}

	FIntPoint OutputResolution = NamedResolution.Resolution;
	switch (ResizeBehavior)
	{
		case EMovieGraphAspectRatioAdaptBehavior::ScaleToWidth:
		{
			OutputResolution.Y = FMath::RoundToInt(OutputResolution.X / CameraAspectRatio);
			break;

		}
		case EMovieGraphAspectRatioAdaptBehavior::ScaleToHeight:
		{
			OutputResolution.X = FMath::RoundToInt(OutputResolution.Y * CameraAspectRatio);
			break;
		}

	default:
		check(0); // Unsupported
	}

	return OutputResolution;
}

FIntPoint UMovieGraphBlueprintLibrary::GetOverscannedResolution(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float DefaultOverscan, float CameraAspectRatio)
{
	if (!InEvaluatedGraph)
    {
    	FFrame::KismetExecutionMessage(TEXT("InEvaluatedGraph cannot be null."), ELogVerbosity::Error);
    	return FIntPoint();
    }

	FIntPoint DesiredOutputResolution = GetDesiredOutputResolution(InEvaluatedGraph, CameraAspectRatio);
	
    constexpr bool bIncludeCDOs = false;
    float RescaledOverscan = DefaultOverscan;
    if (UMovieGraphCameraSettingNode* CameraSetting = InEvaluatedGraph->GetSettingForBranch<UMovieGraphCameraSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs))
    {
    	if (CameraSetting->bOverride_OverscanPercentage)
    	{
    		// The old system used [0-1] range for floats, the new system will use [0-100], so we rescale down before calling through.
    		RescaledOverscan = FMath::Clamp(CameraSetting->OverscanPercentage / 100.f, 0.f, 1.f);
    	}
    }
    
	return UE::MoviePipeline::ScaleResolutionByOverscan(RescaledOverscan, DesiredOutputResolution);
}

FIntPoint UMovieGraphBlueprintLibrary::GetBackbufferResolution(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float DefaultOverscan, float CameraAspectRatio)
{
	if (!InEvaluatedGraph)
	{
		FFrame::KismetExecutionMessage(TEXT("InEvaluatedGraph cannot be null."), ELogVerbosity::Error);
		return FIntPoint();
	}

	FIntPoint OverscannedResolution = GetOverscannedResolution(InEvaluatedGraph, DefaultOverscan, CameraAspectRatio);

	// TODO: When tiling support is added, compute the appropriate backbuffer resolution from OverscannedResolution using tile count and return it
	return OverscannedResolution;
}

FIntRect UMovieGraphBlueprintLibrary::GetOverscanCropRectangle(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float DefaultOverscan, float CameraAspectRatio)
{
	if (!InEvaluatedGraph)
	{
		FFrame::KismetExecutionMessage(TEXT("InEvaluatedGraph cannot be null."), ELogVerbosity::Error);
		return FIntRect();
	}

	const FIntPoint DesiredOutputResolution = GetDesiredOutputResolution(InEvaluatedGraph, CameraAspectRatio);
	const FIntPoint OverscannedResolution = GetOverscannedResolution(InEvaluatedGraph, DefaultOverscan, CameraAspectRatio);

	FIntRect CropRect;

	// Overscan is symmetrical, so this difference is always evenly divisible by 2.
	CropRect.Min = (OverscannedResolution - DesiredOutputResolution) / 2;
	CropRect.Max = CropRect.Min + DesiredOutputResolution;

	return CropRect;
}

FText UMovieGraphBlueprintLibrary::GetJobName(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	return InMovieGraphPipeline ? FText::FromString(InMovieGraphPipeline->GetCurrentJob()->JobName) : FText();
}

FText UMovieGraphBlueprintLibrary::GetJobAuthor(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	return InMovieGraphPipeline ? FText::FromString(UE::MoviePipeline::GetJobAuthor(InMovieGraphPipeline->GetCurrentJob())) : FText();
}

float UMovieGraphBlueprintLibrary::GetCompletionPercentage(const UMovieGraphPipeline* InPipeline)
{
	if (!InPipeline)
	{
		return 0.f;
	}

	int32 OutputFrames;
	int32 TotalOutputFrames;
	GetOverallOutputFrames(InPipeline, OutputFrames, TotalOutputFrames);

	const float CompletionPercentage = FMath::Clamp(OutputFrames / (float)TotalOutputFrames, 0.f, 1.f);
	return CompletionPercentage;
}

void UMovieGraphBlueprintLibrary::GetOverallOutputFrames(const UMovieGraphPipeline* InMovieGraphPipeline, int32& OutCurrentIndex, int32& OutTotalCount)
{
	OutCurrentIndex = 0;
	OutTotalCount = 0;
	
	if (InMovieGraphPipeline && InMovieGraphPipeline->GetTimeStepInstance())
	{
		OutCurrentIndex = InMovieGraphPipeline->GetTimeStepInstance()->GetCalculatedTimeData().OutputFrameNumber;

		for (const TObjectPtr<UMoviePipelineExecutorShot>& Shot : InMovieGraphPipeline->GetActiveShotList())
		{
			OutTotalCount += Shot->ShotInfo.WorkMetrics.TotalOutputFrameCount;
		}
	}
}

FDateTime UMovieGraphBlueprintLibrary::GetJobInitializationTime(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	return InMovieGraphPipeline ? InMovieGraphPipeline->GetInitializationTime() : FDateTime();
}

bool UMovieGraphBlueprintLibrary::GetEstimatedTimeRemaining(const UMovieGraphPipeline* InMovieGraphPipeline, FTimespan& OutEstimate)
{
	if (!InMovieGraphPipeline)
	{
		OutEstimate = FTimespan();
		return false;
	}

	// If they haven't produced a single frame yet, we can't give an estimate.
	int32 OutputFrames;
	int32 TotalOutputFrames;
	GetOverallOutputFrames(InMovieGraphPipeline, OutputFrames, TotalOutputFrames);

	if (OutputFrames <= 0 || TotalOutputFrames <= 0)
	{
		OutEstimate = FTimespan();
		return false;
	}

	const float CompletionPercentage = OutputFrames / static_cast<float>(TotalOutputFrames);
	const FTimespan CurrentDuration = FDateTime::UtcNow() - InMovieGraphPipeline->GetInitializationTime();

	// If it has taken us CurrentDuration to process CompletionPercentage samples, then we can get a total duration
	// estimate by taking (CurrentDuration/CompletionPercentage) and then take that total estimate minus elapsed
	// to get remaining.
	const FTimespan EstimatedTotalDuration = CurrentDuration / CompletionPercentage;
	OutEstimate = EstimatedTotalDuration - CurrentDuration;

	return true;
}

EMovieRenderPipelineState UMovieGraphBlueprintLibrary::GetPipelineState(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	return InMovieGraphPipeline ? InMovieGraphPipeline->GetPipelineState() : EMovieRenderPipelineState::Uninitialized;
}

EMovieRenderShotState UMovieGraphBlueprintLibrary::GetCurrentSegmentState(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	if (InMovieGraphPipeline)
	{
		const int32 ShotIndex = InMovieGraphPipeline->GetCurrentShotIndex();
		
		if (ShotIndex < InMovieGraphPipeline->GetActiveShotList().Num())
		{
			return InMovieGraphPipeline->GetActiveShotList()[ShotIndex]->ShotInfo.State;
		}
	}

	return EMovieRenderShotState::Uninitialized;
}

void UMovieGraphBlueprintLibrary::GetCurrentSegmentName(const UMovieGraphPipeline* InMovieGraphPipeline, FText& OutOuterName, FText& OutInnerName)
{
	if (InMovieGraphPipeline)
    {
    	const int32 ShotIndex = InMovieGraphPipeline->GetCurrentShotIndex();
		
    	if (ShotIndex < InMovieGraphPipeline->GetActiveShotList().Num())
    	{
    		OutOuterName = FText::FromString(InMovieGraphPipeline->GetActiveShotList()[ShotIndex]->OuterName);
    		OutInnerName = FText::FromString(InMovieGraphPipeline->GetActiveShotList()[ShotIndex]->InnerName);
    	}
    }
}

void UMovieGraphBlueprintLibrary::GetOverallSegmentCounts(const UMovieGraphPipeline* InMovieGraphPipeline, int32& OutCurrentIndex, int32& OutTotalCount)
{
	OutCurrentIndex = InMovieGraphPipeline->GetCurrentShotIndex();
	OutTotalCount = InMovieGraphPipeline->GetActiveShotList().Num();
}

FMoviePipelineSegmentWorkMetrics UMovieGraphBlueprintLibrary::GetCurrentSegmentWorkMetrics(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	if (InMovieGraphPipeline)
	{
		const int32 ShotIndex = InMovieGraphPipeline->GetCurrentShotIndex();
		
		if (ShotIndex < InMovieGraphPipeline->GetActiveShotList().Num())
		{
			return InMovieGraphPipeline->GetActiveShotList()[ShotIndex]->ShotInfo.WorkMetrics;
		}
	}

	return FMoviePipelineSegmentWorkMetrics();
}

FTimecode UMovieGraphBlueprintLibrary::GetRootTimecode(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	if (InMovieGraphPipeline)
	{
		return InMovieGraphPipeline->GetTimeStepInstance()->GetCalculatedTimeData().RootTimeCode;
	}

	return FTimecode();
}

FFrameNumber UMovieGraphBlueprintLibrary::GetRootFrameNumber(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	if (InMovieGraphPipeline)
	{
		return InMovieGraphPipeline->GetTimeStepInstance()->GetCalculatedTimeData().RootFrameNumber;
	}

	return FFrameNumber(-1);
}

FTimecode UMovieGraphBlueprintLibrary::GetCurrentShotTimecode(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	if (InMovieGraphPipeline)
	{
		return InMovieGraphPipeline->GetTimeStepInstance()->GetCalculatedTimeData().ShotTimeCode;
	}

	return FTimecode();
}

FFrameNumber UMovieGraphBlueprintLibrary::GetCurrentShotFrameNumber(const UMovieGraphPipeline* InMovieGraphPipeline)
{
	if (InMovieGraphPipeline)
	{
		return InMovieGraphPipeline->GetTimeStepInstance()->GetCalculatedTimeData().ShotFrameNumber;
	}

	return FFrameNumber(-1);
}

float UMovieGraphBlueprintLibrary::GetCurrentFocusDistance(const UMovieGraphPipeline* InMovieGraphPipeline, int32 InCameraIndex)
{
	if (InCameraIndex == INDEX_NONE)
	{
		if (const UCineCameraComponent* CineCameraComponent = UMoviePipelineBlueprintLibrary::Utility_GetCurrentCineCamera(InMovieGraphPipeline->GetWorld()))
		{
			return CineCameraComponent->CurrentFocusDistance;
		}
	}
	else if (UCineCameraComponent* CurrentCameraComponent = GetCurrentCineCamera(InMovieGraphPipeline, InCameraIndex))
	{
		return CurrentCameraComponent->CurrentFocusDistance;
	}

	return -1.f;
}

float UMovieGraphBlueprintLibrary::GetCurrentFocalLength(const UMovieGraphPipeline* InMovieGraphPipeline, int32 InCameraIndex)
{
	if (InCameraIndex == INDEX_NONE)
	{
		if (const UCineCameraComponent* CineCameraComponent = UMoviePipelineBlueprintLibrary::Utility_GetCurrentCineCamera(InMovieGraphPipeline->GetWorld()))
		{
			return CineCameraComponent->CurrentFocalLength;
		}
	}
	else if (UCineCameraComponent* CurrentCameraComponent = GetCurrentCineCamera(InMovieGraphPipeline, InCameraIndex))
	{
		return CurrentCameraComponent->CurrentFocalLength;
	}

	return -1.f;
}

float UMovieGraphBlueprintLibrary::GetCurrentAperture(const UMovieGraphPipeline* InMovieGraphPipeline, int32 InCameraIndex)
{
	if (InCameraIndex == INDEX_NONE)
	{
		if (const UCineCameraComponent* CineCameraComponent = UMoviePipelineBlueprintLibrary::Utility_GetCurrentCineCamera(InMovieGraphPipeline->GetWorld()))
		{
			return CineCameraComponent->CurrentAperture;
		}

	}
	else if (UCineCameraComponent* CurrentCameraComponent = GetCurrentCineCamera(InMovieGraphPipeline, InCameraIndex))
	{
		return CurrentCameraComponent->CurrentAperture;
	}

	return 0.f;
}

UCineCameraComponent* UMovieGraphBlueprintLibrary::GetCurrentCineCamera(const UMovieGraphPipeline* InMovieGraphPipeline, int32 InCameraIndex)
{
	if (InCameraIndex == INDEX_NONE)
	{
		if (UCineCameraComponent* CineCameraComponent = UMoviePipelineBlueprintLibrary::Utility_GetCurrentCineCamera(InMovieGraphPipeline->GetWorld()))
		{
			return CineCameraComponent;
		}
	}
	else if (UMoviePipelineExecutorShot* CurrentShot = GetCurrentExecutorShot(InMovieGraphPipeline))
	{
		// If we're not rendering all cameras, InCameraIndex is -1.
		const bool bRenderAllCameras = InCameraIndex >= 0;
		TArray<UE::MovieGraph::FMinimalCameraInfo> MinimalCameraInfos = InMovieGraphPipeline->GetDataSourceInstance()->GetCameraInformation(CurrentShot, bRenderAllCameras);
		if (!ensure(MinimalCameraInfos.IsValidIndex(InCameraIndex)))
		{
			return nullptr;
		}

		if (const ACineCameraActor* CineCameraActor = Cast<ACineCameraActor>(MinimalCameraInfos[InCameraIndex].ViewActor))
		{
			return CineCameraActor->GetCineCameraComponent();
		}
	}

	return nullptr;
}

FMovieGraphNamedResolution UMovieGraphBlueprintLibrary::NamedResolutionFromProfile(const FName& InResolutionProfileName)
{
	// Find a matching custom entry from Project Settings
	if (const UMovieGraphProjectSettings* MovieGraphProjectSettings =
		GetDefault<UMovieGraphProjectSettings>())
	{
		if (const FMovieGraphNamedResolution* Match = MovieGraphProjectSettings->FindNamedResolutionForOption(InResolutionProfileName))
		{
			return *Match;
		}
	}

	// If we couldn't find it, throw an exception
	FFrame::KismetExecutionMessage(
		*FString::Printf(
			TEXT("%hs: Could not find named resolution with profile name %s in Project Settings."), __FUNCTION__, *InResolutionProfileName.ToString()),
		ELogVerbosity::Error);

	return FMovieGraphNamedResolution();
}

bool UMovieGraphBlueprintLibrary::IsNamedResolutionValid(const FName& InResolutionProfileName)
{
	if (const UMovieGraphProjectSettings* MovieGraphProjectSettings =
		GetDefault<UMovieGraphProjectSettings>())
	{
		if (const FMovieGraphNamedResolution* Match = MovieGraphProjectSettings->FindNamedResolutionForOption(InResolutionProfileName))
		{
			return true;
		}
	}

	return false;
}

FMovieGraphNamedResolution UMovieGraphBlueprintLibrary::NamedResolutionFromSize(const int32 InResX, const int32 InResY)
{
	return FMovieGraphNamedResolution(FMovieGraphNamedResolution::CustomEntryName, FIntPoint(InResX, InResY), FString());
}

UMoviePipelineExecutorShot* UMovieGraphBlueprintLibrary::GetCurrentExecutorShot(const UMovieGraphPipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		const TArray<UMoviePipelineExecutorShot*>& ActiveShotList = InMoviePipeline->GetActiveShotList();
		int32 CurrentShotIndex = InMoviePipeline->GetCurrentShotIndex();
		if (ActiveShotList.IsValidIndex(CurrentShotIndex))
		{
			return ActiveShotList[CurrentShotIndex];
		}
	}

	return nullptr;
}

UWorld* UMovieGraphBlueprintLibrary::GetCurrentPlayWorld()
{
	if (!GEngine)
	{
		return nullptr;
	}

	// Try the standard path first.
	if (UWorld* PlayWorld = GEngine->GetCurrentPlayWorld())
	{
		return PlayWorld;
	}

	// Fallback: search engine world contexts for a PIE or Game world.
	// This covers early MRQ callbacks (e.g., OnJobStart) where
	// GetCurrentPlayWorld() may return null because the world context
	// hasn't been fully registered yet.
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE || Context.WorldType == EWorldType::Game)
		{
			if (Context.World())
			{
				return Context.World();
			}
		}
	}

	return nullptr;
}
