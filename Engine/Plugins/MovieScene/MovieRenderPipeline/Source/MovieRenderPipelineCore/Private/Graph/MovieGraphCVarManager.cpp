// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphCVarManager.h"

#include "Engine/World.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphApplyCVarPresetNode.h"
#include "Graph/Nodes/MovieGraphSetCVarValueNode.h"
#include "Graph/Nodes/MovieGraphSetStartEndConsoleCommandsNode.h"
#include "HAL/IConsoleManager.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/DefaultValueHelper.h"
#include "MoviePipelineQueue.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Sections/MovieSceneConsoleVariableTrackInterface.h"

namespace UE::MovieGraph::Private
{

void FMovieGraphCVarManager::AddCVar(const FString& InName, const float Value)
{
	FMovieGraphCVarOverride* CVarOverride = CVars.FindByPredicate([&InName](const FMovieGraphCVarOverride& Override)
	{
		return Override.Name == InName;
	});

	if (CVarOverride)
	{
		CVarOverride->Value = Value;
	}
	else
	{
		CVars.Add(FMovieGraphCVarOverride(InName, Value));
	}
}

void FMovieGraphCVarManager::AddPreset(const TScriptInterface<IMovieSceneConsoleVariableTrackInterface>& InPreset)
{
	if (!InPreset)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Invalid CVar preset specified. Ignoring.");
		return;
	}

	constexpr bool bOnlyIncludeChecked = true;
	TArray<TTuple<FString, FString>> PresetCVars;
	InPreset->GetConsoleVariablesForTrack(bOnlyIncludeChecked, PresetCVars);

	for (const TTuple<FString, FString>& CVarPair : PresetCVars)
	{
		float CVarFloatValue = 0.0f;
		bool bCvarBoolValue = false;
		if (FDefaultValueHelper::ParseFloat(CVarPair.Value, CVarFloatValue))
		{
			AddCVar(CVarPair.Key, CVarFloatValue);
		}
		else if (FDefaultValueHelper::ParseBool(CVarPair.Value, bCvarBoolValue))
		{
			AddCVar(CVarPair.Key, bCvarBoolValue ? 1.0f : 0.0f);
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to apply CVar \"%ls\" (from preset \"%ls\") because value could not be parsed into a float. Ignoring.",
				*CVarPair.Key, *InPreset.GetObject()->GetName());
		}
	}
}

void FMovieGraphCVarManager::AddStartConsoleCommands(const TArray<FString>& InStartConsoleCommands)
{
	StartConsoleCommands = InStartConsoleCommands;
}

void FMovieGraphCVarManager::AddEndConsoleCommands(const TArray<FString>& InEndConsoleCommands)
{
	EndConsoleCommands = InEndConsoleCommands;
}

void FMovieGraphCVarManager::AddEvaluatedGraph(const UMovieGraphEvaluatedConfig* InEvaluatedGraph)
{
	const FMovieGraphEvaluatedBranchConfig* GlobalsBranchConfig = InEvaluatedGraph->BranchConfigMapping.Find(UMovieGraphNode::GlobalsPinName);
	if (!GlobalsBranchConfig)
	{
		return;
	}

	// Nodes which set cvars can only be created in the Globals branch, so we only need to look for nodes in that branch
	const TArray<TObjectPtr<UMovieGraphNode>> GlobalsNodes = GlobalsBranchConfig->GetNodes();

	// Iterate the evaluated graph in reverse. Nodes which come first in the array appear furthest downstream in the
	// graph, so they should be applied last.
	for (int32 NodeIndex = GlobalsNodes.Num() - 1; NodeIndex >= 0; --NodeIndex)
	{
		const TObjectPtr<UMovieGraphNode>& Node = GlobalsNodes[NodeIndex];

		if (const UMovieGraphSetCVarValueNode* CVarValueNode = Cast<UMovieGraphSetCVarValueNode>(Node))
		{
			AddCVar(CVarValueNode->Name, CVarValueNode->Value);
		}
		else if (const UMovieGraphApplyCVarPresetNode* ApplyPresetNode = Cast<UMovieGraphApplyCVarPresetNode>(Node))
		{
			AddPreset(ApplyPresetNode->ConsoleVariablePreset);

			// Preset nodes can optionally have overrides on top of the preset via promoted pins (and variables connected to them)
			for (const TPair<FString, float>& CvarOverride : ApplyPresetNode->GetConsoleVariableOverrides())
			{
				AddCVar(CvarOverride.Key, CvarOverride.Value);
			}
		}
		else if (const UMovieGraphSetStartEndConsoleCommandsNode* ConsoleCommandsNode = Cast<UMovieGraphSetStartEndConsoleCommandsNode>(Node))
		{
			AddStartConsoleCommands(ConsoleCommandsNode->ConsoleCommands->AddStartCommands);
			AddEndConsoleCommands(ConsoleCommandsNode->ConsoleCommands->AddEndCommands);
		}
	}
}

void FMovieGraphCVarManager::AddShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
// UMoviePipelineExecutorShot and UMoviePipelineExecutorJob don't share a common base class, hence the need for a define vs. a lambda.
#define ADD_CVARS(Owner) \
	for (const FMoviePipelineConsoleVariableEntry& CVarEntry : Owner->ConsoleVariableOverrides) \
	{ \
		if (CVarEntry.bIsEnabled) \
		{ \
			AddCVar(CVarEntry.Name, CVarEntry.Value); \
		} \
	}
	
	if (InShot)
	{
		// Add cvars specified on the parent job first. The shot's cvars will have a chance to override these if needed.
		if (const UMoviePipelineExecutorJob* ParentJob = InShot->GetTypedOuter<UMoviePipelineExecutorJob>())
		{
			ADD_CVARS(ParentJob)
		}

		// Then add the shot's cvar overrides.
		ADD_CVARS(InShot)
	}
#undef ADD_CVARS
}

void FMovieGraphCVarManager::ApplyAllCVars()
{
	PreviousConsoleVariableValues.Reset();
	PreviousConsoleVariableValues.SetNumZeroed(CVars.Num());

	for (int32 CVarIndex = 0; CVarIndex < CVars.Num(); ++CVarIndex)
	{
		const FMovieGraphCVarOverride& CVarEntry = CVars[CVarIndex];
		const FString TrimmedCvar = CVarEntry.Name.TrimStartAndEnd();
	
		if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*TrimmedCvar))
		{
			PreviousConsoleVariableValues[CVarIndex] = CVar->GetFloat();
			ApplyCVar(CVar, CVarEntry.Value);
			UE_LOGF(LogMovieRenderPipeline, Log, "Applying CVar \"%ls\", PreviousValue: %f NewValue: %f", *CVarEntry.Name, PreviousConsoleVariableValues[CVarIndex], CVarEntry.Value);
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to apply CVar \"%ls\" due to no cvar by that name. Ignoring.", *CVarEntry.Name);
		}
	}
}

void FMovieGraphCVarManager::RevertAllCVars()
{
	for (int32 CVarIndex = 0; CVarIndex < CVars.Num(); ++CVarIndex)
	{
		const FMovieGraphCVarOverride& CVarEntry = CVars[CVarIndex];
		const FString TrimmedCvar = CVarEntry.Name.TrimStartAndEnd();
	
		if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*TrimmedCvar))
		{
			ApplyCVar(CVar, PreviousConsoleVariableValues[CVarIndex]);
			UE_LOGF(LogMovieRenderPipeline, Log, "Restoring CVar \"%ls\", PreviousValue: %f NewValue: %f", *CVarEntry.Name, CVarEntry.Value, PreviousConsoleVariableValues[CVarIndex]);
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to revert CVar \"%ls\" due to no cvar by that name. Ignoring.", *CVarEntry.Name);
		}
	}

	CVars.Reset();
	PreviousConsoleVariableValues.Reset();
}

void FMovieGraphCVarManager::RunStartConsoleCommands()
{
	for (const FString& StartCommand : StartConsoleCommands)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Executing Console Command \"%ls\" before shot starts.", *StartCommand);
		UKismetSystemLibrary::ExecuteConsoleCommand(WorldContext, StartCommand, nullptr);
	}
}

void FMovieGraphCVarManager::RunEndConsoleCommands()
{
	for (const FString& EndCommand : EndConsoleCommands)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Executing Console Command \"%ls\" after shot ends.", *EndCommand);
		UKismetSystemLibrary::ExecuteConsoleCommand(WorldContext, EndCommand, nullptr);
	}
}

void FMovieGraphCVarManager::SetWorld(UWorld* InWorld)
{
	WorldContext = InWorld;
}

void FMovieGraphCVarManager::ApplyCVar(IConsoleVariable* InCVar, float InValue)
{
	check(InCVar);

	// When Set is called on a cvar the value is turned into a string. With very large
	// floats this is turned into scientific notation. If the cvar is later retrieved as
	// an integer, the scientific notation doesn't parse into integer correctly. We'll
	// cast to integer first (to avoid scientific notation) if we know the cvar is an integer.
	if (InCVar->IsVariableInt())
	{
		InCVar->SetWithCurrentPriority(static_cast<int32>(InValue), NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}
	else if (InCVar->IsVariableBool())
	{
		InCVar->SetWithCurrentPriority(InValue != 0.f ? true : false, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}
	else
	{
		InCVar->SetWithCurrentPriority(InValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}
}

} // namespace UE::MovieGraph::Private
