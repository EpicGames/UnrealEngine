// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/ScriptInterface.h"

class IMovieSceneConsoleVariableTrackInterface;
class UMovieGraphEvaluatedConfig;
class UMoviePipelineExecutorShot;
class UMovieScene;

namespace UE::MovieGraph::Private
{
	/**
	 * Responsible for applying individual cvars, cvar presets, and console commands. CVars can be added to the manager with the Add*() methods,
	 * then applied via ApplyAllCVars(). Console commands can be added with their associated Add*() methods, and executed via the Run*() methods.
	 */
	class FMovieGraphCVarManager final
	{
	public:
		FMovieGraphCVarManager() = default;

		/**
		 * Adds a cvar that should be applied with the given Value. If the cvar with InName already exists, its value
		 * will be updated.
		 */
		void AddCVar(const FString& InName, const float Value);

		/**
		 * Adds a preset that should be applied. If the preset contains cvars that have already been added via other Add*()
		 * calls, the values in the preset will override values already added.
		 */
		void AddPreset(const TScriptInterface<IMovieSceneConsoleVariableTrackInterface>& InPreset);

		/** Adds console commands that should be run before a shot starts rendering. */
		void AddStartConsoleCommands(const TArray<FString>& InStartConsoleCommands);

		/** Adds console commands that should be run after a shot finishes rendering. */
		void AddEndConsoleCommands(const TArray<FString>& InEndConsoleCommands);

		/** For all cvar, cvar preset, and console command nodes in InEvaluatedGraph, calls either AddCVar(), AddPreset(), or Add*ConsoleCommands(). */
		void AddEvaluatedGraph(const UMovieGraphEvaluatedConfig* InEvaluatedGraph);

		/**
		 * Calls AddCvar() for all console variables that have been added via overrides on the shot, as well as the shot's parent job. Console
		 * variables set on the shot and job have priority over all other cvars that have been set (with shot-based cvars having the highest
		 * priority).
		 */
		void AddShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot);

		/** Applies all cvars that have been gathered via the Add*() methods. */
		void ApplyAllCVars();

		/**
		 * For all cvars that were gathered via the Add*() methods and applied, reverts their values to what they were before
		 * being applied. After calling this, ApplyAllCVars() is a no-op, and the Add*() methods need to be called again.
		 */
		void RevertAllCVars();

		/** Runs all start console commands that have been added. */
		void RunStartConsoleCommands();

		/** Runs all end console commands that have been added. */
		void RunEndConsoleCommands();

		/** Sets the world context that the console commands should use when executing. */
		void SetWorld(UWorld* InWorld);

	private:
		/** Sets the given cvar, InCVar, to InValue. */
		static void ApplyCVar(IConsoleVariable* InCVar, float InValue);

		/** Represents a console variable override. */
		struct FMovieGraphCVarOverride
		{
			FMovieGraphCVarOverride(const FString& InName, const float InValue)
				: Name(InName)
				, Value(InValue)
			{
			}

			/* The name of the console variable. */
			FString Name;

			/* The value of the console variable. */
			float Value;
		};

	private:
		/** The merged result of both individual CVars and CVar presets. */
		TArray<FMovieGraphCVarOverride> CVars;

		/** The values of the gathered cvars before the the manager sets their values. */
		TArray<float> PreviousConsoleVariableValues;

		/** The start console commands that were added. */
		TArray<FString> StartConsoleCommands;

		/** The end console commands that were added. */
		TArray<FString> EndConsoleCommands;

		/** The world that should be used as the context when executing console commands. */
		UWorld* WorldContext;
	};
} // namespace UE::MovieGraph::Private