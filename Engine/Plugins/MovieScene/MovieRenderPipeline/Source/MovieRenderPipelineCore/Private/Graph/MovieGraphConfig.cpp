// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphConfig.h"

#include "Algo/Transform.h"
#include "CineCameraComponent.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphEdge.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphInputNode.h"
#include "Graph/Nodes/MovieGraphOutputNode.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "Graph/Nodes/MovieGraphRemoveRenderSettingNode.h"
#include "Graph/Nodes/MovieGraphRenderLayerNode.h"
#include "Graph/Nodes/MovieGraphRerouteNode.h"
#include "Graph/Nodes/MovieGraphSubgraphNode.h"
#include "Graph/Nodes/MovieGraphVariableNode.h"
#include "Graph/Nodes/MovieGraphSelectNode.h"
#include "Graph/MovieGraphUtils.h"
#include "MovieGraphUtils.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphConfig)

#define LOCTEXT_NAMESPACE "MovieGraphConfig"

namespace
{
	void RecurseUpGlobalsBranchToFindOutputDirectory_Internal(const UMovieGraphConfig* InGraph, const UMovieGraphNode* InNode, FString& OutOutputDirectory, TArray<const UMovieGraphConfig*>& VisitedGraphStack, TSet<const UMovieGraphNode*>& VisitedNodes)
	{
		// If there's no Node, or we've already visited this node, there is no way to continue so we early out.
		if (!InGraph || !InNode || VisitedNodes.Contains(InNode))
		{
			return;
		}

		VisitedNodes.Add(InNode);

		if (const UMovieGraphGlobalOutputSettingNode* SettingsNode = Cast<UMovieGraphGlobalOutputSettingNode>(InNode))
		{
			if (!SettingsNode->IsDisabled() && SettingsNode->bOverride_OutputDirectory)
			{
				OutOutputDirectory = SettingsNode->OutputDirectory.Path;
				return;
			}
		}

		// Only globals can connect to globals linearly, so we only need to look at the first connected input.
		// The only exception being subgraph nodes which will need to be separately evaluated.
		const UMovieGraphPin* DownstreamGlobalsPin = InNode->GetFirstConnectedInputPin();
		if (!DownstreamGlobalsPin)
		{
			return;
		}

		const UMovieGraphPin* UpstreamGlobalsPin = DownstreamGlobalsPin->GetFirstConnectedPin();
		if (!UpstreamGlobalsPin)
		{
			return;
		}

		const UMovieGraphNode* ConnectedNode = UpstreamGlobalsPin->Node;

		// Overrides can be set within Subgraphs.
		if (const UMovieGraphSubgraphNode* SubgraphNode = Cast<UMovieGraphSubgraphNode>(ConnectedNode); SubgraphNode && !SubgraphNode->IsDisabled())
		{
			const UMovieGraphConfig* SubgraphConfig = SubgraphNode->GetSubgraphAsset();

			// Stop recursing if circular references are found.
			if (SubgraphConfig && !VisitedGraphStack.Contains(SubgraphConfig) && OutOutputDirectory.IsEmpty())
			{
				VisitedGraphStack.Push(SubgraphConfig);

				const UMovieGraphPin* SubgraphGlobalsPin =
					SubgraphConfig->GetOutputNode()->GetInputPin(UMovieGraphNode::GlobalsPinName);

				if (SubgraphGlobalsPin && SubgraphGlobalsPin->IsConnected())
				{
					RecurseUpGlobalsBranchToFindOutputDirectory_Internal(SubgraphConfig, SubgraphGlobalsPin->Node, OutOutputDirectory, VisitedGraphStack, VisitedNodes);
				}

				VisitedGraphStack.Pop(EAllowShrinking::No);
			}
		}

		// Keep looking upstream if we haven't found any overrides.
		if (OutOutputDirectory.IsEmpty())
		{
			RecurseUpGlobalsBranchToFindOutputDirectory_Internal(InGraph, UpstreamGlobalsPin->Node, OutOutputDirectory, VisitedGraphStack, VisitedNodes);
		}
	}
}

UMovieGraphConfig* UMovieGraphMember::GetOwningGraph() const
{
	return GetTypedOuter<UMovieGraphConfig>();
}

bool UMovieGraphMember::SetMemberName(const FString& InNewName)
{
	FText UnusedError;
	if (CanRename(FText::FromString(InNewName), UnusedError))
	{
		Modify();
		Name = InNewName;
		
		return true;
	}

	return false;
}

bool UMovieGraphMember::CanRename(const FText& InNewName, FText& OutError) const
{
	static const FString InvalidChars("\"',\n\r\t");
	
	if (InNewName.IsEmptyOrWhitespace())
	{
		OutError = LOCTEXT("InvalidMemberRename_Empty", "The name cannot be empty.");
		return false;
	}

	if (InNewName.ToString() == UMovieGraphNode::GlobalsPinNameString)
	{
		OutError = LOCTEXT("InvalidMemberRename_Globals", "The name cannot be 'Globals'.");
		return false;
	}

	const FString NewNameString = InNewName.ToString();
	if (!FName::IsValidXName(NewNameString, InvalidChars, &OutError))
	{
		return false;
	}

	return true;
}

bool UMovieGraphVariable::IsGlobal() const
{
	return IsA<UMovieGraphGlobalVariable>();
}

const FString& UMovieGraphVariable::GetCategory() const
{
	return Category;
}

void UMovieGraphVariable::SetCategory(const FString& InNewCategory)
{
	// Sets this variable to have a specific category with Pre/Post change events sent properly
	auto SetCategoryWithEvents = [this](const FString& NewCategory)
	{
#if WITH_EDITOR
		FProperty* CategoryProperty = FindFProperty<FProperty>(GetClass(), GET_MEMBER_NAME_CHECKED(UMovieGraphVariable, Category));
		PreEditChange(CategoryProperty);

		Modify();
#endif

		// The category needs to be put through NameToDisplayString() to prevent a host of category-matching issues in the graph's action menu that
		// cannot be controlled/changed on the MRG side. For example, when dragging/dropping a category, the provided category in DroppedOnCategory()
		// will be the *display name* of the category, not the variable's actual category. Sometimes the display name and actual category can differ,
		// (for example, if the variable category was set to "Test1", the display name would be "Test 1"). To mitigate all of these mismatch issues,
		// just put all categories through NameToDisplayString(), which is how the display name is generated. The ReplaceInline() fixes a weird issue
		// with NameToDisplayString().
		constexpr bool bIsBool = false;
		Category = FName::NameToDisplayString(NewCategory, bIsBool);
		Category.ReplaceInline(TEXT("| "), TEXT("|"), ESearchCase::CaseSensitive);

#if WITH_EDITOR
		FPropertyChangedEvent PropertyUpdate(CategoryProperty);
		PostEditChangeProperty(PropertyUpdate);
#endif	// WITH_EDITOR
	};

	// Nothing to do if the category didn't change
	if (Category == InNewCategory)
	{
		return;
	}
	
	const TArray<UMovieGraphVariable*> GraphVariables = GetOwningGraph()->GetVariables();
	
	// Make sure the variable goes in the correct position in the array when changing the category. Variables need to remain sorted by their category,
	// otherwise other operations (eg, moving a variable up/down in the array) will not work as expected when variables are categorized.

	// First, get all the categories that are present.
	TArray<FString> Categories;
	Algo::Transform(GraphVariables, Categories, [](const UMovieGraphVariable* Variable) { return Variable->GetCategory(); });

	// Second, if the newly-set category already exists, then just move the variable to be immediately after the last variable within the category.
	if (Categories.Contains(InNewCategory))
	{
		const int32 LastIndex = GraphVariables.FindLastByPredicate([&InNewCategory](const UMovieGraphVariable* GraphVariable)
		{
			return GraphVariable->GetCategory() == InNewCategory;
		});

		if (LastIndex != INDEX_NONE)
		{
			GetOwningGraph()->MoveVariableToIndex(this, LastIndex + 1);
		}
	}

	// Otherwise, if the newly-set category does not exist, the situation is a bit more complex.
	else
	{
		// If moving the variable into category "Foo|Bar|Baz", generate all potential root categories (Foo, Foo|Bar, and Foo|Bar|Baz).
		TArray<FString> CategoryParts;
		InNewCategory.ParseIntoArray(CategoryParts, TEXT("|"));
		TArray<FString> CategoryCandidates;
		for (int32 CategoryIndex = 0; CategoryIndex < CategoryParts.Num(); ++CategoryIndex)
		{
			if (CategoryIndex == 0)
			{
				CategoryCandidates.Add(CategoryParts[0]);
			}
			else
			{
				CategoryCandidates.Add(CategoryCandidates.Last() + TEXT("|") + CategoryParts[CategoryIndex]);
			}
		}

		// Find the variable with the closest matching category. For example, if setting the category to "Foo|Bar|Baz", if no variable was set in either
		// "Foo|Bar|Baz" or "Foo|Bar", "Foo" would be the closest matching category. Candidates are iterated in reverse because the deepest category
		// should be matched first if possible ("Foo|Bar|Baz" in this example).
		int32 BestIndex = INDEX_NONE;
		for (int32 CategoryIndex = CategoryCandidates.Num() - 1; CategoryIndex >= 0; --CategoryIndex)
		{
			const FString CurrentCategory = CategoryCandidates[CategoryIndex];

			BestIndex = GraphVariables.IndexOfByPredicate([&CurrentCategory](const UMovieGraphVariable* GraphVariable)
			{
				return GraphVariable->GetCategory() == CurrentCategory;
			});

			if (BestIndex != INDEX_NONE)
			{
				break;
			}
		}

		// If there are variables with a matching root set already, move the variable to be immediately before the variable with a matching category.
		if (BestIndex != INDEX_NONE)
		{
			GetOwningGraph()->MoveVariableToIndex(this, BestIndex);
		}
		else
		{
			// Find the last variable with any category set, and move it to be after that. New categories go to the end.
			BestIndex = GraphVariables.FindLastByPredicate([](const UMovieGraphVariable* GraphVariable)
			{
				return !GraphVariable->GetCategory().IsEmpty();
			});
			
			if (BestIndex == INDEX_NONE)
			{
				// If ALL categories are currently empty (meaning no variables have a category set yet) then move this variable to be the first
				// in the array.
				GetOwningGraph()->MoveVariableToIndex(this, 0);
			}
			else if (BestIndex == (GraphVariables.Num() - 1))
			{
				// ALL variables have a category set, so move to the very end.
				GetOwningGraph()->MoveVariableToIndex(this, GraphVariables.Num());
			}
			else
			{
				// Move to after the last variable with a category set 
				GetOwningGraph()->MoveVariableToIndex(this, BestIndex + 1);
			}
		}
	}
	
	SetCategoryWithEvents(InNewCategory);
}

bool UMovieGraphVariable::IsDeletable() const
{
	return true;
}

bool UMovieGraphVariable::CanRename(const FText& InNewName, FText& OutError) const
{
	if (!Super::CanRename(InNewName, OutError))
	{
		return false;
	}

	if (const UMovieGraphConfig* Graph = GetOwningGraph())
	{
		constexpr bool bIncludeGlobal = true;
		if (!IsUniqueNameInMemberArray(InNewName, Graph->GetVariables(bIncludeGlobal)))
		{
			OutError = LOCTEXT("InvalidVariableRename_Exists", "A variable with this name already exists.");
			return false;
		}
	}

	return true;
}

bool UMovieGraphVariable::SetMemberName(const FString& InNewName)
{
	bool bSuccess = Super::SetMemberName(InNewName);

#if WITH_EDITOR
	OnMovieGraphVariableChangedDelegate.Broadcast(this);
#endif
	
	return bSuccess;
}

#if WITH_EDITOR
void UMovieGraphVariable::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnMovieGraphVariableChangedDelegate.Broadcast(this);
}
#endif // WITH_EDITOR

UMovieGraphGlobalVariable::UMovieGraphGlobalVariable()
{
	bIsEditable = false;
}

bool UMovieGraphGlobalVariable::IsDeletable() const
{
	return false;
}

bool UMovieGraphGlobalVariable::CanRename(const FText& InNewName, FText& OutError) const
{
	return false;
}

UMovieGraphGlobalVariable_ShotName::UMovieGraphGlobalVariable_ShotName()
{
	Name = FString(TEXT("shot_name"));

	// SetValueType() will result in a PostEditChangeProperty call; ensure it's only called on the game thread
	AsyncTask(ENamedThreads::GameThread, [WeakThis = TWeakObjectPtr(this)]()
	{
		if (UMovieGraphGlobalVariable_ShotName* ShotVariable = WeakThis.Get())
		{
			ShotVariable->SetValueType(EMovieGraphValueType::String);
		}
	});
}

UMovieGraphGlobalVariable_SequenceName::UMovieGraphGlobalVariable_SequenceName()
{
	Name = FString(TEXT("seq_name"));

	// SetValueType() will result in a PostEditChangeProperty call; ensure it's only called on the game thread
	AsyncTask(ENamedThreads::GameThread, [WeakThis = TWeakObjectPtr(this)]()
	{
		if (UMovieGraphGlobalVariable_SequenceName* SequenceVariable = WeakThis.Get())
		{
			SequenceVariable->SetValueType(EMovieGraphValueType::String);
		}
	});
}

UMovieGraphGlobalVariable_FrameNumber::UMovieGraphGlobalVariable_FrameNumber()
{
	Name = FString(TEXT("frame_num"));

	// SetValueType() will result in a PostEditChangeProperty call; ensure it's only called on the game thread
	AsyncTask(ENamedThreads::GameThread, [WeakThis = TWeakObjectPtr(this)]()
	{
		if (UMovieGraphGlobalVariable_FrameNumber* FrameVariable = WeakThis.Get())
		{
			FrameVariable->SetValueType(EMovieGraphValueType::Int32);
		}
	});
}

UMovieGraphGlobalVariable_CameraName::UMovieGraphGlobalVariable_CameraName()
{
	Name = FString(TEXT("camera_name"));

	// SetValueType() will result in a PostEditChangeProperty call; ensure it's only called on the game thread
	AsyncTask(ENamedThreads::GameThread, [WeakThis = TWeakObjectPtr(this)]()
	{
		if (UMovieGraphGlobalVariable_CameraName* CameraVariable = WeakThis.Get())
		{
			CameraVariable->SetValueType(EMovieGraphValueType::String);
		}
	});
}

void UMovieGraphGlobalVariable_ShotName::UpdateValue(const FMovieGraphTraversalContext* InTraversalContext, const UMovieGraphPipeline* InPipeline)
{
	const TArray<TObjectPtr<UMoviePipelineExecutorShot>>& ShotList = InPipeline->GetActiveShotList();
	
	if (ShotList.IsValidIndex(InTraversalContext->ShotIndex))
	{
		if (const TObjectPtr<UMoviePipelineExecutorShot>& Shot = ShotList[InTraversalContext->ShotIndex])
		{
			SetValueString(Shot->OuterName);
		}
	}
}

void UMovieGraphGlobalVariable_SequenceName::UpdateValue(const FMovieGraphTraversalContext* InTraversalContext, const UMovieGraphPipeline* InPipeline)
{
	SetValueString(InTraversalContext->Job->Sequence.GetAssetName());
}

void UMovieGraphGlobalVariable_FrameNumber::UpdateValue(const FMovieGraphTraversalContext* InTraversalContext, const UMovieGraphPipeline* InPipeline)
{
	SetValueInt32(InTraversalContext->Time.ShotFrameNumber.Value);
}

void UMovieGraphGlobalVariable_CameraName::UpdateValue(const FMovieGraphTraversalContext* InTraversalContext, const UMovieGraphPipeline* InPipeline)
{
	const TArray<TObjectPtr<UMoviePipelineExecutorShot>>& ShotList = InPipeline->GetActiveShotList();
	
	if (ShotList.IsValidIndex(InTraversalContext->ShotIndex))
	{
		if (const TObjectPtr<UMoviePipelineExecutorShot>& Shot = ShotList[InTraversalContext->ShotIndex])
		{
			SetValueString(Shot->InnerName);
		}
	}
}

bool UMovieGraphInput::IsDeletable() const
{
	// The input is deletable as long as it's not the Globals input
	return Name != UMovieGraphNode::GlobalsPinNameString;
}

bool UMovieGraphInput::CanRename(const FText& InNewName, FText& OutError) const
{
	if (!Super::CanRename(InNewName, OutError))
	{
		return false;
	}

	if (const UMovieGraphConfig* Graph = GetOwningGraph())
	{
		if (!IsUniqueNameInMemberArray(InNewName, Graph->GetInputs()))
		{
			OutError = LOCTEXT("InvalidInputRename_Exists", "An input with this name already exists.");
			return false;
		}
	}

	return true;
}

bool UMovieGraphInput::SetMemberName(const FString& InNewName)
{
	bool bSuccess = Super::SetMemberName(InNewName);

#if WITH_EDITOR
	OnMovieGraphInputChangedDelegate.Broadcast(this);
#endif
	return bSuccess;
}

#if WITH_EDITOR
void UMovieGraphInput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	OnMovieGraphInputChangedDelegate.Broadcast(this);
}
#endif // WITH_EDITOR

bool UMovieGraphOutput::IsDeletable() const
{
	// The output is deletable as long as it's not the Globals output
	return Name != UMovieGraphNode::GlobalsPinNameString;
}

bool UMovieGraphOutput::CanRename(const FText& InNewName, FText& OutError) const
{
	if (!Super::CanRename(InNewName, OutError))
	{
		return false;
	}

	if (const UMovieGraphConfig* Graph = GetOwningGraph())
	{
		if (!IsUniqueNameInMemberArray(InNewName, Graph->GetOutputs()))
		{
			OutError = LOCTEXT("InvalidOutputRename_Exists", "An output with this name already exists.");
			return false;
		}
	}

	return true;
}

bool UMovieGraphOutput::SetMemberName(const FString& InNewName)
{
	bool bSuccess = Super::SetMemberName(InNewName);

#if WITH_EDITOR
	OnMovieGraphOutputChangedDelegate.Broadcast(this);
#endif
	
	return bSuccess;
}

#if WITH_EDITOR
void UMovieGraphOutput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnMovieGraphOutputChangedDelegate.Broadcast(this);
}
#endif // WITH_EDITOR

bool UMovieGraphEvaluatedConfig::GetVariableValueByName(const FString& InVariableName, UMovieGraphValueContainer*& OutVariableValue) const
{
	if (const TArray<UMovieGraphValueContainer*>* Variables = VariablesByName.Find(InVariableName))
	{
		if (Variables->Num() > 1)
		{
			MRG_LOG_ONCE_PER_RENDER(DuplicateVariableNameWarning, LogMovieRenderPipeline, Warning,
				TEXT("GetVariableValueByName(): Found a duplicate variable name [%s] within the graphs being used to render (including subgraphs). "
					 "It's suggested that variable names are unique so their values can be fetched reliably from within blueprints."), *InVariableName);
		}

		UMovieGraphValueContainer* FirstVariableWithName = (*Variables)[0];
		check(FirstVariableWithName);

		OutVariableValue = FirstVariableWithName;
		return true;
	}

	// It's not ideal to allocate a new "default" value container in the case of an invalid variable name. However, doing this dramatically simplifies
	// blueprints (eg, you don't have to do complicated control flow branching based on the true/false return value of this function to avoid
	// evaluating a null OutVariableValue) and the allocation is arguably worth the convenience.
	OutVariableValue = NewObject<UMovieGraphValueContainer>(GetTransientPackage());
	return false;
}

bool UMovieGraphEvaluatedConfig::GetVariableStringValueByName(const FString& InVariableName, const FString& InDefaultValue, FString& OutVariableValue) const
{
	UMovieGraphValueContainer* ValueContainer;
	if (GetVariableValueByName(InVariableName, ValueContainer))
	{
		OutVariableValue = ValueContainer->GetValueSerializedString();
		return true;
	}

	OutVariableValue = InDefaultValue;
	return false;
}

const TArray<UMovieGraphConfig*>& UMovieGraphEvaluatedConfig::GetEvaluatedGraphs() const
{
	return EvaluatedGraphs;
}

UMovieGraphConfig::UMovieGraphConfig()
{
	InputNode = CreateDefaultSubobject<UMovieGraphInputNode>(TEXT("DefaultInputNode"));
	OutputNode = CreateDefaultSubobject<UMovieGraphOutputNode>(TEXT("DefaultOutputNode"));

	// Don't add default members in the ctor if this object is being loaded (ie, it's not a new object). Defer that
	// until PostLoad(), otherwise the default members may be overwritten when properties are loaded.
	const bool bIsNewObject = !HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad | RF_NeedPostLoad);
	if (bIsNewObject)
	{
		AddDefaultMembers();
		InputNode->UpdatePins();
		OutputNode->UpdatePins();

		// Offset the default output node so it doesn't overlap the default input node
#if WITH_EDITOR
		constexpr int32 OutputNodeOffset = 900;
		OutputNode->SetNodePosX(OutputNodeOffset);
#endif
	}
}

void UMovieGraphConfig::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// TODO: When the graph has stabilized, we can remove this and replace with a system that solely performs
		// upgrades/deprecations. For now, we assume that each load of the graph should re-initialize all default
		// members.
		AddDefaultMembers();

		// Fire OnGraphVariablesChangedDelegate when the variable changes (name, value, type, etc)
#if WITH_EDITOR
		for (const TObjectPtr<UMovieGraphVariable>& Variable : Variables)
		{
			Variable->OnMovieGraphVariableChangedDelegate.AddWeakLambda(this, [this](UMovieGraphMember*)
			{
				OnGraphVariablesChangedDelegate.Broadcast();
			});
		}

		// Remove all null nodes
		AllNodes.RemoveAll(
			[](UMovieGraphNode* Node)
			{
				if (!Node)
				{
					UE_LOGF(LogMovieRenderPipeline, Warning, "Encountered invalid source node (nullptr) when building Movie Pipeline Editor graph, skipping creating an editor graph node for the invalid source node.")
					return true;
				}
				return false;
			});
#endif
	}
}

template<typename T>
T* UMovieGraphConfig::AddGlobalVariable()
{
	// Don't add duplicate global variables
	const bool VariableExists = GlobalVariables.ContainsByPredicate([](const TObjectPtr<UMovieGraphVariable>& Variable)
	{
		return Variable && (Variable->GetClass() == T::StaticClass());
	});

	if (VariableExists)
	{
		// Don't log here; graphs will typically try to add all available global variables on start-up, even if they
		// already exist in the current graph
		return nullptr;
	}

	// Pass an empty name to AddMember() since globals set their name upon construction
	return AddMember<T>(GlobalVariables, FName());
}

void UMovieGraphConfig::AddDefaultMembers()
{
	const bool InputGlobalsExists = Inputs.ContainsByPredicate([](const UMovieGraphMember* Member)
	{
		return Member && (Member->GetMemberName() == UMovieGraphNode::GlobalsPinNameString);
	});

	const bool OutputGlobalsExists = Outputs.ContainsByPredicate([](const UMovieGraphMember* Member)
	{
		return Member && (Member->GetMemberName() == UMovieGraphNode::GlobalsPinNameString);
	});

	// Ensure there is a Globals input member
	if (!InputGlobalsExists)
	{
		UMovieGraphInput* NewInput = AddInput();

		// Don't call SetMemberName() here, because that will reject setting the name to Globals
		NewInput->Name = UMovieGraphNode::GlobalsPinNameString;

		InputNode->UpdatePins();
	}

	// Ensure there is a Globals output member
	if (!OutputGlobalsExists)
	{
		UMovieGraphOutput* NewOutput = AddOutput();

		// Don't call SetMemberName() here, because that will reject setting the name to Globals
		NewOutput->Name = UMovieGraphNode::GlobalsPinNameString;

		OutputNode->UpdatePins();
	}

	AddGlobalVariable<UMovieGraphGlobalVariable_CameraName>();
	AddGlobalVariable<UMovieGraphGlobalVariable_FrameNumber>();
	AddGlobalVariable<UMovieGraphGlobalVariable_SequenceName>();
	AddGlobalVariable<UMovieGraphGlobalVariable_ShotName>();
}

bool UMovieGraphConfig::AddLabeledEdge(UMovieGraphNode* FromNode, const FName& FromPinLabel, UMovieGraphNode* ToNode, const FName& ToPinLabel)
{
	if (!FromNode || !ToNode)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: Invalid Edge Nodes"), __FUNCTION__),
				ELogVerbosity::Error);
		return false;
	}
	

	UMovieGraphPin* FromPin = FromNode->GetOutputPin(FromPinLabel);
	if (!FromPin)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: FromNode: %s does not have a pin with the label: %s"), __FUNCTION__, *FromNode->GetName(), *FromPinLabel.ToString()),
				ELogVerbosity::Error);
		return false;
	}

	UMovieGraphPin* ToPin = ToNode->GetInputPin(ToPinLabel);
	if (!ToPin)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: ToNode: %s does not have a pin with the label: %s"), __FUNCTION__, *ToNode->GetName(), *ToPinLabel.ToString()),
				ELogVerbosity::Error);
		return false;
	}

	if (!FromPin->CanCreateConnection(ToPin))
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: FromNode %s's pin %s cannot be connected to ToNode %s's pin %s "), __FUNCTION__, *FromNode->GetName(), *FromPinLabel.ToString(), *ToNode->GetName(), *ToPinLabel.ToString()),
				ELogVerbosity::Error);
		return false;
	}

	bool bConnectionBrokeOtherEdges = false;

	// Input pins can only have one edge connected to them at once, so if the pin we're connecting to already
	// has a connection, then we break the existing connection.
	if (!ToPin->AllowsMultipleConnections() && ToPin->EdgeCount() > 0)
	{
		ToPin->BreakAllEdges();
		bConnectionBrokeOtherEdges = true;
	}

	// Add the edge. We do this after the above
	// since that will break all edges first if there's already one.
	FromPin->AddEdgeTo(ToPin);
//
//#if WITH_EDITOR
//	NotifyGraphChanged(EMoviePipelineGraphChangeType::Structural);
//#endif

	return bConnectionBrokeOtherEdges;
}

bool UMovieGraphConfig::RemoveLabeledEdge(UMovieGraphNode* FromNode, const FName& FromPinLabel, UMovieGraphNode* ToNode, const FName& ToPinLabel)
{
	if (!FromNode || !ToNode)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(TEXT("%hs: Invalid Edge Nodes"), __FUNCTION__), ELogVerbosity::Error);
		return false;
	}

	UMovieGraphPin* FromPin = FromNode->GetOutputPin(FromPinLabel);
	if (!FromPin)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: FromNode: %s does not have a pin with the label: %s"), __FUNCTION__, *FromNode->GetName(), *FromPinLabel.ToString()),
				ELogVerbosity::Error);
		return false;
	}

	UMovieGraphPin* ToPin = ToNode->GetInputPin(ToPinLabel);
	if (!ToPin)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: ToNode: %s does not have a pin with the label: %s"), __FUNCTION__, *ToNode->GetName(), *ToPinLabel.ToString()),
				ELogVerbosity::Error);
		return false;
	}

	bool bChanged = ToPin->BreakEdgeTo(FromPin);

//#if WITH_EDITOR
// 	   if(bChanged) {
//	NotifyGraphChanged(EMoviePipelineGraphChangeType::Structural);
// 	   }
//#endif

	return bChanged;
}

bool UMovieGraphConfig::RemoveAllInboundEdges(UMovieGraphNode* InNode)
{
	if (!InNode)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(TEXT("%hs: Invalid Edge Nodes"), __FUNCTION__ ), ELogVerbosity::Error);
		return false;
	}

	bool bChanged = false;
	for (UMovieGraphPin* InputPin : InNode->InputPins)
	{
		bChanged |= InputPin->BreakAllEdges();
	}

	//#if WITH_EDITOR
	// 	   if(bChanged) {
	//	NotifyGraphChanged(EMoviePipelineGraphChangeType::Structural);
	// 	   }
	//#endif

	return bChanged;
}

bool UMovieGraphConfig::RemoveAllOutboundEdges(UMovieGraphNode* InNode)
{
	if (!InNode)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(TEXT("%hs: Invalid Edge Nodes"), __FUNCTION__), ELogVerbosity::Error);
		return false;
	}

	bool bChanged = false;
	for (UMovieGraphPin* OutputPin : InNode->OutputPins)
	{
		bChanged |= OutputPin->BreakAllEdges();
	}

	//#if WITH_EDITOR
	// 	   if(bChanged) {
	//	NotifyGraphChanged(EMoviePipelineGraphChangeType::Structural);
	// 	   }
	//#endif

	return bChanged;
}

bool UMovieGraphConfig::RemoveInboundEdges(UMovieGraphNode* InNode, const FName& InPinName)
{
	if (!InNode)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: Invalid Edge Nodes"), __FUNCTION__),
				ELogVerbosity::Error);
		return false;
	}

	bool bChanged = false;
	if(UMovieGraphPin* InputPin = InNode->GetInputPin(InPinName))
	{
		bChanged |= InputPin->BreakAllEdges();
	}

	//#if WITH_EDITOR
	// 	   if(bChanged) {
	//	NotifyGraphChanged(EMoviePipelineGraphChangeType::Structural);
	// 	   }
	//#endif

	return bChanged;
}

bool UMovieGraphConfig::RemoveOutboundEdges(UMovieGraphNode* InNode, const FName& InPinName)
{
	if (!InNode)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: Invalid Edge Nodes"), __FUNCTION__),
				ELogVerbosity::Error);
		return false;
	}

	bool bChanged = false;
	if (UMovieGraphPin* OutputPin = InNode->GetOutputPin(InPinName))
	{
		bChanged |= OutputPin->BreakAllEdges();
	}

	//#if WITH_EDITOR
	// 	   if(bChanged) {
	//	NotifyGraphChanged(EMoviePipelineGraphChangeType::Structural);
	// 	   }
	//#endif

	return bChanged;
}

UMovieGraphNode* UMovieGraphConfig::InsertBefore(UMovieGraphNode* InTargetNode, const TSubclassOf<UMovieGraphNode> InNodeClass, const FName& InTargetPinLabel)
{
	if (!IsValid(InTargetNode))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "[%s]: InTargetNode is not valid.", __FUNCTION__);
		return nullptr;
	}

	if (!InNodeClass)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "[%s]: InNodeClass is null.", __FUNCTION__);
		return nullptr;
	}

	if (!InTargetNode->GetInputPin(InTargetPinLabel))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "[%s]: Target node '%ls' (%ls) has no input pin '%ls'.",
			__FUNCTION__, *InTargetNode->GetName(), *InTargetNode->GetClass()->GetName(), *InTargetPinLabel.ToString());
		return nullptr;
	}

	// Save info about the node currently upstream of the target before the connection is broken by AddLabeledEdge
	UMovieGraphNode* UpstreamNode = nullptr;
	FName UpstreamOutputPinLabel;
	if (const UMovieGraphPin* ConnectedPin = InTargetNode->GetInputPin(InTargetPinLabel)->GetFirstConnectedPin())
	{
		UpstreamNode = ConnectedPin->Node;
		UpstreamOutputPinLabel = ConnectedPin->Properties.Label;
	}

	Modify();

	UMovieGraphNode* NewNode = CreateNodeByClass(InNodeClass);
	if (!NewNode)
	{
		return nullptr;
	}

	// Connect the new node to the target; input pins only allow one connection, so this auto-breaks
	// the existing UpstreamNode → InTargetNode edge
	AddLabeledEdge(NewNode, FName(), InTargetNode, InTargetPinLabel);

	// Reconnect the previously upstream node to the new node's input
	if (UpstreamNode)
	{
		AddLabeledEdge(UpstreamNode, UpstreamOutputPinLabel, NewNode, FName());
	}

	return NewNode;
}

UMovieGraphNode* UMovieGraphConfig::InsertAfter(UMovieGraphNode* InTargetNode, const TSubclassOf<UMovieGraphNode> InNodeClass, const FName& InTargetPinLabel)
{
	if (!IsValid(InTargetNode))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "[%s]: InTargetNode is not valid.", __FUNCTION__);
		return nullptr;
	}

	if (!InNodeClass)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "[%s]: InNodeClass is null.", __FUNCTION__);
		return nullptr;
	}

	if (!InTargetNode->GetOutputPin(InTargetPinLabel))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "[%s]: Target node '%ls' (%ls) has no output pin '%ls'.",
			__FUNCTION__, *InTargetNode->GetName(), *InTargetNode->GetClass()->GetName(), *InTargetPinLabel.ToString());
		return nullptr;
	}

	// Save info about the node currently downstream of the target so it can be reconnected after insertion.
	// The downstream pin's label is preserved so named branch pins (e.g. on the output node) are handled correctly.
	UMovieGraphNode* DownstreamNode = nullptr;
	FName DownstreamInputPinLabel;
	if (const UMovieGraphPin* ConnectedPin = InTargetNode->GetOutputPin(InTargetPinLabel)->GetFirstConnectedPin())
	{
		DownstreamNode = ConnectedPin->Node;
		DownstreamInputPinLabel = ConnectedPin->Properties.Label;
	}

	Modify();

	UMovieGraphNode* NewNode = CreateNodeByClass(InNodeClass);
	if (!NewNode)
	{
		return nullptr;
	}

	// Explicitly break the existing InTargetNode → DownstreamNode edge before reconnecting; output pins allow
	// multiple connections so AddLabeledEdge alone will not remove the old edge
	if (DownstreamNode)
	{
		RemoveLabeledEdge(InTargetNode, InTargetPinLabel, DownstreamNode, DownstreamInputPinLabel);
	}

	// Connect the target to the new node, then reconnect the new node to what was previously downstream
	AddLabeledEdge(InTargetNode, InTargetPinLabel, NewNode, FName());
	if (DownstreamNode)
	{
		AddLabeledEdge(NewNode, FName(), DownstreamNode, DownstreamInputPinLabel);
	}

	return NewNode;
}

void UMovieGraphConfig::AddNode(UMovieGraphNode* InNode)
{
	if (!InNode)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: No node was specified for add."), __FUNCTION__),
				ELogVerbosity::Error);
		return;
	}

	if (!InNode->CanBeAddedByUser())
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs:Cannot add node of type %s."), __FUNCTION__, *InNode->GetClass()->GetName()),
				ELogVerbosity::Error);
		return;
	}

	InNode->SetFlags(RF_Transactional);

	Modify();

	// Reparent node to this graph
	InNode->Rename(nullptr, this);

	AllNodes.Add(InNode);
}

bool UMovieGraphConfig::RemoveNodes(TArray<UMovieGraphNode*> InNodes)
{
	bool bChanged = false;
	for (UMovieGraphNode* Node : InNodes)
	{
		bChanged |= RemoveNode(Node);
	}
	return bChanged;
}

bool UMovieGraphConfig::RemoveNode(UMovieGraphNode* InNode)
{
	if (!InNode)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: Could not remove invalid node. Is it from a plugin that's not currently loaded?"), __FUNCTION__),
				ELogVerbosity::Error);
		return false;
	}

	Modify();

	RemoveAllInboundEdges(InNode);
	RemoveAllOutboundEdges(InNode);

#if WITH_EDITOR
	TArray<UMovieGraphNode*> RemovedNodes;
	RemovedNodes.Add(InNode);
	OnGraphNodesDeletedDelegate.Broadcast(RemovedNodes);
#endif

	return AllNodes.RemoveSingle(InNode) == 1;
}

template<typename RetType, typename ArrType>
RetType* UMovieGraphConfig::AddMember(TArray<TObjectPtr<ArrType>>& InMemberArray, const FName& InBaseName)
{
	static_assert(std::is_base_of_v<UMovieGraphMember, RetType>, "RetType is not derived from UMovieGraphMember");
	
	Modify();
	
	// TODO: This can be replaced with just CreateDefaultSubobject() when AddDefaultMembers() isn't called from PostLoad()
	//
	// This method will be called in two cases: 1) when default members are being added to a new graph when it is being
	// initially created or loaded via PostLoad(), or 2) a member is being added to the graph by the user. For case 1,
	// when the constructor is running, RF_NeedInitialization will be set. CreateDefaultSubobject() needs to be called
	// in this scenario instead of NewObject().
	const bool bIsNewObject = HasAnyFlags(RF_NeedInitialization);
	RetType* NewMember = bIsNewObject
		? CreateDefaultSubobject<RetType>(MakeUniqueObjectName(this, RetType::StaticClass()))
		: NewObject<RetType>(this, NAME_None);
	
	if (!NewMember)
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: Unable to create new member object in the graph."), __FUNCTION__),
				ELogVerbosity::Error);
		return nullptr;
	}

	InMemberArray.Add(NewMember);
	NewMember->SetFlags(RF_Transactional);
	NewMember->SetGuid(FGuid::NewGuid());

	// Generate and set a unique name. Globals set their name at construction time, so no need to set their name.
	if (!NewMember->template IsA<UMovieGraphGlobalVariable>())
	{
		TArray<FString> ExistingMemberNames;
		Algo::Transform(InMemberArray, ExistingMemberNames, [](const ArrType* Member) { return Member->GetMemberName(); });
		NewMember->SetMemberName(UE::MovieGraph::GetUniqueName(ExistingMemberNames, InBaseName.ToString()));
	}

	return NewMember;
}

UMovieGraphVariable* UMovieGraphConfig::AddVariable(const FName InCustomBaseName)
{
	static const FText VariableBaseName = LOCTEXT("VariableBaseName", "Variable");
	
	UMovieGraphVariable* NewVariable = AddMember<UMovieGraphVariable>(
		Variables, !InCustomBaseName.IsNone() ? InCustomBaseName : FName(*VariableBaseName.ToString()));

	if (NewVariable)
	{
		// Fire OnGraphVariablesChangedDelegate when the variable changes (name, value, type, etc)
#if WITH_EDITOR
		NewVariable->OnMovieGraphVariableChangedDelegate.AddWeakLambda(this, [this](UMovieGraphMember*)
		{
			OnGraphVariablesChangedDelegate.Broadcast();
		});
#endif
	}

#if WITH_EDITOR
	OnGraphVariablesChangedDelegate.Broadcast();
#endif

	return NewVariable;
}

UMovieGraphInput* UMovieGraphConfig::AddInput(const FText& InBaseName)
{
	static const FText InputBaseName = LOCTEXT("InputBaseName", "Input");

	const FText NewInputName = !InBaseName.IsEmpty() ? InBaseName : InputBaseName;
	
	UMovieGraphInput* NewInput = AddMember<UMovieGraphInput>(Inputs, FName(*NewInputName.ToString()));
	InputNode->UpdatePins();
	
#if WITH_EDITOR
	OnGraphInputAddedDelegate.Broadcast(NewInput);
#endif

	return NewInput;
}

UMovieGraphOutput* UMovieGraphConfig::AddOutput(const FText& InBaseName)
{
	static const FText OutputBaseName = LOCTEXT("OutputBaseName", "Output");
	
	const FText NewOutputName = !InBaseName.IsEmpty() ? InBaseName : OutputBaseName;
	
	UMovieGraphOutput* NewOutput = AddMember<UMovieGraphOutput>(Outputs, FName(*NewOutputName.ToString()));
	OutputNode->UpdatePins();

#if WITH_EDITOR
	OnGraphOutputAddedDelegate.Broadcast(NewOutput);
#endif

	return NewOutput;
}

UMovieGraphVariable* UMovieGraphConfig::GetVariableByGuid(const FGuid& InGuid) const
{
	constexpr bool bIncludeGlobal = true;
	for (UMovieGraphVariable* Variable : GetVariables(bIncludeGlobal))
	{
		if (Variable->GetGuid() == InGuid)
		{
			return Variable;
		}
	}

	return nullptr;
}

UMovieGraphVariable* UMovieGraphConfig::GetVariableByName(const FString& InVariableName) const
{
	constexpr bool bIncludeGlobal = true;
	for (UMovieGraphVariable* Variable : GetVariables(bIncludeGlobal))
	{
		if (Variable->GetMemberName() == InVariableName)
		{
			return Variable;
		}
	}

	return nullptr;
}

TArray<UMovieGraphVariable*> UMovieGraphConfig::GetVariables(const bool bIncludeGlobal) const
{
	if (!bIncludeGlobal)
	{
		return Variables;
	}

	TArray<UMovieGraphVariable*> AllVariables = Variables;
	AllVariables.Append(GlobalVariables);

	return AllVariables;
}

void UMovieGraphConfig::UpdateGlobalVariableValues(const UMovieGraphPipeline* InPipeline)
{
	// Note: Although UpdateValue could get the traversal context from the pipeline itself, we fetch it once here
	// to prevent re-creating the context constantly.
	const FMovieGraphTraversalContext TraversalContext = InPipeline->GetCurrentTraversalContext();
	
	for (const TObjectPtr<UMovieGraphGlobalVariable>& GlobalVariable : GlobalVariables)
	{
		GlobalVariable->UpdateValue(&TraversalContext, InPipeline);
	}
}

TArray<UMovieGraphInput*> UMovieGraphConfig::GetInputs() const
{
	return Inputs;
}

TArray<UMovieGraphOutput*> UMovieGraphConfig::GetOutputs() const
{
	return Outputs;
}

bool UMovieGraphConfig::DeleteMember(UMovieGraphMember* MemberToDelete)
{
	if (!MemberToDelete)
	{
		return false;
	}

	if (!MemberToDelete->IsDeletable())
	{
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: The member '%s' cannot be deleted because it is flagged as non-deletable."), __FUNCTION__, *MemberToDelete->GetMemberName()),
				ELogVerbosity::Error);
		return false;
	}

	if (UMovieGraphVariable* GraphVariableToDelete = Cast<UMovieGraphVariable>(MemberToDelete))
	{
		return DeleteVariableMember(GraphVariableToDelete);
	}

	if (UMovieGraphInput* GraphInputToDelete = Cast<UMovieGraphInput>(MemberToDelete))
	{
		return DeleteInputMember(GraphInputToDelete);
	}

	if (UMovieGraphOutput* GraphOutputToDelete = Cast<UMovieGraphOutput>(MemberToDelete))
	{
		return DeleteOutputMember(GraphOutputToDelete);
	}

	return false;
}

UMovieGraphVariable* UMovieGraphConfig::DuplicateVariable(UMovieGraphVariable* InVariableToDuplicate)
{
	if (!InVariableToDuplicate)
	{
		return nullptr;
	}

	Modify();

	// AddVariable() does lots of heavy lifting to make sure variables are added/named correctly. Instead of duplicating lots of that boilerplate
	// here, just manually copy over the value/category/etc from the source variable. DuplicateObject() would work too, but has its own set of
	// clean-up procedures that are needed.
	UMovieGraphVariable* NewVariable = AddVariable(FName(InVariableToDuplicate->GetMemberName()));
	if (NewVariable)
	{
		NewVariable->SetValueType(InVariableToDuplicate->GetValueType(), const_cast<UObject*>(InVariableToDuplicate->GetValueTypeObject()));
		NewVariable->SetValueSerializedString(InVariableToDuplicate->GetValueSerializedString());
		NewVariable->SetCategory(InVariableToDuplicate->GetCategory());
		NewVariable->Description = InVariableToDuplicate->Description;
	}
	
	return NewVariable;
}

bool UMovieGraphConfig::DeleteVariableMember(UMovieGraphVariable* VariableMemberToDelete)
{
	if (!VariableMemberToDelete)
	{
		return false;
	}

	Modify();
	
	// Find all accessor nodes using this graph variable
	TArray<TObjectPtr<UMovieGraphNode>> NodesToRemove =
		AllNodes.FilterByPredicate([VariableMemberToDelete](const TObjectPtr<UMovieGraphNode>& GraphNode)
		{
			if (const UMovieGraphVariableNode* VariableNode = Cast<UMovieGraphVariableNode>(GraphNode))
			{
				const UMovieGraphVariable* GraphVariable = VariableNode->GetVariable();
				if (GraphVariable && (GraphVariable->GetGuid() == VariableMemberToDelete->GetGuid()))
				{
					return true;
				}
			}

			return false;
		});

	// Remove accessor nodes (which broadcasts our node changed delegates)
	TArray<UMovieGraphNode*> RemovedNodes;
	for (const TObjectPtr<UMovieGraphNode>& NodeToRemove : NodesToRemove)
	{
		if (RemoveNode(NodeToRemove.Get()))
		{
			RemovedNodes.Add(NodeToRemove.Get());
		}
	}

	// Remove this variable from the variables tracked by the graph
	Variables.RemoveSingle(VariableMemberToDelete);

#if WITH_EDITOR
	OnGraphVariablesChangedDelegate.Broadcast();
#endif

	return true;
}

#if WITH_EDITOR
void UMovieGraphConfig::SetEditorOnlyNodes(const TArray<TObjectPtr<const UObject>>& InNodes)
{
	EditorOnlyNodes.Empty();

	for (const TObjectPtr<const UObject>& Node : InNodes)
	{
		EditorOnlyNodes.Add(DuplicateObject(Node.Get(), this));
	}
}
#endif	// WITH_EDITOR

bool UMovieGraphConfig::DeleteInputMember(UMovieGraphInput* InputMemberToDelete)
{
	if (InputMemberToDelete)
	{
		Modify();
		
		Inputs.RemoveSingle(InputMemberToDelete);
		RemoveOutboundEdges(InputNode, FName(InputMemberToDelete->GetMemberName()));

		// This calls OnNodeChangedDelegate to update the graph
		InputNode->UpdatePins();

		return true;
	}

	return false;
}

bool UMovieGraphConfig::DeleteOutputMember(UMovieGraphOutput* OutputMemberToDelete)
{
	if (OutputMemberToDelete)
	{
		Modify();
		
		Outputs.RemoveSingle(OutputMemberToDelete);
		RemoveInboundEdges(OutputNode, FName(OutputMemberToDelete->GetMemberName()));

		// This calls OnNodeChangedDelegate to update the graph
		OutputNode->UpdatePins();

		return true;
	}

	return false;
}

FBoolProperty* UMovieGraphConfig::FindOverridePropertyForRealProperty(UClass* InClass, const FProperty* InRealProperty)
{
	if (!ensure(InClass && InRealProperty))
	{
		return nullptr;
	}

	// We can't get access to metadata in shipping builds, so we need to just rely on a naming pattern of bOverride_<PropertyName>
	const FString DesiredPropertyName = FString::Printf(TEXT("bOverride_%s"), *InRealProperty->GetName());

	for (TFieldIterator<FProperty> PropertyIterator(InClass); PropertyIterator; ++PropertyIterator)
	{
		FProperty* CheckProperty = *PropertyIterator;
		if (CheckProperty && CheckProperty->IsA<FBoolProperty>())
		{
			FBoolProperty* PropertyAsBool = CastFieldChecked<FBoolProperty>(CheckProperty);
			if (PropertyAsBool->GetName() == DesiredPropertyName)
			{
				return PropertyAsBool;
			}
		}
	}

	return nullptr;
}

void UMovieGraphConfig::VisitUpstreamNodes(UMovieGraphNode* FromNode, const FVisitNodesCallback& VisitCallback) const
{
	TSet<UMovieGraphNode*> VisitedNodes;
	VisitUpstreamNodes_Recursive(FromNode, VisitCallback, VisitedNodes);
}

void UMovieGraphConfig::VisitDownstreamNodes(UMovieGraphNode* FromNode, const FVisitNodesCallback& VisitCallback) const
{
	TSet<UMovieGraphNode*> VisitedNodes;
	VisitDownstreamNodes_Recursive(FromNode, VisitCallback, VisitedNodes);
}

TArray<FString> UMovieGraphConfig::GetDownstreamBranchNames(UMovieGraphNode* FromNode, const UMovieGraphPin* FromPin, const bool bStopAtSubgraph) const
{
	TArray<FString> BranchNames;

	// FromNode itself might be the Outputs node, so check before visiting the downstream nodes
	if (FromNode->IsA<UMovieGraphOutputNode>() && FromPin)
	{
		BranchNames.AddUnique(FromPin->Properties.Label.ToString());
	}

	VisitDownstreamNodes(FromNode, FVisitNodesCallback::CreateLambda(
		[&BranchNames, bStopAtSubgraph](const UMovieGraphNode* VisitedNode, const UMovieGraphPin* VisitedPin)
		{
			if (VisitedNode->IsA<UMovieGraphSubgraphNode>() && VisitedPin)
			{
				BranchNames.AddUnique(VisitedPin->Properties.Label.ToString());

				if (bStopAtSubgraph)
				{
					return false;	// Stop traversing nodes
				}
			}
			
			if (VisitedNode->IsA<UMovieGraphOutputNode>() && VisitedPin)
			{
				BranchNames.AddUnique(VisitedPin->Properties.Label.ToString());
			}

			return true;
		}));

	return BranchNames;
}

TArray<FString> UMovieGraphConfig::GetUpstreamBranchNames(UMovieGraphNode* FromNode, const UMovieGraphPin* FromPin, const bool bStopAtSubgraph) const
{
	TArray<FString> BranchNames;

	// FromNode itself might be the Inputs node, so check before visiting the upstream nodes
	if (FromNode->IsA<UMovieGraphInputNode>() && FromPin)
	{
		BranchNames.AddUnique(FromPin->Properties.Label.ToString());
	}

	VisitUpstreamNodes(FromNode, FVisitNodesCallback::CreateLambda(
		[&BranchNames, bStopAtSubgraph](const UMovieGraphNode* VisitedNode, const UMovieGraphPin* VisitedPin)
		{
			if (VisitedNode->IsA<UMovieGraphSubgraphNode>() && VisitedPin)
			{
				BranchNames.AddUnique(VisitedPin->Properties.Label.ToString());

				if (bStopAtSubgraph)
				{
					return false;	// Stop traversing nodes
				}
			}
			
			if (VisitedNode->IsA<UMovieGraphInputNode>() && VisitedPin)
			{
				BranchNames.AddUnique(VisitedPin->Properties.Label.ToString());
			}

			return true;
		}));

	return BranchNames;
}

void UMovieGraphConfig::GetAllContainedSubgraphs(TSet<UMovieGraphConfig*>& OutSubgraphs) const
{
	for (const TObjectPtr<UMovieGraphNode>& Node : GetNodes())
	{
		if (!Node)
		{
			continue;
		}
		
		if (const UMovieGraphSubgraphNode* SubgraphNode = Cast<UMovieGraphSubgraphNode>(Node))
		{
			UMovieGraphConfig* SubgraphConfig = SubgraphNode->GetSubgraphAsset();

			if (!SubgraphConfig) // A subgraph may not have been assigned yet
			{
				continue;
			}
			
			// Don't recurse into this graph if it was already added (to prevent infinite recursion)
			if (!OutSubgraphs.Contains(SubgraphConfig))
			{
				OutSubgraphs.Add(SubgraphConfig);
				SubgraphConfig->GetAllContainedSubgraphs(OutSubgraphs);
			}
		}
	}
}

void UMovieGraphConfig::RecurseUpGlobalsBranchToFindOutputDirectory(const UMovieGraphNode* InNode, FString& OutOutputDirectory, TArray<const UMovieGraphConfig*>& VisitedGraphStack) const
{
	TSet<const UMovieGraphNode*> VisitedNodes;
	const bool bPushedThisGraph = !VisitedGraphStack.Contains(this);
	if (bPushedThisGraph)
	{
		VisitedGraphStack.Push(this);
	}

	RecurseUpGlobalsBranchToFindOutputDirectory_Internal(this, InNode, OutOutputDirectory, VisitedGraphStack, VisitedNodes);

	if (bPushedThisGraph)
	{
		VisitedGraphStack.Pop(EAllowShrinking::No);
	}
};

TArray<FName> UMovieGraphConfig::GetBranchNames() const
{
	TArray<FName> BranchNames;
	for (const UMovieGraphOutput* Output : GetOutputs())
	{
		BranchNames.AddUnique(FName(Output->GetMemberName()));
	}
	return BranchNames;
}

UMovieGraphNode* UMovieGraphConfig::GetNodeForBranch(UClass* InClass, const FName& InBranchName, bool bExactMatch) const
{
	TArray<UMovieGraphNode*> ResultNodes = GetNodesForBranch(InClass, InBranchName, bExactMatch);
	if (ResultNodes.Num() > 0)
	{
		return ResultNodes[0];
	}

	return nullptr;
}

TArray<UMovieGraphNode*> UMovieGraphConfig::GetNodesForBranch(UClass* InClass, const FName& InBranchName, bool bExactMatch) const
{
	TArray<UMovieGraphNode*> ResultNodes;

	// We need to be able to detect when we've reached the end of traversal for the current branch,
	// which will either be at the input node, or when there's nothing connected to the left of it.
	bool bIsTraversingBranch = false;
	UMovieGraphNode* LocalInputNode = GetInputNode();

	VisitUpstreamNodes(GetOutputNode(), UMovieGraphConfig::FVisitNodesCallback::CreateLambda(
		[&InBranchName, &bIsTraversingBranch, &ResultNodes, LocalInputNode, bExactMatch, InClass](const UMovieGraphNode* VisitedNode, const UMovieGraphPin* VisitedPin)
		{
			check(VisitedNode);
			constexpr bool bFollowRerouteNodes = true;

			// If we return true from this function, then we will follow that pin. Unfortunately it only visits the first
			// child (ie: not the output node), so we have to check if the downstream pin was their desired pin
			if (!bIsTraversingBranch)
			{
				UMovieGraphPin* FirstOutputPin = VisitedNode->GetFirstConnectedOutputPin();
				if (FirstOutputPin)
				{
					for (UMovieGraphEdge* Edge : FirstOutputPin->Edges)
					{
						UMovieGraphPin* OtherPin = Edge->GetOtherPin(FirstOutputPin, bFollowRerouteNodes);
						if (OtherPin)
						{
							const bool bBranchMatch = OtherPin->Properties.Label == InBranchName;
							if (bBranchMatch)
							{
								bIsTraversingBranch = true;
							}
						}
					}
				}
			}

			if (bIsTraversingBranch)
			{
				// If there's nothing to our left on this node (or we're the Input node) then we've fully explored this branch
				if (VisitedNode->GetInputPins().Num() == 0 || VisitedNode == LocalInputNode)
				{
					bIsTraversingBranch = false;
				}
				else
				{
					// If we started traversing the branch, we need to keep adding all the nodes we find, even if they're not directly connected
					bool bMatches = bExactMatch ? VisitedNode->GetClass() == InClass : VisitedNode->IsA(InClass);
					if (bMatches)
					{
						ResultNodes.AddUnique(const_cast<UMovieGraphNode*>(VisitedNode));
					}
				}
			}

			// We return true here anywhere on the correct branch so that we keep going.
			return bIsTraversingBranch; 
		}));

	return ResultNodes;
}

UMovieGraphNode* UMovieGraphConfig::GetNodeForTag(const FString& ScriptTag, UClass* OptionalClassFilter, const FName OptionalBranchFilter, bool bExactMatch) const
{
	TArray<UMovieGraphNode*> ResultNodes = GetNodesForTag(ScriptTag, OptionalClassFilter, OptionalBranchFilter, bExactMatch);
	if (ResultNodes.Num() > 0)
	{
		return ResultNodes[0];
	}

	return nullptr;
}

TArray<UMovieGraphNode*> UMovieGraphConfig::GetNodesForTag(const FString& ScriptTag, UClass* OptionalClassFilter, const FName OptionalBranchFilter, bool bExactMatch) const
{
	TArray<UMovieGraphNode*> AllNodesToConsider;
	if (OptionalBranchFilter != NAME_None)
	{
		UClass* ClassFilter = OptionalClassFilter ? OptionalClassFilter : UMovieGraphNode::StaticClass();
		AllNodesToConsider = GetNodesForBranch(ClassFilter, OptionalBranchFilter, bExactMatch);
	}
	else
	{
		AllNodesToConsider.Reserve(GetNodes().Num());
		for (const TObjectPtr<UMovieGraphNode>& Node : GetNodes())
		{
			AllNodesToConsider.Add(Node.Get());
		}
	}

	TArray<UMovieGraphNode*> AllNodesWithTag;
	for (UMovieGraphNode* GraphNode : AllNodesToConsider)
	{
		// We re-run the class filter here because if they weren't using branch filters, they didn't get class-filtered yet.
		const bool bHasClassFilter = OptionalClassFilter != nullptr;
		const bool bPassesClassFilter = bExactMatch ? GraphNode->GetClass() == OptionalClassFilter : GraphNode->IsA(OptionalClassFilter);

		const bool bConsider = bHasClassFilter ? bPassesClassFilter : true;
		if (bConsider)
		{
			if (GraphNode->ScriptTags.Contains(ScriptTag))
			{
				AllNodesWithTag.Add(GraphNode);
			}
		}
	}

	return AllNodesWithTag;
}

void UMovieGraphConfig::GetOutputDirectory(FString& OutOutputDirectory) const
{
	check (OutputNode);

	// Clear out input strings
	OutOutputDirectory = FString();

	// We only traverse up the globals branch in order to find the output directory and file name format
	const UMovieGraphPin* GlobalsPin = OutputNode->GetInputPin(UMovieGraphNode::GlobalsPinName);

	if (GlobalsPin && GlobalsPin->IsConnected())
	{
		TArray<const UMovieGraphConfig*> VisitedGraphStack;
		RecurseUpGlobalsBranchToFindOutputDirectory(OutputNode, OutOutputDirectory, VisitedGraphStack);

		if (OutOutputDirectory.IsEmpty())
		{
			// If we didn't find any overrides, use the CDO values
			UMovieGraphGlobalOutputSettingNode* CDO = Cast<UMovieGraphGlobalOutputSettingNode>(UMovieGraphGlobalOutputSettingNode::StaticClass()->GetDefaultObject(false));
			check(CDO);
			
			OutOutputDirectory = CDO->OutputDirectory.Path;
		}
	}
}

void UMovieGraphConfig::MoveVariableBefore(UMovieGraphVariable* InTargetVariable, UMovieGraphVariable* InBeforeVariable)
{
	const int32 TargetVariableIndex = Variables.Find(InTargetVariable);
	const int32 BeforeVariableIndex = Variables.Find(InBeforeVariable);
	if ((TargetVariableIndex == INDEX_NONE) || (BeforeVariableIndex == INDEX_NONE) || (TargetVariableIndex == BeforeVariableIndex))
	{
		return;
	}

#if WITH_EDITOR
	Modify();
	InTargetVariable->Modify();
#endif

	// Moving the target before another variable means that the target should inherit the other variable's category
	InTargetVariable->Category = InBeforeVariable->Category;

	Variables.RemoveSingle(InTargetVariable);
	if (BeforeVariableIndex < TargetVariableIndex)
	{
		Variables.Insert(InTargetVariable, BeforeVariableIndex);
	}
	else
	{
		Variables.Insert(InTargetVariable, BeforeVariableIndex - 1);
	}

#if WITH_EDITOR
	if (OnGraphVariablesChangedDelegate.IsBound())
	{
		OnGraphVariablesChangedDelegate.Broadcast();
	}
#endif
}

void UMovieGraphConfig::MoveVariableToIndex(UMovieGraphVariable* InTargetVariable, int32 NewIndex)
{
	if (!InTargetVariable)
	{
		return;
	}
	
	// Check if the index is valid. IsValidIndex() won't work here because we may want to insert at one-past-the-end, where IsValidIndex() will
	// return false.
	const bool bIsValidIndex = (NewIndex >= 0) && (NewIndex <= Variables.Num());
	if (!bIsValidIndex)
	{
		return;
	}
	
	const int32 CurrentIndex = Variables.Find(InTargetVariable);
	if ((CurrentIndex == NewIndex) || (CurrentIndex == INDEX_NONE))
	{
		return;
	}

#if WITH_EDITOR
	Modify();
#endif

	Variables.RemoveAt(CurrentIndex);
	if (NewIndex < CurrentIndex)
	{
		Variables.Insert(InTargetVariable, NewIndex);
	}
	else
	{
		Variables.Insert(InTargetVariable, NewIndex - 1);
	}
}

void UMovieGraphConfig::MoveCategoryBefore(const FString& InCategoryToMove, const FString& InCategoryBefore)
{
	// Determines if a variable is part of the given category. Matches by exact category, or if the variable is a child under the given category
	// (eg, if "Foo" is the provided category, and the variable's category is "Foo|Bar", the variable would be a match).
	auto VariableBelongsToCategory = [](const UMovieGraphVariable* InVariable, const FString& InCategory)
	{
		if (!InVariable)
		{
			return false;
		}
		
		const FString ParentCategoryPrefix = FString::Format(TEXT("{0}|"), {InCategory});
		const FString& VariableCategory = InVariable->GetCategory();
		
		return (VariableCategory == InCategory) || VariableCategory.StartsWith(ParentCategoryPrefix);
	};
	
	// If dragging a category into a subcategory (eg, Foo|Bar) we have to use the root category (Foo) as the category to move before. Variable
	// reparenting via category moves is not supported currently.
	FString CategoryBefore = InCategoryBefore;
	if (CategoryBefore.Contains(TEXT("|")))
	{
		TArray<FString> CategoryParts;
		CategoryBefore.ParseIntoArray(CategoryParts, TEXT("|"));

		if (!CategoryParts.IsEmpty())
		{
			CategoryBefore = CategoryParts[0];
		}
	}
	
	// Cache all variables that will be moved
	const TArray<UMovieGraphVariable*> SourceVariables = Variables.FilterByPredicate([&InCategoryToMove, &VariableBelongsToCategory](const UMovieGraphVariable* InVariable)
	{
		return VariableBelongsToCategory(InVariable, InCategoryToMove);
	});

	// Ensure that a "before" variable can be found before removing the "from" variables
	const TObjectPtr<UMovieGraphVariable>* BeforeVariable = Variables.FindByPredicate([&CategoryBefore, &VariableBelongsToCategory](const TObjectPtr<UMovieGraphVariable>& InVariable)
	{
		return VariableBelongsToCategory(InVariable, CategoryBefore);
	});
	if (!BeforeVariable)
	{
		return;
	}

#if WITH_EDITOR
	Modify();
#endif

	// Remove all the variables that will be moved
	Variables.RemoveAll([&InCategoryToMove, &VariableBelongsToCategory](const UMovieGraphVariable* InVariable)
	{
		return VariableBelongsToCategory(InVariable, InCategoryToMove);
	});

	// Find the first variable that marks the variable to be "before" variable
	int32 BeforeVariableIndex = INDEX_NONE;
	for (int32 VariableIndex = 0; VariableIndex < Variables.Num(); ++VariableIndex)
	{
		if (VariableBelongsToCategory(Variables[VariableIndex], CategoryBefore))
		{
			BeforeVariableIndex = VariableIndex;
			break;
		}
	}

	// If found, move all "from" variables to be immediately before the "before" variable.
	if (BeforeVariableIndex != INDEX_NONE)
	{
		for (int32 SourceVariableIndex = 0; SourceVariableIndex < SourceVariables.Num(); ++ SourceVariableIndex)
		{
			Variables.Insert(SourceVariables[SourceVariableIndex], BeforeVariableIndex + SourceVariableIndex);
		}
	}

#if WITH_EDITOR
	if (OnGraphVariablesChangedDelegate.IsBound())
	{
		OnGraphVariablesChangedDelegate.Broadcast();
	}
#endif
}

void UMovieGraphConfig::InitializeFlattenedNode(UMovieGraphNode* InNode)
{
	// We go through each of the bOverride_ properties on this new instance and set
	// them all as "not overridden", so that as we traverse the graph, we can only
	// update the node the first time we hit another property that is overridden.
	// If they never override the values anywhere in the chain then it's okay, because
	// it will use the values from the CDO.
	for (TFieldIterator<FProperty> PropertyIterator(InNode->GetClass()); PropertyIterator; ++PropertyIterator)
	{
		FProperty* CheckProperty = *PropertyIterator;
		FBoolProperty* EditConditionProperty = FindOverridePropertyForRealProperty(InNode->GetClass(), CheckProperty);
		if (EditConditionProperty)
		{
			EditConditionProperty->SetPropertyValue_InContainer(InNode, false);
		}
	}

	// Initialize the dynamic properties so they can be updated during traversal like UPROPERTY properties
	InNode->UpdateDynamicProperties();
}

void UMovieGraphConfig::CopyOverriddenProperties(UMovieGraphNode* FromNode, UMovieGraphNode* ToNode, const FMovieGraphEvaluationContext& InEvaluationContext)
{
	if (!ensure(FromNode && ToNode))
	{
		return;
	}

	if (!ensureMsgf(FromNode->GetClass() == ToNode->GetClass(), TEXT("Cross-Class Property copying is not supported at this time.")))
	{
		return;
	}

	for (const FProperty* DestNodeProperty : ToNode->GetAllOverrideableProperties())
	{
		if (!DestNodeProperty)
		{
			continue;
		}

		const FName PropertyName = DestNodeProperty->GetFName();
		
		// For each property on the destination node, decide if we need to try to update it (we don't update bOverride_ properties)
		//
		// We use the existence of a matching edit condition node to signal that this property is overrideable, ie:
		// float MyFoo would have a bOverride_MyFoo, so FindOverridePropertyForRealProperty would match it. However when
		// looking at the bOverride_MyFoo property it would look for bOverride_bOverride_MyFoo, which wouldn't exist, successfully
		// filtering us to only the "real" overrideable properties.

		const bool bIsDynamic = ToNode->GetDynamicPropertyDescriptions().ContainsByPredicate([&PropertyName](const FPropertyBagPropertyDesc& PropDesc)
		{
			return PropDesc.Name == PropertyName;
		});
		
		const bool bIsExposed = FromNode->GetExposedProperties().ContainsByPredicate([&PropertyName](const FMovieGraphPropertyInfo& ExposedPropertyInfo)
		{
			return ExposedPropertyInfo.Name == PropertyName;
		});

		// Ensure there's a property (the "bOverride_*" property) that tracks whether or not this property has already
		// been set/overridden
		const FBoolProperty* EditConditionProperty = bIsDynamic
			? ToNode->FindOverridePropertyForDynamicProperty(PropertyName)
			: FindOverridePropertyForRealProperty(ToNode->GetClass(), DestNodeProperty);
		if (!EditConditionProperty)
		{
			continue;
		}

		// If our destination node already has this property marked as overridden, then some other node in the graph has
		// taken priority and set the value to something, so we don't want to override it. The exception to this is
		// an object implementing IMovieGraphTraversableObject -- they determine when/how property values are updated.
		const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(DestNodeProperty);
		const bool bIsMergeableObject = ObjectProperty && ObjectProperty->PropertyClass->ImplementsInterface(UMovieGraphTraversableObject::StaticClass());
		const bool bAlreadyOverriddenOnDestNode = bIsDynamic
			? ToNode->IsDynamicPropertyOverridden(PropertyName)
			: EditConditionProperty->GetPropertyValue_InContainer(ToNode) && !bIsMergeableObject;
		if (bAlreadyOverriddenOnDestNode)
		{
			continue;
		}

		// If this property (dynamic or not) has been exposed, attempt to get its value via the connection to it (if any)
		if (bIsExposed)
		{
			if (UMovieGraphPin* InputPin = FromNode->GetInputPin(PropertyName))
			{
				TArray<UMovieGraphPin*> ConnectionPath;
				
				// Iterate up the connection chain and find all pins which might have a value that can be resolved.
				FMovieGraphEvaluationContext ValueConnectionContext = InEvaluationContext;
				ValueConnectionContext.PinBeingFollowed = InputPin;
				TArray<UMovieGraphPin*> ConnectedValuePins = InputPin->Node->EvaluatePinsToFollow(ValueConnectionContext);

				FString ResolvedValue;
				const bool bFoundResolvedValue = UE::MovieGraph::ResolveConnectedPinValue(InputPin, ConnectedValuePins, ValueConnectionContext, ResolvedValue);
				if (bFoundResolvedValue)
				{
					if (bIsDynamic)
					{
						ToNode->SetDynamicPropertyValue(PropertyName, ResolvedValue);
						ToNode->SetDynamicPropertyOverridden(PropertyName, true);
					}
					else
					{
						DestNodeProperty->ImportText_Direct(*ResolvedValue, DestNodeProperty->ContainerPtrToValuePtr<uint8>(ToNode), nullptr, PPF_None);
						EditConditionProperty->SetPropertyValue_InContainer(ToNode, true);
					}

					// The property value was set via a connected pin; move on to the next property
					continue;
				}
			}
		}

		// We know it's not already overridden, so now we should check to see if the incoming node wants to override it.
		const bool bSourceNodeOverwrites = bIsDynamic
			? FromNode->IsDynamicPropertyOverridden(PropertyName)
			: EditConditionProperty->GetPropertyValue_InContainer(FromNode);
		if (!bSourceNodeOverwrites)
		{
			// The source node didn't have the override flag checked, so we don't copy the value from it.
			continue;
		}

		// Okay at this point we know that on our target node, no one has overridden it yet, and our source node wants to override this property.
		// First, we update the booleans to say that yes, this property has been overridden on the target node.
		if (bIsDynamic)
		{
			ToNode->SetDynamicPropertyOverridden(PropertyName, true);
		}
		else
		{
			EditConditionProperty->SetPropertyValue_InContainer(ToNode, true);
		}

		// Before using the normal property copying procedure, check to see if this property is an IMovieGraphTraversableObject.
		// These objects define a particular way they should have their properties merged.
		if (bIsMergeableObject)
		{
			const IMovieGraphTraversableObject* SourceTraversableObject = Cast<IMovieGraphTraversableObject>(
				ObjectProperty->GetObjectPropertyValue(ObjectProperty->ContainerPtrToValuePtr<void>(FromNode)));
			IMovieGraphTraversableObject* DestTraversableObject = Cast<IMovieGraphTraversableObject>(
				ObjectProperty->GetObjectPropertyValue(ObjectProperty->ContainerPtrToValuePtr<void>(ToNode)));
			
			if (DestTraversableObject && SourceTraversableObject)
			{
				DestTraversableObject->Merge(SourceTraversableObject);

				// Property has been copied via Merge(), don't run the normal copy procedure
				continue;
			}
		}

		// Now we need to copy the value from the source to the destination
		if (bIsDynamic)
		{
			FString SourceValue;
			if (FromNode->GetDynamicPropertyValue(PropertyName, SourceValue))
			{
				ToNode->SetDynamicPropertyValue(PropertyName, SourceValue);
			}
		}
		else
		{
			DestNodeProperty->CopyCompleteValue_InContainer(ToNode, FromNode);
		}
	}
}

bool UMovieGraphConfig::CreateFlattenedGraph_Recursive(UMovieGraphEvaluatedConfig* InOwningConfig, FMovieGraphEvaluatedBranchConfig& OutBranchConfig,
	FMovieGraphEvaluationContext& InEvaluationContext, UMovieGraphPin* InPinToFollow)
{
	if (!InPinToFollow)
	{
		InEvaluationContext.TraversalError = LOCTEXT("TraversalError_InvalidPin", "Found an invalid pin during graph traversal.");
		return false;
	}

	UMovieGraphNode* Node = InPinToFollow->Node;
	if (!Node)
	{
		InEvaluationContext.TraversalError = LOCTEXT("TraversalError_InvalidNode", "Found an invalid node during graph traversal.");
		return false;
	}

	// We only follow execution pins during traversal.
	if (!ensureMsgf(InPinToFollow->Properties.bIsBranch, TEXT("Only Branch pins should be contained by InPinToFollow!")))
	{
		InEvaluationContext.TraversalError = LOCTEXT("TraversalError_NonBranchPin", "Attempting to follow a non-branch pin during graph traversal.");
		return false;
	}

	InEvaluationContext.PinBeingFollowed = InPinToFollow;

	// Add this node to the set of visited nodes so it can be checked for cycles later. Get the graph from GetTypedOuter() rather than "this" because
	// this method will be called recursively, potentially on pins within subgraphs.
	UMovieGraphConfig* OwningGraph = InPinToFollow->GetTypedOuter<UMovieGraphConfig>();
	ensure(OwningGraph);
	TSet<TObjectPtr<UMovieGraphNode>>& VisitedNodeSet = InEvaluationContext.VisitedNodesByOwningGraph.FindOrAdd(OwningGraph).VisitedNodes;
	VisitedNodeSet.Add(Node);

	// The evaluated config also keeps track of all the graphs that were visited.
	InOwningConfig->EvaluatedGraphs.AddUnique(OwningGraph);
	
	const bool bShouldIncludeNode =
		Node->IsA<UMovieGraphSettingNode>() &&
		!Node->IsDisabled() &&
		!InEvaluationContext.NodeTypesToRemoveStack.Contains(Node->GetClass());

	if (bShouldIncludeNode)
	{
#if WITH_EDITOR
		// Normally we copy properties if we find a matching bOverride_ property. Unfortunately this creates a somewhat common
		// scenario where you've created a bOverride_ property but typo'd the real property name, so the real property doesn't
		// actually get updated, but we don't produce a warning (as it's valid to have properties with no matching bOverride_).
		// So to avoid this we have this editor ensure to prompt you when we find a bOverride_ property with no matching "real" property.
		for (TFieldIterator<FProperty> PropertyIterator(Node->GetClass()); PropertyIterator; ++PropertyIterator)
		{
			// If we're looking at an override property... 
			if (PropertyIterator->GetName().StartsWith(TEXT("bOverride_")))
			{
				const FString RealPropertyName = PropertyIterator->GetName().RightChop(10); // Chop off bOverride_ to get the name of the property we're searching for.
				bool bFoundProperty = false;
				for (TFieldIterator<FProperty> InnerPropertyIterator(Node->GetClass()); InnerPropertyIterator; ++InnerPropertyIterator)
				{
					if (InnerPropertyIterator->GetName() == RealPropertyName)
					{
						bFoundProperty = true;
						break;
					}
				}

				ensureAlwaysMsgf(bFoundProperty, TEXT("Found override property named %s, but could not find real property named %s"), *PropertyIterator->GetName(), *RealPropertyName);
			}
		}
#endif

		// Initialize (or copy properties to) the node in the evaluated graph. Reroute nodes aren't included in this because they're
		// passthrough and have no properties.
		if (!Node->IsA<UMovieGraphRerouteNode>())
		{
			const UMovieGraphSettingNode* NodeAsSetting = CastChecked<UMovieGraphSettingNode>(Node);
			const FString& NodeInstanceName = NodeAsSetting->GetNodeInstanceName();
		
			UMovieGraphSettingNode* ExistingNode = Cast<UMovieGraphSettingNode>(OutBranchConfig.GetNodeByClassExactMatch(Node->GetClass(), NodeInstanceName));
			if (!ExistingNode)
			{
				// Create a new instance of this node inside our flattened eval graph
				ExistingNode = NewObject<UMovieGraphSettingNode>(InOwningConfig, Node->GetClass());
				OutBranchConfig.NamedNodes.FindOrAdd(NodeInstanceName).NodeInstances.Add(ExistingNode);

				// This shouldn't be abused by nodes, but in some very rare cases the flattened node needs to be "primed" before being put through the
				// evaluation process.
				ExistingNode->PrepareForFlattening(NodeAsSetting);

				// Set all of the boolean edit condition values to false, so we can use "true" to indicate
				// that the value was overridden already during traversal.
				InitializeFlattenedNode(ExistingNode);
			}

			// Now do a property-copy from this node onto our flattened one. We don't use the generic property
			// copy routines in the engine because we have special handling (we want to check if the property
			// is actually marked for override, and also skip if this has already been overridden).
			CopyOverriddenProperties(Node, ExistingNode, InEvaluationContext);
		}
	}

	// If this is a special "removal" node, keep track of the type that should be removed. Since this method is recursive,
	// a stack is used to keep track of the types. The graph is iterated starting from the Outputs node, so all matching
	// nodes that are *upstream* of the removal node will be removed.
	const UMovieGraphRemoveRenderSettingNode* RemovalNode = Cast<UMovieGraphRemoveRenderSettingNode>(Node);
	const bool bIsARemovalNode = RemovalNode && !Node->IsDisabled() && (RemovalNode->NodeType.Get() != nullptr);
	if (bIsARemovalNode)
	{
		InEvaluationContext.NodeTypesToRemoveStack.Push(RemovalNode->NodeType);
	}
	
	// Now that we've potentially resolved the values on this node, continue to travel up-stream along any execution pins,
	// potentially following re-route nodes, sub-graph nodes, through branches, etc.
	TArray<UMovieGraphPin*> NewPinsToFollow = Node->EvaluatePinsToFollow(InEvaluationContext);

	// Immediately stop traversal if a circular subgraph reference was found. This is done after EvaluatePinsToFollow() because
	// subgraph nodes will set bCircularGraphReferenceFound in EvaluatePinsToFollow().
	if (InEvaluationContext.bCircularGraphReferenceFound)
	{
		// Generate a string illustrating the problematic subgraph stack
		FString GraphCycleTraversalPath;
		for (const TObjectPtr<const UMovieGraphSubgraphNode>& SubgraphNode : InEvaluationContext.SubgraphStack)
		{
			if (const UMovieGraphConfig* SubgraphAsset = SubgraphNode->GetSubgraphAsset())
			{
				GraphCycleTraversalPath += FString::Printf(TEXT("\n%s -> "), *SubgraphAsset->GetName());
			}
		}

		InEvaluationContext.TraversalError = FText::Format(
			LOCTEXT("TraversalError_CircularGraphReference", "Circular subgraph reference found during traversal.{0}"), FText::FromString(GraphCycleTraversalPath));
		
		return false;
	}
	
	for (UMovieGraphPin* Pin : NewPinsToFollow)
	{
		for (UMovieGraphEdge* Edge : Pin->Edges)
		{
			// If the edge is invalid, it's most likely due to a node from an unloaded plugin. Graph evaluation can't continue at this point.
			// Ideally we could provide the branch name here, but it isn't available currently.
			if (!IsValid(Edge))
			{
				InEvaluationContext.TraversalError = LOCTEXT("TraversalError_InvalidConnectionInBranch", "Found an invalid connection, likely from an invalid node. Aborting graph evaluation.");

				return false;
			}
			
			constexpr bool bFollowRerouteConnections = true;
			UMovieGraphPin* OtherPin = Edge->GetOtherPin(Pin, bFollowRerouteConnections);
			if (!OtherPin)
			{
				continue;
			}
			
			UMovieGraphNode* OtherNode = OtherPin->Node;

			// Detect cycles within node connections
			if (VisitedNodeSet.Contains(OtherNode))
			{
				// Generate a string illustrating the problematic node connections
				FString NodeCycleTraversalPath;
				for (const TObjectPtr<UMovieGraphNode>& VisitedNode : VisitedNodeSet)
				{
					NodeCycleTraversalPath += FString::Printf(TEXT("\n%s -> "), *VisitedNode->GetName());
				}
				
				InEvaluationContext.TraversalError = FText::Format(
					LOCTEXT("TraversalError_CircularNodeReference", "Node connection cycle found during traversal.{0}"), FText::FromString(NodeCycleTraversalPath));
				
				return false;
			}

			// If no cycle detected, continue following the pin
			const bool bSuccess = CreateFlattenedGraph_Recursive(InOwningConfig, OutBranchConfig, InEvaluationContext, OtherPin);
			if (!bSuccess)
			{
				return false;
			}
		}
	}

	// Done with this removal node now; pop it off the stack so it doesn't affect other branches
	if (bIsARemovalNode)
	{
		InEvaluationContext.NodeTypesToRemoveStack.Pop();
	}

	return true;
}

void UMovieGraphConfig::VisitUpstreamNodes_Recursive(UMovieGraphNode* FromNode,	const FVisitNodesCallback& VisitCallback, TSet<UMovieGraphNode*>& VisitedNodes) const
{
	if (VisitedNodes.Contains(FromNode))
	{
		// Cycle detected, stop recursing down this path. This is not necessarily an error, so don't log. For example:
		//  |----| ---> N2 
		//  | N1 | ---> N3 
		//  |----| ---> N4
		// Where nodes N2, N3, and N4 are nodes feeding out of node N1. N2 may have visited N1 already, so when N3 visits
		// N1, just stop.
		return;
	}
	
	VisitedNodes.Add(FromNode);
	
	for (const UMovieGraphPin* Pin : FromNode->GetInputPins())
	{
		for (const UMovieGraphPin* ConnectedPin : Pin->GetAllConnectedPins())
		{
			if (!ConnectedPin)
			{
				continue;
			}

			// Branches are followed during traversal, but we also want to traverse reroute nodes as well since they're passthrough
			if (ConnectedPin->Properties.bIsBranch || (Pin->Node && Pin->Node->IsA<UMovieGraphRerouteNode>()))
			{
				bool bContinueVisiting = true;
				if (VisitCallback.IsBound())
				{
					bContinueVisiting = VisitCallback.Execute(ConnectedPin->Node, ConnectedPin);
				}

				if (bContinueVisiting)
				{
					VisitUpstreamNodes_Recursive(ConnectedPin->Node, VisitCallback, VisitedNodes);
				}
			}
		}
	}
}

void UMovieGraphConfig::VisitDownstreamNodes_Recursive(UMovieGraphNode* FromNode, const FVisitNodesCallback& VisitCallback, TSet<UMovieGraphNode*>& VisitedNodes) const
{
	if (VisitedNodes.Contains(FromNode))
	{
		// Cycle detected, stop recursing down this path. This is not necessarily an error, so don't log. For example:
		// N1 ---> |----|
		// N2 ---> | N4 |
		// N3 ---> |----|
		// Where nodes N1, N2, and N3 are nodes feeding into node N4. N1 may have visited N4 already, so when N2 visits
		// N4, just stop.
		return;
	}

	VisitedNodes.Add(FromNode);

	for (const UMovieGraphPin* Pin : FromNode->GetOutputPins())
	{
		for (const UMovieGraphPin* ConnectedPin : Pin->GetAllConnectedPins())
		{
			if (!ConnectedPin)
			{
				continue;
			}

			// Branches are followed during traversal, but we also want to traverse reroute nodes as well since they're passthrough
			if (ConnectedPin->Properties.bIsBranch || (Pin->Node && Pin->Node->IsA<UMovieGraphRerouteNode>()))
			{
				bool bContinueVisiting = true;
				if (VisitCallback.IsBound())
				{
					bContinueVisiting = VisitCallback.Execute(ConnectedPin->Node, ConnectedPin);
				}

				if (bContinueVisiting)
				{
					VisitDownstreamNodes_Recursive(ConnectedPin->Node, VisitCallback, VisitedNodes);
				}
			}
		}
	}
}

UMovieGraphEvaluatedConfig* UMovieGraphConfig::CreateFlattenedGraph(const FMovieGraphTraversalContext& InContext, FString& OutError)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MRQ_CreateFlattenedGraph);
	LLM_SCOPE_BYNAME(TEXT("MovieGraph/CreateFlattenedGraph"));

	OutError.Empty();

	// Create the evaluated config with the transient package as the outer. The transient package is used here in order to avoid some potentially
	// tricky GC-related issues. Evaluated configs are frequently held via TStrongObjectPtr, which means they're added to the root set. This will cause
	// everything in the evaluated config's outer chain to not be GC'd until the strong object ptr is deleted. If there's an issue in the pipeline
	// somewhere that causes the strong object ptr to not be cleaned up when expected, it can cause a cascade that causes multiple objects (the outers)
	// to not be GC'd in a timely manner, leading to issues like the PIE world not being GC'd (since the pipeline has the PIE world as the outer). Avoid
	// these (unlikely) issues altogether by using the transient package as the outer.
	UMovieGraphEvaluatedConfig* NewContext = NewObject<UMovieGraphEvaluatedConfig>(GetTransientPackage(), NAME_None, RF_Transient);

	if (OutputNode)
	{
		TArray<UMovieGraphPin*> InputPinsToFollow = OutputNode->GetInputPins();
		UMovieGraphPin* GlobalsPin = OutputNode->GetInputPin(UMovieGraphNode::GlobalsPinName);

		// For each input pin, we create an instance of the config (including Globals, since some queries are made with no context)
		// but we also want each named branch to have a complete, resolved set of configs.
		for (UMovieGraphPin* InputPin : OutputNode->GetInputPins())
		{
			if (!InputPin->Properties.bIsBranch)
			{
				// TODO: For now, only branches can be evaluated
				continue;
			}
			
			const FName BranchName = InputPin->Properties.Label;
			FMovieGraphEvaluatedBranchConfig& BranchConfig = NewContext->BranchConfigMapping.Add(BranchName);
			
			// The stack evaluation context is per-branch
			FMovieGraphEvaluationContext StackContext;
			StackContext.UserContext = InContext;

			// Follow the branch connected to this pin
			for (UMovieGraphEdge* Edge : InputPin->Edges)
			{
				if (!IsValid(Edge))
				{
					const FText InvalidNodeError = FText::Format(LOCTEXT("TraversalError_InvalidNodeInBranch", "Found an invalid node in branch [{0}]. Aborting graph evaluation."), { FText::FromName(BranchName) });
					UE_LOGF(LogMovieRenderPipeline, Error, "%ls", *InvalidNodeError.ToString());
					OutError = InvalidNodeError.ToString();
					return nullptr;
				}
				
				UMovieGraphPin* OtherPin = Edge->GetOtherPin(InputPin);
				if (OtherPin)
				{
					const bool bTraversalSuccessful = CreateFlattenedGraph_Recursive(NewContext, /*InOut*/ BranchConfig, StackContext, OtherPin);
					if (!bTraversalSuccessful)
					{
						UE_LOGF(LogMovieRenderPipeline, Error, "%ls", *StackContext.TraversalError.ToString());
						OutError = StackContext.TraversalError.ToString();
						return nullptr;
					}
				}
			}

			// Now, if this isn't the Globals branch, we apply the Globals branch after the branch settings.
			// This allows users to override things on a per-branch basis, and if they don't set it, then the Globals
			// branch has a chance to set "defaults" (which then fall back to CDO values if the Globals branch doesn't 
			// set it). We skip doing this for the Globals branch because the above loop just did that.
			if (BranchName != UMovieGraphNode::GlobalsPinName && GlobalsPin)
			{
				// We use a new context here as we consider every branch independent.
				FMovieGraphEvaluationContext GlobalStackContext;
				GlobalStackContext.UserContext = InContext;
				for (UMovieGraphEdge* Edge : GlobalsPin->Edges)
				{
					if (!IsValid(Edge))
					{
						continue;
					}
					
					UMovieGraphPin* OtherPin = Edge->GetOtherPin(GlobalsPin);
					if (OtherPin)
					{
						const bool bTraversalSuccessful = CreateFlattenedGraph_Recursive(NewContext, /*InOut*/ BranchConfig, GlobalStackContext, OtherPin);
						if (!bTraversalSuccessful)
						{
							UE_LOGF(LogMovieRenderPipeline, Error, "%ls", *StackContext.TraversalError.ToString());
							OutError = StackContext.TraversalError.ToString();
                            return nullptr;
						}
					}
				}
			}
		}		
	}

	// Give the IMovieGraphEvaluationNodeInjector interface a chance to inject nodes now that the evaluation has finished.
	for (TPair<FName, FMovieGraphEvaluatedBranchConfig>& Pair : NewContext->BranchConfigMapping)
	{
		const FName& BranchName = Pair.Key;
		FMovieGraphEvaluatedBranchConfig& BranchConfig = Pair.Value;

		TArray<UMovieGraphSettingNode*> InjectedNodes;
		
		for (const TObjectPtr<UMovieGraphNode>& Node : BranchConfig.GetNodes())
		{
			if (IMovieGraphEvaluationNodeInjector* NodeInjector = Cast<IMovieGraphEvaluationNodeInjector>(Node))
			{
				NodeInjector->InjectNodesPostEvaluation(BranchName, NewContext, InjectedNodes);
			}
		}

		// Add the injected nodes, but make sure that there isn't a node of the same instance name + type already in the branch.
		for (UMovieGraphSettingNode* InjectedNode : InjectedNodes)
		{
			TArray<TObjectPtr<UMovieGraphNode>>& NodesWithInstanceName = BranchConfig.NamedNodes.FindOrAdd(InjectedNode->GetNodeInstanceName()).NodeInstances;

			for (const TObjectPtr<UMovieGraphNode>& ExistingNode : NodesWithInstanceName)
			{
				if (ExistingNode && (ExistingNode->GetClass() == InjectedNode->GetClass()))
				{
					const FText InvalidNodeTypesError = FText::Format(LOCTEXT("TraversalError_SameTypeSameBranchConflict", "Found multiple nodes with type [{0}] and the same instance name [{1}] on the same branch [{2}]. Some may have been injected by other nodes. Aborting graph evaluation."),
						{ FText::FromString(InjectedNode->GetClass()->GetName()), FText::FromString(InjectedNode->GetNodeInstanceName()), FText::FromName(BranchName) });
					UE_LOGF(LogMovieRenderPipeline, Error, "%ls", *InvalidNodeTypesError.ToString());
					OutError = InvalidNodeTypesError.ToString();
					return nullptr;
				}
			}
			
			NodesWithInstanceName.Add(InjectedNode);
		}

		InjectedNodes.Reset();
	}

	// We need to give nodes a chance to perform a Post-Load type upgrade to the flattened graph.
	// This is because their scripting pipeline may have modified the source graph this flattened graph was created from to set legacy properties,
	// so we need to give the flattened graphs a chance to migrate that data and warn the user of the need to update their scripts.
	for (const TPair<FName, FMovieGraphEvaluatedBranchConfig>& Pair : NewContext->BranchConfigMapping)
	{
		for (UMovieGraphNode* Node : Pair.Value.GetNodes())
		{
			if (UMovieGraphSettingNode* Setting = Cast<UMovieGraphSettingNode>(Node))
			{
				Setting->PostFlatten();	
			}
		}
	}

	// For properties that contain {tokens}, resolve their values.
	{
		// The identifier needs to be updated for each branch that the tokens are resolved on. Some properties in the identifier cannot be resolved
		// here (like {camera_name}) because there's not enough context at this point.
		FMovieGraphRenderDataIdentifier Identifier = InContext.RenderDataIdentifier;

		// This lambda, in combination with the resolve params, are used to resolve the format strings. The resolve args aren't used here, but are
		// required (they're reset because ResolveFormatArguments() appends to them, and there's no reason to continuously allocate a larger and
		// larger array).
		FMovieGraphFilenameResolveParams ResolveParams =
			FMovieGraphFilenameResolveParams::MakeResolveParams(Identifier, nullptr, InContext.Time.EvaluatedConfig, InContext, {});
		FMovieGraphResolveArgs ResolveArgs;
		TFunction<void(FString&)> ResolveToken = [&ResolveParams, &ResolveArgs, &Identifier](FString& InValue) -> void
		{
			ResolveArgs.FileMetadata.Reset();
			ResolveArgs.FilenameArguments.Reset();

			// The identifier is updated for each branch/layer
			ResolveParams.RenderDataIdentifier = Identifier;

			// Assign the resolved value back to the property that's being resolved
			InValue = UMovieGraphBlueprintLibrary::ResolveFormatArguments(InValue, ResolveParams, ResolveArgs);
		};

		// Give each setting node a chance to resolve its properties
		FMovieGraphTokenResolveContext AdditionalContext;
		for (const TPair<FName, FMovieGraphEvaluatedBranchConfig>& Pair : NewContext->BranchConfigMapping)
		{
			for (TObjectPtr<UMovieGraphNode>& Node : Pair.Value.GetNodes())
			{
				if (UMovieGraphSettingNode* Setting = Cast<UMovieGraphSettingNode>(Node))
				{
					UMovieGraphRenderLayerNode* RenderLayerNode =
						Cast<UMovieGraphRenderLayerNode>(Pair.Value.GetNodeByClassExactMatch(UMovieGraphRenderLayerNode::StaticClass(), FString()));

					// Provide additional context for the resolve; some of the identifier's properties can't be populated here, though. These updates
					// will be picked up by the ResolveToken() lambda above.
					Identifier.RootBranchName = Pair.Key;
					Identifier.LayerName = RenderLayerNode ? RenderLayerNode->LayerName : TEXT("Unknown");

					AdditionalContext.RenderDataIdentifier = &Identifier;
					AdditionalContext.TraversalContext = &InContext;
					Setting->ResolveTokenContainingProperties(ResolveToken, AdditionalContext);
				}
			}
		}
	}

	// Check all nodes for validation errors. If any, these will be logged, and the render will be halted.
	TArray<FText> ValidationErrors;
	for (const TPair<FName, FMovieGraphEvaluatedBranchConfig>& Pair : NewContext->BranchConfigMapping)
	{
		for (UMovieGraphNode* Node : Pair.Value.GetNodes())
		{
			if (UMovieGraphSettingNode* Setting = Cast<UMovieGraphSettingNode>(Node))
			{
				Setting->GetNodeValidationErrors(Pair.Key, NewContext, ValidationErrors);
			}
		}
	}

	if (!ValidationErrors.IsEmpty())
	{
		const FText AllValidationErrors = FText::Join(INVTEXT("\n"), ValidationErrors);
		const FText ValidationErrorMsg = FText::Format(LOCTEXT("TraversalError_NodeValidationErrors", "Found node validation errors while evaluating the graph.\n{0}"), AllValidationErrors);
		
		OutError = ValidationErrorMsg.ToString();
		
		return nullptr;
	}

	// Update the evaluated config with all of the evaluated variable values. These values are typically only used by scripting or blueprints.
	for (const UMovieGraphConfig* EvaluatedGraph : NewContext->EvaluatedGraphs)
	{
		for (UMovieGraphVariable* GraphVariable : EvaluatedGraph->GetVariables())
		{
			TArray<UMovieGraphValueContainer*>& VariablesWithName = NewContext->VariablesByName.FindOrAdd(GraphVariable->GetMemberName());

			TObjectPtr<UMovieGraphValueContainer> VariableValue;
			if (UMovieGraphVariableNode::GetResolvedVariableValue(GraphVariable, &InContext, VariableValue))
			{
				// *All* variables with the given name are added. This is done so that we can warn that multiple variables were found with a specific
				// name. May also have future uses if we need to get a resolved variable value from a specific graph.
				VariablesWithName.Add(VariableValue);
			}
		}
	}
	
	bool bHasRenderLayerNode = false;

	for (const TPair<FName, FMovieGraphEvaluatedBranchConfig>& Pair : NewContext->BranchConfigMapping)
	{
		for (UMovieGraphNode* Node : Pair.Value.GetNodes())
		{
			for (TFieldIterator<FProperty> PropertyIterator(Node->GetClass()); PropertyIterator; ++PropertyIterator)
			{
				FProperty* CheckProperty = *PropertyIterator;
				FBoolProperty* EditConditionProperty = FindOverridePropertyForRealProperty(Node->GetClass(), CheckProperty);
				if (EditConditionProperty)
				{
					FString ExportText;
					CheckProperty->ExportText_InContainer(0, ExportText, Node, Node, Node, 0);
				}
			}
			
			bHasRenderLayerNode |= Node->IsA<UMovieGraphRenderLayerNode>();
		}
	}

	if (!bHasRenderLayerNode)
	{
		// NOTE: While this doesn't cover all cases, we ensure the presence of at least one render layer node.
		UE_CALL_ONCE([] { UE_LOGF(LogMovieRenderPipeline, Error, "For render jobs to succeed, one or more render layer node(s) must be present."); });
	}

	return NewContext;
}
#undef LOCTEXT_NAMESPACE
