// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphCoreTimeStep.h"

#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "Graph/Nodes/MovieGraphCameraNode.h"
#include "MoviePipelineQueue.h"
#include "MovieRenderPipelineCoreModule.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "Camera/PlayerCameraManager.h"
#include "Camera/CameraComponent.h"
#include "Camera/CameraActor.h"
#include "Interfaces/Interface_PostProcessVolume.h"
#include "GameFramework/WorldSettings.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "MoviePipelineUtils.h"
#include "Graph/Nodes/MovieGraphSamplingMethodNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphCoreTimeStep)

UMovieGraphCoreTimeStep::UMovieGraphCoreTimeStep()
{
	// This set up our internal state to pick up on the first temporal sub-sample in the pattern
	ResetForEndOfOutputFrame();
}

void UMovieGraphCoreTimeStep::Shutdown()
{
	// Clear out the evaluated config during shutdown. It's a strong object ptr, so references to it should be removed to ensure everything
	// gets cleaned up properly.
	CurrentFrameData.EvaluatedConfig.Reset();
	if (UWorld* World = GetWorld())
	{
		World->GetWorldSettings()->SetAllowTimeDilation(bPrevWorldAllowTimeDilation);
	}
	Super::Shutdown();
}

void UMovieGraphCoreTimeStep::Initialize()
{
	Super::Initialize();
	if (UWorld* World = GetWorld())
	{
		bPrevWorldAllowTimeDilation = World->GetWorldSettings()->bAllowTimeDilation;
	}
}

void UMovieGraphCoreTimeStep::TickProducingFrames()
{
	int32 CurrentShotIndex = GetOwningGraph()->GetCurrentShotIndex();
	const TArray<TObjectPtr<UMoviePipelineExecutorShot>>& ActiveShotList = GetOwningGraph()->GetActiveShotList();
	const TObjectPtr<UMoviePipelineExecutorShot>& CurrentCameraCut = ActiveShotList[CurrentShotIndex];


	// We cache the frame metrics for the duration of a single output frame, instead of recalculating them every tick.
	// This is required for stochastic timesteps (which will jump around the actual evaluated time randomly within one
	// output frame), but it helps resolve some of our issues related to time dilation. Historically when doing it every
	// tick, we didn't know if we were going to overrun the end range of time represented until we had actually done it,
	// partway through the frame, at which point we had to backtrack and abandon the work. So now we calculate it all
	// in advance, check if our new time would overrun the end of the current camera cut, and if so, skip actually starting
	// that frame. Unfortunately this does cause some complications with actor time dilations - the time dilation track
	// will be updating each tick within the output sample and trying to change the world time dilation which we don't want.
	
	// ToDo: Still tbd how we'll fix that, currently the plan is to allow custom clocksources to disable world time dilation
	// support, at which point we'll handle the time dilation by affecting the whole engine tick instead of just world tick.
	//if (IsFirstTemporalSample())
	{

		// Now that we've calculated the total range of time we're trying to represent, we can check to see
		// if this would put us beyond our range of time this shot is supposed to represent.

	}
	
	// Some state transitions (such as moving to a previous point in time when paused) require a Jump operation
	// to correctly evaluate the new time, so setting this flag will issue an extra Jump instruction once the
	// final time has been calculated.
	bool bShouldJump = false;

	// Handle shot initialization
	if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::Uninitialized)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "MovieGraph Initializing Camera Cut [%d/%d] OuterName: [%ls] InnerName: %ls.",
			CurrentShotIndex + 1, ActiveShotList.Num(), *CurrentCameraCut->OuterName, *CurrentCameraCut->InnerName);

		// We need some evaluation context to be able to properly set up this shot, but we
		// re-evaluate before actually doing anything.
		FMovieGraphTraversalContext Context = GetOwningGraph()->GetCurrentTraversalContext();
		
		// Update global variables before evaluating the graph
		Context.RootGraph->UpdateGlobalVariableValues(GetOwningGraph());

		FString OutError;
		CurrentFrameData.EvaluatedConfig = TStrongObjectPtr<UMovieGraphEvaluatedConfig>(Context.RootGraph->CreateFlattenedGraph(Context, OutError));
		CurrentFrameData.TemporalSampleCount = GetTemporalSampleCount();
		UpdateFrameMetrics();

		// Ensure we've set it in the CurrentTimeStepData so things can fetch from it below.
		CurrentTimeStepData.EvaluatedConfig = TObjectPtr<UMovieGraphEvaluatedConfig>(CurrentFrameData.EvaluatedConfig.Get());
		
		// Get ready to begin tracking relative shot frame count
		CurrentTimeStepData.ShotOutputFrameNumber = -1;

		// Prepare the shot hierarchy and cvars before the first shot evaluation. The initial eval may be
		// back-tracked into warmup, so the incoming shot needs to be expanded and soloed before jumping there.
		GetOwningGraph()->BeginShotSetup(CurrentCameraCut);

		// Shot setup can apply cvars and game overrides that affect time dilation, so pick them up before
		// calculating the warmup back-track used for the initial data source evaluation.
		CurrentFrameData.TemporalSampleCount = GetTemporalSampleCount();
		UpdateFrameMetrics();
		
		FFrameTime EvalTime = CurrentCameraCut->ShotInfo.CurrentTimeInRoot;

		// Fold the warmup back-track into the FIRST eval. Otherwise InitializeShot's JumpDataSource
		// triggers a sequence evaluation at the shot's actual start, which caches MovieSceneAudioSystem
		// lifespan state under the new shot's range. When warmup then back-tracks the sequence into
		// the previous shot, the entities created by that initial eval are torn down and their cached
		// pre-animated state is restored - destroying any audio component whose section ends within
		// the new shot. The result is a fresh audio component started at the back-tracked time, which
		// re-records audio that the previous shot's segment already captured (audible as a duplicated
		// word at the cut).
		const bool bWillBackTrack = CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining > 0
			&& !CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur;
		if (bWillBackTrack)
		{
			EvalTime = EvalTime - (CurrentFrameMetrics.FrameTimePerOutputFrame * CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining);
		}

		// Seeks the external data source to match. This is done before setting up render resources
		// so that Spawnables, cameras, etc. will be spawned.
		GetOwningGraph()->GetDataSourceInstance()->InitializeShot(CurrentCameraCut, EvalTime);
		
		// Sets up render resources that depend on the data source having evaluated at least once.
		GetOwningGraph()->FinishShotSetup(CurrentCameraCut);

		// Generate render layers from the evaluated graph
		GetOwningGraph()->CreateLayersInRenderLayerSubsystem(CurrentTimeStepData.EvaluatedConfig);

		// Update the contents of the render layers
		// TODO: This should probably be done per-frame
		GetOwningGraph()->UpdateLayerContentsInRenderLayerSubsystem(CurrentTimeStepData.EvaluatedConfig);

		// Setting up the data source may cause a evaluation which can apply Time Dilation, so we should
		// pick up the new time dilation before we calculate things.
		UpdateFrameMetrics();

		// We have three possible options we can go to after the initialization state. If they have no warm up frames,
		// and don't want to emulate motion blur, we go directly into the Rendering State (which starts on Frame 0, then Frame 1, etc.)
		// If they do have warm up frames and are not emulating motion blur, we "walk" the level sequence up to the first frame.
		// If they have warm up frames and are emulating motion blur, then we evaluate Frame 0, wait the specified number of frames,
		// and then do motion blur emulation.
		CurrentCameraCut->ShotInfo.WorkMetrics.TotalEngineWarmUpFrameCount = CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining;

		if (CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining == 0 && !CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur)
		{
			// If there's no warm-up frames and they don't want to emulate motion blur, skip to rendering state next frame.
			CurrentCameraCut->ShotInfo.State = EMovieRenderShotState::Rendering;
			
			// Set up a fake "Previous" range that is just 1 frame further back than the start.
			FFrameTime UpperBound = CurrentCameraCut->ShotInfo.CurrentTimeInRoot;
			CurrentFrameData.LastOutputFrameRange = TRange<FFrameTime>(UpperBound - CurrentFrameMetrics.FrameTimePerOutputFrame, UpperBound);
			CurrentFrameData.LastSampleRange = CurrentFrameData.LastOutputFrameRange;

			GetOwningGraph()->GetDataSourceInstance()->PlayDataSource();
		}
		else
		{
			const bool bHasWarmUpFrames = CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining > 0;
			const bool bEmulateMotionBlur = CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur;

			if(bHasWarmUpFrames)
			{
				CurrentCameraCut->ShotInfo.State = EMovieRenderShotState::WarmingUp;

				// If they have warm up frames and are not emulating motion blur, we need to actually move their start range back in the sequence.
				int32 NumFramesToGoBack = bEmulateMotionBlur ? 0 : CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining;

				FFrameTime UpperBound = CurrentCameraCut->ShotInfo.CurrentTimeInRoot;
				FFrameTime NewStartTime = UpperBound - (CurrentFrameMetrics.FrameTimePerOutputFrame * NumFramesToGoBack);
				CurrentFrameData.LastOutputFrameRange = TRange<FFrameTime>(NewStartTime - CurrentFrameMetrics.FrameTimePerOutputFrame, NewStartTime);
				CurrentFrameData.LastSampleRange = CurrentFrameData.LastOutputFrameRange;

				if(bEmulateMotionBlur)
				{
					GetOwningGraph()->GetDataSourceInstance()->PauseDataSource();
					// We have to assign this in this scenario, because we don't automatically advance forward
					// when we're warming up, but emulating motion blur (meaning we sit on frame zero)
					TRangeBound<FFrameTime> EndOfPreviousFrame = CurrentFrameData.LastOutputFrameRange.GetUpperBound();
					CurrentFrameData.CurrentOutputFrameRange = TRange<FFrameTime>(EndOfPreviousFrame.GetValue(), EndOfPreviousFrame.GetValue() + CurrentFrameMetrics.FrameTimePerOutputFrame);

					// Because we're going backwards in time (while paused) we need to issue a jump command.
					bShouldJump = true;
				}
				else
				{
					GetOwningGraph()->GetDataSourceInstance()->PlayDataSource();
				}
			}
			else
			{
				// If there are no warm up frames, then they're already in the motion blur state.
				CurrentCameraCut->ShotInfo.State = EMovieRenderShotState::MotionBlur;

				// Set up our frame time to just be the first frame, with a fake past.
				FFrameTime UpperBound = CurrentCameraCut->ShotInfo.CurrentTimeInRoot;
				FFrameTime NewStartTime = UpperBound;
				CurrentFrameData.LastOutputFrameRange = TRange<FFrameTime>(NewStartTime - CurrentFrameMetrics.FrameTimePerOutputFrame, NewStartTime);
				CurrentFrameData.LastSampleRange = CurrentFrameData.LastOutputFrameRange;
				
				// This doens't get automatically incremented below, so we need to specify the output range they're working on.
				CurrentFrameData.CurrentOutputFrameRange = TRange<FFrameTime>(NewStartTime, NewStartTime + CurrentFrameMetrics.FrameTimePerOutputFrame);


				// We pause the sequence so it accurately reflects what we're actually doing with the time.
				GetOwningGraph()->GetDataSourceInstance()->PauseDataSource();
			}
		}

		// We intentionally fall through to the below states so that we don't have extra frames between state changes.
		// Extra frames are problematic for things like motion blur which depend on the data from the previous engine tick.
		UE_LOGF(LogMovieRenderPipeline, Log, "MovieGraph Finished initializing Camera Cut [%d/%d] OuterName: [%ls]  InnerName: %ls.",
			CurrentShotIndex + 1, ActiveShotList.Num(), *CurrentCameraCut->OuterName, *CurrentCameraCut->InnerName);
	}

	// We only automatically advance the CurrentOutputFrameRange during Rendering, or when warm-up frames
	// are counting down (if we're not going to emulate motion blur).
	const bool bIncrementBecauseRenderingState = CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::Rendering || CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::CoolingDown;
	const bool bIncrementBecauseWarmUpState = CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::WarmingUp && !CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur;
	bool bIncrementInternalCounters = bIncrementBecauseRenderingState || bIncrementBecauseWarmUpState;
	
	// Due to the motion blur emulation frame only using one TS, we need to potentially override the flags we send to the renderer,
	// otherwise the renderer gets confused because it's asked to render a frame that wasn't scheduled.
	TOptional<bool> bIsFirstTemporalSampleOverride;
	TOptional<bool> bIsLastTemporalSampleOverride;

	if (IsFirstTemporalSample())
	{
		// Update our context and stats
		FMovieGraphTraversalContext Context = GetOwningGraph()->GetCurrentTraversalContext();

		// Update global variables before evaluating the graph
		FString OutError;
		Context.RootGraph->UpdateGlobalVariableValues(GetOwningGraph());
		CurrentFrameData.EvaluatedConfig = TStrongObjectPtr<UMovieGraphEvaluatedConfig>(Context.RootGraph->CreateFlattenedGraph(Context, OutError));

		// The temporal sample count can change every frame due to graph evaluations, so when we're on our first temporal
		// sub-sample of the new frame, we need to re-fetch the value. 
		
		// ToDo: The time-step implementations can use the active
		// shot state to decide not to do a full TS count (ie: during warm-up) if applicable.
		CurrentFrameData.TemporalSampleCount = GetTemporalSampleCount();

		// Re-calculate some timing statistics about the given TS count
		UpdateFrameMetrics();

		// We create a range of time that we want to represent with this frame in absolute root sequence
		// space. This is because each frame can have a different temporal sample count, and when we jump
		// over the time period the shutter is closed, it's no longer an easy fixed number. So instead,
		// we build a time range we wish to represesnt with this frame, and then split that into sub-regions
		// and simply move between pre-calculated regions. This lets us handle the fact that the delta time
		// between output frames is going to be different when moving between different TS counts on frames.

		// To figure out what range of time we want to represent, we actually need to look at the last frame's
		// calculated total output range (ignoring all sub-sampling). The new frame is then multiplied by time
		// dilation, so our entire output range is shortened. We can't just jump between whole frames (due to
		// time dilation) so we have to take the last range, figure out how long the new frame is, and then
		// build ontop of that.
		TRangeBound<FFrameTime> EndOfPreviousFrame = CurrentFrameData.LastOutputFrameRange.GetUpperBound();

		if (bIncrementInternalCounters)
		{
			// The upper bound is exclusive, so we initialize a new TRange with just the value to start inclusively.
			CurrentFrameData.CurrentOutputFrameRange = TRange<FFrameTime>(EndOfPreviousFrame.GetValue(), EndOfPreviousFrame.GetValue() + CurrentFrameMetrics.FrameTimePerOutputFrame);
		}

		// Update CurrentFrameData.RangeShutterOpen and CurrentFrameData.RangeShutterClosed
		UpdateShutterRanges();
		// Update CurrentFrameData.TemporalRanges
		UpdateTemporalRanges();

		// Warm-up Frames follow similar logic to the main rendering. They need to support temporal sub-sampling because
		// the Path Tracer requires many samples to produce a high quality image, and the high quality image is important
		// to seed a denoiser (which requires looking at previous frames for temporal stability). So we now render temporal
		// sub-samples during warm-up.
		if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::WarmingUp)
		{
			CurrentTimeStepData.bDiscardOutput = true;

			// If we're going to emulate motion blur, we don't actually want to move the 
			// evaluated point in the sequence, otherwise we repeatedly jump back and forth
			// between temporal sub-sample locations when we're supposed to be warming up
			// only the first frame. We could possibly do this by overriding the temporal
			// sample count to 1, but we want to avoid having different behavior between
			// emulating motion blur and not.
			if (CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur)
			{
				for (int32 Index = 1; Index < CurrentFrameData.TemporalRanges.Num(); Index++)
				{
					CurrentFrameData.TemporalRanges[Index] = CurrentFrameData.TemporalRanges[0];
				}
			}

			if (CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining == 0)
			{
				GetOwningGraph()->GetDataSourceInstance()->PlayDataSource();
				CurrentCameraCut->ShotInfo.State = CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur ? EMovieRenderShotState::MotionBlur : EMovieRenderShotState::Rendering;
			}

			// We decrement at the end of this as we check before to see if we should move to the next state, but want to ensure we do at least one warm up frame.
			CurrentCameraCut->ShotInfo.WorkMetrics.EngineWarmUpFrameIndex++;
			CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining--;
		}

		if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::MotionBlur)
		{
			CurrentTimeStepData.bDiscardOutput = true;

			// This fires the first frame of motion blur evaluation, where we want to set their position to the end of the first temporal sub-sample
			if (!CurrentCameraCut->ShotInfo.bHasEvaluatedMotionBlurFrame)
			{
				// Instead of jumping all the way to the end of the first temporal sub-sample, we jump to one tick less. 
				// This is because if you have a sequence that is 1 frame long, and 1 TS, the end of the Temporal Sub-Sample
				// will be out of bounds for evaluation, and you won't properly emulate motion blur.
				FFrameTime TemporalUpperBound = CurrentFrameData.TemporalRanges[0].GetUpperBoundValue() - FFrameTime(FFrameNumber(1));

				// The code below is going to pick the lower bound of the next temporal range (which will be the first range)
				// We're going to rewrite the range to just be our nearly empty range, and then the rest of the timing should
				// just fall out.
				TRange<FFrameTime> NewRange = TRange<FFrameTime>(TemporalUpperBound, CurrentFrameData.TemporalRanges[0].GetUpperBoundValue());
				CurrentFrameData.TemporalRanges.Empty();
				CurrentFrameData.TemporalRanges.Add(NewRange);
				bIsFirstTemporalSampleOverride = true;
				bIsLastTemporalSampleOverride = true;

				GetOwningGraph()->GetDataSourceInstance()->PlayDataSource();
				CurrentCameraCut->ShotInfo.bHasEvaluatedMotionBlurFrame = true;
			}
			else
			{
				// Update the temporal ranges for this output frame to match their actual TS settings again
				// (we modified them above to make the motion blur emulation frame jump to a non-standard time).
				UpdateTemporalRanges();

				// We have to override this back to true because we're falling through into the Rendering State,
				// and the rendering state should increment.
				bIncrementInternalCounters = true;

				// We've done the motion blur emulation frame (above), but we didn't increment internal counters, so
				// we can safely fall through to rendering here and it'll start on the rendering frame like normal. 
				// We _must_ fall through to rendering this tick, as we need the motion vectors generated by the difference 
				// between last frame and this one to be used for the renders this frame.
				CurrentCameraCut->ShotInfo.State = EMovieRenderShotState::Rendering;
				GetOwningGraph()->GetDataSourceInstance()->PlayDataSource();
			}
		}

		if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::Rendering)
		{
			CurrentTimeStepData.bDiscardOutput = false;

			// We only track work metrics during Rendering
			CurrentCameraCut->ShotInfo.WorkMetrics.OutputFrameIndex++;

			// Now that we've calculated the total range of time we're trying to represent, we can check to see
			// if this would put us beyond our range of time this shot is supposed to represent.
			if (CurrentFrameData.CurrentOutputFrameRange.GetUpperBoundValue() > CurrentCameraCut->ShotInfo.TotalOutputRangeRoot.GetUpperBoundValue())
			{
				// We've reached the end of the sequence. We're going
				// to transition into the Cooling Down state (which we may
				// immediately leave if there are no cooling down frames
				// to actually render).
				CurrentCameraCut->ShotInfo.State = EMovieRenderShotState::CoolingDown;
			}
		}

		// If we've run past the end of the sequence, we'll be in the CoolingDown state now.
		if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::CoolingDown)
		{
			// We may or may not actually have any CoolingDown frames to process. If there's no frames left,
			// then we'll move to the Finished state and tear-down the render process for this frame.
			if (CurrentCameraCut->ShotInfo.NumEngineCoolDownFramesRemaining <= 0)
			{
				// We're going to spend this frame tearing down the shot (not rendering anything), next frame
				// we'll re-enter this loop and pick up the start of the next shot.
				ResetForEndOfOutputFrame();

				// The delta time isn't very relevant here but you must specify at _a_ delta time each frame to ensure
				// we never have a frame that uses a stale delta time.
				GetOwningGraph()->GetCustomEngineTimeStep()->SetCachedFrameTiming(UMovieGraphEngineTimeStep::FTimeStepCache(CurrentFrameMetrics.FrameRate.AsInterval()));

				CurrentCameraCut->ShotInfo.State = EMovieRenderShotState::Finished;
				GetOwningGraph()->TeardownShot(CurrentCameraCut);
				return;
			}
			else
			{
				// Only jump backwards in time if there's actually a cooldown frames to run.
				// Cooling down repeats the last frame of the render entirely, spatial and temporal samples included. This is because 
				// we want to generate good looking images to send to the denoiser (to denoise the normal final frames), so we need to
				// generate them like they would have been in the normal render.
				//CurrentFrameData.CurrentOutputFrameRange = CurrentFrameData.LastOutputFrameRange;
				//CurrentFrameData.LastSampleRange = CurrentFrameData.LastOutputFrameRange;

				// Go back FrameCount output frame for the current output frame range, and FrameCount+1 output frame for the last output frame.
				FFrameTime UpperBound = EndOfPreviousFrame.GetValue() - CurrentFrameMetrics.FrameTimePerOutputFrame;
				FFrameTime NewStartTime = UpperBound;
				CurrentFrameData.LastOutputFrameRange = TRange<FFrameTime>(UpperBound - CurrentFrameMetrics.FrameTimePerOutputFrame, UpperBound);
				CurrentFrameData.LastSampleRange = CurrentFrameData.LastOutputFrameRange;
				CurrentFrameData.CurrentOutputFrameRange = TRange<FFrameTime>(NewStartTime, NewStartTime + CurrentFrameMetrics.FrameTimePerOutputFrame);

				// Update CurrentFrameData.RangeShutterOpen and CurrentFrameData.RangeShutterClosed
				UpdateShutterRanges();
				// Update CurrentFrameData.TemporalRanges
				UpdateTemporalRanges();

				// We're going backwards so we need to flag the jump
				bShouldJump = true;

				// Cooldown frames 
				// ToDo: We need to separate the idea of "should we send this to the output merger" (false during cooldown)
				// and "should we render and schedule a readback" (true during cooldown)
				CurrentTimeStepData.bDiscardOutput = false;
				CurrentCameraCut->ShotInfo.NumEngineCoolDownFramesRemaining--;
			}
		}
	}

	// The delta time for this frame is the difference between the current range index, and the last range index.
	const TRange<FFrameTime>& PreviousRange = CurrentFrameData.LastSampleRange;
	const TRange<FFrameTime>& NextRange = CurrentFrameData.TemporalRanges[GetNextTemporalRangeIndex()];

	// Because some time-steps may not advance linearly through time, the abs value of the ranges needs to be taken
	// to avoid negative delta times.
	FFrameTime FrameDeltaTime = NextRange.GetLowerBoundValue() > PreviousRange.GetLowerBoundValue()
		? NextRange.GetLowerBoundValue() - PreviousRange.GetLowerBoundValue()
		: PreviousRange.GetLowerBoundValue() - NextRange.GetLowerBoundValue();

	// There are valid scenarios where FrameDeltaTime is zero (such as warming up and using motion blur emulation,
	// the sequence isn't actually progressing forward), but we can't propagate zero delta times into the engine.
	if (FrameDeltaTime.GetFrame() == 0 && FrameDeltaTime.GetSubFrame() == 0.f)
	{
		FrameDeltaTime = CurrentFrameData.TemporalRanges[0].Size<FFrameTime>();
	}

	// Cloth needs to increase the number of iterations when solving during the "long"
	// frame of MRQ (shutter closed time).
	{
		double Ratio = FrameDeltaTime.FloorToFrame().Value / (double)CurrentFrameMetrics.FrameTimePerTemporalSample.FloorToFrame().Value;

		// Slomo can end up trying to do less than one iteration, we don't want that.	
		int32 DivisionMultiplier = FMath::Max(FMath::FloorToInt(Ratio), 1);
		UE::MoviePipeline::SetSkeletalMeshClothSubSteps(DivisionMultiplier, GetWorld(), GetOwningGraph()->GetClothSimCache());
	}

	// Because we know what time range we're supposed to represent, we can just assign the CurrentTimeInRoot absolutely,
	// instead of accumulating delta times.
	CurrentCameraCut->ShotInfo.CurrentTimeInRoot = NextRange.GetLowerBoundValue();

	FFrameTime FinalEvalTime = CurrentCameraCut->ShotInfo.CurrentTimeInRoot + CurrentFrameMetrics.MotionBlurCenteringOffsetTime + CurrentFrameMetrics.ShutterOffsetFrameTime;
	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "FinalEvalTime: %ls (Tick: %ls SOffset: %ls MOffset: %ls)",
		*LexToString(FinalEvalTime),
		*LexToString(CurrentCameraCut->ShotInfo.CurrentTimeInRoot), 
		*LexToString(CurrentFrameMetrics.ShutterOffsetFrameTime),
		*LexToString(CurrentFrameMetrics.MotionBlurCenteringOffsetTime));

	// Now we need to fill out some of our current timestep data for the renderer portion to use.
 	double FrameDeltaTimeAsSeconds = CurrentFrameMetrics.TickResolution.AsSeconds(FrameDeltaTime);
	CurrentTimeStepData.FrameDeltaTime = FrameDeltaTimeAsSeconds;
	CurrentTimeStepData.WorldSeconds = CurrentTimeStepData.WorldSeconds + FrameDeltaTimeAsSeconds;
	CurrentTimeStepData.WorldTimeDilation = CurrentFrameMetrics.TimeDilation;
	CurrentTimeStepData.FrameRate = CurrentFrameMetrics.FrameRate;

	// Once a sequence starts using time dilation (due to non 1.0 timescale), we need to remember that for the rest
	// of the render. If they're using {frame_number_shot} or {frame_number} tokens, when we first encounter time dilation
	// the downstream system will force those tokens into their _rel versions. Sequences can drift in and out of using time
	// dilation, ie: they could start at 0.5 and stay that way the whole time, or start at 1.0 then go to 0.5, or start at 0.5
	// and go to 1.0. 
	//
	// To solve this, we track if time dilation has been used at any point within this level sequence. If it has, then all subsequent
	// frames will need to use relative frame numbers to avoid overwriting each other.
	const bool bTimeDilationUsed = !FMath::IsNearlyEqual(CurrentTimeStepData.WorldTimeDilation, 1.0f);
	GetOwningGraph()->GetCustomEngineTimeStep()->SharedTimeStepData.bHasTimeDilationBeenUsed |= bTimeDilationUsed;
	CurrentTimeStepData.bHasRelativeTimeBeenUsed = GetOwningGraph()->GetCustomEngineTimeStep()->SharedTimeStepData.bHasTimeDilationBeenUsed;

	// The combination of shutter angle percentage, non-uniform render frame delta times and dividing by sample
	// count produce the correct length for motion blur in all cases.
	CurrentTimeStepData.MotionBlurFraction = CurrentFrameMetrics.MotionBlurAmount / CurrentFrameData.TemporalSampleCount;
	CurrentTimeStepData.bIsFirstTemporalSampleForFrame = bIsFirstTemporalSampleOverride.Get(IsFirstTemporalSample());
	CurrentTimeStepData.bIsLastTemporalSampleForFrame = bIsLastTemporalSampleOverride.Get(IsLastTemporalSample());
	CurrentTimeStepData.bRequiresAccumulator = CurrentFrameData.TemporalSampleCount > 1 && !CurrentTimeStepData.bDiscardOutput;
	CurrentTimeStepData.OutputFrameNumber = GetOwningGraph()->GetCustomEngineTimeStep()->SharedTimeStepData.OutputFrameNumber;
	CurrentTimeStepData.RenderedFrameNumber = GetOwningGraph()->GetCustomEngineTimeStep()->SharedTimeStepData.RenderedFrameNumber;
	CurrentTimeStepData.TemporalSampleCount = CurrentFrameData.TemporalSampleCount;
	CurrentTimeStepData.TemporalSampleIndex = CurrentFrameData.TemporalSampleIndex;
	CurrentTimeStepData.EvaluatedConfig = TObjectPtr<UMovieGraphEvaluatedConfig>(CurrentFrameData.EvaluatedConfig.Get());

	//UE_LOGF(LogTemp, Warning, "F# %d bFirst: %d bLast: %d bReqAc: %d",
	//	CurrentTimeStepData.OutputFrameNumber, CurrentTimeStepData.bIsFirstTemporalSampleForFrame,
	//	CurrentTimeStepData.bIsLastTemporalSampleForFrame, CurrentTimeStepData.bRequiresAccumulator);

	// Calculate frame numbers and timecodes for the current sequence (root) and shot
	{
		constexpr bool bIncludeCDOs = true;
		UMovieGraphGlobalOutputSettingNode* OutputSetting = CurrentFrameData.EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs);

		// "Closest" isn't straightforward when using temporal sub-sampling, ie: A large enough shutter angle pushes a sample over the half way point and it rounds to
		// the wrong one. Because temporal sub-sampling isn't centered around a frame (the centering is done via the final eval time) we can just subtract TSI*TPS to get our centered value.
		const FFrameTime CenteringOffset = CurrentFrameData.TemporalSampleIndex * CurrentFrameMetrics.FrameTimePerTemporalSample;
		FFrameTime CenteredFrameTime = CurrentCameraCut->ShotInfo.CurrentTimeInRoot - CenteringOffset;
		FFrameTime ShotCenteredFrameTime = CenteredFrameTime * CurrentCameraCut->ShotInfo.OuterToInnerTransform;
	
		const FFrameRate SourceFrameRate = GetOwningGraph()->GetDataSourceInstance()->GetDisplayRate();
		const FFrameRate EffectiveFrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputSetting, SourceFrameRate);
		const FFrameRate TickResolution = GetOwningGraph()->GetDataSourceInstance()->GetTickResolution();

		// Enable DF timecodes if the framerate is 29.97 and the user requested DF timecodes
		const FFrameRate TwentyNineNineSeven = FFrameRate(30000, 1001);
		const bool bDropFrame = OutputSetting->bDropFrameTimecode && (EffectiveFrameRate == TwentyNineNineSeven);

		// If using a custom timecode start, the frame number for the root and shot need to be changed
		const FFrameNumber RootFrameNumber = OutputSetting->bOverride_CustomTimecodeStart
			? CurrentTimeStepData.OutputFrameNumber + OutputSetting->CustomTimecodeStart.ToFrameNumber(EffectiveFrameRate).Value
			: FFrameRate::TransformTime(CenteredFrameTime, TickResolution, EffectiveFrameRate).RoundToFrame();
		const FFrameNumber ShotFrameNumber = OutputSetting->bOverride_CustomTimecodeStart
			? CurrentTimeStepData.OutputFrameNumber + OutputSetting->CustomTimecodeStart.ToFrameNumber(EffectiveFrameRate).Value
			: FFrameRate::TransformTime(ShotCenteredFrameTime, TickResolution, EffectiveFrameRate).RoundToFrame();

		CurrentTimeStepData.RootFrameNumber = RootFrameNumber;
		CurrentTimeStepData.RootTimeCode = FTimecode::FromFrameNumber(CurrentTimeStepData.RootFrameNumber, EffectiveFrameRate, bDropFrame);

		// Calculate metrics for the shot as well
		CurrentTimeStepData.ShotFrameNumber = ShotFrameNumber;
		CurrentTimeStepData.ShotTimeCode = FTimecode::FromFrameNumber(CurrentTimeStepData.ShotFrameNumber, EffectiveFrameRate, bDropFrame);

		// Update the lightweight tick info in the module for other modules (Niagara) to have information
		// about the temporal data for better simulation. We do this here since we calculated the effective
		// frame rate above.
		{
			FMoviePipelineLightweightTickInfo TickInfo;
			TickInfo.bIsActive = true;
			TickInfo.TemporalSampleCount = CurrentTimeStepData.TemporalSampleCount;
			TickInfo.TemporalSampleIndex = CurrentTimeStepData.TemporalSampleIndex;
			TickInfo.SequenceFPS = EffectiveFrameRate.AsDecimal();
			FMovieRenderPipelineCoreModule::SetTickInfo(TickInfo);
		}
	}

	// Set our time step for the next frame. We use the undilated delta time for the Custom Timestep as the engine will
	// apply the time dilation to the world tick for us, so we don't want to double up time dilation.
	double UndilatedDeltaTime = CurrentFrameMetrics.TickResolution.AsSeconds(FrameDeltaTime);
	GetOwningGraph()->GetCustomEngineTimeStep()->SetCachedFrameTiming(UMovieGraphEngineTimeStep::FTimeStepCache(UndilatedDeltaTime));
	GetOwningGraph()->GetDataSourceInstance()->SyncDataSourceTime(FinalEvalTime);
	if (bShouldJump)
	{
		GetOwningGraph()->GetDataSourceInstance()->JumpDataSource(FinalEvalTime);
	}

	// ToDo: This should be converted back to an 'effective' frame number (source frame in external data asset)
	// so you can line up profiling with the acutal content on screen.
	TRACE_BOOKMARK(TEXT("MRQ Frame %d [TS: %d]"), CurrentTimeStepData.OutputFrameNumber, CurrentFrameData.TemporalSampleIndex);

	// Increment various post-frame counters to set them up for the next frame. This is okay
	// because the rest of the Movie Graph Pipeline system is based on CurrentTimeStepData which accurately
	// reflects which frame we're on.
	{
		// Update the last sample range to the one we just rendered.
		CurrentFrameData.LastSampleRange = NextRange;

		if ( bIsLastTemporalSampleOverride.Get(IsLastTemporalSample()))
		{
			GetOwningGraph()->GetCustomEngineTimeStep()->SharedTimeStepData.RenderedFrameNumber++;
			CurrentFrameData.LastOutputFrameRange = CurrentFrameData.CurrentOutputFrameRange;
				
			// Increment the output frame number only on the last temporal sample, and only if
			// we're actually rendering frames to disk.
			if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::Rendering)
			{
				GetOwningGraph()->GetCustomEngineTimeStep()->SharedTimeStepData.OutputFrameNumber++;
				CurrentTimeStepData.ShotOutputFrameNumber++;
			}
				
			// If we've rendered the last temporal sub-sample, we've started a new output frame
			// and we need to reset our temporal sample index.
			CurrentFrameData.TemporalSampleIndex = 0;
			CurrentCameraCut->ShotInfo.WorkMetrics.OutputSubSampleIndex = 0;
		}
		else
		{
			// We skip this on the motion blur state (which should only be true here on the one 
			// motion blur emulation frame), as the next frame we're going to reset back to
			// the rendering moe, and we want the renderer to start on the correct temporal
			// sample index to start a frame (0).
			if (CurrentCameraCut->ShotInfo.State != EMovieRenderShotState::MotionBlur) // ToDo: maybe not needed?
			{
				// Each tick we increment the temporal sample we're on.
				CurrentFrameData.TemporalSampleIndex++;
				CurrentCameraCut->ShotInfo.WorkMetrics.OutputSubSampleIndex++;
			}
		}
	}

	if (!ensure(CurrentCameraCut->ShotInfo.CurrentTickInRoot < CurrentCameraCut->ShotInfo.TotalOutputRangeRoot.GetUpperBoundValue()))
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Shot ran past evaluation range, this shouldn't be possible.");
	}

	if (!ensureMsgf(!FMath::IsNearlyZero(GetOwningGraph()->GetCustomEngineTimeStep()->TimeCache.UndilatedDeltaTime), TEXT("An incorrect or uninitialized time step was used!")))
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "An incorrect or uninitialized time step was used!");
	}
}

void UMovieGraphCoreTimeStep::UpdateShutterRanges()
{
	// Figure out ranges for how long the shutter is open and closed
	TArray<TRange<FFrameTime>> SplitRange = CurrentFrameData.CurrentOutputFrameRange.Split(CurrentFrameData.CurrentOutputFrameRange.GetLowerBoundValue() + CurrentFrameMetrics.FrameTimeWhileShutterOpen);

	// We will have two ranges when the shutter angle isn't 360/360
	if (ensure(SplitRange.Num() > 0))
	{
		if (SplitRange.Num() == 2)
		{
			CurrentFrameData.RangeShutterOpen = SplitRange[0];
			CurrentFrameData.RangeShutterClosed = SplitRange[1];
		}
		else if (SplitRange.Num() == 1)
		{
			// If the shutter is fully open, we create an empty range for closed as it is never actually closed.
			CurrentFrameData.RangeShutterOpen = SplitRange[0];
			CurrentFrameData.RangeShutterClosed = TRange<FFrameTime>::Empty();
		}
	}
}

void UMovieGraphCoreTimeStep::UpdateTemporalRanges()
{
	// Split the range the shutter is open per temporal sample.
	CurrentFrameData.TemporalRanges.Reset();
	CurrentFrameData.TemporalRanges.Reserve(CurrentFrameData.TemporalSampleCount);
	FFrameTime SplitStartTime = CurrentFrameData.RangeShutterOpen.GetLowerBoundValue();
	TRange<FFrameTime> RemainingRange = CurrentFrameData.RangeShutterOpen;
	for (int32 Index = 0; Index < CurrentFrameData.TemporalSampleCount - 1; Index++)
	{
		SplitStartTime += CurrentFrameMetrics.FrameTimePerTemporalSample;
		TArray<TRange<FFrameTime>> NewRanges = RemainingRange.Split(SplitStartTime);
		if (ensure(NewRanges.Num() == 2))
		{
			CurrentFrameData.TemporalRanges.Add(NewRanges[0]);
			RemainingRange = NewRanges[1];
		}
	}

	// Add the remaining range, it's the leftover from the last split.
	CurrentFrameData.TemporalRanges.Add(RemainingRange);

	//for (int32 Index = 0; Index < CurrentFrameData.TemporalRanges.Num(); Index++)
	//{
	//	UE_LOGF(LogTemp, Warning, "Range: [%ls, %ls)",
	//		*LexToString(CurrentFrameData.TemporalRanges[Index].GetLowerBoundValue()),
	//		*LexToString(CurrentFrameData.TemporalRanges[Index].GetUpperBoundValue()));
	//}
}

bool UMovieGraphCoreTimeStep::IsFirstTemporalSample() const
{
	return CurrentFrameData.TemporalSampleIndex == 0;
}

bool UMovieGraphCoreTimeStep::IsLastTemporalSample() const
{
	return CurrentFrameData.TemporalSampleIndex == CurrentFrameData.TemporalSampleCount - 1;
}

void UMovieGraphCoreTimeStep::ResetForEndOfOutputFrame()
{
	CurrentFrameData.TemporalSampleIndex = 0;
}

bool UMovieGraphCoreTimeStep::IsExpansionForTSRequired(const TObjectPtr<UMovieGraphEvaluatedConfig>& InConfig) const
{
	return GetTemporalSampleCountFromConfig(InConfig) > 1;
}

void UMovieGraphCoreTimeStep::UpdateFrameMetrics()
{
	FOutputFrameMetrics FrameData;

	// We inherit a tick resolution from the level sequence so that we can represent the same
	// range of time that the level sequence does.
	FrameData.TickResolution = GetOwningGraph()->GetDataSourceInstance()->GetTickResolution();
	FrameData.FrameRate = GetOwningGraph()->GetDataSourceInstance()->GetDisplayRate();

	// We don't always have a config set up when this function is called.
	if (CurrentFrameData.EvaluatedConfig)
	{
		if (UMovieGraphGlobalOutputSettingNode* OutputSetting = CurrentFrameData.EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName))
		{
			FrameData.FrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputSetting, FrameData.FrameRate);
		}
	}

	FrameData.FrameTimePerOutputFrame = FFrameRate::TransformTime(FFrameTime(FFrameNumber(1)), FrameData.FrameRate, FrameData.TickResolution);

	// Temporarily override time dilation so we can figure out what time dilation
	// the engine wanted, then turn support back off so we can implement it.
	GetWorld()->GetWorldSettings()->SetAllowTimeDilation(true);
	float TimeDilation = GetWorld()->GetWorldSettings()->GetEffectiveTimeDilation();
	GetWorld()->GetWorldSettings()->SetAllowTimeDilation(false);
	
	if(!FMath::IsNearlyEqual(TimeDilation, 1.0f))
	{
		FrameData.FrameTimePerOutputFrame = FrameData.FrameTimePerOutputFrame * TimeDilation;
	}
	
	FrameData.TimeDilation = TimeDilation;

	// Manually perform blending of the Post Process Volumes/Camera/Camera Modifiers to match what the renderer will do.
	// This uses the primary camera specified by the PlayerCameraManager to get the motion blur amount so in the event of
	// multi-camera rendering, all cameras will end up using the same motion blur amount defined by the primary camera).
	FrameData.MotionBlurAmount = GetBlendedMotionBlurAmount();

	// Calculate how long of a duration we want to represent where the camera shutter is open.
	// If the shutter angle is effectively zero, lie about how long a frame is to prevent divide by zero
	if (FrameData.MotionBlurAmount < (1.0 / 360.0))
	{
		FrameData.FrameTimeWhileShutterOpen = FrameData.FrameTimePerOutputFrame * (1.0 / 360.0);
	}
	else
	{
		FrameData.FrameTimeWhileShutterOpen = FrameData.FrameTimePerOutputFrame * FrameData.MotionBlurAmount;
	}

	// Now that we know how long the shutter is open, figure out how long each temporal sub-sample gets.
	FrameData.FrameTimePerTemporalSample = FrameData.FrameTimeWhileShutterOpen / CurrentFrameData.TemporalSampleCount;

	// The amount of time closed + time open should add up to exactly how long a frame is.
	FrameData.FrameTimeWhileShutterClosed = FrameData.FrameTimePerOutputFrame - FrameData.FrameTimeWhileShutterOpen;

	// Shutter timing is a bias applied to the final evaluation time to let us change what we consider a frame
	// ie: Do we consider a frame the start of the timespan we captured? Or is the frame the end of the timespan?
	// We default to Centered so that the center of your evaluated time is what you see in Level Sequences.
	EMoviePipelineShutterTiming ShutterTiming = EMoviePipelineShutterTiming::FrameCenter;
	
	// We don't always have a config set up when this function is called.
	if (CurrentFrameData.EvaluatedConfig)
	{
		if (UMovieGraphCameraSettingNode* CameraSetting = CurrentFrameData.EvaluatedConfig->GetSettingForBranch<UMovieGraphCameraSettingNode>(UMovieGraphNode::GlobalsPinName))
		{
			ShutterTiming = CameraSetting->ShutterTiming;
		}
	}
	
	switch (ShutterTiming)
	{
	// Subtract the entire time the shutter is open.
	case EMoviePipelineShutterTiming::FrameClose:
		FrameData.ShutterOffsetFrameTime = -FrameData.FrameTimeWhileShutterOpen;
		break;
	// Only subtract half the time the shutter is open.
	case EMoviePipelineShutterTiming::FrameCenter:
		FrameData.ShutterOffsetFrameTime = -FrameData.FrameTimeWhileShutterOpen / 2.0;
		break;
	// No offset needed
	case EMoviePipelineShutterTiming::FrameOpen:
		FrameData.ShutterOffsetFrameTime = FFrameTime(0);
		break;
	}

	// Then, calculate our motion blur offset. Motion Blur in the engine is always
	// centered around the object so we offset our time sampling by half of the
	// motion blur distance so that the distance blurred represents that time.
	FrameData.MotionBlurCenteringOffsetTime = FrameData.FrameTimePerTemporalSample / 2.0;

	CurrentFrameMetrics = FrameData;
}

float UMovieGraphCoreTimeStep::GetBlendedMotionBlurAmount()
{
	// 0.5f is the default engine motion blur in the event no Post Process/Camera overrides it.
	float FinalMotionBlurAmount = 0.5f;

	APlayerCameraManager* PlayerCameraManager = GetWorld()->GetFirstPlayerController()->PlayerCameraManager;
	if (PlayerCameraManager)
	{
		// Apply any motion blur settings from post process volumes in the world
		FVector ViewLocation = PlayerCameraManager->GetCameraLocation();
		for (IInterface_PostProcessVolume& PPVolume : GetWorld()->GetPostProcessVolumeIterator())
		{
			const FPostProcessVolumeProperties VolumeProperties = PPVolume.GetProperties();

			// Skip any volumes which are either disabled or don't modify blur amount
			if (!VolumeProperties.bIsEnabled || !VolumeProperties.Settings->bOverride_MotionBlurAmount)
			{
				continue;
			}

			float LocalWeight = FMath::Clamp(VolumeProperties.BlendWeight, 0.0f, 1.0f);

			if (!VolumeProperties.bIsUnbound)
			{
				float DistanceToPoint = 0.0f;
				PPVolume.EncompassesPoint(ViewLocation, 0.0f, &DistanceToPoint);

				if (DistanceToPoint >= 0 && DistanceToPoint < VolumeProperties.BlendRadius)
				{
					LocalWeight *= FMath::Clamp(1.0f - DistanceToPoint / VolumeProperties.BlendRadius, 0.0f, 1.0f);
				}
				else
				{
					LocalWeight = 0.0f;
				}
			}

			if (LocalWeight > 0.0f)
			{
				FinalMotionBlurAmount = FMath::Lerp(FinalMotionBlurAmount, VolumeProperties.Settings->MotionBlurAmount, LocalWeight);
			}
		}

		// Now try from the camera, which takes priority over post processing volumes.
		ACameraActor* CameraActor = Cast<ACameraActor>(PlayerCameraManager->GetViewTarget());
		if (CameraActor)
		{
			UCameraComponent* CameraComponent = CameraActor->GetCameraComponent();
			if (CameraComponent && CameraComponent->PostProcessSettings.bOverride_MotionBlurAmount)
			{
				FinalMotionBlurAmount = FMath::Lerp(FinalMotionBlurAmount, CameraComponent->PostProcessSettings.MotionBlurAmount, CameraComponent->PostProcessBlendWeight);
			}
		}

		// Apply any motion blur settings from post processing blends attached to the camera manager
		TArray<FPostProcessSettings> const* CameraAnimPPSettings;
		TArray<float> const* CameraAnimPPBlendWeights;
		PlayerCameraManager->GetCachedPostProcessBlends(CameraAnimPPSettings, CameraAnimPPBlendWeights);
		for (int32 PPIdx = 0; PPIdx < CameraAnimPPBlendWeights->Num(); ++PPIdx)
		{
			if ((*CameraAnimPPSettings)[PPIdx].bOverride_MotionBlurAmount)
			{
				FinalMotionBlurAmount = FMath::Lerp(FinalMotionBlurAmount, (*CameraAnimPPSettings)[PPIdx].MotionBlurAmount, (*CameraAnimPPBlendWeights)[PPIdx]);
			}
		}
	}

	return FinalMotionBlurAmount;
}

UMovieGraphEngineTimeStep::UMovieGraphEngineTimeStep()
	: PrevMinUndilatedFrameTime(0.f)
	, PrevMaxUndilatedFrameTime(0.f)
{
}

bool UMovieGraphEngineTimeStep::Initialize(UEngine* InEngine)
{
	// We cache the Min/Max times the world has because we modify these during
	// renders to match our sample length, otherwise stuff gets out of sync.
	if (AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings())
	{
		PrevMinUndilatedFrameTime = WorldSettings->MinUndilatedFrameTime;
		PrevMaxUndilatedFrameTime = WorldSettings->MaxUndilatedFrameTime;
	}

	return true;
}

void UMovieGraphEngineTimeStep::Shutdown(UEngine* InEngine)
{
	if (AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings())
	{
		WorldSettings->MinUndilatedFrameTime = PrevMinUndilatedFrameTime;
		WorldSettings->MaxUndilatedFrameTime = PrevMaxUndilatedFrameTime;
	}
}

void UMovieGraphEngineTimeStep::SetCachedFrameTiming(const FTimeStepCache& InTimeCache)
{
	if (ensureMsgf(!FMath::IsNearlyZero(InTimeCache.UndilatedDeltaTime), TEXT("An incorrect or uninitialized time step was used! Delta Time of 0 isn't allowed.")))
	{
		TimeCache = InTimeCache;
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "SetCachedFrameTiming called with zero delta time, falling back to 1/24");
		TimeCache = FTimeStepCache(1 / 24.0);
	}
}

bool UMovieGraphEngineTimeStep::UpdateTimeStep(UEngine* /*InEngine*/)
{
	if (ensureMsgf(!FMath::IsNearlyZero(TimeCache.UndilatedDeltaTime), TEXT("An incorrect or uninitialized time step was used! Delta Time of 0 isn't allowed.")))
	{
		FApp::UpdateLastTime();
		FApp::SetDeltaTime(TimeCache.UndilatedDeltaTime);
		FApp::SetCurrentTime(FApp::GetCurrentTime() + FApp::GetDeltaTime());

		// The UWorldSettings can clamp the delta time inside the Level Tick function, this creates a mismatch between component
		// velocity and render thread velocity and becomes an issue at high temporal sample counts. 
		if (AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings())
		{
			WorldSettings->MinUndilatedFrameTime = TimeCache.UndilatedDeltaTime;
			WorldSettings->MaxUndilatedFrameTime = TimeCache.UndilatedDeltaTime;
		}
	}

	// Clear our cached time to ensure we're always explicitly telling this what to do and never relying on the last set value.
	// (This will cause the ensure above to check on the next tick if someone didn't reset our value.)
	TimeCache = FTimeStepCache();

	// Return false so the engine doesn't run its own logic to overwrite FApp timings.
	return false;
}

int32 UMovieGraphCoreTimeStep::GetTemporalSampleCountFromConfig(UMovieGraphEvaluatedConfig* InConfig) const
{
	if (!ensure(InConfig))
	{
		return 1;
	}

	constexpr bool bIncludeCDOs = true;
	const UMovieGraphSamplingMethodNode* SamplingMethod =
		InConfig->GetSettingForBranch<UMovieGraphSamplingMethodNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs);
	if (SamplingMethod->TemporalSampleCount <= 0)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Sampling Method > Temporal Sample Count was zero, this is not allowed. Forcing value to 1!");
	}

	return FMath::Max(SamplingMethod->TemporalSampleCount, 1);
}
