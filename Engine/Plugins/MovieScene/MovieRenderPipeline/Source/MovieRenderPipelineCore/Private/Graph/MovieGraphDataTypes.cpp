// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphSequenceDataSource.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphDefaultAudioRenderer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphDataTypes)

FMovieGraphInitConfig::FMovieGraphInitConfig()
{
	RendererClass = UMovieGraphDefaultRenderer::StaticClass();
	DataSourceClass = UMovieGraphSequenceDataSource::StaticClass();
	AudioRendererClass = UMovieGraphDefaultAudioRenderer::StaticClass();
	bRenderViewport = false;
}

UMovieGraphPipeline* UMovieGraphTimeStepBase::GetOwningGraph() const
{ 
	return GetTypedOuter<UMovieGraphPipeline>();
}

UMovieGraphPipeline* UMovieGraphRendererBase::GetOwningGraph() const
{ 
	return GetTypedOuter<UMovieGraphPipeline>();
}

UMovieGraphPipeline* UMovieGraphDataSourceBase::GetOwningGraph() const
{
	return GetTypedOuter<UMovieGraphPipeline>();
}

UMovieGraphPipeline* UMovieGraphAudioRendererBase::GetOwningGraph() const
{
	return GetTypedOuter<UMovieGraphPipeline>();
}

const MoviePipeline::FAudioState& UMovieGraphAudioRendererBase::GetAudioState() const
{
	return AudioState;
}

UE::MovieGraph::FMovieGraphRenderDataValidationInfo UE::MovieGraph::FMovieGraphOutputMergerFrame::GetValidationInfo(const FMovieGraphRenderDataIdentifier& InRenderID, bool bInDiscardCompositedRenders) const
{
	const FName& ActiveBranchName = InRenderID.RootBranchName;
	const FString& ActiveRendererName = InRenderID.RendererName;
	const FString& ActiveCameraName = InRenderID.CameraName;

	FMovieGraphRenderDataValidationInfo ValidationInfo;

	TSet<FString>	LayerCounts;
	TSet<FName>		BranchCounts;
	TSet<FString>	BranchRendererCounts;
	TSet<FString>	CameraNameCounts;

	const int32 ReserveNum = ExpectedRenderPasses.Num();
	LayerCounts.Reserve(ReserveNum);
	BranchCounts.Reserve(ReserveNum);
	BranchRendererCounts.Reserve(ReserveNum);
	CameraNameCounts.Reserve(ReserveNum);

	for (const FMovieGraphRenderDataIdentifier& PassIdentifier : ExpectedRenderPasses)
	{
		LayerCounts.Add(PassIdentifier.LayerName);
		BranchCounts.Add(PassIdentifier.RootBranchName);

		// We only count renderers on the active branch and active camera
		if (PassIdentifier.RootBranchName == ActiveBranchName && PassIdentifier.CameraName == ActiveCameraName)
		{
			BranchRendererCounts.Add(PassIdentifier.RendererName);

			// Since renderers will be resolved separately, we only count subresources on the active renderer for the active branch
			if (PassIdentifier.RendererName == ActiveRendererName)
			{
				ValidationInfo.ActiveRendererSubresourceCount++;
			}
		}

		// Figure out how many cameras exist for this branch
		if (PassIdentifier.RootBranchName == ActiveBranchName)
		{
			if (!CameraNameCounts.Contains(PassIdentifier.CameraName))
			{
				ValidationInfo.ActiveCameraCount++;
			}
			CameraNameCounts.Add(PassIdentifier.CameraName);

		}
	}

	// Remove any renderers that will be composited on later
	if (bInDiscardCompositedRenders)
	{
		for (const TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& RenderData : ImageOutputData)
		{
			UE::MovieGraph::FMovieGraphSampleState* Payload = RenderData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
			check(Payload);
			if (Payload->bCompositeOnOtherRenders)
			{
				if (RenderData.Key.RootBranchName == ActiveBranchName && RenderData.Key.CameraName == ActiveCameraName)
				{
					BranchRendererCounts.Remove(RenderData.Key.RendererName);

					if (RenderData.Key.RendererName == ActiveRendererName)
					{
						ValidationInfo.ActiveRendererSubresourceCount--;
					}
				}
			}
		}
	}

	ValidationInfo.LayerCount = LayerCounts.Num();
	ValidationInfo.BranchCount = BranchCounts.Num();
	ValidationInfo.ActiveBranchRendererCount = BranchRendererCounts.Num();

	return ValidationInfo;
}
