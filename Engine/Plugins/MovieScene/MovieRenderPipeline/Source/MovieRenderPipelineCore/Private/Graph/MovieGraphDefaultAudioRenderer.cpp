// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphDefaultAudioRenderer.h"

#include "AudioDevice.h"
#include "AudioMixerDevice.h"
#include "AudioMixerPlatformNonRealtime.h"
#include "AudioMixerSubmix.h"
#include "DSP/BufferVectorOperations.h"
#include "Engine/Engine.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphSequenceDataSource.h"
#include "Graph/MovieGraphUtils.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "HAL/IConsoleManager.h"
#include "LevelSequencePlayer.h"
#include "MoviePipelineQueue.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieRenderPipelineDataTypes.h"

#if WITH_EDITOR
#include "Settings/LevelEditorMiscSettings.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphDefaultAudioRenderer)

static TAutoConsoleVariable<int32> CVarAudioLatencyCompensationFrames(
	TEXT("MovieRenderPipeline.AudioLatencyCompensationFrames"),
	7,
	TEXT("Number of output frames to drop from the start of the first captured audio segment to compensate\n")
	TEXT("for inherent submix output latency (EQ lookahead, audio pipeline buffering, etc.).\n")
	TEXT("Default 7 matches the empirically-measured latency in the new-process executor's audio pipeline.\n")
	TEXT("Only used for remote renders; this cvar is forced to 0 for local renders.\n"),
	ECVF_Default);

namespace
{
	// Sets the FMovieSceneAudioRecordingCapability flag on the pipeline's sequence player, if it
	// exposes one. When true, the audio-component playhead is preserved across shot boundaries.
	void SetMovieSceneAudioRecording(const UMovieGraphPipeline* InPipeline, const bool bInIsRecording)
	{
		if (!InPipeline)
		{
			return;
		}

		const UMovieGraphSequenceDataSource* SequenceDataSource = Cast<UMovieGraphSequenceDataSource>(InPipeline->GetDataSourceInstance());
		if (!SequenceDataSource)
		{
			return;
		}

		UMovieSceneSequencePlayer* SequencePlayer = SequenceDataSource->GetSequencePlayer();
		if (!SequencePlayer)
		{
			return;
		}

		const TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = SequencePlayer->GetSharedPlaybackState();
		if (UE::MovieScene::FMovieSceneAudioRecordingCapability* AudioRecordingCapability = SharedPlaybackState->FindCapability<UE::MovieScene::FMovieSceneAudioRecordingCapability>())
		{
			AudioRecordingCapability->bIsRecordingAudio = bInIsRecording;
		}
	}
}

void UMovieGraphDefaultAudioRenderer::StartAudioRecording()
{
	AudioState.bIsRecordingAudio = true;

	if (Audio::FMixerDevice* MixerDevice = UE::MovieGraph::Audio::GetAudioMixerDeviceFromWorldContext(this))
	{
		const TWeakPtr<Audio::FMixerSubmix> MasterSubmix = MixerDevice->GetMasterSubmix();
		if (MasterSubmix.Pin())
		{
			AudioState.ActiveSubmixes.Add(MasterSubmix);
		}

		for (TWeakPtr<Audio::FMixerSubmix>& WeakSubmix : AudioState.ActiveSubmixes)
		{
			constexpr float ExpectedDuration = 30.f;
			WeakSubmix.Pin()->OnStartRecordingOutput(ExpectedDuration);
		}
	}
}

void UMovieGraphDefaultAudioRenderer::StopAudioRecording()
{
	AudioState.bIsRecordingAudio = false;

	const int32 CurrentShotIndex = GetOwningGraph()->GetCurrentShotIndex();
	const int32 LatencyCompensationFrames = CVarAudioLatencyCompensationFrames.GetValueOnGameThread();

	// If this is the last shot of the render and compensation is active, drain the submix pipeline by ticking
	// the NRT mixer for LatencyCompensationFrames worth of seconds before OnStopRecordingOutput fires. The
	// submix's effect chain holds N frames of in-flight audio (lookahead-delayed samples for the last N picture
	// frames). For mid-render shots those samples naturally land in shot N+1's head recording, but the last
	// shot has no N+1 - without an explicit drain the tail audio is lost, leaving the final shot N frames short.
	// Combined with the head drop on shot 0, this restores full audio coverage.
	const TArray<TObjectPtr<UMoviePipelineExecutorShot>>& ActiveShotList = GetOwningGraph()->GetActiveShotList();
	const bool bIsLastShot = ActiveShotList.IsValidIndex(CurrentShotIndex) && (CurrentShotIndex == ActiveShotList.Num() - 1);
	if (bIsLastShot && LatencyCompensationFrames > 0)
	{
		if (Audio::FMixerDevice* DrainMixerDevice = UE::MovieGraph::Audio::GetAudioMixerDeviceFromWorldContext(this))
		{
			const Audio::IAudioMixerPlatformInterface* DrainPlatform = DrainMixerDevice->GetAudioMixerPlatform();
			if (DrainPlatform && DrainPlatform->IsNonRealtime())
			{
				Audio::FMixerPlatformNonRealtime* DrainNRTPlatform = static_cast<Audio::FMixerPlatformNonRealtime*>(DrainMixerDevice->GetAudioMixerPlatform());

				const UMovieGraphDataSourceBase* DataSourceInstance = GetOwningGraph()->GetDataSourceInstance();
				const FFrameRate OutputFrameRate = DataSourceInstance ? DataSourceInstance->GetDisplayRate() : FFrameRate(30, 1);
				const double DrainSeconds = LatencyCompensationFrames * OutputFrameRate.AsInterval();

				DrainMixerDevice->Update(true);
				DrainNRTPlatform->RenderAudio(DrainSeconds);
			}
		}
	}

	for (TWeakPtr<Audio::FMixerSubmix>& WeakSubmix : AudioState.ActiveSubmixes)
	{
		if (const TSharedPtr<Audio::FMixerSubmix> Submix = WeakSubmix.Pin())
		{
			MoviePipeline::FAudioState::FAudioSegment& NewSegment = AudioState.FinishedSegments.AddDefaulted_GetRef();

			// Copy the data returned by the submix
			const Audio::AlignedFloatBuffer& AudioRecording = Submix->OnStopRecordingOutput(NewSegment.NumChannels, NewSegment.SampleRate);
			NewSegment.SegmentData = AudioRecording;

			// Submix output latency compensation: drop the first N output frames from the very first segment captured.
			// Main submix effects (limiter lookahead, EQ phase delay, etc.) introduce a uniform latency in the
			// submix's output pipeline, so the recording's leading samples don't yet contain audio for the shot's
			// first picture frames. Since the offset is uniform across the entire render, dropping from shot 0
			// effectively shifts the whole concatenated audio earlier by N frames.
			if (AudioState.FinishedSegments.Num() == 1 && LatencyCompensationFrames > 0 && NewSegment.SampleRate > 0 && NewSegment.NumChannels > 0)
			{
				const UMovieGraphDataSourceBase* DataSourceInstance = GetOwningGraph()->GetDataSourceInstance();
				const FFrameRate OutputFrameRate = DataSourceInstance ? DataSourceInstance->GetDisplayRate() : FFrameRate(30, 1);
				const int32 SamplesPerFrame = static_cast<int32>(NewSegment.SampleRate * OutputFrameRate.AsInterval());
				const int32 DropCount = LatencyCompensationFrames * SamplesPerFrame * NewSegment.NumChannels;

				if (DropCount > 0 && DropCount <= NewSegment.SegmentData.Num())
				{
					NewSegment.SegmentData.RemoveAt(0, DropCount, EAllowShrinking::No);
				}
				else if (DropCount > NewSegment.SegmentData.Num())
				{
					UE_LOGF(LogMovieRenderPipeline, Warning, "Audio latency compensation (%d frames = %d samples) exceeds shot 0 segment size (%d samples). Skipping compensation.",
						LatencyCompensationFrames, DropCount, NewSegment.SegmentData.Num());
				}
			}
		}
	}

	AudioState.ActiveSubmixes.Reset();
}

void UMovieGraphDefaultAudioRenderer::ProcessAudioTick()
{
	// Only supported on the new audio mixer (with the non-realtime device, windows only).
	Audio::FMixerDevice* MixerDevice = UE::MovieGraph::Audio::GetAudioMixerDeviceFromWorldContext(this);
	if (!MixerDevice)
	{
		return;
	}

	const Audio::IAudioMixerPlatformInterface* AudioMixerPlatform = MixerDevice->GetAudioMixerPlatform();
	Audio::FMixerPlatformNonRealtime* NRTPlatform = nullptr;
	if (AudioMixerPlatform && AudioMixerPlatform->IsNonRealtime())
	{
		// There is only one non-realtime audio platform at this time, so we can safely static_cast this due to the IsNonRealtime check.
		NRTPlatform = static_cast<Audio::FMixerPlatformNonRealtime*>(MixerDevice->GetAudioMixerPlatform());
	}

	if (!NRTPlatform)
	{
		return;
	}

	const TArray<TObjectPtr<UMoviePipelineExecutorShot>>& ActiveShotList = GetOwningGraph()->GetActiveShotList();
	const int32 CurrentShotIndex = GetOwningGraph()->GetCurrentShotIndex();
	const TObjectPtr<UMoviePipelineExecutorShot> CurrentShot = ActiveShotList[CurrentShotIndex];

	if (CurrentShot->ShotInfo.State == EMovieRenderShotState::WarmingUp)
	{
		SetMovieSceneAudioRecording(GetOwningGraph(), true);
	}

	// Start capturing any produced samples on the same frame we start submitting samples that will make it to disk.
	// This comes before we process samples for this frame (below).
	if ((CurrentShot->ShotInfo.State == EMovieRenderShotState::Rendering) && !AudioState.bIsRecordingAudio)
	{
		StartAudioRecording();
	}

	// Tick the NRT submix only during the shot recording window (Rendering, CoolingDown).
	bool bCanRenderAudio = (CurrentShot->ShotInfo.State == EMovieRenderShotState::Rendering)
						|| (CurrentShot->ShotInfo.State == EMovieRenderShotState::CoolingDown);
	double AudioDeltaTime = FApp::GetDeltaTime();

	if (CurrentShot->ShotInfo.State == EMovieRenderShotState::Rendering)
	{
		const UMovieGraphTimeStepBase* TimeStepInstance = GetOwningGraph()->GetTimeStepInstance();
		const UMovieGraphDataSourceBase* DataSourceInstance = GetOwningGraph()->GetDataSourceInstance();
		
		constexpr bool bIncludeCDOs = true;
		constexpr bool bExactMatch = true;
		const TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedGraph = TimeStepInstance->GetCalculatedTimeData().EvaluatedConfig;
		UMovieGraphGlobalOutputSettingNode* OutputSettingNode = EvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);

		const FFrameRate SourceFrameRate = DataSourceInstance->GetDisplayRate();
		const FFrameRate EffectiveFrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputSettingNode, SourceFrameRate);
		
		// The non-real time audio renderer desires even engine time steps. Unfortunately, when using temporal sampling
		// we don't have an even time-step. However, because it's non-real time, and we're accumulating the results into
		// a single frame anyways, we can bunch up the audio work and then process it when we've reached the end of a frame.
		bCanRenderAudio = TimeStepInstance->GetCalculatedTimeData().bIsLastTemporalSampleForFrame;
		
		// Process work that has been submitted from the game thread to the audio thread over the temporal samples of this frame.
		AudioDeltaTime = EffectiveFrameRate.AsInterval();
	}

	// Handle any game logic that changed Audio State.
	MixerDevice->Update(true);

	// Process work that has been submitted from the game thread to the audio thread over the temporal samples of this frame.
	if (bCanRenderAudio)
	{
		NRTPlatform->RenderAudio(AudioDeltaTime);
	}
}

void UMovieGraphDefaultAudioRenderer::SetupAudioRendering()
{
	SetMovieSceneAudioRecording(GetOwningGraph(), true);

	// Ensure that we try to play audio at full volume, even if we're unfocused.
	AudioState.PrevUnfocusedAudioMultiplier = FApp::GetUnfocusedVolumeMultiplier();
	FApp::SetUnfocusedVolumeMultiplier(1.f);

#if WITH_EDITOR
	// Force background audio on so audio is captured even when the editor is in the background
	{
		ULevelEditorMiscSettings* MiscSettings = GetMutableDefault<ULevelEditorMiscSettings>();
		AudioState.bPrevAllowBackgroundAudio = MiscSettings->bAllowBackgroundAudio;
		MiscSettings->bAllowBackgroundAudio = true;
	}
#endif

	// Will be null if the NRT module wasn't loaded
	if (IConsoleVariable* AudioRenderEveryTickCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.nrt.RenderEveryTick")))
	{
		AudioState.PrevRenderEveryTickValue = AudioRenderEveryTickCvar->GetInt();
		
		// Override it to prevent it from automatically ticking, we'll control this below
		AudioRenderEveryTickCvar->SetWithCurrentPriority(0, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}

	// Ensure that the NRT audio doesn't get muted
	if (IConsoleVariable* NeverMuteNRTAudioCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.NeverMuteNonRealtimeAudioDevices")))
	{
		AudioState.PrevNeverMuteNRTAudioValue = NeverMuteNRTAudioCvar->GetInt();
		
		NeverMuteNRTAudioCvar->SetWithCurrentPriority(1, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}

	// Suppress audio latency compensation for PIE renders only. The cvar's default (and any user-set value)
	// is the project's compensation for new-process renders; PIE's per-world NRT device is aligned natively
	// and doesn't need it. Snapshot the current value either way so Teardown restores it cleanly.
	{
		AudioState.PrevAudioLatencyCompensationFrames = CVarAudioLatencyCompensationFrames.GetValueOnGameThread();

		const UWorld* SetupWorldContext = GetOwningGraph() ? GetOwningGraph()->GetWorld() : nullptr;
		const bool bSetupIsPIE = SetupWorldContext && SetupWorldContext->IsPlayInEditor();
		if (bSetupIsPIE)
		{
			CVarAudioLatencyCompensationFrames.AsVariable()->SetWithCurrentPriority(0, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
		}
	}
}

void UMovieGraphDefaultAudioRenderer::TeardownAudioRendering() const
{
	// Clear the audio-recording capability flag so non-MRG sequence playback restores normal lifespan handling.
	SetMovieSceneAudioRecording(GetOwningGraph(), false);

	// Restore previous unfocused audio multiplier, to no longer force audio when unfocused
	FApp::SetUnfocusedVolumeMultiplier(AudioState.PrevUnfocusedAudioMultiplier);

#if WITH_EDITOR
	// Restore the original bAllowBackgroundAudio setting
	GetMutableDefault<ULevelEditorMiscSettings>()->bAllowBackgroundAudio = AudioState.bPrevAllowBackgroundAudio;
#endif

	// Restore our cached CVar values
	
	// This will be null if the NRT wasn't used (module not loaded)
	if (IConsoleVariable* AudioRenderEveryTickCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.nrt.RenderEveryTick")))
	{
		AudioRenderEveryTickCvar->SetWithCurrentPriority(AudioState.PrevRenderEveryTickValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}

	if (IConsoleVariable* NeverMuteNRTAudioCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.NeverMuteNonRealtimeAudioDevices")))
	{
		NeverMuteNRTAudioCvar->SetWithCurrentPriority(AudioState.PrevNeverMuteNRTAudioValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}

	// Restore the audio latency compensation cvar to its pre-render value.
	CVarAudioLatencyCompensationFrames.AsVariable()->SetWithCurrentPriority(AudioState.PrevAudioLatencyCompensationFrames, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
}
