// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphDefaultRenderer.h"

#include "DumpGPU.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphUtils.h"
#include "Graph/Nodes/MovieGraphCameraNode.h"
#include "Graph/Nodes/MovieGraphDebugNode.h"
#include "Graph/Nodes/MovieGraphGlobalGameOverrides.h"
#include "Graph/Nodes/MovieGraphRenderLayerNode.h"
#include "Graph/Nodes/MovieGraphRenderPassNode.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineSurfaceReader.h"
#include "MovieRenderPipelineCoreModule.h"
#include "RenderCaptureInterface.h"
#include "RenderingThread.h"
#include "UObject/Package.h"

// For flushing async systems
#include "AssetCompilingManager.h"
#include "ContentStreaming.h"
#include "EngineModule.h"
#include "EngineUtils.h"
#include "LandscapeSubsystem.h"
#include "Materials/MaterialInterface.h"
#include "MeshCardRepresentation.h"
#include "RendererInterface.h"
#include "ShaderCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphDefaultRenderer)

void UMovieGraphDefaultRenderer::SetupRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot)
{
	// Cached values for render-pass telemetry; other values in the struct are kept as-is
	CachedRenderPassTelemetry = FMoviePipelineEndShotRenderTelemetryContext();

	// Iterate through the graph config and look for Render Layers.
	UMovieGraphConfig* RootGraph = GetOwningGraph()->GetRootGraphForShot(InShot);


	struct FMovieGraphPass
	{
		TSubclassOf<UMovieGraphRenderPassNode> ClassType;

		/**
		 * Maps a named branch to the specific render pass nodes that are assigned to render it. Typically there is only one render pass node of a
		 * given class type in the same branch; however, there are exceptions when node instance names are used (eg, burn-in nodes).
		 */
		TMap<FMovieGraphRenderDataIdentifier, TArray<TWeakObjectPtr<UMovieGraphRenderPassNode>>> BranchRenderers;
	};

	TArray<FMovieGraphPass> OutputPasses;

	// Start by getting our root set of branches we should follow
	const FMovieGraphTimeStepData& TimeStepData = GetOwningGraph()->GetTimeStepInstance()->GetCalculatedTimeData();
	UMovieGraphEvaluatedConfig* EvaluatedConfig = TimeStepData.EvaluatedConfig;
	TArray<FName> GraphBranches = EvaluatedConfig->GetBranchNames();

	// Figure out how many cameras we're rendering and generate fguids so render layers can fetch
	// the appropriate camera later.
	UMovieGraphCameraSettingNode* CameraNode = EvaluatedConfig->GetSettingForBranch<UMovieGraphCameraSettingNode>(UMovieGraphNode::GlobalsPinName);
	TArray<int32> CameraIndexes;
	if (CameraNode->bRenderAllCameras)
	{
		// When using sidecar cameras, the primary one is included as one of them.
		const TArray<UE::MovieGraph::FMinimalCameraInfo> MinimalCameraInfos = GetOwningGraph()->GetDataSourceInstance()->GetCameraInformation(InShot, true);
		for (int32 Index = 0; Index < InShot->SidecarCameras.Num(); Index++)
		{
			if (MinimalCameraInfos.IsValidIndex(Index) && MinimalCameraInfos[Index].ViewActor.IsValid())
			{
				CameraIndexes.Add(Index);
			}
			else
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "Skipping unresolved sidecar camera '%ls' while setting up Movie Graph rendering for shot '%ls'.",
					*InShot->GetCameraName(Index), *InShot->InnerName);
			}
		}
	}
	else
	{
		// "-1" means primary camera instead of any sidecar cameras
		CameraIndexes.Add(-1);
	}

	CameraOverscanCache.Empty();
	bHasWarnedAboutAnimatedOverscan = false;

	int32 RenderLayerIndex = 0;
	TMap<FMovieGraphRenderDataIdentifier, int32> IdentifierToLayerIndex;
	
	for (const FName& Branch : GraphBranches)
	{
		// We follow each branch looking for Render Layer nodes to figure out what render layer this should be. We assume a render layer is named
		// after the branch it is on, unless they specifically add a UMovieGraphRenderLayerNode to rename it.
		const bool bIncludeCDOs = false;
		UMovieGraphRenderLayerNode* RenderLayerNode = EvaluatedConfig->GetSettingForBranch<UMovieGraphRenderLayerNode>(Branch, bIncludeCDOs);
			
		// A RenderLayerNode is required for now to indicate you wish to actually render something.
		if (!RenderLayerNode)
		{
			continue;
		}

		// Now we need to figure out which renderers are on this branch.
		const bool bExactMatch = false;
		TArray<UMovieGraphRenderPassNode*> Renderers = EvaluatedConfig->GetSettingsForBranch<UMovieGraphRenderPassNode>(Branch, bIncludeCDOs, bExactMatch);
		if (RenderLayerNode && Renderers.Num() == 0)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Found RenderLayer: \"%ls\" but no Renderers defined.", *Branch.ToString());
		}

		for (UMovieGraphRenderPassNode* RenderPassNode : Renderers)
		{
			FMovieGraphPass* ExistingPass = OutputPasses.FindByPredicate([RenderPassNode](const FMovieGraphPass& Pass)
				{ return Pass.ClassType == RenderPassNode->GetClass(); });
			
			if (!ExistingPass)
			{
				ExistingPass = &OutputPasses.AddDefaulted_GetRef();
				ExistingPass->ClassType = RenderPassNode->GetClass();
			}

			FMovieGraphRenderDataIdentifier Identifier;
			Identifier.RootBranchName = Branch;
			Identifier.LayerName = RenderLayerNode->LayerName;

			IdentifierToLayerIndex.Add(Identifier, RenderLayerIndex++);
			
			ExistingPass->BranchRenderers.FindOrAdd(Identifier).Add(RenderPassNode);
		}
	}

	// Tracks sources that have already had OnBeginFrame() called this frame.
	// A source may be shared across multiple nodes, so this prevents duplicate calls.
	TArray<TSharedPtr<FMovieGraphRenderCameraSource>> ProcessedCameraSources;

	//UE_LOGF(LogMovieRenderPipeline, Warning, "Found: %d Render Passes:", OutputPasses.Num());
	int32 TotalLayerCount = 0;
	int32 CameraCount = 1;
	for (const FMovieGraphPass& Pass : OutputPasses)
	{
		// ToDo: This should probably come from the Renderers themselves, as they can internally produce multiple
		// renders (such as ObjectID passes).
		TotalLayerCount += Pass.BranchRenderers.Num();
		
		FMovieGraphRenderPassSetupData SetupData;
		SetupData.Renderer = this;
		SetupData.EvaluatedConfig = EvaluatedConfig;
		
		for (const TTuple<FMovieGraphRenderDataIdentifier, TArray<TWeakObjectPtr<UMovieGraphRenderPassNode>>>& BranchRenderer : Pass.BranchRenderers)
		{
			// Generate setup data for every render node / camera combination
			for (const TWeakObjectPtr<UMovieGraphRenderPassNode>& RenderNode : BranchRenderer.Value)
			{
				// Create the custom camera source for this node; null if the node uses default sequence cameras.
				const TSharedPtr<FMovieGraphRenderCameraSource> RenderCameraSource = RenderNode->CreateRenderCameraSource(GetOwningGraph(), EvaluatedConfig);
				if (RenderCameraSource.IsValid() && !ProcessedCameraSources.Contains(RenderCameraSource))
				{
					// Prepare the source for the new frame; must be called before querying camera data.
					RenderCameraSource->OnBeginFrame(GetOwningGraph(), &TimeStepData);
					ProcessedCameraSources.Add(RenderCameraSource);
				}

				// Custom source drives camera count; otherwise fall back to the sequence camera index list.
				const int32 NumCameras = RenderCameraSource ? RenderCameraSource->GetNumCameras() : CameraIndexes.Num();

				// One layer entry per camera — each becomes an independent render pass.
				for (int32 CameraIndex = 0; CameraIndex < NumCameras; CameraIndex++)
				{
					FMovieGraphRenderPassLayerData& LayerData = SetupData.Layers.AddDefaulted_GetRef();
					LayerData.BranchName = BranchRenderer.Key.RootBranchName;
					LayerData.LayerName = BranchRenderer.Key.LayerName;
					LayerData.LayerIndex = *IdentifierToLayerIndex.Find(BranchRenderer.Key);
					LayerData.RenderPassNode = RenderNode;

					// With a custom camera source the loop index is used directly (provider space, 0..N-1).
					// Without one, CameraIndexes translates the loop index to -1 (primary) or 0..N-1 (sidecar).
					LayerData.CameraIndex = RenderCameraSource ? CameraIndex : CameraIndexes[CameraIndex];
					LayerData.NumCameras = NumCameras;
				
					// We provide the name here so that the setup functions of the renderers can create RenderLayerIdentifiers.
					UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = GetCameraInfo(LayerData.CameraIndex, RenderCameraSource.Get());
					LayerData.CameraName = CameraInfo.CameraName;
				}
			}
		}

		UMovieGraphRenderPassNode* RenderPassCDO = Pass.ClassType->GetDefaultObject<UMovieGraphRenderPassNode>();
		RenderPassCDO->Setup(SetupData);

		RenderPassesInUse.Add(RenderPassCDO);
	}

	UE_LOGF(LogMovieRenderPipeline, Log, "Finished initializing %d Render Passes (with %d total layers and %d cameras).", OutputPasses.Num(), TotalLayerCount, CameraCount);
}

void UMovieGraphDefaultRenderer::TeardownRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MRQ_TeardownRenderingPipeline);

	// Ensure the GPU has actually rendered all of the frames
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MRQ_WaitForRenderWorkFinished);
		FlushRenderingCommands();
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MRQ_WaitForGPUReadbackFinished);
		// Make sure all of the data has actually been copied back from the GPU and accumulation tasks started.
		for (const TPair<UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams, FMoviePipelineSurfaceQueuePtr>& SurfaceQueueIt : PooledSurfaceQueues)
		{
			if (SurfaceQueueIt.Value.IsValid())
			{
				SurfaceQueueIt.Value->Shutdown();
			}
		}

	}
	
	// Stall until the task graph has completed any pending accumulations.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MRQ_WaitForAccumulationTasksFinished);

		UE::Tasks::Wait(OutstandingTasks);
		OutstandingTasks.Empty();
	}
	

	{
		// Gather telemetry from render passes before they're torn down.
		for (const TObjectPtr<UMovieGraphRenderPassNode>& RenderPass : RenderPassesInUse)
		{
			RenderPass->UpdateRenderPassTelemetry(CachedRenderPassTelemetry);
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(MRQ_TeardownRenderPasses);
		
		for (const TObjectPtr<UMovieGraphRenderPassNode>& RenderPass : RenderPassesInUse)
		{
			RenderPass->Teardown();
		}

		RenderPassesInUse.Reset();
	}

	// ToDo: This could probably be preserved across shots to avoid allocations
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MRQ_ResetPooledData);
		PooledViewRenderTargets.Reset();
		PooledAccumulators.Reset();
		PooledSurfaceQueues.Reset();
	}
}

void UMovieGraphDefaultRenderer::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
	UMovieGraphDefaultRenderer* This = CastChecked<UMovieGraphDefaultRenderer>(InThis);

	// Can't be a const& due to AddStableReference API
	for (TPair<UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewDataPoolParams, TObjectPtr<UTextureRenderTarget2D>>& KVP : This->PooledViewRenderTargets)
	{
		Collector.AddStableReference(&KVP.Value);
	}
}

void UMovieGraphDefaultRenderer::Render(const FMovieGraphTimeStepData& InTimeStepData)
{
	// Flush built-in systems before we render anything. This maximizes the likelihood that the data is prepared for when
	// the render thread uses it.
	FlushAsyncEngineSystems(InTimeStepData.EvaluatedConfig);

	// Empty the camera overscan cache at the start of each frame
	if (InTimeStepData.bIsFirstTemporalSampleForFrame)
	{
		CameraOverscanCache.Empty();
	}
	
	// Housekeeping: Clean up any tasks that were completed since the last frame. This lets us have a 
	// better idea of how much work we're concurrently doing. 
	{
		// Don't iterate through the array if someone is actively modifying it.
		TRACE_CPUPROFILER_EVENT_SCOPE(MRQ_RemoveCompletedOutstandingTasks);
		FScopeLock ScopeLock(&OutstandingTasksMutex);
		for (int32 Index = 0; Index < OutstandingTasks.Num(); Index++)
		{
			const UE::Tasks::FTask& Task = OutstandingTasks[Index];
			if (Task.IsCompleted())
			{
				OutstandingTasks.RemoveAtSwap(Index);
		
				// We swapped the end array element into the current spot,
				// so we need to check this element again, otherwise we skip
				// checking some things.
				Index--;
			}
		}
	}

	// Clean up and shutdown any stale surface queues. This is necessary for anything that changes resolution between frames, such as animated overscan.
	// The surface queue pool is keyed off of resolution, so if every frame has a new resolution, a new surface queue is created, and subsequently,
	// only one surface is ever added to the queue (that for the frame that needed that resolution of surface queue). However, when a surface queue isn't full
	// it can't properly mark surfaces as complete and ready for readback because surface queues natively track "staleness" by how far from the current surface in the queue
	// a previously queued surface is. So, in order to prevent the surface queue from growing too large, and to force surfaces to complete their readback,
	// we track the last frame the queue was used on, and if it has been enough frames, we clean it up, forcing any surfaces to read back. This staleness amount
	// should give any queued surfaces enough frames to complete rendering so that they can be read back by the time Shutdown is called
	{
		for (auto Iter = PooledSurfaceQueues.CreateIterator(); Iter; ++Iter)
		{
			if (Iter->Value->IsStale())
			{
				Iter->Value->Shutdown();
				Iter.RemoveCurrent();
			}
		}
	}

	// Clean up any stale view render targets
	{
		for (auto Iter = PooledViewRenderTargets.CreateIterator(); Iter; ++Iter)
		{
			const UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewPoolState& State = PooledViewRenderTargetStates[Iter->Key];
			if (State.IsStale())
			{
				PooledViewRenderTargetStates.Remove(Iter->Key);
				Iter.RemoveCurrent();
			}
		}
	}
	
	// Hide the progress widget before we render anything. This allows widget captures to not include the progress bar.
	GetOwningGraph()->SetPreviewWidgetVisible(false);

	const FMoviePipelineCameraCutInfo& CurrentCameraCut = GetOwningGraph()->GetActiveShotList()[GetOwningGraph()->GetCurrentShotIndex()]->ShotInfo;
	const FMovieGraphTraversalContext CurrentTraversalContext = GetOwningGraph()->GetCurrentTraversalContext();

	// Allocate a new output merger frame and determine the render passes in use. However, only do this if the pipeline state is Rendering (ie, don't
	// do this if warm-ups are happening because they do not output to disk).
	if (InTimeStepData.bIsFirstTemporalSampleForFrame && (CurrentCameraCut.State == EMovieRenderShotState::Rendering))
	{
		// If this is the first sample for this output frame, then we need to 
		// talk to all of our render passes and ask them for what data they will
		// produce, and set the Output Merger up with that knowledge.
		UE::MovieGraph::FMovieGraphOutputMergerFrame& NewOutputFrame = GetOwningGraph()->GetOutputMerger()->AllocateNewOutputFrame_GameThread(InTimeStepData.OutputFrameNumber);

		// Get the Traversal Context (not specific to any render pass) at the first sample. This is so
		// we can easily fetch things that are shared between all render layers later.
		NewOutputFrame.TraversalContext = CurrentTraversalContext;
		NewOutputFrame.EvaluatedConfig = TStrongObjectPtr<UMovieGraphEvaluatedConfig>(InTimeStepData.EvaluatedConfig);

		for (const TObjectPtr<UMovieGraphRenderPassNode>& RenderPass : RenderPassesInUse)
		{
			RenderPass->GatherOutputPasses(InTimeStepData.EvaluatedConfig, NewOutputFrame.ExpectedRenderPasses);
		}

		// Register the frame with our render statistics as being worked on

		UE::MovieGraph::FRenderTimeStatistics* TimeStats = GetRenderTimeStatistics(NewOutputFrame.TraversalContext.Time.OutputFrameNumber);
		if (ensure(TimeStats))
		{
			TimeStats->StartTime = FDateTime::UtcNow();
		}
	}

	// There is some work we need to signal to the renderer for only the first view of a frame,
	// so we have to track when any of our *FSceneView* render passes submit stuff (UI renderers don't count)
	bHasRenderedFirstViewThisFrame = false;

	// Support for RenderDoc and DumpGPU captures of just the MRQ work
#if WITH_EDITOR && !UE_BUILD_SHIPPING
	TUniquePtr<RenderCaptureInterface::FScopedCapture> ScopedRenderDocCapture;
	TUniquePtr<UE::RenderCore::DumpGPU::FDumpScope> ScopedDumpGPUCapture;
	{
	
		const UMovieGraphDebugSettingNode* DebugSettings = InTimeStepData.EvaluatedConfig->GetSettingForBranch<UMovieGraphDebugSettingNode>(UMovieGraphNode::GlobalsPinName);
		check(DebugSettings);
		const bool bIsRendering = GetOwningGraph()->GetActiveShotList()[GetOwningGraph()->GetCurrentShotIndex()]->ShotInfo.State == EMovieRenderShotState::Rendering;
		const bool bWantsRenderDocDebugCapture = DebugSettings->bCaptureFramesWithRenderDoc;
		const bool bWantsDumpGpuDebugCapture = DebugSettings->bCaptureFramesWithDumpGpu;
		const bool bIsRenderDocCorrectFrame = InTimeStepData.RootFrameNumber == DebugSettings->RenderDocCaptureFrame;
		const bool bIsDumpGPUCorrectFrame = (DebugSettings->DumpGpuCaptureFrameStart < 0 || InTimeStepData.RootFrameNumber >= DebugSettings->DumpGpuCaptureFrameStart) &&
											(DebugSettings->DumpGpuCaptureFrameStop  < 0 || InTimeStepData.RootFrameNumber <  DebugSettings->DumpGpuCaptureFrameStop);

		if (bIsRendering && bWantsRenderDocDebugCapture && bIsRenderDocCorrectFrame)
		{
			ScopedRenderDocCapture = MakeUnique<RenderCaptureInterface::FScopedCapture>(true, *FString::Printf(TEXT("MRQ Frame: %d"), InTimeStepData.RootFrameNumber.Value));
		}
		if (bIsRendering && bWantsDumpGpuDebugCapture && bIsDumpGPUCorrectFrame)
		{
			const FString FinalPath = UE::MovieGraph::Capture::PrepareCapturePath(GetOwningGraph(), InTimeStepData.EvaluatedConfig, DebugSettings->DumpGpuCaptureFilePathFormat, TEXT(""), true);
			const FString Cmd = FString::Printf(TEXT("-noexplorer -path=\"%s\" -unified_filter=\"%s\""), *FinalPath, *DebugSettings->DumpGpuCaptureUnifiedPassResourceFilters);
			const FString Context = FString::Printf(TEXT("MRQ Frame: %d"), InTimeStepData.RootFrameNumber.Value);
			ScopedDumpGPUCapture = MakeUnique<UE::RenderCore::DumpGPU::FDumpScope>(true, *Cmd, *Context);
		}
	}
#endif

	// Workaround for UE-202937, we need to detect when there will be multiple scene views rendered for a given frame
	// to have grooms handle motion blur correctly when there are multiple views being rendered.
	int32 NumSceneViewsRendered = 0;
	for (const TObjectPtr<UMovieGraphRenderPassNode>& RenderPass : RenderPassesInUse)
	{
		NumSceneViewsRendered += RenderPass->GetNumSceneViewsRendered();
	}

	if (NumSceneViewsRendered > 1)
	{
		IConsoleVariable* HairStrandsCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HairStrands.Strands.MotionVectorCheckViewID"));
		if (HairStrandsCVar)
		{
			HairStrandsCVar->SetWithCurrentPriority(0, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
		}
	}

	// Collect all camera sources
	TArray<TSharedRef<FMovieGraphRenderCameraSource>> AllCameraSources;
	for (const TObjectPtr<UMovieGraphRenderPassNode>& RenderPass : RenderPassesInUse)
	{
		RenderPass->GatherRenderCameraSources(AllCameraSources);
	}

	// Notify once per frame 
	for (const TSharedRef<FMovieGraphRenderCameraSource>& CameraSourceIt : AllCameraSources)
	{
		CameraSourceIt->OnBeginFrame(GetOwningGraph(), &InTimeStepData);
	}

	for (const TObjectPtr<UMovieGraphRenderPassNode>& RenderPass : RenderPassesInUse)
	{
		// Pass in a copy of the traversal context so the renderer can decide what to do with it.
		RenderPass->Render(CurrentTraversalContext, InTimeStepData);
	}

	if (NumSceneViewsRendered > 1)
	{
		// Restore the default value afterwards.
		IConsoleVariable* HairStrandsCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HairStrands.Strands.MotionVectorCheckViewID"));
		if (HairStrandsCVar)
		{
			HairStrandsCVar->SetWithCurrentPriority(1, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
		}
	}

	// Re-enable the progress widget so when the player viewport is drawn to the preview window, it shows.
	GetOwningGraph()->SetPreviewWidgetVisible(true);
}

void UMovieGraphDefaultRenderer::FlushAsyncEngineSystems(const TObjectPtr<UMovieGraphEvaluatedConfig>& InConfig) const
{
	const UMovieGraphGlobalGameOverridesNode* GameOverrides = InConfig->GetSettingForBranch<UMovieGraphGlobalGameOverridesNode>(UMovieGraphNode::GlobalsPinName);
	if (!GameOverrides)
	{
		return;
	}

	const UMovieGraphCameraSettingNode* CameraSettingNode = InConfig->GetSettingForBranch<UMovieGraphCameraSettingNode>(UMovieGraphNode::GlobalsPinName);
	if (!CameraSettingNode)
	{
		return;
	}

	// Block until level streaming is completed, we do this at the end of the frame
	// so that level streaming requests made by Sequencer level visibility tracks are
	// accounted for.
	if (GameOverrides->bFlushLevelStreaming && GetOwningGraph()->GetWorld())
	{
		GetOwningGraph()->GetWorld()->BlockTillLevelStreamingCompleted();
	}

	// Flush all assets still being compiled asynchronously.
	// A progress bar is already in place so the user can get feedback while waiting for everything to settle.
	if (GameOverrides->bFlushAssetCompiler)
	{
		FAssetCompilingManager::Get().FinishAllCompilation();
	}

	// Ensure we have complete shader maps for all materials used by primitives in the world.
	// This way we will never render with the default material.
	if (GameOverrides->bFlushShaderCompiler)
	{
		UMaterialInterface::SubmitRemainingJobsForWorld(GetOwningGraph()->GetWorld());
	}

	// Flush virtual texture tile calculations.
	// In its own scope just to minimize the duration FSyncScope has a lock.
	{
		UE::RenderCommandPipe::FSyncScope SyncScope;
		
		ERHIFeatureLevel::Type FeatureLevel = GetWorld()->GetFeatureLevel();
		ENQUEUE_RENDER_COMMAND(VirtualTextureSystemFlushCommand)(
			[FeatureLevel](FRHICommandListImmediate& RHICmdList)
			{
				GetRendererModule().LoadPendingVirtualTextureTiles(RHICmdList, FeatureLevel);
			});
	}

	// Flush any outstanding work waiting in Streaming Manager implementations (texture streaming, nanite, etc.)
	// Note: This isn't a magic fix for gpu-based feedback systems, if the work hasn't made it to the streaming
	// manager, it can't flush it. This just ensures that work that has been requested is done before we render.
	if (GameOverrides->bFlushStreamingManagers)
	{
		FStreamingManagerCollection& StreamingManagers = IStreamingManager::Get();
		constexpr bool bProcessEverything = true;
		StreamingManagers.UpdateResourceStreaming(GetOwningGraph()->GetWorld()->GetDeltaSeconds(), bProcessEverything);
		StreamingManagers.BlockTillAllRequestsFinished();
	}

	// If there are async tasks to build more grass, wait for them to finish so there aren't missing patches
	// of grass. If you have way too dense grass this option can cause you to OOM.
	if (GameOverrides->bFlushGrassStreaming)
	{
		if (ULandscapeSubsystem* LandscapeSubsystem = GetWorld()->GetSubsystem<ULandscapeSubsystem>())
		{
			constexpr bool bFlushGrass = false; // Flush means a different thing to grass system
			constexpr bool bInForceSync = true;

			UMoviePipelineExecutorShot* CurrentShot = GetOwningGraph()->GetActiveShotList()[GetOwningGraph()->GetCurrentShotIndex()];

			TArray<FVector> CameraLocations;
			GetCameraLocationsForFrame(CameraLocations, CurrentShot, CameraSettingNode->bRenderAllCameras);

			LandscapeSubsystem->RegenerateGrass(bFlushGrass, bInForceSync, MakeArrayView(CameraLocations));
		}
	}
}

void UMovieGraphDefaultRenderer::GetCameraLocationsForFrame(TArray<FVector>& OutLocations, UMoviePipelineExecutorShot* InShot, bool bIncludeSidecar) const
{
	// Collect custom camera sources from all active render pass nodes.
	TArray<TSharedRef<FMovieGraphRenderCameraSource>> AllCameraSources;
	for (const TObjectPtr<UMovieGraphRenderPassNode>& NodeIt : RenderPassesInUse)
	{
		if (NodeIt)
		{
			NodeIt->GatherRenderCameraSources(AllCameraSources);
		}
	}

	// Add a location for each camera provided by custom sources.
	for (const TSharedRef<FMovieGraphRenderCameraSource>& CameraSourceIt : AllCameraSources)
	{
		const int32 NumCameras = CameraSourceIt->GetNumCameras();
		for (int32 CameraIndex = 0; CameraIndex < NumCameras; CameraIndex++)
		{
			FMinimalViewInfo MinimalViewInfo;
			if (CameraSourceIt->GetCameraViewInfo(CameraIndex, MinimalViewInfo))
			{
				OutLocations.AddUnique(MinimalViewInfo.Location);
			}
		}
	}

	// Always include default sequence cameras (active game camera and optionally sidecar cameras).
	TArray<UE::MovieGraph::FMinimalCameraInfo> MinimalCameraInfos = GetOwningGraph()->GetDataSourceInstance()->GetCameraInformation(InShot, bIncludeSidecar);
	for (const UE::MovieGraph::FMinimalCameraInfo& MinimalCameraInfo : MinimalCameraInfos)
	{
		// Skip default placeholders for unresolved sidecar bindings; otherwise, grass streaming would include (0,0,0).
		if (bIncludeSidecar && !MinimalCameraInfo.ViewActor.IsValid())
		{
			continue;
		}

		OutLocations.Add(MinimalCameraInfo.ViewInfo.Location);
	}
}

void UMovieGraphDefaultRenderer::AddOutstandingRenderTask_AnyThread(UE::Tasks::FTask InTask)
{
	// We might be looping through the array to remove previously completed tasks,
	// so don't modify until that is completed.
	FScopeLock ScopeLock(&OutstandingTasksMutex);
	OutstandingTasks.Add(MoveTemp(InTask));
}

UE::MovieGraph::DefaultRenderer::FCameraInfo UMovieGraphDefaultRenderer::GetCameraInfoImpl(const int32 InCameraIndex) const
{
	UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo;
	

	UMoviePipelineExecutorShot* CurrentShot = GetOwningGraph()->GetActiveShotList()[GetOwningGraph()->GetCurrentShotIndex()];

	// When not using multi-camera the index is "-1" (but we store the data in the 0th array) so we remap,
	// but the GetCameraName function is -1 aware.
	int32 LocalArrayIndex = InCameraIndex;
	if (InCameraIndex < 0)
	{
		LocalArrayIndex = 0;
		const int32 CameraIndex = INDEX_NONE;
		CameraInfo.CameraName = CurrentShot->GetCameraName(CameraIndex);
	}
	else
	{
		CameraInfo.CameraName = CurrentShot->GetCameraName(InCameraIndex);
	}


	// If we're not rendering all cameras, InCameraIndex is -1.
	const bool bRenderAllCameras = InCameraIndex >= 0;
	TArray<UE::MovieGraph::FMinimalCameraInfo> MinimalCameraInfos = GetOwningGraph()->GetDataSourceInstance()->GetCameraInformation(GetOwningGraph()->GetActiveShotList()[GetOwningGraph()->GetCurrentShotIndex()], bRenderAllCameras);
	if (!ensureAlways(MinimalCameraInfos.IsValidIndex(LocalArrayIndex)))
	{
		return CameraInfo;
	}

	if (bRenderAllCameras && !MinimalCameraInfos[LocalArrayIndex].ViewActor.IsValid())
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Unable to resolve sidecar camera '%ls' while fetching Movie Graph camera info for shot '%ls'.",
			*CameraInfo.CameraName, *CurrentShot->InnerName);
		return CameraInfo;
	}

	CameraInfo.ViewInfo = MinimalCameraInfos[LocalArrayIndex].ViewInfo;
	CameraInfo.ViewActor = MinimalCameraInfos[LocalArrayIndex].ViewActor.Get();

	return CameraInfo;
}

UE::MovieGraph::DefaultRenderer::FCameraInfo UMovieGraphDefaultRenderer::GetCameraInfo(	
	int32 InCameraIndex,
	const FMovieGraphRenderCameraSource* InRenderCameraSource) const
{
	UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo;

	// When a custom camera source is active, query it directly instead of the default sequence cameras.
	if (InRenderCameraSource && InCameraIndex >= 0)
	{
		FMinimalViewInfo ViewInfo;
		if (InRenderCameraSource->GetCameraViewInfo(InCameraIndex, ViewInfo))
		{
			CameraInfo.ViewInfo = ViewInfo;

			// GetCameraViewActor() and GetCameraName() are optional — the source may defer
			// either to the default sequence camera by returning false.
			const bool bHasCameraViewActor = InRenderCameraSource->GetCameraViewActor(InCameraIndex, CameraInfo.ViewActor);
			const bool bHasCameraName = InRenderCameraSource->GetCameraName(InCameraIndex, CameraInfo.CameraName);
			if (!bHasCameraViewActor || !bHasCameraName)
			{
				const UE::MovieGraph::DefaultRenderer::FCameraInfo DefaultCameraInfo = GetCameraInfoImpl(INDEX_NONE);
				if (!bHasCameraViewActor)
				{
					CameraInfo.ViewActor = DefaultCameraInfo.ViewActor;
				}
				if (!bHasCameraName)
				{
					CameraInfo.CameraName = DefaultCameraInfo.CameraName;
				}
			}

			return CameraInfo;
		}

		// Source exists but couldn't resolve this index — fall back to the primary sequence camera.
		// INDEX_NONE is the sentinel for "primary camera" in the default camera path below.
		return GetCameraInfoImpl(INDEX_NONE);
	}

	return GetCameraInfoImpl(InCameraIndex);
}

float UMovieGraphDefaultRenderer::GetCameraOverscanImpl(int32 InCameraIndex)
{
	if (CameraOverscanCache.Contains(InCameraIndex))
	{
		return CameraOverscanCache[InCameraIndex];
	}

	const float CameraOverscan = GetCameraInfo(InCameraIndex).ViewInfo.GetOverscan();
	CameraOverscanCache.Add(InCameraIndex, CameraOverscan);
	return CameraOverscan;
}

float UMovieGraphDefaultRenderer::GetCameraOverscan(
	int32 InCameraIndex,
	const FMovieGraphRenderCameraSource* InRenderCameraSource)
{
	// When CameraSource overrides default cameras, caches can't be used. The cache is keyed only by
	// InCameraIndex, but a CameraSource supplies a completely different set of cameras than the default
	// sequence cameras. The same index can therefore resolve to a different view with a different
	// overscan, making any cached value incorrect.
	// Note: INDEX_NONE reserved for default camera.
	if (InRenderCameraSource)
	{
		float RenderCameraSourceOverscan = 0.f;
		if (InRenderCameraSource->GetCameraOverscan(InCameraIndex, RenderCameraSourceOverscan))
		{
			return RenderCameraSourceOverscan;
		}

		// fallback to default camera
		return GetCameraOverscanImpl(INDEX_NONE);
	}

	return GetCameraOverscanImpl(InCameraIndex);
}

void UMovieGraphDefaultRenderer::WarnAboutAnimatedOverscan(float InInitialOverscan)
{
	if (!bHasWarnedAboutAnimatedOverscan)
	{
		UMoviePipelineExecutorShot* CurrentShot = GetOwningGraph()->GetActiveShotList()[GetOwningGraph()->GetCurrentShotIndex()];
		
		UE_LOGF(
			LogMovieRenderPipeline,
			Warning,
			"Overscan on camera %ls changed since start of frame in shot %ls, using cached overscan value of %f instead",
			*CurrentShot->OuterName,
			*CurrentShot->InnerName,
			InInitialOverscan);

		bHasWarnedAboutAnimatedOverscan = true;
	}
}

UTextureRenderTarget2D* UMovieGraphDefaultRenderer::GetOrCreateViewRenderTarget(const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InInitParams, const FMovieGraphRenderDataIdentifier& InIdentifier)
{
	UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewDataPoolParams CombinedParams;
	CombinedParams.RenderInitParams = InInitParams;
	CombinedParams.Identifier = InIdentifier;

	if (const TObjectPtr<UTextureRenderTarget2D>* ExistingViewRenderTarget = PooledViewRenderTargets.Find(CombinedParams))
	{
		PooledViewRenderTargetStates[CombinedParams].UpdateLastUsedFrame();
		return *ExistingViewRenderTarget;
	}

	const TObjectPtr<UTextureRenderTarget2D> NewViewRenderTarget = CreateViewRenderTarget(InInitParams);
	PooledViewRenderTargets.Emplace(CombinedParams, NewViewRenderTarget);
	PooledViewRenderTargetStates.Emplace(CombinedParams, UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewPoolState());
	
	return NewViewRenderTarget.Get();
}

void UMovieGraphDefaultRenderer::UpdateRenderPassTelemetry(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const
{
	InOutTelemetry.MergeRenderPassTelemetry(CachedRenderPassTelemetry);
}

TObjectPtr<UTextureRenderTarget2D> UMovieGraphDefaultRenderer::CreateViewRenderTarget(const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InInitParams) const
{
	TObjectPtr<UTextureRenderTarget2D> NewTarget = NewObject<UTextureRenderTarget2D>(GetTransientPackage());
	NewTarget->ClearColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
	NewTarget->TargetGamma = InInitParams.TargetGamma;
	NewTarget->InitCustomFormat(InInitParams.Size.X, InInitParams.Size.Y, InInitParams.PixelFormat, InInitParams.bForceLinearGamma);
	int32 ResourceSizeBytes = NewTarget->GetResourceSizeBytes(EResourceSizeMode::Type::EstimatedTotal);
	UE_LOGF(LogMovieRenderPipeline, Log, "Allocated a View Render Target sized: (%d, %d), Bytes: %d", InInitParams.Size.X, InInitParams.Size.Y, ResourceSizeBytes);

	return NewTarget;
}

FMoviePipelineSurfaceQueuePtr UMovieGraphDefaultRenderer::GetOrCreateSurfaceQueue(const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InInitParams)
{
	if (const FMoviePipelineSurfaceQueuePtr* ExistingSurfaceQueue = PooledSurfaceQueues.Find(InInitParams))
	{
		(*ExistingSurfaceQueue)->UpdateLastUsedFrame();
		return *ExistingSurfaceQueue;
	}

	const FMoviePipelineSurfaceQueuePtr NewSurfaceQueue = CreateSurfaceQueue(InInitParams);
	PooledSurfaceQueues.Emplace(InInitParams, NewSurfaceQueue);

	return NewSurfaceQueue;
}

FMoviePipelineSurfaceQueuePtr UMovieGraphDefaultRenderer::CreateSurfaceQueue(const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InInitParams) const
{
	// ToDo: Refactor these to be dynamically sized, but also allow putting a cap on them. We need at least enough to submit everything for one render
	FMoviePipelineSurfaceQueuePtr SurfaceQueue = MakeShared<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe>(InInitParams.Size, InInitParams.PixelFormat, 6, true);
	
	UE_LOGF(LogMovieRenderPipeline, Log, "Allocated a Surface Queue sized: (%d, %d)", InInitParams.Size.X, InInitParams.Size.Y);
	return SurfaceQueue;
}

UE::MovieGraph::FRenderTimeStatistics* UMovieGraphDefaultRenderer::GetRenderTimeStatistics(const int32 InFrameNumber)
{
	return &RenderTimeStatistics.FindOrAdd(InFrameNumber);
}

TArray<FMovieGraphImagePreviewData> UMovieGraphDefaultRenderer::GetPreviewData() const
{ 
	TArray<FMovieGraphImagePreviewData> Results;

	uint64 MostRecentFrame = 0;
	
	TSet<FString> CameraNamesInUse;
	for (const TPair<UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewDataPoolParams, TObjectPtr<UTextureRenderTarget2D>>& RenderTarget : PooledViewRenderTargets)
	{
		CameraNamesInUse.Add(RenderTarget.Key.Identifier.CameraName);
		MostRecentFrame = FMath::Max(PooledViewRenderTargetStates[RenderTarget.Key].LastUsedFrame, MostRecentFrame);
	}

	for (const TPair<UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewDataPoolParams, TObjectPtr<UTextureRenderTarget2D>>& RenderTarget : PooledViewRenderTargets)
	{
		const FString RendererName = RenderTarget.Key.Identifier.RendererName;
		
		// Skip render targets that have opted out of showing in the preview window.
		if (!RenderTarget.Key.RenderInitParams.bIncludeInPreviewWindow)
		{
			continue;
		}

		// In cases of changing resolution between frames (e.g animated overscan) the render target pool will have a render target for each resolution, but
		// that render target may be for a past frame. So filter out any render targets that aren't from the most recent frame
		const UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewPoolState RenderTargetState = PooledViewRenderTargetStates[RenderTarget.Key];
		if (RenderTargetState.LastUsedFrame == MostRecentFrame)
		{
			FMovieGraphImagePreviewData& Data = Results.AddDefaulted_GetRef();
			Data.Identifier = RenderTarget.Key.Identifier;
			Data.Texture = RenderTarget.Value.Get();
			Data.bMultipleCameraNames = CameraNamesInUse.Num() > 1;

			// Let any active render pass contribute per-tile banner messages
			for (const TObjectPtr<UMovieGraphRenderPassNode>& RenderPass : RenderPassesInUse)
			{
				if (RenderPass)
				{
					RenderPass->GetPreviewBannerMessages(Data.Identifier, Data.BannerMessages);
				}
			}
		}
	}

	return Results;
}
