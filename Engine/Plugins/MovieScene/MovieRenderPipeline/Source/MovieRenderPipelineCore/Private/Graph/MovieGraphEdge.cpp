// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphEdge.h"

#include "Graph/Nodes/MovieGraphRerouteNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphEdge)

bool UMovieGraphEdge::IsValid() const
{
	return InputPin.Get() && OutputPin.Get();
}

UMovieGraphPin* UMovieGraphEdge::GetOtherPin(const UMovieGraphPin* InFirstPin, const bool bFollowRerouteConnections)
{
	check(InFirstPin == InputPin || InFirstPin == OutputPin);
	UMovieGraphPin* OtherPin = InFirstPin == InputPin ? OutputPin : InputPin;

	if (!OtherPin)
	{
		return nullptr;
	}

	// If the other pin is connected to the reroute node, recursively follow the chain until a non-reroute node is reached
	if (bFollowRerouteConnections)
	{
		if (const UMovieGraphRerouteNode* RerouteNode = Cast<UMovieGraphRerouteNode>(OtherPin->Node))
		{
			UMovieGraphPin* PassThroughPin = RerouteNode->GetPassThroughPin(OtherPin);
			if (PassThroughPin && !PassThroughPin->Edges.IsEmpty())
			{
				OtherPin = PassThroughPin->Edges[0]->GetOtherPin(PassThroughPin, bFollowRerouteConnections);
			}
		}
	}
		
	return OtherPin;
}
