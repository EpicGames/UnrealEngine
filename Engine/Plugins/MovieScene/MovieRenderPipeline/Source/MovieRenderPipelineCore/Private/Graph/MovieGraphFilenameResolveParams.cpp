// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphFilenameResolveParams.h"

#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "MoviePipelineQueue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphFilenameResolveParams)

FMovieGraphFilenameResolveParams FMovieGraphFilenameResolveParams::MakeResolveParams(
	const FMovieGraphRenderDataIdentifier& InRenderId,
	const UMovieGraphPipeline* InPipeline,
	const TObjectPtr<UMovieGraphEvaluatedConfig>& InEvaluatedConfig,
	const FMovieGraphTraversalContext& InTraversalContext,
	const TMap<FString, FString>& InAdditionalFormatArgs,
	const UE::MovieGraph::FMovieGraphSampleState* InSamplePayload)
{
    FMovieGraphFilenameResolveParams Params = FMovieGraphFilenameResolveParams();
	
	int32 RootFrameNumberRelOffset = 0;
	int32 ShotFrameNumberRelOffset = 0;

	// Initialize the job/shot with the pipeline if available; otherwise, fall back to the traversal context.
	// Using the traversal context may result in a slightly under-initialized set of resolve parameters, but generally
	// these parameters aren't needed in the contexts that a pipeline isn't available.
	if (IsValid(InPipeline))
	{
		Params.InitializationTime = InPipeline->GetInitializationTime();
		Params.InitializationTimeOffset = InPipeline->GetInitializationTimeOffset();
		Params.Job = InPipeline->GetCurrentJob();
		
		if (InPipeline->GetActiveShotList().IsValidIndex(InTraversalContext.ShotIndex))
		{
			Params.Shot = InPipeline->GetActiveShotList()[InTraversalContext.ShotIndex];
		}

		// Calculate the offset (in frames) that relative frame numbers need to add to themselves so that they are correctly
		// offset by the starting frame of the root sequence/shot (to match new updated relative behavior).
		// We use the first shot to calculate the root version of the relative frame number offset, because that'll have the correct
		// root time for rendered frames (that should account for handle frames (included) vs warmup (not)). It's safe to assume index
		// 0 exists here because we don't null check ShotIndex and it's either going to be >= 0.
		const TObjectPtr<UMoviePipelineExecutorShot>& RootShot = InPipeline->GetActiveShotList()[0];

		RootFrameNumberRelOffset = FFrameRate::TransformTime(RootShot->ShotInfo.InitialTimeInRoot, RootShot->ShotInfo.CachedTickResolution, RootShot->ShotInfo.CachedFrameRate).FloorToFrame().Value;
	}
	else
	{
		Params.Job = InTraversalContext.Job;
		Params.Shot = InTraversalContext.Shot;
	}

	if (const TObjectPtr<UMoviePipelineExecutorShot>& Shot = Params.Shot)
	{
		Params.Version = Shot->ShotInfo.VersionNumber;
		
		ShotFrameNumberRelOffset = FFrameRate::TransformTime(Shot->ShotInfo.InitialTimeInShot, Shot->ShotInfo.CachedTickResolution, Shot->ShotInfo.CachedFrameRate).FloorToFrame().Value;
	}
        
	Params.RenderDataIdentifier = InRenderId;
	
	Params.RootFrameNumber = InTraversalContext.Time.RootFrameNumber.Value;
	Params.ShotFrameNumber = InTraversalContext.Time.ShotFrameNumber.Value;

	// Starting in MRG, relative file numbers are now relative to the first frame of the shot/sequence, and not
	// to zero. To do this, we use our zero-relative numbers and offset them by the starting point of the shot/sequence.
	Params.RootFrameNumberRel = InTraversalContext.Time.OutputFrameNumber + RootFrameNumberRelOffset;
	Params.ShotFrameNumberRel = InTraversalContext.Time.ShotOutputFrameNumber + ShotFrameNumberRelOffset;

	if (InEvaluatedConfig)
	{
		const UMovieGraphGlobalOutputSettingNode* OutputSettingNode = InEvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName);
		if (IsValid(OutputSettingNode))
		{
			Params.ZeroPadFrameNumberCount = OutputSettingNode->ZeroPadFrameNumbers;
			Params.FrameNumberOffset = OutputSettingNode->FrameNumberOffset;
		}
		Params.EvaluatedConfig = InEvaluatedConfig;
	}

	const bool bTimeDilationUsed = !FMath::IsNearlyEqual(InTraversalContext.Time.WorldTimeDilation, 1.f) || InTraversalContext.Time.bHasRelativeTimeBeenUsed;
    Params.bForceRelativeFrameNumbers = bTimeDilationUsed;
	Params.bEnsureAbsolutePath = true;
	Params.FileNameFormatOverrides = InAdditionalFormatArgs;

	if (InSamplePayload)
	{
		Params.FileNameFormatOverrides.Append(InSamplePayload->AdditionalFormatArgs);
		Params.FileMetadata.Append(InSamplePayload->AdditionalFileMetadata);
	}

	return Params;
}
