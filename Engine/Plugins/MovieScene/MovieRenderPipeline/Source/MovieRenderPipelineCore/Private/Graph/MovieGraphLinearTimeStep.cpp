// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphLinearTimeStep.h"

#include "Graph/MovieGraphPipeline.h"
#include "MoviePipelineQueue.h"
#include "MovieRenderPipelineCoreModule.h"

#include "Graph/Nodes/MovieGraphSamplingMethodNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphLinearTimeStep)

static TAutoConsoleVariable<int32> CVarNumWarmUpFramesWithTemporalSampling(
	TEXT("MovieRenderPipeline.NumWarmUpFramesWithTemporalSampling"),
	5,
	TEXT("The number of warm-up frames that should have temporal sampling applied to them.\n"),
	ECVF_Default);

int32 UMovieGraphLinearTimeStep::GetNextTemporalRangeIndex() const
{
	// Linear time step just steps through the temporal ranges in order
	return CurrentFrameData.TemporalSampleIndex;
}

int32 UMovieGraphLinearTimeStep::GetTemporalSampleCount() const
{
	const int32 CurrentShotIndex = GetOwningGraph()->GetCurrentShotIndex();
	const TArray<TObjectPtr<UMoviePipelineExecutorShot>>& ActiveShotList = GetOwningGraph()->GetActiveShotList();
	const TObjectPtr<UMoviePipelineExecutorShot>& CurrentCameraCut = ActiveShotList[CurrentShotIndex];

	// If we're not near the end of the warm-up frames, don't do any temporal sampling. This reduces the time it takes to do warm-ups, and
	// performing temporal samples in these warm-up frames usually has no practical use. Allowing a few warm-up frames to do temporal sampling
	// will allow systems like temporal denoising to have full-quality frames to base the denoising from. Cloth simulation could also be impacted
	// by this.
	if (CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining > CVarNumWarmUpFramesWithTemporalSampling.GetValueOnGameThread())
	{
		return 1;
	}

	return GetTemporalSampleCountFromConfig(CurrentFrameData.EvaluatedConfig.Get());
}
