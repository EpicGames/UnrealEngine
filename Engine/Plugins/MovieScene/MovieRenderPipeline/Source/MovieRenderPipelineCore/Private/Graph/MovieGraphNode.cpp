// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphRerouteNode.h"
#include "UObject/TextProperty.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphNode)

FName UMovieGraphNode::GlobalsPinName = FName("Globals");
FString UMovieGraphNode::GlobalsPinNameString = FString("Globals");

namespace UE::MovieGraph::Private
{
	EMovieGraphValueType GetValueTypeFromProperty(const FProperty* InSourceProperty, TObjectPtr<const UObject>& OutValueTypeObject)
	{
		if (CastField<FBoolProperty>(InSourceProperty))
		{
			return EMovieGraphValueType::Bool;
		}
		if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InSourceProperty))
		{
			if (ByteProperty->IsEnum())
			{
				OutValueTypeObject = ByteProperty->Enum.Get();
				return EMovieGraphValueType::Enum;
			}
			
			return EMovieGraphValueType::Byte;
		}
		if (CastField<FIntProperty>(InSourceProperty))
		{
			return EMovieGraphValueType::Int32;
		}
		if (CastField<FInt64Property>(InSourceProperty))
		{
			return EMovieGraphValueType::Int64;
		}
		if (CastField<FFloatProperty>(InSourceProperty))
		{
			return EMovieGraphValueType::Float;
		}
		if (CastField<FDoubleProperty>(InSourceProperty))
		{
			return EMovieGraphValueType::Double;
		}
		if (CastField<FNameProperty>(InSourceProperty))
		{
			return EMovieGraphValueType::Name;
		}
		if (CastField<FStrProperty>(InSourceProperty))
		{
			return EMovieGraphValueType::String;
		}
		if (CastField<FTextProperty>(InSourceProperty))
		{
			return EMovieGraphValueType::Text;
		}
		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InSourceProperty))
		{
			OutValueTypeObject = EnumProperty->GetEnum();
			return EMovieGraphValueType::Enum;
		}
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty))
		{
			OutValueTypeObject = StructProperty->Struct.Get();
			return EMovieGraphValueType::Struct;
		}
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InSourceProperty))
		{
			OutValueTypeObject = ObjectProperty->PropertyClass.Get();
			return EMovieGraphValueType::Object;
		}
		if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(InSourceProperty))
		{
			OutValueTypeObject = SoftObjectProperty->PropertyClass.Get();
			return EMovieGraphValueType::SoftObject;
		}
		if (const FClassProperty* ClassProperty = CastField<FClassProperty>(InSourceProperty))
		{
			OutValueTypeObject = ClassProperty->PropertyClass.Get();
			return EMovieGraphValueType::Class;
		}
		if (const FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(InSourceProperty))
		{
			OutValueTypeObject = SoftClassProperty->PropertyClass.Get();
			return EMovieGraphValueType::SoftClass;
		}

		// Handle array property
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InSourceProperty))
		{
			return GetValueTypeFromProperty(ArrayProperty->Inner, OutValueTypeObject);	
		}

		return EMovieGraphValueType::None;
	}
}

UMovieGraphNode::UMovieGraphNode()
{
	
}

TArray<FMovieGraphPinProperties> UMovieGraphNode::GetExposedPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;

	// Exposed properties are tracked by name; convert to pin properties
	for (const FMovieGraphPropertyInfo& PropertyInfo : GetExposedProperties())
	{
		if (PropertyInfo.ValueType == EMovieGraphValueType::None)
		{
			continue;
		}
		
		constexpr bool bAllowMultipleConnections = false;
		Properties.Add(FMovieGraphPinProperties(PropertyInfo.Name, PropertyInfo.ValueType, PropertyInfo.ValueTypeObject, bAllowMultipleConnections));
	}

	return Properties;
}

void UMovieGraphNode::TogglePromotePropertyToPin(const FName& PropertyName)
{
	bool bIsAlreadyExposedProperty = ExposedPropertyInfo.ContainsByPredicate([PropertyName](const FMovieGraphPropertyInfo& ExposedProperty)
	{
		return ExposedProperty.Name == PropertyName;
	});
	
	// Both Promote and Unpromote will call Modify() and UpdatePins() on their own.
	if(bIsAlreadyExposedProperty)
	{
		UnpromotePropertyFromPin(PropertyName);
	}
	else
	{
		PromotePropertyToPin(PropertyName);
	}
}

void UMovieGraphNode::PromotePropertyToPin(const FName& PropertyName)
{
	Modify();
	
	// Ensure this is a property that can be promoted
	bool bIsPromotableProperty = false;
	FMovieGraphPropertyInfo OverrideablePropertyInfo;
	for (const FMovieGraphPropertyInfo& ExposablePropertyInfo : GetOverrideablePropertyInfo())
	{
		if (ExposablePropertyInfo.Name == PropertyName)
		{
			bIsPromotableProperty = true;
			OverrideablePropertyInfo = ExposablePropertyInfo;
			break;
		}
	}
	
	if (bIsPromotableProperty)
	{
		ExposedPropertyInfo.AddUnique(MoveTemp(OverrideablePropertyInfo));
	}
	else
	{
		// They requested a property that didn't exist, throw a kismet exception so scripts can know.
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: Could not find a property with the name '%s' to promote to pin."), __FUNCTION__, *PropertyName.ToString()),
			ELogVerbosity::Error);
	}
	
	UpdatePins();
}

void UMovieGraphNode::UnpromotePropertyFromPin(const FName& PropertyName)
{
	Modify();
	
	bool bIsExistingProperty = ExposedPropertyInfo.ContainsByPredicate([PropertyName](const FMovieGraphPropertyInfo& ExposedProperty)
	{
		return ExposedProperty.Name == PropertyName;
	});
	
	if(bIsExistingProperty)
	{
		// Remove the requested pin from our ExposedPropertyInfo array. We use ExposedPropertyInfo here (and not GetOverrideablePropertyInfo)
		// because this reflects the actual currently exposed pins, even if they're no longer something you could override. This allows you to
		// unpromote an old property.
		ExposedPropertyInfo.RemoveAll([&PropertyName](const FMovieGraphPropertyInfo& ExposedInfo)
		{
			return ExposedInfo.Name == PropertyName;
		});
	}
	else
	{
		// They requested a property that didn't exist, throw a kismet exception so scripts can know.
		FFrame::KismetExecutionMessage(
			*FString::Printf(
				TEXT("%hs: Could not find a property with the name '%s' to promote to pin."), __FUNCTION__, *PropertyName.ToString()),
			ELogVerbosity::Error);
	}
	
	UpdatePins();
}

bool UMovieGraphNode::GetCustomPromotionDefaultValue(const FMovieGraphPropertyInfo& InPropertyInfo, FString& OutSerializedValue) const
{
	return false;
}

TArray<const FProperty*> UMovieGraphNode::GetAllOverrideableProperties() const
{
	TArray<const FProperty*> AllProperties;
	
	for (TFieldIterator<FProperty> PropertyIterator(GetClass()); PropertyIterator; ++PropertyIterator)
	{
		if (UMovieGraphConfig::FindOverridePropertyForRealProperty(GetClass(), *PropertyIterator))
		{
			AllProperties.Add(*PropertyIterator);
		}
	}

	if (const UPropertyBag* PropertyBag = DynamicProperties.GetPropertyBagStruct())
	{
		for (const FPropertyBagPropertyDesc& Desc : PropertyBag->GetPropertyDescs())
		{
			if (FindOverridePropertyForDynamicProperty(Desc.Name))
			{
				AllProperties.Add(Desc.CachedProperty);
			}
		}
	}

	return AllProperties;
}

void UMovieGraphNode::UpdatePins()
{
	TArray<FMovieGraphPinProperties> InputPinProperties = GetInputPinProperties();
	TArray<FMovieGraphPinProperties> OutputPinProperties = GetOutputPinProperties();

	// Include the exposed dynamic properties in the input pins
	InputPinProperties.Append(GetExposedPinProperties());

	bool bChangedPins = false;

	auto UpdatePins = [this, &bChangedPins](TArray<UMovieGraphPin*>& Pins, const TArray<FMovieGraphPinProperties>& PinProperties)
	{
		bool bAppliedEdgeChanges = false;

		// Find unmatched pins vs. properties (via name matching)
		TArray<UMovieGraphPin*> UnmatchedPins;
		for (UMovieGraphPin* Pin : Pins)
		{
			if (const FMovieGraphPinProperties* MatchingProperties = PinProperties.FindByPredicate([Pin](const FMovieGraphPinProperties& Prop) { return Prop.Label == Pin->Properties.Label; }))
			{
				if (Pin->Properties != *MatchingProperties)
				{
					Pin->Modify();
					Pin->Properties = *MatchingProperties;
					//bAppliedEdgeCHanges |= Pin->BreakAllIncompatibleEdges();
					bChangedPins = true;
				}
			}
			else
			{
				UnmatchedPins.Add(Pin);
			}
		}

		// Now do the opposite, find any properties that don't have pins
		TArray<FMovieGraphPinProperties> UnmatchedProperties;
		for (const FMovieGraphPinProperties& Properties : PinProperties)
		{
			if (!Pins.FindByPredicate([&Properties](const UMovieGraphPin* Pin) { return Pin->Properties.Label == Properties.Label; }))
			{
				UnmatchedProperties.Add(Properties);
			}
		}

		if (!UnmatchedPins.IsEmpty() || !UnmatchedProperties.IsEmpty())
		{
			Modify();
			bChangedPins = true;
		}

		// Remove old pins
		for (int32 UnmatchedPinIndex = UnmatchedPins.Num() - 1; UnmatchedPinIndex >= 0; UnmatchedPinIndex--)
		{
			const int32 PinIndex = Pins.IndexOfByKey(UnmatchedPins[UnmatchedPinIndex]);
			check(PinIndex >= 0);

			//bAppliedEdgeCHanges |= Pins[PinIndex]->BreakAllEdges();
			Pins.RemoveAt(PinIndex);
		}

		// Add new pins
		for (const FMovieGraphPinProperties& UnmatchedProperty : UnmatchedProperties)
		{
			const int32 InsertIndex = PinProperties.IndexOfByKey(UnmatchedProperty);
			UMovieGraphPin* NewPin = NewObject<UMovieGraphPin>(this, NAME_None, RF_Transactional);
			NewPin->Node = this;
			NewPin->Properties = UnmatchedProperty;
			Pins.Insert(NewPin, InsertIndex);
		}

		// return bAppliedEdgeChanges ? EPCGChangeType::Edge : None) | (bChangedPins ? EPCGChangeType::Node : None
	};

	UpdatePins(MutableView(InputPins), InputPinProperties);
	UpdatePins(MutableView(OutputPins), OutputPinProperties);

	if (bChangedPins)
	{
		OnNodeChangedDelegate.Broadcast(this);
	}
}

void UMovieGraphNode::UpdateDynamicProperties()
{
	TArray<FPropertyBagPropertyDesc> DesiredDynamicProperties = GetDynamicPropertyDescriptions();

	// Check to see if we need to remake our property bag
	bool bHasAllProperties = DynamicProperties.GetNumPropertiesInBag() == DesiredDynamicProperties.Num();
	if (bHasAllProperties)
	{
		// If there's still the same number of properties before/after, then we need to do the more expensive
		// check, which is to see if every property in the desired list is already inside the property bag.
		for (const FPropertyBagPropertyDesc& Desc : DesiredDynamicProperties)
		{
			bool bBagContainsProperty = DynamicProperties.FindPropertyDescByName(Desc.Name) != nullptr;
			if (!bBagContainsProperty)
			{
				bHasAllProperties = false;
				break;
			}
		}
	}

	// If we don't have all the properties in our bag already, we need to generate a new bag with the correct
	// layout, and then we have to migrate the existing bag over (so existing, matching, values stay).
	if (!bHasAllProperties)
	{
		FInstancedPropertyBag NewPropertyBag;
		NewPropertyBag.AddProperties(DesiredDynamicProperties);

		DynamicProperties.MigrateToNewBagInstance(NewPropertyBag);
	}
}

UMovieGraphConfig* UMovieGraphNode::GetGraph() const
{
	return Cast<UMovieGraphConfig>(GetOuter());
}

FInstancedPropertyBag* UMovieGraphNode::GetMutableDynamicProperties_Unsafe() const
{
	return const_cast<FInstancedPropertyBag*>(&DynamicProperties);
}

UMovieGraphPin* UMovieGraphNode::GetInputPin(const FName& InPinLabel, const EMovieGraphPinQueryRequirement PinRequirement) const
{
	for (UMovieGraphPin* InputPin : InputPins)
	{
		if (InputPin->Properties.Label != InPinLabel)
		{
			continue;
		}

		if (PinRequirement == EMovieGraphPinQueryRequirement::BuiltInOrDynamic)
		{
			return InputPin;
		}

		const bool bIsBuiltIn = InputPin->Properties.bIsBuiltIn;
		if ((PinRequirement == EMovieGraphPinQueryRequirement::BuiltIn) && bIsBuiltIn)
		{
			return InputPin;
		}
		
		if ((PinRequirement == EMovieGraphPinQueryRequirement::Dynamic) && !bIsBuiltIn)
		{
			return InputPin;
		}
	}

	return nullptr;
}

UMovieGraphPin* UMovieGraphNode::GetOutputPin(const FName& Label) const
{
	for (UMovieGraphPin* OutputPin : OutputPins)
	{
		if (OutputPin->Properties.Label == Label)
		{
			return OutputPin;
		}
	}

	return nullptr;
}

UMovieGraphPin* UMovieGraphNode::GetFirstConnectedInputPin() const
{
	for (const TObjectPtr<UMovieGraphPin>& InputPin : InputPins)
	{
		if (InputPin->IsConnected())
		{
			return InputPin.Get();
		}
	}

	return nullptr;
}

UMovieGraphPin* UMovieGraphNode::GetFirstConnectedOutputPin() const
{
	for (const TObjectPtr<UMovieGraphPin>& OutputPin : OutputPins)
	{
		if (OutputPin->IsConnected())
		{
			return OutputPin.Get();
		}
	}

	return nullptr;
}

bool UMovieGraphNode::CanBeDisabled() const
{
	// By default, all nodes can be disabled
	return true;
}

void UMovieGraphNode::SetDisabled(const bool bNewDisableState)
{
	bIsDisabled = bNewDisableState;
}

bool UMovieGraphNode::IsDisabled() const
{
	return bIsDisabled;
}

bool UMovieGraphNode::GetNodeValidationErrors(const FName& InBranchName, const UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<FText>& OutValidationErrors) const
{
	// No validation errors by default
	return false;
}

#if WITH_EDITOR
FLinearColor UMovieGraphNode::GetNodeTitleColor() const
{
	return FLinearColor::Black;
}

FSlateIcon UMovieGraphNode::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = FLinearColor::White;
	return FSlateIcon();
}

TArray<TPair<FName, UClass*>> UMovieGraphNode::GetHiddenProperties() const
{
	// By default, no properties are hidden.
	return {};
}
#endif // WITH_EDITOR

void UMovieGraphNode::PostLoad()
{
	Super::PostLoad();

	RegisterDelegates();
}

void UMovieGraphNode::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	// New graphs are typically created by duplicating a template. Make sure the nodes in the template have their delegates registered.
	RegisterDelegates();
}

bool UMovieGraphNode::SetDynamicPropertyValue(const FName PropertyName, const FString& InNewValue)
{
	return DynamicProperties.SetValueSerializedString(PropertyName, InNewValue) == EPropertyBagResult::Success;
}

bool UMovieGraphNode::GetDynamicPropertyValue(const FName PropertyName, FString& OutValue)
{
	const TValueOrError<FString, EPropertyBagResult> Result = DynamicProperties.GetValueSerializedString(PropertyName);
	if (Result.IsValid())
	{
		OutValue = Result.GetValue();
	}

	return Result.IsValid();
}

UMovieGraphMutableValueView* UMovieGraphNode::GetDynamicPropertiesView() const
{
	// Lazily instantiate the view
	if (!DynamicPropertiesView)
	{
		UMovieGraphNode* MutableThis = const_cast<UMovieGraphNode*>(this);
		DynamicPropertiesView = NewObject<UMovieGraphMutableValueView>(MutableThis, TEXT("DynamicPropertiesView"));
		DynamicPropertiesView->ViewProperties(&MutableThis->DynamicProperties);
	}

	return DynamicPropertiesView.Get();
}

const FBoolProperty* UMovieGraphNode::FindOverridePropertyForDynamicProperty(const FName& InPropertyName) const
{
	const FString OverridePropertyName = FString::Printf(TEXT("bOverride_%s"), *InPropertyName.ToString());
	const FPropertyBagPropertyDesc* Desc = DynamicProperties.FindPropertyDescByName(FName(OverridePropertyName));

	if (!Desc)
	{
		return nullptr;
	}

	if (const FBoolProperty* DescBoolProperty = CastField<FBoolProperty>(Desc->CachedProperty))
	{
		return DescBoolProperty;
	}
	
	return nullptr;
}

bool UMovieGraphNode::IsDynamicPropertyOverridden(const FName& InPropertyName) const
{
	if (const FBoolProperty* OverrideProperty = FindOverridePropertyForDynamicProperty(InPropertyName))
	{
		TValueOrError<bool, EPropertyBagResult> Result = DynamicProperties.GetValueBool(OverrideProperty->GetFName());
		return Result.IsValid() && Result.GetValue();
	}

	return false;
}

void UMovieGraphNode::SetDynamicPropertyOverridden(const FName& InPropertyName, const bool bIsOverridden)
{
	if (const FBoolProperty* OverrideProperty = FindOverridePropertyForDynamicProperty(InPropertyName))
	{
		DynamicProperties.SetValueBool(OverrideProperty->GetFName(), bIsOverridden);
	}
}

TArray<FMovieGraphPropertyInfo> UMovieGraphNode::GetOverrideablePropertyInfo() const
{
	TArray<FMovieGraphPropertyInfo> OverrideableProperties;
	
	for (TFieldIterator<FProperty> PropertyIterator(GetClass()); PropertyIterator; ++PropertyIterator)
	{
		// Only allow overriding properties that are shown in the details panel (ie, has EditAnywhere/VisibleAnywhere property flags -- CPF_Edit). However,
		// if the property has VisibleAnywhere (CPF_EditConst) then hide the property from being overrideable. Setting the property as VisibleAnywhere
		// gives details customizations the opportunity to pick up the property for further processing, but because of the complicated nature of these
		// properties, VisibleAnywhere serves as a filtering mechanism to prevent it from being overridden.
		if (!PropertyIterator->HasAnyPropertyFlags(CPF_Edit) || PropertyIterator->HasAnyPropertyFlags(CPF_EditConst))
		{
			continue;
		}
		
		if (UMovieGraphConfig::FindOverridePropertyForRealProperty(GetClass(), *PropertyIterator))
		{
			FMovieGraphPropertyInfo Info;
			Info.Name = PropertyIterator->GetFName();
			Info.bIsDynamicProperty = false;
			Info.ValueType = UE::MovieGraph::Private::GetValueTypeFromProperty(*PropertyIterator, Info.ValueTypeObject);
			
			OverrideableProperties.Add(MoveTemp(Info));
		}
	}

	for (const FPropertyBagPropertyDesc& Desc : GetDynamicPropertyDescriptions())
	{
		if (FindOverridePropertyForDynamicProperty(Desc.Name))
		{
			FMovieGraphPropertyInfo Info;
			Info.Name = Desc.Name;
			Info.bIsDynamicProperty = true;
			Info.ValueType = static_cast<EMovieGraphValueType>(Desc.ValueType);
			Info.ValueTypeObject = Desc.ValueTypeObject.Get();

			OverrideableProperties.Add(MoveTemp(Info));
		}
	}

	// Some properties exist only via GetInputProperties(). Some of them may be non-branch-typed so they should be exposed as overrideable.
	for (const FMovieGraphPinProperties& InputPinProperty : GetInputPinProperties())
	{
		if (InputPinProperty.bIsBranch)
		{
			continue;
		}
		
		FMovieGraphPropertyInfo Info;
		Info.Name = InputPinProperty.Label;
		Info.ValueType = InputPinProperty.Type;
		Info.ValueTypeObject = InputPinProperty.TypeObject;
		Info.bIsPermanentlyExposed = InputPinProperty.bIsBuiltIn;

		OverrideableProperties.Add(MoveTemp(Info));
	}

	return OverrideableProperties;
}

TArray<UMovieGraphPin*> UMovieGraphNode::EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const
{
	TArray<UMovieGraphPin*> PinsToFollow;
	
	if (!ensure(InContext.PinBeingFollowed))
    {
    	return PinsToFollow;
    }
    
	// If the node is disabled, only follow the first connected pin.
    // This is only important for branch connections. For data pins, just continue following the connection.
	if (IsDisabled() && InContext.PinBeingFollowed->Properties.bIsBranch)
	{
		if (UMovieGraphPin* GraphPin = GetFirstConnectedInputPin())
		{
			PinsToFollow.Add(GraphPin);
		}

		return PinsToFollow;
	}

	// If this is a data (non-branch) connection, just return the first connected pin (following any reroute nodes).
	if (!InContext.PinBeingFollowed->Properties.bIsBranch)
	{
		const TObjectPtr<UMovieGraphNode> OwningNode = InContext.PinBeingFollowed->Node;
		
		// Only do this if the pin is an input though. For data output pins, the generic case does not have enough information to determine what
		// the upstream connected data pin is. If this pin is on a reroute node, keep following the connection.
		if (InContext.PinBeingFollowed->IsInputPin() || (OwningNode && OwningNode->IsA<UMovieGraphRerouteNode>()))
		{
			constexpr bool bFollowRerouteConnections = true;
			if (UMovieGraphPin* ConnectedInputPin = InContext.PinBeingFollowed->GetFirstConnectedPin(bFollowRerouteConnections))
			{
				PinsToFollow.Add(ConnectedInputPin);
			}
		}
		
		return PinsToFollow;
	}

	// By default we provide all Input Pins to this node that are the Branch Type.
	// You should override this in downstream nodes that need custom logic, such
	// as branch or switch nodes.
	for (UMovieGraphPin* InputPin : GetInputPins())
	{
		if (InputPin->Properties.bIsBranch)
		{
			PinsToFollow.Add(InputPin);
		}
	}
	
	return PinsToFollow;
}

TArray<FMovieGraphPinProperties> UMovieGraphSettingNode::GetInputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;
	Properties.Add(FMovieGraphPinProperties::MakeBranchProperties());
	return Properties;
}

TArray<FMovieGraphPinProperties> UMovieGraphSettingNode::GetOutputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;
	Properties.Add(FMovieGraphPinProperties::MakeBranchProperties());
	return Properties;
}

void UMovieGraphSettingNode::ResolveTokenContainingProperties(TFunction<void(FString&)>& ResolveFunc, const FMovieGraphTokenResolveContext& InContext)
{
	// The default node has no properties that contain {tokens} that need to be resolved.

	// Dev note: Ideally we could use UPROPERTY metadata to define these properties. However, UPROPERTY metadata isn't available in runtime
	// (non-editor) builds, so it's not a viable solution.
}

void IMovieGraphEvaluationNodeInjector::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	
}