// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphOCIOHelper.h"

#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphDataTypes.h"
#include "MovieRenderPipelineCoreModule.h"

#if WITH_OCIO
#include "ImageCore.h" // For GetImageView()
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOColorTransform.h"
#endif // WITH_OCIO

#if WITH_OCIO
FMovieGraphOCIOHelper::FOpenColorIOPixelPreProcessor::FOpenColorIOPixelPreProcessor(FOpenColorIOWrapperProcessor&& InProcessor)
	: Processor(InProcessor)
{
	
}

void FMovieGraphOCIOHelper::FOpenColorIOPixelPreProcessor::operator()(const FImagePixelData* PixelData) const
{
	check(PixelData);
	Processor.TransformImage(PixelData->GetImageView());
}

bool FMovieGraphOCIOHelper::GenerateOcioPixelPreProcessor(const UE::MovieGraph::FMovieGraphSampleState* InPayload, const UMovieGraphPipeline* InPipeline, const TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FOpenColorIODisplayConfiguration& InConfiguration, const TMap<FString, FString>& InContext, TArray<FPixelPreProcessor>& OutPixelPreProcessors)
{
	TMap<FString, FString> ResolvedOCIOContext;

	const TObjectPtr<UOpenColorIOConfiguration>& ConfigurationAsset = InConfiguration.ColorConfiguration.ConfigurationSource;
	if (IsValid(ConfigurationAsset))
	{
		ResolvedOCIOContext = ConfigurationAsset->Context;
	}

	ResolvedOCIOContext.Append(InContext);

	ResolvedOCIOContext = ResolveOpenColorIOContext(
		ResolvedOCIOContext,
		InPayload->TraversalContext.RenderDataIdentifier,
		InPipeline,
		InEvaluatedConfig,
		InPayload->TraversalContext
	);

	return GenerateOcioPixelPreProcessorWithContext(InPayload, InConfiguration, ResolvedOCIOContext, OutPixelPreProcessors);
}

bool FMovieGraphOCIOHelper::GenerateOcioPixelPreProcessorWithContext(const UE::MovieGraph::FMovieGraphSampleState* InPayload, const FOpenColorIODisplayConfiguration& InConfiguration, const TMap<FString, FString>& InResolvedContext, TArray<FPixelPreProcessor>& OutPixelPreProcessors)
{
	if (!InConfiguration.bIsEnabled || !InPayload->bAllowOCIO)
	{
		return false;
	}
	
	ValidateDisableTonecurve(*InPayload);

	FPixelPreProcessor OCIOPixelPreProcessor = CreateOpenColorIOPixelPreProcessor(
		InConfiguration.ColorConfiguration,
		InResolvedContext
	);
	
	if (OCIOPixelPreProcessor)
	{
		OutPixelPreProcessors.Emplace(MoveTemp(OCIOPixelPreProcessor));
		return true;
	}

	return false;
}
#endif	// WITH_OCIO

TMap<FString, FString> FMovieGraphOCIOHelper::ResolveOpenColorIOContext(const TMap<FString, FString>& InContext, const FMovieGraphRenderDataIdentifier& InRenderId, const UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FMovieGraphTraversalContext& InTraversalContext)
{
	TMap<FString, FString> OutContext;
	OutContext.Reserve(InContext.Num());

	FMovieGraphFilenameResolveParams Params = FMovieGraphFilenameResolveParams::MakeResolveParams(InRenderId, InPipeline, InEvaluatedConfig, InTraversalContext);

	for (const TPair<FString, FString>& Pair : InContext)
	{
		FMovieGraphResolveArgs FormatArgs;
		UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(Pair.Value, Params, FormatArgs);

		FStringFormatNamedArguments NamedArgs;
		for (const TPair<FString, FString>& Argument : FormatArgs.FilenameArguments)
		{
			NamedArgs.Add(Argument.Key, Argument.Value);
		}

		const FString& ResolvedValue = OutContext.Add(Pair.Key, FString::Format(*Pair.Value, NamedArgs));
		UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "OCIO Context Key/Value: %ls / %ls", *Pair.Key, *ResolvedValue);
	}

	return OutContext;
}

#if WITH_OCIO
FPixelPreProcessor FMovieGraphOCIOHelper::CreateOpenColorIOPixelPreProcessor(const FOpenColorIOColorConversionSettings& InConversionSettings, const TMap<FString, FString>& InContext)
{
	const TObjectPtr<UOpenColorIOConfiguration>& ConfigurationSource = InConversionSettings.ConfigurationSource;
	if (IsValid(ConfigurationSource))
	{
		const TObjectPtr<const UOpenColorIOColorTransform> ColorTransform = ConfigurationSource->FindTransform(InConversionSettings);
		if (IsValid(ColorTransform))
		{
			FOpenColorIOWrapperProcessor Processor;
			if (ColorTransform->GetTransformProcessor(Processor, InContext))
			{
				return FOpenColorIOPixelPreProcessor(MoveTemp(Processor));
			}
		}
	}

	UE_LOGF(LogMovieRenderPipeline, Warning, "Invalid configuration source or conversion settings, bypassing OpenColorIO transform.");

	return {};
}
#endif	// WITH_OCIO

void FMovieGraphOCIOHelper::ValidateDisableTonecurve(const UE::MovieGraph::FMovieGraphSampleState& InPayload)
{
	if (InPayload.SceneCaptureSource != SCS_FinalColorHDR)
	{
		UE_CALL_ONCE([]
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "The OCIO transform did not receive scene-referred linear colors, which most standard workflows expect."
					"You may wish to disable the tonecurve on your renderer node(s).");
			});
	}
}