// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphOutputMerger.h"

#include "Async/Async.h"
#include "ImageWriteQueue.h"
#include "ImageWriteTask.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphRenderDataIdentifier.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "Modules/ModuleManager.h"
#include "MoviePipelineQueue.h"

namespace UE::MovieGraph
{
	FMovieGraphOutputMerger::FMovieGraphOutputMerger(UMovieGraphPipeline* InOwningMoviePipeline)
		: WeakMoviePipeline(MakeWeakObjectPtr(InOwningMoviePipeline))
	{
		Debug_ImageWriteQueue = &FModuleManager::Get().LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue").GetWriteQueue();
	}

	void FMovieGraphOutputMerger::AbandonOutstandingWork()
	{
		FScopeLock ScopeLock(&PendingDataMutex);
		FinishedFrames.Empty();
		PendingData.Empty();
	}

	FMovieGraphOutputMergerFrame& FMovieGraphOutputMerger::AllocateNewOutputFrame_GameThread(const int32 InRenderedFrameNumber)
	{
		FScopeLock ScopeLock(&PendingDataMutex);

		// Ensure this frame hasn't already been entered somehow.
		check(!PendingData.Find(InRenderedFrameNumber));

		FMovieGraphOutputMergerFrame& NewFrame = PendingData.Add(InRenderedFrameNumber);
		return NewFrame;
	}

	FMovieGraphOutputMergerFrame& FMovieGraphOutputMerger::GetOutputFrame_GameThread(const int32 InRenderedFrameNumber)
	{
		check(PendingData.Find(InRenderedFrameNumber));
		return *PendingData.Find(InRenderedFrameNumber);
	}
	
	void FMovieGraphOutputMerger::OnSingleSampleDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData)
	{
		// This is to support outputting individual samples (skipping accumulation) for debug reasons,
		// or because you want to post-process them yourself. We just forward this directly on for output to disk.

		AsyncTask(ENamedThreads::GameThread, [LocalData = MoveTemp(InData), LocalWeakPipeline = WeakMoviePipeline, LocalDebugQueue = Debug_ImageWriteQueue]() mutable
		{
			if (!ensureAlwaysMsgf(LocalWeakPipeline.IsValid(), TEXT("A memory lifespan issue has left an output builder alive without an owning Movie Pipeline.")))
			{
				return;
			}
			
			const UE::MovieGraph::FMovieGraphSampleState* Payload = LocalData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
			const FMovieGraphTraversalContext& TraversalContext = Payload->TraversalContext;
			const TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig = TraversalContext.Time.EvaluatedConfig;
			
			// Resolve the file output path
			FString FinalFilePath;
			{
				// TODO: Add Tile X/Y when tiling is available
				FString OutputName;
				if (Payload->Debug_OverrideFilename.IsEmpty())
				{

					OutputName = FString::Printf(TEXT("%s_%s_%s_%s_%s_SS_%d_TS_%d.%d"),
						*TraversalContext.Shot->OuterName,
						*TraversalContext.RenderDataIdentifier.LayerName,
						*TraversalContext.RenderDataIdentifier.RendererName,
						*TraversalContext.RenderDataIdentifier.SubResourceName,
						*TraversalContext.RenderDataIdentifier.CameraName,
						TraversalContext.Time.SpatialSampleIndex,
						TraversalContext.Time.TemporalSampleIndex,
						TraversalContext.Time.OutputFrameNumber);
				}
				else
				{
					// If provided, we assume they've already resolved the formatting for the exact name they want.
					OutputName = Payload->Debug_OverrideFilename;
				}

				const UMovieGraphGlobalOutputSettingNode* OutputSettings =
					EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName);

				const FString OutputDirectory = OutputSettings->OutputDirectory.Path;
				const FString FileNameFormatString = OutputDirectory / OutputName;

				TMap<FString, FString> AdditionalFormatArgs;
				AdditionalFormatArgs.Add(TEXT("ext"), TEXT("exr"));

				FMovieGraphResolveArgs MergedFormatArgs;
				const FMovieGraphFilenameResolveParams ResolveParams = FMovieGraphFilenameResolveParams::MakeResolveParams(
					TraversalContext.RenderDataIdentifier,
					LocalWeakPipeline.Get(),
					EvaluatedConfig.Get(),
					Payload->TraversalContext,
					AdditionalFormatArgs,
					Payload);

				FinalFilePath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FileNameFormatString, ResolveParams, MergedFormatArgs);
			}

			TUniquePtr<FImageWriteTask> TileImageTask = MakeUnique<FImageWriteTask>();
			TileImageTask->Format = EImageFormat::EXR;
			TileImageTask->CompressionQuality = static_cast<int32>(EImageCompressionQuality::Default);
			TileImageTask->Filename = FinalFilePath;
			TileImageTask->PixelData = MoveTemp(LocalData);	// The pixel data is currently owned by the async task; transfer ownership to the image write task

			LocalDebugQueue->Enqueue(MoveTemp(TileImageTask));
		});
	}
	
	void FMovieGraphOutputMerger::OnCompleteRenderPassDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData)
	{	
		// Lock the ActiveData when we're updating what data has been gathered.
		FScopeLock ScopeLock(&PendingDataMutex);
		
		// Fetch our payload from the data. If this check fails then you didn't attach the payload to the image.
		FMovieGraphSampleState* Payload = InData->GetPayload<FMovieGraphSampleState>();
		check(Payload);
		
		const int32 IndexedFrameNumber = Payload->TraversalContext.Time.OutputFrameNumber;

		// See if we can find the frame this data is for. This should always be valid, if it's not
		// valid it means they either forgot to declare they were going to produce it, or this is
		// coming in after the system already thinks it's finished that frame.
		FMovieGraphOutputMergerFrame* OutputFrame = PendingData.Find(IndexedFrameNumber);
		
		// Make sure we expected this frame number
		if (!ensureAlwaysMsgf(OutputFrame, TEXT("Received data for unknown frame. Frame was either already processed or not queued yet!")))
		{
			return;
		}
		
		// Make sure this render pass identifier was expected as well.
		FMovieGraphRenderDataIdentifier NewLayerId = Payload->TraversalContext.RenderDataIdentifier;
		if (!ensureAlwaysMsgf(OutputFrame->ExpectedRenderPasses.Contains(NewLayerId), TEXT("Received data for unexpected render pass: %s"), *LexToString(NewLayerId)))
		{
			return;
		}
		
		// Put the new data inside this output frame.
		OutputFrame->ImageOutputData.FindOrAdd(NewLayerId) = MoveTemp(InData);
		
		// Check to see if this was the last piece of data needed for this frame.
		const int32 TotalPasses = OutputFrame->ExpectedRenderPasses.Num();
		const int32 FinishedPasses = OutputFrame->ImageOutputData.Num();
		
		if (FinishedPasses == TotalPasses)
		{
			// Merge in any metadata produced by the render (there may be multiple passes that need to be merged)
			for (const FMovieGraphPassData& PassData : OutputFrame->ImageOutputData)
			{
				const FMovieGraphSampleState* PassPayload = PassData.Value->GetPayload<FMovieGraphSampleState>();
				check(PassPayload);

				OutputFrame->FileMetadata.Append(PassPayload->AdditionalFileMetadata);
			}
			
			// Sort the output frames. This is only really important for multi-channel formats like EXR, but it lets passes
			// specify which one should be the thumbnail/default rgba channels instead of a first-come-first-serve.
			OutputFrame->ImageOutputData.ValueStableSort([](const TUniquePtr<FImagePixelData>& First, const TUniquePtr<FImagePixelData>& Second) -> bool
				{
					FMovieGraphSampleState* FirstPayload = First->GetPayload<FMovieGraphSampleState>();
					FMovieGraphSampleState* SecondPayload = Second->GetPayload<FMovieGraphSampleState>();

					return FirstPayload->CompositingSortOrder < SecondPayload->CompositingSortOrder;
				}
			);
		}

		// Frames can arrive out-of-order in some cases. We should process them in order though, which is especially important for movies (since
		// those write to one file, vs. many individual files for image sequences, so it's important to provide movie writers the frames in order).
		// Note: PendingData is kept sorted by its key (frame number).
		TArray<int32> PendingFrameNumbers;
		PendingData.GetKeys(PendingFrameNumbers);
		for (const int32 PendingFrameNumber : PendingFrameNumbers)
		{
			const FMovieGraphOutputMergerFrame& OutputMergerFrame = PendingData[PendingFrameNumber];
			const bool bFrameIsComplete = OutputMergerFrame.ExpectedRenderPasses.Num() == OutputMergerFrame.ImageOutputData.Num();
			if (!bFrameIsComplete)
			{
				// This frame hasn't finished yet, so delay pushing it (and any other out-of-order frames) to FinishedFrames until it's finished.
				break;
			}
			
			// Move this frame into our FinishedFrames array so the Game Thread can read it at its leisure
			FMovieGraphOutputMergerFrame FinalFrame;
			ensureMsgf(PendingData.RemoveAndCopyValue(PendingFrameNumber, FinalFrame), TEXT("Could not find frame in pending data, output will be skipped!"));
			
			// TQueue is thread safe so it's okay to just push the data into it.
			FinishedFrames.Enqueue(MoveTemp(FinalFrame));
		}
	}
}		
