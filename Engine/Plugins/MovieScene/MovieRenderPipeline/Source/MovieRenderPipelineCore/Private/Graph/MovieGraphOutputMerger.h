// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphDataTypes.h"
#include "Containers/Queue.h"

class IImageWriteQueue;
class UMovieGraphPipeline;

namespace UE::MovieGraph
{
	/**
	* When MRQ produces a single output frame, it's built out of many individual parts that are spread
	* out over time. Different render passes within a single output frame may take longer to produce
	* due to splitting out work onto the Task Graph, so this class is responsible for handling finished
	* data for each render pass coming in at an arbitrary time, and once all of the pieces are there
	* only then does this release it onto the actual pipeline saying the frame is finished.
	*/
	class FMovieGraphOutputMerger : public UE::MovieGraph::IMovieGraphOutputMerger
	{
	public:
		FMovieGraphOutputMerger(UMovieGraphPipeline* InOwningMoviePipeline);

		// IMovieGraphOutputMerger Interface
		virtual FMovieGraphOutputMergerFrame& AllocateNewOutputFrame_GameThread(const int32 InRenderedFrameNumber) override;
		virtual FMovieGraphOutputMergerFrame& GetOutputFrame_GameThread(const int32 InRenderedFrameNumber) override;
		virtual void OnCompleteRenderPassDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) override;
		virtual void OnSingleSampleDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) override;
		virtual void AbandonOutstandingWork() override;
		virtual int32 GetNumOutstandingFrames() const override { return PendingData.Num(); }
		virtual TQueue<FMovieGraphOutputMergerFrame>& GetFinishedFrames() override { return FinishedFrames; }
		// ~IMovieGraphOutputMerger Interface

	public:
		/** 
		* A thread safe FIFO queue of finished frames that have had all of their render passes complete. 
		* All frames in the queue are processed on the next Game Thread tick and moved onto the output containers.
		*/
		TQueue<UE::MovieGraph::FMovieGraphOutputMergerFrame> FinishedFrames;

	private:
		/** The Movie Pipeline that owns us. */
		TWeakObjectPtr<UMovieGraphPipeline> WeakMoviePipeline;

		/** Data that is expected but not fully available yet. Sorted by frame number. */
		TSortedMap<int32, FMovieGraphOutputMergerFrame> PendingData;

		/** Mutex that protects adding/updating/removing from PendingData */
		FCriticalSection PendingDataMutex;

		/** A debug image sequence writer (EXR-only). Typically used for writing out individual samples, bypassing accumulation and output containers. */
		IImageWriteQueue* Debug_ImageWriteQueue;
	};
}