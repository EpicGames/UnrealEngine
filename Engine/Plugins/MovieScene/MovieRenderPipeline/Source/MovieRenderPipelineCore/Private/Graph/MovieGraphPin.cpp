// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphPin.h"

#include "Algo/Find.h"
#include "EdGraph/EdGraphSchema.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphEdge.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/Nodes/MovieGraphRerouteNode.h"
#include "Graph/Nodes/MovieGraphSubgraphNode.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphPin)

#if WITH_EDITOR
bool UMovieGraphPin::Modify(bool bAlwaysMarkDirty)
{
	SetFlags(RF_Transactional);
	return Super::Modify(bAlwaysMarkDirty);
}
#endif

bool UMovieGraphPin::AddEdgeTo(UMovieGraphPin* InOtherPin)
{
	if (!InOtherPin)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "AddEdgeTo: Invalid InOtherPin");
		return false;
	}

	// Check to make sure the connection doesn't already exist
	for (UMovieGraphEdge* Edge : Edges)
	{
		if (Edge->GetOtherPin(this) == InOtherPin)
		{
			return false;
		}
	}

	// Don't allow connection between two output streams
	const bool bThisPinIsUpstream = IsOutputPin();
	const bool bOtherPinIsUpstream = InOtherPin->IsOutputPin();
	if (!ensure(bThisPinIsUpstream != bOtherPinIsUpstream))
	{
		return false;
	}

	Modify();
	InOtherPin->Modify();

	UMovieGraphEdge* NewEdge = NewObject<UMovieGraphEdge>(this);
	Edges.Add(NewEdge);
	InOtherPin->Edges.Add(NewEdge);

	NewEdge->InputPin = bThisPinIsUpstream ? this : InOtherPin;
	NewEdge->OutputPin = bThisPinIsUpstream ? InOtherPin : this;
	
	// When an edge is made, the pin type may need to be propagated (eg, to reroute nodes that could currently be wildcards)
	PropagatePinType(NewEdge->OutputPin, NewEdge->InputPin);
	
	return true;
}

bool UMovieGraphPin::BreakEdgeTo(UMovieGraphPin* InOtherPin)
{
	if (!InOtherPin)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "BreakEdgeTo: Invalid InOtherPin");
		return false;
	}

	for (UMovieGraphEdge* Edge : Edges)
	{
		if (Edge->GetOtherPin(this) == InOtherPin)
		{
			Modify();
			InOtherPin->Modify();
			ensure(InOtherPin->Edges.Remove(Edge));

			Edges.Remove(Edge);

			// After a disconnection, some pins may need to revert to being wildcard (eg, on reroute nodes)
			MaybeRevertToWildcard(this);
			MaybeRevertToWildcard(InOtherPin);

			return true;
		}
	}

	return false;
}

bool UMovieGraphPin::BreakAllEdges()
{
	bool bChanged = false;
	if (!Edges.IsEmpty())
	{
		Modify();
	}

	for (UMovieGraphEdge* Edge : Edges)
	{
		if (!Edge)
		{
			continue;
		}

		if (UMovieGraphPin* OtherPin = Edge->GetOtherPin(this))
		{
			OtherPin->Modify();
			ensure(OtherPin->Edges.Remove(Edge));
			bChanged = true;

			// After a disconnection, some pins may need to revert to being wildcard (eg, on reroute nodes)
			MaybeRevertToWildcard(this);
			MaybeRevertToWildcard(OtherPin);
		}
	}

	Edges.Reset();
	return bChanged;
}

FPinConnectionResponse UMovieGraphPin::CanCreateConnection_PinConnectionResponse(const UMovieGraphPin* InOtherPin) const
{
	if (!InOtherPin)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, NSLOCTEXT("MoviePipeline", "InvalidPinError", "InOtherPin is invalid!"));
	}
	
	// No Circular Connections
	if (Node == InOtherPin->Node)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, NSLOCTEXT("MoviePipeline", "CircularPinError", "No Circular Connections!"));
	}

	const bool bBothPinsAreBranch = Properties.bIsBranch && InOtherPin->Properties.bIsBranch;
	const bool bPinIsWildcard = Properties.bIsWildcard || InOtherPin->Properties.bIsWildcard;

	const bool bBothPinsAreSameType = bPinIsWildcard ||					// Any connection can be made to a wildcard pin or
		bBothPinsAreBranch ||											// Both are branches or
		(!Properties.bIsBranch && !InOtherPin->Properties.bIsBranch &&	// Neither is branch and
		IsTypeCompatibleWith(InOtherPin) &&								// They have compatible types and
		Properties.TypeObject == InOtherPin->Properties.TypeObject);	// They have the same type object (for enums, structs, objects, classes)

	// Pins need to be the same type
	if (!bBothPinsAreSameType)				
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, NSLOCTEXT("MoviePipeline", "PinTypeMismatchError", "Pin types don't match!"));
	}

	if (!IsPinDirectionCompatibleWith(InOtherPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, NSLOCTEXT("MoviePipeline", "PinDirectionMismatchError", "Directions are not compatible!"));
	}

	// Determine if the connection would violate branch restrictions enforced by the nodes involved in the connection.
	FText BranchRestrictionError;
	if (bBothPinsAreBranch && !IsConnectionToBranchAllowed(InOtherPin, BranchRestrictionError))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, BranchRestrictionError);
	}

	// We don't allow multiple things to be connected to an Input Pin
	const UMovieGraphPin* InputPin = IsInputPin() ? this : InOtherPin;
	if(InputPin->GetAllConnectedPins().Num() > 0)
	{
		const ECanCreateConnectionResponse ReplyBreakInputs = IsInputPin() ? CONNECT_RESPONSE_BREAK_OTHERS_A : CONNECT_RESPONSE_BREAK_OTHERS_B;
		return FPinConnectionResponse(ReplyBreakInputs, NSLOCTEXT("MoviePipeline", "PinInputReplaceExisting","Replace existing input connections"));
	}
	
	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, NSLOCTEXT("MoviePipeline", "PinConnect", "Connect nodes"));
}

bool UMovieGraphPin::CanCreateConnection(const UMovieGraphPin* InOtherPin) const
{
	return CanCreateConnection_PinConnectionResponse(InOtherPin).Response != CONNECT_RESPONSE_DISALLOW;
}

bool UMovieGraphPin::IsConnected() const
{
	for (UMovieGraphEdge* Edge : Edges)
	{
		if (Edge && Edge->IsValid())
		{
			return true;
		}
	}

	return false;
}

bool UMovieGraphPin::IsInputPin() const
{
	check(Node);
	return Node->GetInputPin(Properties.Label) == this;
}

bool UMovieGraphPin::IsOutputPin() const
{
	check(Node);
	return Node->GetOutputPin(Properties.Label) == this;
}

int32 UMovieGraphPin::EdgeCount() const
{
	// Includes invalid edges, if any
	return Edges.Num();
}

bool UMovieGraphPin::AllowsMultipleConnections() const
{
	// Always allow multiple connection on output pin
	return IsOutputPin() || Properties.bAllowMultipleConnections;
}

UMovieGraphPin* UMovieGraphPin::GetFirstConnectedPin(const bool bFollowRerouteConnections) const
{
	if (Edges.IsEmpty())
	{
		return nullptr;
	}

	if (ensureMsgf(Edges[0], TEXT("Null edge found when trying to get connected pin!")))
	{
		return Edges[0]->GetOtherPin(this, bFollowRerouteConnections);
	}

	return nullptr;
}

TArray<UMovieGraphPin*> UMovieGraphPin::GetAllConnectedPins() const
{
	TArray<UMovieGraphPin*> ConnectedPins;

	for (UMovieGraphEdge* Edge : Edges)
	{
		if (!IsValid(Edge))
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "GetAllConnectedPins() - Found an invalid edge. This is typically caused by a node that is from a plugin that is not currently loaded.");
			continue;
		}

		ConnectedPins.Add(Edge->GetOtherPin(this));
	}

	return ConnectedPins;
}

TArray<UMovieGraphNode*> UMovieGraphPin::GetConnectedNodes() const
{
	TArray<TObjectPtr<UMovieGraphNode>> OutNodes;
	for (const TObjectPtr<UMovieGraphEdge>& Edge : Edges)
	{
		if (!Edge)
		{
			continue;
		}

		UMovieGraphPin* OtherPin = Edge->GetOtherPin(this);
		if (!OtherPin)
		{
			continue;
		}

		if (!OtherPin->Node)
		{
			continue;
		}

		OutNodes.Add(OtherPin->Node);
	}

	return OutNodes;
}

bool UMovieGraphPin::IsConnectionToBranchAllowed(const UMovieGraphPin* OtherPin, FText& OutError) const
{
	const UMovieGraphPin* InputPin = IsInputPin() ? this : OtherPin;
	const UMovieGraphPin* OutputPin = !IsInputPin() ? this : OtherPin;
	
	if (!InputPin || !OutputPin)
	{
		OutError = NSLOCTEXT("MovieGraph", "PinDirectionMismatchError", "Directions are not compatible!");
		return false;
	}
	
	const TObjectPtr<UMovieGraphNode> ToNode = InputPin->Node;
	const TObjectPtr<UMovieGraphNode> FromNode = OutputPin->Node;
	check(ToNode && FromNode);
	const UMovieGraphConfig* GraphConfig = ToNode->GetGraph();

	const bool ToNodeIsSubgraph = ToNode->IsA<UMovieGraphSubgraphNode>();
	const bool FromNodeIsSubgraph = FromNode->IsA<UMovieGraphSubgraphNode>();

	// Test High-Level Node Restrictions
	const EMovieGraphBranchRestriction FromNodeRestriction = FromNode->GetBranchRestriction();
	const EMovieGraphBranchRestriction ToNodeRestriction = ToNode->GetBranchRestriction();
	if (FromNodeRestriction != ToNodeRestriction &&						// If BranchRestrictions are not the same
		FromNodeRestriction != EMovieGraphBranchRestriction::Any &&		// And neither Node is an 'Any' Node
		ToNodeRestriction != EMovieGraphBranchRestriction::Any)			// Then do not allow connection
	{
		OutError = NSLOCTEXT("MovieGraph", "HighLevelPerNodeBranchRestrictionError", "Cannot connect a Globals-only Node to a RenderLayer-only Node!");
		return false;
	}

	// Get all upstream/downstream nodes that occur on the connection -- these are the nodes that need to be checked for branch restrictions.
	// FromNode/ToNode themselves also needs to be part of the validation checks.
	//
	// If the FromNode is a subgraph, there's no need to visit upstream nodes. The subgraph node will enforce branch restrictions, since it
	// effectively represents an Inputs node. The same logic applies to the ToNode behaving like an Outputs node.
	TArray<UMovieGraphNode*> NodesToCheck = {FromNode, ToNode};
	if (!FromNodeIsSubgraph)
	{
		GraphConfig->VisitUpstreamNodes(FromNode, UMovieGraphConfig::FVisitNodesCallback::CreateLambda(
			[&NodesToCheck](UMovieGraphNode* VisitedNode, const UMovieGraphPin* VisitedPin)
			{
				if (VisitedNode->IsA<UMovieGraphSubgraphNode>())
				{
					return false;	// Don't visit more upstream nodes
				}
			
				NodesToCheck.Add(VisitedNode);
				return true;
			}));
	}

	if (!ToNodeIsSubgraph)
	{
		GraphConfig->VisitDownstreamNodes(ToNode, UMovieGraphConfig::FVisitNodesCallback::CreateLambda(
			[&NodesToCheck](UMovieGraphNode* VisitedNode, const UMovieGraphPin* VisitedPin)
			{
				if (VisitedNode->IsA<UMovieGraphSubgraphNode>())
				{
					return false;	// Don't visit more downstream nodes
				}
				
				NodesToCheck.Add(VisitedNode);
				return true;
			}));
	}

	const FName InputName = InputPin->Properties.Label;
	const FName OutputName = OutputPin->Properties.Label;
	const bool bInputIsGlobals = (InputName == UMovieGraphNode::GlobalsPinName);
	const bool bOutputIsGlobals = (OutputName == UMovieGraphNode::GlobalsPinName);
	constexpr bool bStopAtSubgraph = true;

	// Determine which branch(es) are connected to this node up/downstream. If the To/From node is a subgraph, skip trying to traverse the graph past
	// the subgraph, because for the purposes of determining connection validity, the subgraph's input/output pin is enough.
	const TArray<FString> DownstreamBranchNames = ToNodeIsSubgraph
		? TArray{InputName.ToString()}
		: GraphConfig->GetDownstreamBranchNames(ToNode, InputPin, bStopAtSubgraph);
	
	const TArray<FString> UpstreamBranchNames = FromNodeIsSubgraph
		? TArray{OutputName.ToString()}
		: GraphConfig->GetUpstreamBranchNames(FromNode, OutputPin, bStopAtSubgraph);
	
	// Consider Globals to be up/downstream if the connection is directly to the Globals branch, to any node already connected to Globals, or to a
	// node that has a Globals-only branch restriction (this last check is important when a Globals-only node is not yet connected to the Globals branch).
	const bool bGlobalsIsDownstream = bInputIsGlobals || DownstreamBranchNames.Contains(UMovieGraphNode::GlobalsPinNameString) || (ToNode->GetBranchRestriction() == EMovieGraphBranchRestriction::Globals);
	const bool bGlobalsIsUpstream = bOutputIsGlobals || UpstreamBranchNames.Contains(UMovieGraphNode::GlobalsPinNameString) || (FromNode->GetBranchRestriction() == EMovieGraphBranchRestriction::Globals);

	const bool bDownstreamBranchExistsAndIsntOnlyGlobals =
		!DownstreamBranchNames.IsEmpty() && ((DownstreamBranchNames.Num() != 1) || (DownstreamBranchNames[0] != UMovieGraphNode::GlobalsPinNameString));
	const bool bUpstreamBranchExistsAndIsntOnlyGlobals =
		!UpstreamBranchNames.IsEmpty() && ((UpstreamBranchNames.Num() != 1) || (UpstreamBranchNames[0] != UMovieGraphNode::GlobalsPinNameString));

	// Subgraph nodes are a special case -- they can be connected to both Globals and render layer branches at the same time
	if (ToNodeIsSubgraph || FromNodeIsSubgraph)
	{
		// Only allow Globals -> Globals connections
		if ((ToNodeIsSubgraph && bInputIsGlobals && bUpstreamBranchExistsAndIsntOnlyGlobals) ||
			(FromNodeIsSubgraph && bOutputIsGlobals && bDownstreamBranchExistsAndIsntOnlyGlobals))
		{
			OutError = NSLOCTEXT("MovieGraph", "SubgraphGlobalsBranchMismatchError", "A subgraph Globals branch can only be connected to another Globals branch or Globals-only nodes.");
			return false;
		}

		// Only allow non-Globals -> non-Globals connections
		if ((ToNodeIsSubgraph && !bInputIsGlobals && bGlobalsIsUpstream) ||
			(FromNodeIsSubgraph && !bOutputIsGlobals && bGlobalsIsDownstream))
		{
			OutError = NSLOCTEXT("MovieGraph", "SubgraphNonGlobalsBranchMismatchError", "A subgraph non-Globals branch cannot be connected to the Globals branch or Globals-only nodes.");
			return false;
		}
	}
	else
	{
		// Globals branches can only be connected to Globals branches
		if ((bGlobalsIsDownstream && bUpstreamBranchExistsAndIsntOnlyGlobals) || (bGlobalsIsUpstream && bDownstreamBranchExistsAndIsntOnlyGlobals))
		{
			OutError = NSLOCTEXT("MovieGraph", "GlobalsBranchMismatchError", "Globals branches and Globals-only nodes can only be connected to other Globals branches and Globals-only nodes.");
			return false;
		}
	}

	// Error out if any of the nodes that are part of the connection cannot be connected to the upstream/downstream branches.
	for (const UMovieGraphNode* NodeToCheck : NodesToCheck)
	{
		if (NodeToCheck->GetBranchRestriction() == EMovieGraphBranchRestriction::Globals)
		{
			// Globals-specific nodes have to be connected such that the only upstream/downstream branches are Globals.
			// If either the upstream/downstream branches are empty (ie, the node isn't connected to Inputs/Outputs yet)
			// then the connection is OK for now -- the branch restriction will be enforced when nodes are connected to
			// Inputs/Outputs.
			if (bDownstreamBranchExistsAndIsntOnlyGlobals || bUpstreamBranchExistsAndIsntOnlyGlobals)
			{
				OutError = FText::Format(
					NSLOCTEXT("MovieGraph", "GlobalsBranchRestrictionError", "The node '{0}' can only be connected to the Globals branch."),
						FText::FromString(NodeToCheck->GetName()));
				return false;
			}
		}

		// Check that render-layer-only nodes aren't connected to Globals.
		if (NodeToCheck->GetBranchRestriction() == EMovieGraphBranchRestriction::RenderLayer)
		{
			if (bGlobalsIsDownstream || bGlobalsIsUpstream)
			{
				OutError = FText::Format(
					NSLOCTEXT("MovieGraph", "RenderLayerBranchRestrictionError", "The node '{0}' can only be connected to a render layer branch."),
						FText::FromString(NodeToCheck->GetName()));
				return false;
			}
		}
	}

	return true;
}

bool UMovieGraphPin::IsPinDirectionCompatibleWith(const UMovieGraphPin* OtherPin) const
{
	return IsInputPin() != OtherPin->IsInputPin();
}

bool UMovieGraphPin::IsTypeCompatibleWith(const UMovieGraphPin* InOtherPin) const
{
	// There's one exception to the pin-types-must-match rule. Float and double are compatible with each other.
	const bool bFirstPinIsFloatOrDouble = (Properties.Type == EMovieGraphValueType::Float) || (Properties.Type == EMovieGraphValueType::Double);
	const bool bSecondPinIsFloatOrDouble = (InOtherPin->Properties.Type == EMovieGraphValueType::Float) || (InOtherPin->Properties.Type == EMovieGraphValueType::Double);

	return (Properties.Type == InOtherPin->Properties.Type) || (bFirstPinIsFloatOrDouble && bSecondPinIsFloatOrDouble);
}

void UMovieGraphPin::PropagatePinType(const UMovieGraphPin* InInputPin, const UMovieGraphPin* InOutputPin)
{
	// If the pin's nodes aren't valid, don't proceed. This shouldn't normally happen, but could theoretically occur if one of the nodes could not
	// be loaded properly (eg, it comes from a plugin that isn't loaded).
	if ((InInputPin && !InInputPin->Node) || (InOutputPin && !InOutputPin->Node))
	{
		return;
	}

	// Find the first downstream pin with a non-wildcard type
	const UMovieGraphPin* DownstreamPinWithType = InInputPin && InInputPin->Properties.bIsWildcard ? nullptr : InInputPin;
	if (!DownstreamPinWithType && InInputPin)
	{
		InInputPin->Node->GetGraph()->VisitDownstreamNodes(InInputPin->Node, UMovieGraphConfig::FVisitNodesCallback::CreateLambda(
			[&DownstreamPinWithType](const UMovieGraphNode* VisitedNode, const UMovieGraphPin* VisitedPin)
			{
				if (VisitedPin && !VisitedPin->Properties.bIsWildcard)
				{
					DownstreamPinWithType = VisitedPin;
					return false;
				}

				return true;
			}));
	}

	// Find the first pin upstream with a non-wildcard type
	const UMovieGraphPin* UpstreamPinWithType = InOutputPin && InOutputPin->Properties.bIsWildcard ? nullptr : InOutputPin;
	if (!UpstreamPinWithType && InOutputPin)
	{
		InOutputPin->Node->GetGraph()->VisitUpstreamNodes(InOutputPin->Node, UMovieGraphConfig::FVisitNodesCallback::CreateLambda(
			[&UpstreamPinWithType](const UMovieGraphNode* VisitedNode, const UMovieGraphPin* VisitedPin)
			{
				if (VisitedPin && !VisitedPin->Properties.bIsWildcard)
				{
					UpstreamPinWithType = VisitedPin;
					return false;
				}

				return true;
			}));
	}

	// Prioritize pins downstream with a type; if one is found, propagate that pin type upstream
	if (DownstreamPinWithType)
	{
		constexpr bool bPropagateUpstream = true;
		DownstreamPinWithType->PropagatePinProperties(DownstreamPinWithType->Properties, bPropagateUpstream);
		return;
	}

	// Otherwise, propagate the type from upstream (if possible)
	if (UpstreamPinWithType)
	{
		constexpr bool bPropagateUpstream = false;
		UpstreamPinWithType->PropagatePinProperties(UpstreamPinWithType->Properties, bPropagateUpstream);
	}
}

void UMovieGraphPin::PropagatePinProperties(const FMovieGraphPinProperties PinProperties, const bool bPropagateUpstream) const
{
	if (bPropagateTypeRecursionGuard)
	{
		return;
	}

	TGuardValue<bool> RecursionGuard(bPropagateTypeRecursionGuard, true);
	
	const UMovieGraphRerouteNode* RerouteNode = Cast<UMovieGraphRerouteNode>(Node);

	// Get the pins to propagate to. If propagating upstream, this will only be one pin. For downstream, there may be multiple.
	TArray<UMovieGraphPin*> NextPins;
	if (bPropagateUpstream)
	{
		NextPins.Add(IsInputPin()
			? GetFirstConnectedPin()
			: (RerouteNode ? RerouteNode->GetPassThroughPin(this) : nullptr));
	}
	else
	{
		// When propagating downstream, there may be multiple connections to propagate to
		if (IsInputPin())
		{
			if (RerouteNode)
			{
				NextPins.Add(RerouteNode->GetPassThroughPin(this));
			}
		}
		else
		{
			NextPins.Append(GetAllConnectedPins());
		}
	}

	// Propagate the type for all connected pins
	for (const UMovieGraphPin* NextPin : NextPins)
	{
		if (NextPin && NextPin->Node && NextPin->Node->IsA<UMovieGraphRerouteNode>())
		{
			UMovieGraphRerouteNode* NextRerouteNode = Cast<UMovieGraphRerouteNode>(NextPin->Node);

			// "PinProperties" isn't supplied directly to SetPinProperties() because there are members within the properties (like the label) that
			// need to remain the same.
			FMovieGraphPinProperties PropertiesToUpdate = NextRerouteNode->GetPinProperties();
			PropertiesToUpdate.Type = PinProperties.Type;
			PropertiesToUpdate.TypeObject = PinProperties.TypeObject;
			PropertiesToUpdate.bIsBranch = PinProperties.bIsBranch;
			PropertiesToUpdate.bIsWildcard = PinProperties.bIsWildcard;
			NextRerouteNode->SetPinProperties(PropertiesToUpdate);
			
			NextRerouteNode->UpdatePins();

			// Continue propagating the type through the connection chain
			NextPin->PropagatePinProperties(PinProperties, bPropagateUpstream);

			// When propagating upstream, there may be a situation where propagation also needs to fork off into a separate downstream propagation.
			// For example, if we're coming from pin C, into pin B, propagation also needs to continue to A.
			//
			//  /``````A
			// B
			//  \......C
			if (bPropagateUpstream)
			{
				TArray<UMovieGraphPin*> DownstreamPins = NextPin->GetAllConnectedPins();

				// Remove this pin from the downstream pins (eg, NextPin is B, remove C)
				DownstreamPins.Remove(const_cast<UMovieGraphPin*>(this));
				
				if (!DownstreamPins.IsEmpty())
				{
					NextPin->PropagatePinProperties(PinProperties, !bPropagateUpstream);
				}
			}
		}
	}
}

void UMovieGraphPin::MaybeRevertToWildcard(const UMovieGraphPin* InPin)
{
	check(InPin);

	if (!InPin->Node)
	{
		return;
	}

	// Traverse the graph up/downstream and determine if there are only reroute nodes left in the connection chain (ie, this is an isolated island of
	// only reroute nodes). If that's the case, then revert all of the reroute nodes to wildcards.
	bool bFoundNonRerouteNodes = !InPin->Node->IsA<UMovieGraphRerouteNode>();

	// Determine if there are any upstream non-reroute nodes
	InPin->Node->GetGraph()->VisitUpstreamNodes(InPin->Node, UMovieGraphConfig::FVisitNodesCallback::CreateLambda(
	[&bFoundNonRerouteNodes](const UMovieGraphNode* VisitedNode, const UMovieGraphPin* VisitedPin)
	{
		if (VisitedNode && !VisitedNode->IsA<UMovieGraphRerouteNode>())
		{
			bFoundNonRerouteNodes = true;
			return false;
		}

		return true;
	}));

	// Determine if there are any downstream non-reroute nodes
	InPin->Node->GetGraph()->VisitDownstreamNodes(InPin->Node, UMovieGraphConfig::FVisitNodesCallback::CreateLambda(
	[&bFoundNonRerouteNodes](const UMovieGraphNode* VisitedNode, const UMovieGraphPin* VisitedPin)
	{
		if (VisitedNode && !VisitedNode->IsA<UMovieGraphRerouteNode>())
		{
			bFoundNonRerouteNodes = true;
			return false;
		}

		return true;
	}));

	// If no non-reroute nodes were found, revert everything in the connection chain to wildcard
	if (!bFoundNonRerouteNodes)
	{
		const FMovieGraphPinProperties WildcardProperties = FMovieGraphPinProperties::MakeWildcardProperties();
		
		bool bPropagateUpstream = true;
		InPin->PropagatePinProperties(WildcardProperties, bPropagateUpstream);
		bPropagateUpstream = false;
		InPin->PropagatePinProperties(WildcardProperties, bPropagateUpstream);
	}
}
