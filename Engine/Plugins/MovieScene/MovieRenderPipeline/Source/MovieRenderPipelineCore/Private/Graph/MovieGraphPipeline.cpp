// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphPipeline.h"

#include "MoviePipelineBasicConfig.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieScene.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphCVarManager.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphLinearTimeStep.h"
#include "Graph/MovieGraphOutputMerger.h"
#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "Graph/MovieGraphUtils.h"
#include "Graph/Nodes/MovieGraphCollectionNode.h"
#include "Graph/Nodes/MovieGraphDebugNode.h"
#include "Graph/Nodes/MovieGraphExecuteScriptNode.h"
#include "Graph/Nodes/MovieGraphFileOutputNode.h"
#include "Graph/Nodes/MovieGraphGlobalGameOverrides.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "Graph/Nodes/MovieGraphModifierNode.h"
#include "Graph/Nodes/MovieGraphRenderLayerNode.h"
#include "Graph/Nodes/MovieGraphSamplingMethodNode.h"
#include "Graph/Nodes/MovieGraphSubgraphNode.h"
#include "Graph/Nodes/MovieGraphWarmUpSettingNode.h"
#include "Graph/Nodes/MovieGraphRenderPassNode.h"

#include "HAL/PlatformFileManager.h"
#include "RenderingThread.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphPipeline)

FString UMovieGraphPipeline::DefaultPreviewWidgetAsset = TEXT("/MovieRenderPipeline/Blueprints/UI_MovieGraphPipelineScreenOverlay.UI_MovieGraphPipelineScreenOverlay_C");

UMovieGraphPipeline::UMovieGraphPipeline()
	: CurrentShotIndex(-1)
	, bIsTransitioningState(false)
	, bIsTearingDownShot(false)
	, PipelineState(EMovieRenderPipelineState::Uninitialized)
	, bDidStartInsightsCapture(false)
{
	OutputMerger = MakeShared<UE::MovieGraph::FMovieGraphOutputMerger>(this);
	CustomEngineTimeStep = CreateDefaultSubobject<UMovieGraphEngineTimeStep>("MovieGraphEngineTimeStep");
}

void UMovieGraphPipeline::Initialize(UMoviePipelineExecutorJob* InJob, const FMovieGraphInitConfig& InitConfig)
{
	UE_LOGF(LogMovieRenderPipeline, Log, "Initializing MovieGraph Render");
	if (!ensureAlwaysMsgf(InJob, TEXT("MovieGraph cannot be initialized with a null job. Aborting.")))
	{
		//Shutdown(true);
		return;
	}

	if (!ensureAlwaysMsgf(InJob->GetGraphPreset() || InJob->IsUsingBasicConfiguration(), TEXT("MovieGraphPipeline cannot be initialized with a null job configuration. Make sure you've assigned a Graph or Basic config to this job. Aborting.")))
	{
		//Shutdown(true);
		return;
	}

	// ToDo: If we cache presets into the job, update the job config here. I don't think we want to do that though
	// due to the recursive nature of linked graphs.
	// But we could (should?) at least copy the primary preset asset into the primary config slot.

	if (!ensureAlwaysMsgf(PipelineState == EMovieRenderPipelineState::Uninitialized, TEXT("Pipeline cannot be reused. Create a new pipeline to execute a job.")))
	{
		Shutdown(true);
		return;
	}

	// Ensure this object has the World as part of its Outer (so that it has context to spawn things)
	if (!ensureAlwaysMsgf(GetWorld(), TEXT("Pipeline does not contain the world as an outer.")))
	{
		Shutdown(true);
		return;
	}

	// Register any additional engine callbacks needed.
	{
		// Called before the Custom Timestep is updated. This gives us a chance to calculate
		// what we want the frame to look like and then cache that information so that the
		// Custom Timestep doesn't have to perform its own logic.
		FCoreDelegates::OnBeginFrame.AddUObject(this, &UMovieGraphPipeline::OnEngineTickBeginFrame);
		// Called at the end of the frame after everything has been ticked and rendered for the frame.
		FCoreDelegates::OnEndFrame.AddUObject(this, &UMovieGraphPipeline::OnEngineTickEndFrame);
	}

	CurrentJob = InJob;
	bWasInitializedWithBasicConfig = CurrentJob->IsUsingBasicConfiguration();
	CurrentShotIndex = 0;
	GraphInitializationTime = FDateTime::UtcNow();
	InitializationTimeOffset = FDateTime::Now() - FDateTime::UtcNow();
	CVarManager = MakeShared<UE::MovieGraph::Private::FMovieGraphCVarManager>();

	DuplicateJobAndConfiguration();
	ExecutePreJobScripts();

	// Create instances of our different classes from the InitConfig
	GraphRendererInstance = NewObject<UMovieGraphRendererBase>(this, InitConfig.RendererClass);
	GraphDataSourceInstance = NewObject<UMovieGraphDataSourceBase>(this, InitConfig.DataSourceClass);
	GraphAudioRendererInstance = NewObject<UMovieGraphAudioRendererBase>(this, InitConfig.AudioRendererClass);

	// Now that we've created our various systems, we will start using them. First thing we do is cache data about
	// the world, job, player viewport, etc, before we make any modifications. These will be restored at the end
	// of the render.
	GraphDataSourceInstance->CacheDataPreJob(InitConfig);

	// Construct the viewport preview UI and bind it to this instance.
	LoadPreviewWidget();

	GraphAudioRendererInstance->SetupAudioRendering();

	// Update our list of shots from our data source, and then
	// create our list of active shots, so we don't try to render
	// a shot the user has deactivated.
	BuildShotListFromDataSource();

	const FMovieGraphTraversalContext Context = GetCurrentTraversalContext(false);
	FString OutError;
	UMovieGraphConfig* GraphPreset = GetCurrentJob()->IsUsingGraphConfiguration() ? GetCurrentJob()->GetGraphPreset() : nullptr;
	UMovieGraphEvaluatedConfig* FlattenedConfig = GraphPreset ? GraphPreset->CreateFlattenedGraph(Context, OutError) : nullptr;

	// Lay groundwork for Unreal Insights; this is not supported with Basic configs.
	if (FlattenedConfig)
	{
		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = true;
		const UMovieGraphDebugSettingNode* DebugSetting =
			FlattenedConfig->GetSettingForBranch<UMovieGraphDebugSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);
		if (DebugSetting && DebugSetting->bCaptureUnrealInsightsTrace)
		{
			StartUnrealInsightsCapture(FlattenedConfig);
		}
	}
	else
	{
		// Although the evaluated graph was created for Unreal Insights, it will lead to a general failure to initialize the pipeline.
		UE_LOGF(LogMovieRenderPipeline, Error, "Unable to initialize Movie Pipeline. The following error(s) were encountered: %ls", *OutError);
	}

	//for (ULevelStreaming* Level : GetWorld()->GetStreamingLevels())
	//{
	//	UClass* StreamingClass = Level->GetClass();
	//
	//	if (StreamingClass == ULevelStreamingDynamic::StaticClass())
	//	{
	//		const FString NonPrefixedLevelName = UWorld::StripPIEPrefixFromPackageName(Level->GetWorldAssetPackageName(), GetWorld()->StreamingLevelsPrefix);
	//		UE_LOGF(LogMovieRenderPipeline, Warning, "Sub-level '%ls' is set to blueprint streaming and will not be visible during a render unless a Sequencer Visibility Track controls its visibility or you have written other code to handle loading it.",
	//			*NonPrefixedLevelName);
	//	}
	//}

	// ToDo: Print some information, Job Name, Level Sequence, etc.
	UE_LOGF(LogMovieRenderPipeline, Log, "Finished initializing MovieGraph Render");

	const bool bNoShotsToRender = ActiveShotList.Num() == 0;
	if (bNoShotsToRender)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "No shots detected to render. Either all outside playback range, or disabled via shot mask. Shutting down.");
	}

	// If shutting down immediately, we'll transition directly to finish as there is no work to do.
	// If the flattened config isn't valid at this point, evaluation failed, and a shutdown is required.
	if (bNoShotsToRender || !FlattenedConfig)
	{
		// We have to transition twice as Uninitialized -> n state is a no-op, so the second tick will take us to Finished which shuts down.
		TransitionToState(EMovieRenderPipelineState::Export);
		TransitionToState(EMovieRenderPipelineState::Finished);
	}
	else
	{
		TransitionToState(EMovieRenderPipelineState::ProducingFrames);
	}
}

void UMovieGraphPipeline::DuplicateJobAndConfiguration()
{
	// Scripting is likely to want to modify both the job (to set variable assignments) and 
	// the configuration itself (to add nodes, or override an output directory, etc. If scripts
	// directly modified the job/configuration it would lead to a lot of unintentional mutation
	// of assets and queues, so we instead choose to duplicate the job and configurations for
	// the duration of a render, and all of the Graph Pipeline code should look at the duplicates.
	FObjectDuplicationParameters JobDuplicationParms = FObjectDuplicationParameters(CurrentJob, this);
	JobDuplicationParms.DestName = FName(FString::Format(TEXT("{0}_Duplicate"), {CurrentJob->GetFName().ToString()}));
	JobDuplicationParms.FlagMask = RF_AllFlags & ~RF_Transactional;
	JobDuplicationParms.ApplyFlags = RF_Transient;
	CurrentJobDuplicate = Cast<UMoviePipelineExecutorJob>(StaticDuplicateObjectEx(JobDuplicationParms));

	// Convert Basic config to Graph on the transient duplicate (before the existing duplication logic processes it)
	if (CurrentJobDuplicate->IsUsingBasicConfiguration())
	{
		// Generate job-level graph from the basic config
		UMovieGraphConfig* GeneratedGraph = UMoviePipelineBasicConfig::GenerateGraph(
			CurrentJobDuplicate->GetBasicConfig(), CurrentJobDuplicate);

		if (!GeneratedGraph)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "DuplicateJobAndConfiguration: Failed to generate graph from Basic config. The job will have no graph configuration.");
		}

		// Pre-populate the duplication map so DuplicateConfigRecursive skips these already-transient graphs
		if (GeneratedGraph)
		{
			DuplicatedGraphs.Add(GeneratedGraph, GeneratedGraph);
		}

		// Generate per-shot graphs (each shot gets a merged config of job + shot overrides)
		for (UMoviePipelineExecutorShot* Shot : CurrentJobDuplicate->ShotInfo)
		{
			if (const UMoviePipelineBasicConfig* ShotOverrides = Shot->GetBasicShotConfig())
			{
				UMoviePipelineBasicConfig* MergedConfig = NewObject<UMoviePipelineBasicConfig>(Shot);
				CurrentJobDuplicate->GetBasicConfig()->MergeOverrides(ShotOverrides, *MergedConfig);
				UMovieGraphConfig* ShotGraph = UMoviePipelineBasicConfig::GenerateGraph(MergedConfig, Shot);
				Shot->SetGraphPreset(ShotGraph, false);

				if (ShotGraph)
				{
					DuplicatedGraphs.Add(ShotGraph, ShotGraph);
				}
			}
		}

		// Set the job graph after shot graphs are set
		CurrentJobDuplicate->SetGraphPreset(GeneratedGraph, false);
	}

	// The duplicate job is a mix of duplicated objects and non-duplicated objects. Objects that
	// don't have the job as their outer will not have been duplicate (which is good for the World/Sequence),
	// but this also means that the graph configurations were not duplicated (as they are assets), so we need
	// to manually duplicate them and update the pointers in the duplicated job.
	// Note: We use CurrentJobDuplicate here (not CurrentJob) so that Basic-config-generated graphs are also processed.
	UMovieGraphConfig* DuplicatePrimaryConfig = MoviePipeline::DuplicateConfigRecursive(CurrentJobDuplicate->GetGraphPreset(), DuplicatedGraphs);
	UpdateVariableAssignmentsHelper(CurrentJobDuplicate.Get(), DuplicatedGraphs);

	// Set the graph preset WITHOUT updating assignments. We're doing all updates manually here. SetGraphPreset() will attempt to update the
	// assignments in the shots as well, but those have not yet been updated to use the duplicated graphs. Therefore, skip updating assignments
	// completely to avoid the shot assignments from being wiped out.
	constexpr bool bUpdateVariableAssignments = false;
	CurrentJobDuplicate->SetGraphPreset(DuplicatePrimaryConfig, bUpdateVariableAssignments);

	for (int32 Index = 0; Index < CurrentJobDuplicate->ShotInfo.Num(); Index++)
	{
		// Now for each shot we need to duplicate its config (if any)
		// Note: We use CurrentJobDuplicate here (not CurrentJob) so that Basic-config-generated graphs are also processed.
		if (UMovieGraphConfig* ShotGraphPreset = CurrentJobDuplicate->ShotInfo[Index]->GetGraphPreset())
		{
			UMovieGraphConfig* DuplicateShotConfig = MoviePipeline::DuplicateConfigRecursive(ShotGraphPreset, DuplicatedGraphs);
			CurrentJobDuplicate->ShotInfo[Index]->SetGraphPreset(DuplicateShotConfig, bUpdateVariableAssignments);
		}

		// Always update the variable assignments, regardless of whether the shot has a graph assigned to it (it may be overriding the primary graph's
		// variable assignments)
		UpdateVariableAssignmentsHelper(CurrentJobDuplicate->ShotInfo[Index].Get(), DuplicatedGraphs);
	}

	// We only look in the primary configuration for the job for script nodes (and not shot specific overrides). If we looked
	// in the shot-specific overrides, we would end up creating instances of the scripts for those shots. This can become really
	// confusing if a shot specifies an Execute Script node with the same script as the Primary Graph, now you'll have two separate
	// instances of your script, one which receives 4 callbacks (pre/post job, pre/post shot) and one that only receives pre/post shot.
	FMovieGraphTraversalContext Context;
	Context.Job = CurrentJobDuplicate;
	Context.RootGraph = DuplicatePrimaryConfig;
	FString OutError;

	if (UMovieGraphEvaluatedConfig* FlattenedConfig = DuplicatePrimaryConfig->CreateFlattenedGraph(Context, OutError))
	{
		constexpr bool bIncludeCDOs = false;
		TArray<UMovieGraphExecuteScriptNode*> ScriptNodes = FlattenedConfig->GetSettingsForBranch<UMovieGraphExecuteScriptNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs);

		// We now have the duplicated job, shots, and configs and we've fixed up the pointers, so we can now pass the duplicates
		// to the user-defined callbacks, allowing them to modify most things without worrying about accidental mutation. If they
		// choose to modify shared state (ie: the level sequence) then there's not much we can do.
		for (const UMovieGraphExecuteScriptNode* Node : ScriptNodes)
		{
			// We instantiate an instance of each script and store it, and we'll call all subsequent callbacks on these (ignoring the nodes)
			// so that we ensure that only one copy of the scripts exist and that they can store state during a render.
			UMovieGraphScriptBase* ScriptInstance = Node->AllocateScriptInstance();
		
			// It's valid for a node to return a nullptr script instance (invalid class or no class set).
			if (ScriptInstance)
			{
				CurrentScriptInstances.Add(ScriptInstance);
			}
		}
	}
}

template <typename JobType>
void UMovieGraphPipeline::UpdateVariableAssignmentsHelper(JobType* InTargetJob, TMap<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>>& InOriginalToDuplicateGraphMap)
{
	// Remaps the provided variable assignments to point to the the duplicated graphs.
	auto UpdateVariableAssignments = [&InOriginalToDuplicateGraphMap](TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& InVariableAssignments)
	{
		for (const TObjectPtr<UMovieJobVariableAssignmentContainer>& VariableAssignment : InVariableAssignments)
		{
			for (const TPair<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>>& GraphMapping : InOriginalToDuplicateGraphMap)
			{
				if (VariableAssignment->GetGraphConfig().LoadSynchronous() == GraphMapping.Key.Get())
				{
					VariableAssignment->SetGraphConfig(MakeSoftObjectPtr(GraphMapping.Value.Get()));
					break;
				}
			}
		}
	};

	// Update the variable assignments according to the original-to-duplicate map. Applies to primary jobs and shots.
	constexpr bool bUpdateAssignments = false;
	UpdateVariableAssignments(InTargetJob->GetGraphVariableAssignments(bUpdateAssignments));

	// Update the primary-level graph (and subgraph) assignments on shots as well.
	if constexpr (std::is_same_v<JobType, UMoviePipelineExecutorShot>)
	{
		UpdateVariableAssignments(InTargetJob->GetPrimaryGraphVariableAssignments(bUpdateAssignments));
	}
}

void UMovieGraphPipeline::LoadPreviewWidget()
{
	// ToDo: Allow overriding this widget so that users can style it to match their project.
	if (PreviewWidgetClassToUse.Get() == nullptr)
	{
		PreviewWidgetClassToUse = LoadClass<UMovieGraphRenderPreviewWidget>(nullptr, *DefaultPreviewWidgetAsset, nullptr, LOAD_None, nullptr);
	}

	if (PreviewWidgetClassToUse.Get() != nullptr)
	{
		PreviewWidget = CreateWidget<UMovieGraphRenderPreviewWidget>(GetWorld(), PreviewWidgetClassToUse.Get());
		if (PreviewWidget)
		{
			PreviewWidget->OnInitializedForPipeline(this);
			PreviewWidget->AddToViewport();
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to create Preview Screen UMG Widget. No in-game overlay available.");
		}
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to find Preview Screen UMG Widget class. No in-game overlay available.");
	}
}

void UMovieGraphPipeline::SetPreviewWidgetVisibleImpl(bool bInIsVisible)
{
	if (PreviewWidget)
	{
		PreviewWidget->SetVisibility(bInIsVisible ? ESlateVisibility::Visible : ESlateVisibility::Collapsed);
	}
}

TArray<FMovieGraphRenderOutputData>& UMovieGraphPipeline::GetGeneratedOutputData()
{
	return GeneratedOutputData;
}

void UMovieGraphPipeline::CreateLayersInRenderLayerSubsystem(const UMovieGraphEvaluatedConfig* EvaluatedConfig) const
{
	UMovieGraphRenderLayerSubsystem* LayerSubsystem = GetWorld()->GetSubsystem<UMovieGraphRenderLayerSubsystem>();
	if (!LayerSubsystem)
	{
		return;
	}
	
	LayerSubsystem->Reset();

	// One render layer is generated per branch
	for (const FName& BranchName : EvaluatedConfig->GetBranchNames())
	{
		// Don't add the branch unless it has an active Render Layer node
		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = true;
		const UMovieGraphRenderLayerNode* RenderLayerNode =
			EvaluatedConfig->GetSettingForBranch<UMovieGraphRenderLayerNode>(BranchName, bIncludeCDOs, bExactMatch);
		
		if (RenderLayerNode && !RenderLayerNode->IsDisabled())
		{
			UMovieGraphRenderLayer* RenderLayer = NewObject<UMovieGraphRenderLayer>();
			RenderLayer->SetRenderLayerName(BranchName);
			LayerSubsystem->AddRenderLayer(RenderLayer);
		}
	}
}

void UMovieGraphPipeline::UpdateLayerContentsInRenderLayerSubsystem(const UMovieGraphEvaluatedConfig* EvaluatedConfig) const
{
	UMovieGraphRenderLayerSubsystem* LayerSubsystem = GetWorld()->GetSubsystem<UMovieGraphRenderLayerSubsystem>();
	if (!LayerSubsystem)
	{
		return;
	}

	for (UMovieGraphRenderLayer* RenderLayer : LayerSubsystem->GetRenderLayers())
	{
		const FName& LayerName = RenderLayer->GetRenderLayerName();
		
		// Gather all collection and modifier nodes from the evaluated graph
		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = true;
		const TArray<UMovieGraphCollectionNode*> CollectionNodes =
			EvaluatedConfig->GetSettingsForBranch<UMovieGraphCollectionNode>(LayerName, bIncludeCDOs, bExactMatch);
		TArray<IMovieGraphModifierNodeInterface*> ModifierNodes =
			EvaluatedConfig->GetSettingsImplementing<IMovieGraphModifierNodeInterface>(UMovieGraphModifierNodeInterface::StaticClass(), LayerName);

		// Graph evaluation discovers nodes working from the Outputs node, moving towards the Inputs node. Therefore modifiers (even if they are
		// nodes which override an existing modifier) that are furthest downstream occur first in the array of modifiers returned by the evaluated
		// graph. For example:
		//
		// A (definition) -> B (definition) -> C (definition) -> B (override) -> A (override) -> C (override)
		//
		// ... would be returned from the graph in the order of C, A, B
		//
		// However, users expect the *evaluation* order of the modifiers to be B, A, C, where the nodes furthest downstream execute last.
		// Therefore, the modifier node array returned by the evaluated graph needs to be reversed.
		Algo::Reverse(ModifierNodes);
		
		for (const IMovieGraphModifierNodeInterface* ModifierNode : ModifierNodes)
		{
			// For modifiers that don't use collections, just add all of the modifiers unconditionally.
			if (!ModifierNode->SupportsCollections())
			{
				for (UMovieGraphModifierBase* Modifier : ModifierNode->GetAllModifiers())
				{
					RenderLayer->AddLayerModifier(Modifier);
				}
				
				continue;
			}
			
			// For each modifier instance, find the collection(s) that it is modifying
			for (UMovieGraphModifierBase* ModifierInstance : ModifierNode->GetAllModifiers())
			{
				// All of these modifiers should support collections (due to the SupportsCollections() call), but double-check anyway
				UMovieGraphCollectionModifier* CollectionModifierInstance = Cast<UMovieGraphCollectionModifier>(ModifierInstance);
				if (!ensureMsgf(CollectionModifierInstance, TEXT("Found a modifier node which declares it supports collections, but provided modifiers that don't use collections.")))
				{
					continue;
				}
				
				CollectionModifierInstance->SetCollections({});
				TArray<UMovieGraphCollection*> ModifierCollections;

				for (const FName& ModifiedCollectionName : ModifierNode->GetAllCollections())
				{
					bool bFoundModifiedCollection = false;
					
					for (const UMovieGraphCollectionNode* CollectionNode : CollectionNodes)
					{
						if (!CollectionNode || !CollectionNode->Collection)
						{
							continue;
						}

						if (CollectionNode->Collection->GetCollectionName() == ModifiedCollectionName)
						{
							bFoundModifiedCollection = true;

							// Collections can be disabled within a modifier; only include it if the collection is enabled
							if (ModifierNode->IsCollectionEnabled(ModifiedCollectionName))
							{
								ModifierCollections.Add(CollectionNode->Collection);
							}
							
							break;
						}
					}

					if (!bFoundModifiedCollection)
					{
						UE_LOGF(LogMovieRenderPipeline, Warning, "The modifier '%ls' specified a collection '%ls', but the collection couldn't be found.",
							*CollectionModifierInstance->GetName(), *ModifiedCollectionName.ToString());
					}
				}

				// If the modifier had valid collection(s) added to it, add the modifier to the render layer. The modifier
				// won't have any effect without collection(s) to act on.
				if (!ModifierCollections.IsEmpty())
				{
					CollectionModifierInstance->SetCollections(ModifierCollections);

					RenderLayer->AddLayerModifier(CollectionModifierInstance);
				}
			}
		}
	}
}

void UMovieGraphPipeline::BuildShotListFromDataSource()
{
	// Synchronize our shot list with our target data source. New shots will be added and outdated shots removed.
	// Shots that are already in the list will be updated but their enable flag will be respected. 
	GraphDataSourceInstance->UpdateShotList();

	GraphTimeStepInstances.Reset();
	GraphTimeStepInstances.Reserve(GetCurrentJob()->ShotInfo.Num());

	for (int32 ShotIndex = 0; ShotIndex < GetCurrentJob()->ShotInfo.Num(); ShotIndex++)
	{
		const TObjectPtr<UMoviePipelineExecutorShot>& Shot = GetCurrentJob()->ShotInfo[ShotIndex];

		// Once the shot list is built, we now need to do some work to the calculated data. Handle Frames 
		// should expand the range provided by the shot list, and we need to do this to all shots in advance.
		// We do it in advance because to get the total number of frames we're going to render, we add up the
		// range sizes for each shot, so handle frames need to be included in that range. Each shot can have
		// different settings, so we build an evaluation context for each shot, flatten the graph and then
		// read the handle frames config values.
		// ToDo: This is being run on disabled shots as well, is that intended? The old system did this as well.

		FMovieGraphTraversalContext CurrentContext;
		CurrentContext.ShotIndex = ShotIndex;
		CurrentContext.ShotCount = GetActiveShotList().Num();
		CurrentContext.Job = GetCurrentJob();
		CurrentContext.Shot = Shot;
		CurrentContext.RootGraph = GetRootGraphForShot(Shot);

		// Frame Rate, Handle Frames, Warm-up Frames are all global settings so we provide an empty time context.
		FMovieGraphTimeStepData TimeContext;
		CurrentContext.Time = TimeContext;

		FString OutError;
		TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig = CurrentContext.RootGraph->CreateFlattenedGraph(CurrentContext, OutError);

		// Shut down if there was an error when generating the evaluated graph
		if (!OutError.IsEmpty())
		{
			constexpr bool bIsError = true;
			Shutdown(bIsError);
			return;
		}
		
		// Create the time step instance for this shot. The time step method cannot vary per branch or per frame, so it
		// is fetched from the Globals branch. This is the earliest point in the pipeline where the evaluated graph is
		// available, hence why the instances are generated all at once. Doing it at this point also prevents a circular
		// dependency between the graph and the time step node. Generally the time step class generates the evaluated
		// graph, but the evaluated graph also specifies the time step class to use. Therefore, we need to determine the
		// time step class to use by evaluating the graph outside of the time step class.
		UMovieGraphSamplingMethodNode* SamplingMethodNode = EvaluatedConfig->GetSettingForBranch<UMovieGraphSamplingMethodNode>(UMovieGraphSettingNode::GlobalsPinName);
		if (UClass* SamplingMethodClass = SamplingMethodNode->SamplingMethodClass.TryLoadClass<UMovieGraphTimeStepBase>())
		{
			GraphTimeStepInstances.Add(NewObject<UMovieGraphTimeStepBase>(this, SamplingMethodClass));
		}
		else
		{
			GraphTimeStepInstances.Add(NewObject<UMovieGraphLinearTimeStep>(this));
			UE_LOGF(LogMovieRenderPipeline, Warning, "The shot '%ls' specified a time step of type '%ls', but it could not be loaded. Defaulting to linear.",
				*Shot->OuterName, *SamplingMethodNode->SamplingMethodClass.GetAssetName());
		}
		
		UMovieGraphGlobalOutputSettingNode* OutputNode = EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphSettingNode::GlobalsPinName);

		const FFrameRate SourceFrameRate = GetDataSourceInstance()->GetDisplayRate();
		const FFrameRate FinalFrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputNode, SourceFrameRate);
		const FFrameRate TickResolution = GetDataSourceInstance()->GetTickResolution();

		UMovieGraphWarmUpSettingNode* WarmUpNode = EvaluatedConfig->GetSettingForBranch<UMovieGraphWarmUpSettingNode>(UMovieGraphSettingNode::GlobalsPinName);

		Shot->ShotInfo.NumTemporalSamples = FMath::Max(SamplingMethodNode->TemporalSampleCount, 1);
		Shot->ShotInfo.NumSpatialSamples = 1;
		Shot->ShotInfo.NumTiles = FIntPoint(1,1);
		Shot->ShotInfo.CachedFrameRate = FinalFrameRate;
		Shot->ShotInfo.CachedTickResolution = Shot->ShotInfo.CachedShotTickResolution = TickResolution;
		if (Shot->ShotInfo.SubSectionHierarchy.IsValid() && Shot->ShotInfo.SubSectionHierarchy->MovieScene.IsValid())
		{
			Shot->ShotInfo.CachedShotTickResolution = Shot->ShotInfo.SubSectionHierarchy->MovieScene->GetTickResolution();
		}
		
		// Query the max frame count.
		int32 MaxCoolingDownFrameCount = 0;
		for (const FName& BranchName : EvaluatedConfig->GetBranchNames())
		{
			const bool bIncludeCDOs = false;
			const bool bExactMatch = false;
			TArray<UMovieGraphRenderPassNode*> Renderers = EvaluatedConfig->GetSettingsForBranch<UMovieGraphRenderPassNode>(BranchName, bIncludeCDOs, bExactMatch);
			for (const UMovieGraphRenderPassNode* Render : Renderers)
			{
				MaxCoolingDownFrameCount = FMath::Max(Render->GetCoolingDownFrameCount(), MaxCoolingDownFrameCount);
			}
		}

		// When using cooldown, we need at least that many warm-up frames even if they have otherwise chosen not to do warm-ups.
		Shot->ShotInfo.NumEngineCoolDownFramesRemaining = MaxCoolingDownFrameCount;
		Shot->ShotInfo.NumEngineWarmUpFramesRemaining = FMath::Max(WarmUpNode->NumWarmUpFrames, MaxCoolingDownFrameCount);
		Shot->ShotInfo.bEmulateFirstFrameMotionBlur = WarmUpNode->bEmulateMotionBlur;

		const bool bPrePass = true;
		const bool bExpandForTemporalSubSample = GraphTimeStepInstances.Last()->IsExpansionForTSRequired(EvaluatedConfig);
		
		// We need to wait until the NumEngineWarmUpFramesRemaining has been set by either actual warm up frames, or by cool-down frames before we expand.
		ExpandShot(Shot, OutputNode->HandleFrameCount, bExpandForTemporalSubSample, bPrePass, FinalFrameRate, TickResolution, Shot->ShotInfo.NumEngineWarmUpFramesRemaining);

		Shot->ShotInfo.CurrentTimeInRoot = Shot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue();
		Shot->ShotInfo.InitialTimeInRoot = Shot->ShotInfo.CurrentTimeInRoot;
		Shot->ShotInfo.InitialTimeInShot = Shot->ShotInfo.CurrentTimeInRoot * Shot->ShotInfo.OuterToInnerTransform;
		Shot->ShotInfo.CalculateWorkMetrics();
		Shot->ShotInfo.VersionNumber = ResolveVersionForShot(Shot, EvaluatedConfig);
	}

	// The active shot-list is a subset of the whole shot-list; The ShotInfo contains information about every range it detected to render
	// but if the user has turned the shot off in the UI then we don't want to render it.
	ActiveShotList.Empty();
	for (UMoviePipelineExecutorShot* Shot : GetCurrentJob()->ShotInfo)
	{
		if (Shot->ShouldRender())
		{
			ActiveShotList.Add(Shot);
		}
	}
}

void UMovieGraphPipeline::OnEngineTickBeginFrame()
{
	LLM_SCOPE_BYNAME(TEXT("MovieGraphBeginFrame"));


	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "MovieGraph OnEngineTickBeginFrame");
	switch (PipelineState)
	{
	case EMovieRenderPipelineState::Uninitialized:
		// We shouldn't register this delegate until we're initialized.
		check(false);
		break;
	case EMovieRenderPipelineState::ProducingFrames:
		TickProducingFrames();
		break;
	case EMovieRenderPipelineState::Finalize:
		// Don't flush the finalize to keep the UI responsive.
		TickFinalizeOutputContainers(false);
		break;
	case EMovieRenderPipelineState::Export:
		// Don't flush the export to keep the UI responsive.
		TickPostFinalizeExport(false);
		break;
	}
}

void UMovieGraphPipeline::TickProducingFrames()
{
	// Move any output frames that have been finished from the Output Merger
	// into the actual outputs. This will generate new futures (for actual 
	// disk writes) which we keep track of below.
	ProcessOutstandingFinishedFrames();
	
	// Process any files that have finished writing to disk and push them into our list of 
	// files made by this shot.
	ProcessOutstandingFutures();

	// The callback for this function does not get registered until Initialization has been called, which sets
	// the state to Render. If it's not, we have a initialization order/flow issue!
	//check(PipelineState == EMovieRenderPipelineState::ProducingFrames);

	// We should not be calling this once we have completed all the shots.
	//check(CurrentShotIndex >= 0 && CurrentShotIndex < ActiveShotList.Num());
	//
	//ProcessOutstandingFutures();
	//
	
	if (bShutdownRequested)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "MovieGraph TickProducingFrames: Async Shutdown Requested, abandoning remaining work and moving to Finalize.");
		TransitionToState(EMovieRenderPipelineState::Finalize);
		return;
	}

	// When start up we want to override the engine's Custom Timestep with our own.
	// This gives us the ability to completely control the engine tick/delta time before the frame
	// is started so that we don't have to always be thinking of delta times one frame ahead. We need
	// to do this only once we're ready to set the timestep though, as Initialize can be called as
	// a result of a OnBeginFrame, meaning that Initialize is called on the frame before TickProducingFrames
	// so there would be one frame where it used the custom timestep (after initialize) before TPF was called.
	if (GEngine->GetCustomTimeStep() != CustomEngineTimeStep)
	{
		PrevCustomEngineTimeStep = GEngine->GetCustomTimeStep();
		GEngine->SetCustomTimeStep(CustomEngineTimeStep);
	}

	// If we don't have a graph time step instance assigned at all and TickProducingFrames is being called,
	// then we set the first one as the pending one (because we haven't initialized any shots yet), that way it gets
	// Initialize called on it, and we avoid anyone relying on a not-initialized GraphTimeStepInstance if 
	// they tried accessing it before the first shot was started.
	if (!GraphTimeStepInstance)
	{
		PendingTimeStepInstance = GraphTimeStepInstances[0];
	}

	// We can switch between time-step instances between shots,
	// but the SetupShot/TeardownShot are called by the current instance,
	// so we defer the actual pointer change until the next tick.
	if (PendingTimeStepInstance)
	{
		if(GraphTimeStepInstance)
		{
			GraphTimeStepInstance->Shutdown();
		}
		GraphTimeStepInstance = PendingTimeStepInstance;
		GraphTimeStepInstance->Initialize();
		
		PendingTimeStepInstance = nullptr;
	}



	GetTimeStepInstance()->TickProducingFrames();
}

void UMovieGraphPipeline::TickFinalizeOutputContainers(const bool bInForceFinish)
{
	// Tick all containers until they all report that they have finalized.
	bool bAllContainsFinishedProcessing;

	while(true)
	{
		bAllContainsFinishedProcessing = true;

		for (const TObjectPtr<UMovieGraphFileOutputNode>& Node : GetOutputNodesUsed())
		{
			bAllContainsFinishedProcessing &= Node->IsFinishedWritingToDisk();
		}
	
		// If we aren't forcing a finish, early out after one loop to keep
		// the editor/ui responsive.
		if (!bInForceFinish || bAllContainsFinishedProcessing)
		{
			break;
		}
	
		// If they've reached here, they're forcing them to finish so we'll sleep for a touch to give
		// everyone a chance to actually do work before asking them if they're done.
		FPlatformProcess::Sleep(0.1f);
	}

	// If an output container is still working, we'll early out to keep the UI responsive.
	// If they've forced a finish this will have to be true before we can reach this block.
	if (!bAllContainsFinishedProcessing)
	{
		return;
	}

	// Notify all output nodes that they should finalize
	for (const TObjectPtr<UMovieGraphFileOutputNode>& Node : GetOutputNodesUsed())
	{
		Node->OnAllFramesFinalized(this, PostRenderEvaluatedGraph);
	}

	TransitionToState(EMovieRenderPipelineState::Export);
}

void UMovieGraphPipeline::TickPostFinalizeExport(const bool bInForceFinish)
{
	// This step assumes you have produced data and filled the data structures.
	check(PipelineState == EMovieRenderPipelineState::Export);
	
	UE_LOGF(LogMovieRenderPipeline, Verbose, "[%lld] PostFinalize Export (Start).", GFrameCounter);

	// Loop through any extensions (such as XML export) and let them export using all of the
	// data that was generated during this run such as containers, output names and lengths.
	// Tick all containers until they all report that they have finalized.
	bool bAllContainsFinishedProcessing = true;

	do
	{
		bAllContainsFinishedProcessing = true;

		// Ask the nodes if they're all done processing.
		const TArray<IMovieGraphPostRenderNode*> PostRenderNodes =
			PostRenderEvaluatedGraph->GetSettingsImplementing<IMovieGraphPostRenderNode>(UMovieGraphPostRenderNode::StaticClass(), UMovieGraphNode::GlobalsPinName);
		for (IMovieGraphPostRenderNode* PostRenderNode : PostRenderNodes)
		{
			bAllContainsFinishedProcessing &= PostRenderNode->HasFinishedExporting();
		}
	
		// If we aren't forcing a finish, early out after one loop to keep the editor/ui responsive.
		if (!bInForceFinish || bAllContainsFinishedProcessing)
		{
			break;
		}
	
		// If they've reached here, they're forcing them to finish so we'll sleep for a touch to give
		// everyone a chance to actually do work before asking them if they're done.
		FPlatformProcess::Sleep(1.f);
	} while (true);

	UE_LOGF(LogMovieRenderPipeline, Verbose, "[%lld] PostFinalize Export (End).", GFrameCounter);

	// If an output container is still working, we'll early out to keep the UI responsive.
	// If they've forced a finish this will have to be true before we can reach this block.
	if (!bAllContainsFinishedProcessing)
	{
		return;
	}

	TransitionToState(EMovieRenderPipelineState::Finished);
}

void UMovieGraphPipeline::BeginFinalize()
{
	// Notify our output nodes that no more frames will be submitted. This allows
	// them to put fences into queues for file writes, etc.
	for (const TObjectPtr<UMovieGraphFileOutputNode>& Node : GetOutputNodesUsed())
	{
		Node->OnAllFramesSubmitted(this, PostRenderEvaluatedGraph);
	}
}

void UMovieGraphPipeline::BeginExport()
{
	if (!PostRenderEvaluatedGraph)
	{
		// The generation of the evaluated graph would have emitted a warning if it failed; no need to generate another warning here 
		return;
	}

	// Exports are run per primary job, so it only makes sense to look for nodes in the primary job's graph in the Globals branch. Shot graphs are not
	// relevant at this point.
	const TArray<IMovieGraphPostRenderNode*> PostRenderNodes =
		PostRenderEvaluatedGraph->GetSettingsImplementing<IMovieGraphPostRenderNode>(UMovieGraphPostRenderNode::StaticClass(), UMovieGraphNode::GlobalsPinName);
	
	for (IMovieGraphPostRenderNode* PostRenderNode : PostRenderNodes)
	{
		PostRenderNode->BeginExport(this, PostRenderEvaluatedGraph);
	}
}

void UMovieGraphPipeline::StartUnrealInsightsCapture(UMovieGraphEvaluatedConfig* EvaluatedConfig)
{
	check(EvaluatedConfig);

	const bool bIncludeCDOs = false;
	const bool bExactMatch = true;
	const UMovieGraphDebugSettingNode* DebugSetting =
		EvaluatedConfig->GetSettingForBranch<UMovieGraphDebugSettingNode>(UMovieGraphSettingNode::GlobalsPinName, bIncludeCDOs, bExactMatch);

	if (!ensureAlwaysMsgf(DebugSetting, TEXT("Failed to find UMovieGraphDebugSettingNode. Aborting.")))
	{
		return;
	}

	const FString FinalFilePath = UE::MovieGraph::Capture::PrepareCapturePath(this, EvaluatedConfig, DebugSetting->UnrealInsightsTraceFileNameFormat, TEXT("utrace"), false);

	const bool bTraceStarted = FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::File, *FinalFilePath);

	if (bTraceStarted)
	{
		bDidStartInsightsCapture = true;
		UE_LOGF(LogMovieRenderPipeline, Log, "Started capturing UnrealInsights trace file to %ls", *FinalFilePath);
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to start capturing UnrealInsights trace. Is there already a trace session in progress?");
	}
}

void UMovieGraphPipeline::StopUnrealInsightsCapture()
{
	FTraceAuxiliary::Stop();
}

void UMovieGraphPipeline::SetupShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	BeginShotSetup(InShot);
	FinishShotSetup(InShot);
}

void UMovieGraphPipeline::BeginShotSetup(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	CurrentShotStartOutputFrameNumber = GetCustomEngineTimeStep()->SharedTimeStepData.OutputFrameNumber;

	// Set the new shot as the active shot. This enables the specified shot section and disables all other shot sections.
	SetSoloShot(InShot);

	// Loop through just our primary settings and let them know which shot we're about to start.
	//TArray<UMovieGraphSetting*> Settings = GetPipelinePrimaryConfig()->GetAllSettings();
	//Algo::SortBy(Settings, [](const UMovieGraphSetting* Setting) { return Setting->GetPriority(); });
	//for (UMovieGraphSetting* Setting : Settings)
	//{
	//	Setting->OnSetupForShot(InShot);
	//}
	//
	//if (InShot->GetShotOverrideConfiguration() != nullptr)
	//{
	//	// Any shot-specific overrides haven't had first time initialization. So we'll do that now.
	//	TArray<UMovieGraphSetting*> ShotSettings = InShot->GetShotOverrideConfiguration()->GetUserSettings();
	//	Algo::SortBy(ShotSettings, [](const UMovieGraphSetting* Setting) { return Setting->GetPriority(); });
	//	for (UMovieGraphSetting* Setting : ShotSettings)
	//	{
	//		Setting->OnMoviePipelineInitialized(this);
	//	}
	//}

	const FMovieGraphTimeStepData& TimeStepData = GetTimeStepInstance()->GetCalculatedTimeData();
	UMovieGraphEvaluatedConfig* EvaluatedConfig = TimeStepData.EvaluatedConfig;

	// Apply any global game overrides, which includes cvars. This needs to be done before the CVarManager sets cvars
	// so any user-specified cvars can override cvars set via the global game overrides. Note that the CDO is intentionally
	// not fetched here so users have a way of opting out of this node if needed.
	constexpr bool bIncludeCDOs = false;
	constexpr bool bExactMatch = true;
	if (UMovieGraphGlobalGameOverridesNode* GlobalGameOverridesNode = EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalGameOverridesNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch))
	{
		constexpr bool bOverrideValues = true;
		GlobalGameOverridesNode->ApplySettings(bOverrideValues, GetWorld());
	}

	// Apply cvars and console commands for the shot
	CVarManager->SetWorld(GetWorld());
	CVarManager->AddEvaluatedGraph(EvaluatedConfig);
	CVarManager->AddShot(InShot);
	CVarManager->ApplyAllCVars();
	CVarManager->RunStartConsoleCommands();

}

void UMovieGraphPipeline::FinishShotSetup(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	const FMovieGraphTimeStepData& TimeStepData = GetTimeStepInstance()->GetCalculatedTimeData();
	UMovieGraphEvaluatedConfig* EvaluatedConfig = TimeStepData.EvaluatedConfig;

	ExecutePreShotScripts(InShot);

	// Setup required rendering architecture for all passes in this shot.
	GraphRendererInstance->SetupRenderingPipelineForShot(InShot);

	FMoviePipelineBeginShotRenderTelemetryContext BeginShotTelemetry;
	BeginShotTelemetry.Shot = InShot;
	BeginShotTelemetry.EvaluatedConfig = EvaluatedConfig;
	BeginShotTelemetry.bIsBasicConfig = bWasInitializedWithBasicConfig;
	FMoviePipelineTelemetry::SendBeginShotRenderTelemetry(BeginShotTelemetry);
}

void UMovieGraphPipeline::TeardownShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	// No re-entrancy. Multiple TeardownShot() calls can occur in some situations when the pipeline is shutting down early.
	if (bIsTearingDownShot)
	{
		return;
	}

	TGuardValue<bool> TeardownShotGuard(bIsTearingDownShot, true);
	
	// Teardown happens at the start of the first frame the shot is finished so we'll stop recording
	// audio, which will prevent it from capturing any samples for this frame. We don't do a similar
	// start in InitializeShot() because we don't want to record samples during warm up/motion blur.
	GraphAudioRendererInstance->StopAudioRecording();
	
	// Teardown any rendering architecture for this shot. This needs to happen first because it'll flush outstanding rendering commands
	GraphRendererInstance->TeardownRenderingPipelineForShot(InShot);

	for (const TObjectPtr<UMoviePipelineExecutorShot>& Shot : GetCurrentJob()->ShotInfo)
	{
		GetDataSourceInstance()->RestoreHierarchyForShot(Shot);
	}

	const FMovieGraphTimeStepData& TimeStepData = GetTimeStepInstance()->GetCalculatedTimeData();
	TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig = TimeStepData.EvaluatedConfig;

	ProcessOutstandingFinishedFrames();

	const bool bIsPostShotCallbackNeeded = IsPostShotCallbackNeeded();

	// Run any post-render file generation that the nodes need to do
	for (const TObjectPtr<UMovieGraphFileOutputNode>& FileOutputNode : GetOutputNodesUsed())
	{
		const bool bFlushToDisk = bIsPostShotCallbackNeeded;
		FileOutputNode->OnAllShotFramesSubmitted(this, InShot, EvaluatedConfig, bFlushToDisk);
	}

	// Ensure all of our Futures have been converted to the GeneratedOutputData. If we have a post-shot callback, then all futures need to be completed
	// at this point in the pipeline.
	const bool bStallAndProcessAllFutures = bIsPostShotCallbackNeeded;
	ProcessOutstandingFutures(bStallAndProcessAllFutures);

	// Run any shot-based exports now that file output nodes have had a chance to run. This includes nodes like the Command Line Encoder node.
	const TArray<IMovieGraphPostRenderNode*> PostRenderNodes =
		EvaluatedConfig->GetSettingsImplementing<IMovieGraphPostRenderNode>(UMovieGraphPostRenderNode::StaticClass(), UMovieGraphNode::GlobalsPinName);
	for (IMovieGraphPostRenderNode* PostRenderNode : PostRenderNodes)
	{
		PostRenderNode->BeginShotExport(this);
	}

	// Wait for the export nodes to finish before going on to the next shot
	bool bFinishedExporting = true;
	do
	{
		bFinishedExporting = true;
		
		for (IMovieGraphPostRenderNode* PostRenderNode : PostRenderNodes)
		{
			bFinishedExporting &= PostRenderNode->HasFinishedExporting();
		}

		if (bFinishedExporting)
		{
			break;
		}
		
		// Sleep for a while to give the export processes time to do some work
		FPlatformProcess::Sleep(1.f);
	} while (true);

	// Revert the cvar values that were initially applied for the shot
	CVarManager->RevertAllCVars();
	CVarManager->RunEndConsoleCommands();

	UE::MoviePipeline::RestoreSkeletalMeshClothSubSteps(ClothSimCache);
	ClothSimCache.Reset();

	// Revert cvars set by the global game overrides. Needs to be done after the CVarManager reverts (since the global
	// game overrides are applied first in SetupShot).
	constexpr bool bIncludeCDOs = false;
	constexpr bool bExactMatch = true;
	if (UMovieGraphGlobalGameOverridesNode* GlobalGameOverridesNode = EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalGameOverridesNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch))
	{
		constexpr bool bOverrideValues = false;
		GlobalGameOverridesNode->ApplySettings(bOverrideValues, GetWorld());
	}

	const int32 CurrentShotEndOutputFrameNumber = GetCustomEngineTimeStep()->SharedTimeStepData.OutputFrameNumber;
	FMoviePipelineEndShotRenderTelemetryContext EndShotTelemetry;
	EndShotTelemetry.bIsGraph = true;
	EndShotTelemetry.bIsBasicConfig = bWasInitializedWithBasicConfig;
	EndShotTelemetry.bWasSuccessful = !bShutdownSetErrorFlag;
	EndShotTelemetry.bWasCanceled = bShutdownRequested;
	EndShotTelemetry.TotalRenderedFrameCount = FMath::Max(0, CurrentShotEndOutputFrameNumber - CurrentShotStartOutputFrameNumber);

	if (const UMovieGraphDefaultRenderer* DefaultRenderer = Cast<UMovieGraphDefaultRenderer>(GraphRendererInstance))
	{
		DefaultRenderer->UpdateRenderPassTelemetry(EndShotTelemetry);
	}

	FMoviePipelineTelemetry::SendEndShotRenderTelemetry(EndShotTelemetry);

	if (bIsPostShotCallbackNeeded)
	{
		// Even though these methods were called above, there may be a point in the future where post-render nodes (like the Command Line Encoder
		// node) generate files, so we need to ensure that their futures have a chance to update the generated file list.
		ProcessOutstandingFinishedFrames();
		ProcessOutstandingFutures(bStallAndProcessAllFutures);

		FMoviePipelineOutputData Params;
		Params.Pipeline = this;
		Params.Job = GetCurrentJob();
		Params.bSuccess = !bShutdownSetErrorFlag;

		// We only provide data that the current shot generated during post-shot callbacks.
		TArray< FMovieGraphRenderOutputData> SingleOutputData;
		if (CurrentShotIndex < GeneratedOutputData.Num())
		{
			// If a shot has been started but canceled during warm-up, nothing will have been pushed to the GeneratedOutputData for that shot,
			// so we have to put a range check on this.
			SingleOutputData.Add(GeneratedOutputData[CurrentShotIndex]);
		}
		Params.GraphData = SingleOutputData;

		ExecutePostShotScripts(Params);
	}

	// Check to see if this was the last shot in the Pipeline, otherwise on the next
	// tick the new shot will be initialized and processed.
	if (CurrentShotIndex >= (ActiveShotList.Num() - 1))
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "MovieGraph Finished rendering last shot. Moving to Finalize to finish writing items to disk.");
		TransitionToState(EMovieRenderPipelineState::Finalize);
	}

	CurrentShotIndex++;

	// At the end of each shot, set our Pending instance to be the next one, so that all initialization logic is handled within
	// a single TimeStepInstance instead of being split between the last one and the new one (which would happen if we use SetupShot to do this)
	if (CurrentShotIndex < GraphTimeStepInstances.Num())
	{
		PendingTimeStepInstance = GraphTimeStepInstances[CurrentShotIndex];
	}
}

void UMovieGraphPipeline::SetSoloShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	// We need to 'solo' shots whichs means disabling any other sections that may overlap with the one currently being
	// rendered. This is because temporal samples, handle frames, warmup frames, etc. all need to evaluate outside of
	// their original bounds and we don't want to end up evaluating something that should have been clipped by the shot bounds.
	for (const TObjectPtr<UMoviePipelineExecutorShot>& Shot : GetCurrentJob()->ShotInfo)
	{
		// Cache and mute all shots
		GetDataSourceInstance()->CacheHierarchyForShot(Shot);
		GetDataSourceInstance()->MuteShot(Shot);
	}

	// Historically shot expansion was done all at once up front, however this creates a lot of complications when a movie scene isn't filled with unique data
	// such as re-using shots or using different parts of shots. To resolve this, we expand the entire tree needed for a given range, render it, and then restore the original
	// values before moving onto the next shot so that each shot has no effect on the others.
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Expanding Shot %d/%d (Shot: %ls Camera: %ls)", CurrentShotIndex + 1, ActiveShotList.Num(), *InShot->OuterName, *InShot->InnerName);

		// Enable the one hierarchy we do want for rendering. We will re-disable it later when we restore the current Sequence state.
		GetDataSourceInstance()->UnmuteShot(InShot);

		const FMovieGraphTimeStepData& TimeStepData = GetTimeStepInstance()->GetCalculatedTimeData();
		TObjectPtr<UMovieGraphEvaluatedConfig> Config = TimeStepData.EvaluatedConfig;
		UMovieGraphGlobalOutputSettingNode* OutputNode = Config->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphSettingNode::GlobalsPinName);

		const FFrameRate SourceFrameRate = GetDataSourceInstance()->GetDisplayRate();
		const FFrameRate FinalFrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputNode, SourceFrameRate);
		const FFrameRate TickResolution = GetDataSourceInstance()->GetTickResolution();

		
		// Expand the shot to encompass handle frames (+warmup, etc.). This will modify the sections required for expansion, etc.
		const bool bIsPrePass = false;
		const bool bExpandForTemporalSubSample = GetTimeStepInstance()->IsExpansionForTSRequired(Config);
		
		ExpandShot(InShot, OutputNode->HandleFrameCount, bExpandForTemporalSubSample, bIsPrePass, FinalFrameRate, TickResolution, InShot->ShotInfo.NumEngineWarmUpFramesRemaining);
	}
}

void UMovieGraphPipeline::ExpandShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot, const int32 InNumHandleFrames, const bool bInHasMultipleTemporalSamples, const bool bIsPrePass,
	const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution, const int32 InWarmUpFrames)
{
	int32 LeftDeltaFrames = 0;
	int32 RightDeltaFrames = 0;

	// Calculate the number of ticks added for warmup frames. These are added to both sides. The rendering
	// code is unaware of handle frames, we just pretend the shot is bigger than it actually is.
	LeftDeltaFrames += InNumHandleFrames;
	RightDeltaFrames += InNumHandleFrames;

	// We only expand the left side for temporal sub-sampling, as no camera timing allows you to beyond the end of frame.
	if (bInHasMultipleTemporalSamples)
	{
		LeftDeltaFrames += 1;
	}

	// Check to see if the detected range was not aligned to a whole frame on the root. We produce a warning here because 
	// if your shot starts on a sub-frame (say frame 3.5) the output frame will say "3", but when you go to look at frame 3
	// in Sequencer, it will show different content than was actually evaluated. So we warn + round down to get them aligned
	// on whole frames.
	FFrameTime StartTimeInRoot = FFrameRate::TransformTime(InShot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue(), InTickResolution, InDisplayRate);
	if (bIsPrePass && StartTimeInRoot.GetSubFrame() != 0.f)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Shot/Camera \"%ls [%ls]\" starts on a sub-frame. Rendered range has been rounded to the previous frame to make output numbers align with Sequencer.", 
			*InShot->OuterName, *InShot->InnerName);
		FFrameNumber NewStartFrame = FFrameRate::TransformTime(FFrameTime(StartTimeInRoot.GetFrame()), InDisplayRate, InTickResolution).FloorToFrame();
		InShot->ShotInfo.TotalOutputRangeRoot.SetLowerBoundValue(NewStartFrame);
	}

	// We auto-expand into the warm-up ranges, but users are less concerned about 'early' data there. So we cache how many frames
	// the user expects to check beforehand, so we can use this for a warning later.
	const int32 LeftDeltaFramesUserPoV = LeftDeltaFrames;

	// Warm Up frames are only on the left side. This comes after the above section so that we don't warn about partial data in the warm up section.
	LeftDeltaFrames += InWarmUpFrames;

	GetDataSourceInstance()->ExpandShot(InShot, LeftDeltaFrames, LeftDeltaFramesUserPoV, RightDeltaFrames, bIsPrePass);

	// Expand the Total Output Range Root by Handle Frames. The expansion of TotalOutputRangeRoot has to come after we do partial evaluation checks,
	// which is done by ExpandShot above, otherwise the expanded range makes it check the wrong area for partial evaluations.
	if (bIsPrePass)
	{
		// We expand on the pre-pass so that we have the correct number of frames set up in our datastructures before we reach each shot so that metrics
		// work as expected.
		FFrameNumber LeftHandleTicks = FFrameRate::TransformTime(FFrameTime(InNumHandleFrames), InDisplayRate, InTickResolution).CeilToFrame().Value;
		FFrameNumber RightHandleTicks = FFrameRate::TransformTime(FFrameTime(InNumHandleFrames), InDisplayRate, InTickResolution).CeilToFrame().Value;

		InShot->ShotInfo.TotalOutputRangeRoot = UE::MovieScene::DilateRange(InShot->ShotInfo.TotalOutputRangeRoot, -LeftHandleTicks, RightHandleTicks);
	}
}

UMovieGraphTimeStepBase* UMovieGraphPipeline::GetTimeStepInstance() const
{
	return GraphTimeStepInstance;
}

void UMovieGraphPipeline::OnEngineTickEndFrame()
{
	LLM_SCOPE_BYNAME(TEXT("MovieGraphEndFrame"));

	// Don't try to submit anything to the renderer if the shot isn't initialized yet, or has
	// finished. We tick the engine when transitioning between shot states.
	EMovieRenderShotState CurrentShotState = GetActiveShotList()[GetCurrentShotIndex()]->ShotInfo.State;
	if (CurrentShotState == EMovieRenderShotState::Uninitialized ||
		CurrentShotState == EMovieRenderShotState::Finished)
	{
		return;
	}

	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "MovieGraph OnEngineTickEndFrame (Start)");

	GraphAudioRendererInstance->ProcessAudioTick();
	
	RenderFrame();

	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "MovieGraph OnEngineTickEndFrame (End)");
}

void UMovieGraphPipeline::RenderFrame()
{
	check(GraphRendererInstance);

	const FMovieGraphTimeStepData& TimeStepData = GetTimeStepInstance()->GetCalculatedTimeData();

	GraphRendererInstance->Render(TimeStepData);
}

int32 UMovieGraphPipeline::ResolveVersionForShot(const TObjectPtr<UMoviePipelineExecutorShot>& Shot, const TObjectPtr<UMovieGraphEvaluatedConfig>& EvaluatedConfig)
{	
	int32 HighestVersionFound = 1;
	
	FMovieGraphFilenameResolveParams ResolveParams;
	ResolveParams.EvaluatedConfig = EvaluatedConfig;
	ResolveParams.Shot = Shot;
	ResolveParams.Job = GetCurrentJob();

	const FMovieGraphTimeStepData& ShotTimeStepInstance = GraphTimeStepInstances.Last()->GetCalculatedTimeData();

	constexpr bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	const UMovieGraphGlobalOutputSettingNode* BranchOutputSettingNode =
		EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);
	
	// TODO: The RenderDataIdentifier needs to be fully populated in order to resolve the version reliably. For
	// example, {renderer_name} may be used as a directory, and discovering files correctly depends on that token
	// being resolved. We can loop over all renderer and file output nodes here since the branches are already being
	// iterated, and resolve a version for each (still using the highest version found as the final resolved
	// version). Note that to do this correctly, we need a way of asking these nodes for their renderer name and all
	// possible sub-resource names.
	ResolveParams.InitializationTime = GetInitializationTime();
	ResolveParams.InitializationTimeOffset = GetInitializationTimeOffset();
	ResolveParams.DefaultFrameRate = ShotTimeStepInstance.FrameRate;
	ResolveParams.FrameNumberOffset = BranchOutputSettingNode->FrameNumberOffset;
	ResolveParams.RenderDataIdentifier.CameraName = Shot->InnerName;
	// ResolveParams.RenderDataIdentifier.RendererName = ?
	// ResolveParams.RenderDataIdentifier.SubResourceName = ?
	ResolveParams.RootFrameNumber = ShotTimeStepInstance.RootFrameNumber.Value;
	ResolveParams.ShotFrameNumber = ShotTimeStepInstance.ShotFrameNumber.Value;
	ResolveParams.bForceRelativeFrameNumbers = false;	// TODO: This should not be hardcoded
	// ResolveParams.FileNameFormatOverrides = ?
	ResolveParams.RootFrameNumberRel = ShotTimeStepInstance.OutputFrameNumber;
	// ResolveParams.ShotFrameNumberRel = ?
	ResolveParams.ZeroPadFrameNumberCount = BranchOutputSettingNode->ZeroPadFrameNumbers;

	for (const FName& BranchName : EvaluatedConfig->GetBranchNames())
	{
		ResolveParams.RenderDataIdentifier.RootBranchName = BranchName;

		const int32 BranchVersion = UMovieGraphBlueprintLibrary::ResolveVersionNumber(ResolveParams);
		if (BranchVersion > HighestVersionFound)
		{
			HighestVersionFound = BranchVersion;
		}
	}
	
	return HighestVersionFound;
}

void UMovieGraphPipeline::RequestShutdownImpl(bool bIsError)
{
	// It's possible for a previous call to RequestionShutdown to have set an error before this call that may not
	// We don't want to unset a previously set error state
	if (bIsError)
	{
		bShutdownSetErrorFlag = true;
	}

	// The user has requested a shutdown, it will be read the next available chance and possibly acted on.
	bShutdownRequested = true;
	switch (PipelineState)
	{
		// It is valid to call Shutdown at any point during these two states.
	case EMovieRenderPipelineState::Uninitialized:
	case EMovieRenderPipelineState::ProducingFrames:
		break;
		// You can call Shutdown during these two, but they won't do anything as we're already shutting down at that point.
	case EMovieRenderPipelineState::Finalize:
	case EMovieRenderPipelineState::Export:
		UE_LOGF(LogMovieRenderPipeline, Log, "MovieGraph Async Shutdown Requested, ignoring due to already being on the way to shutdown.");
		break;
	}
}

void UMovieGraphPipeline::ShutdownImpl(bool bIsError)
{
	check(IsInGameThread());

	// We flag this so you can check if the shutdown was requested even when we do a stall-stop.
	bShutdownRequested = true;

	// It's possible for a previous call to RequestShutdown to have set an error before this call and
	// we don't want to just blow away the error flag even if this function was then called normally.
	// (ie: Don't accidentally unset error state)
	if (bIsError)
	{
		bShutdownSetErrorFlag = true;
	}

	// This is a blocking operation which abandons any outstanding work to be submitted but finishes
	// the existing work already processed.
	if (PipelineState == EMovieRenderPipelineState::Uninitialized)
	{
		// If initialize is not called, no need to do anything.
		return;
	}

	if (PipelineState == EMovieRenderPipelineState::Finished)
	{
		// If already shut down, no need to do anything.
		return;
	}

	if (PipelineState == EMovieRenderPipelineState::ProducingFrames)
	{
		// Teardown the currently active shot (if there is one). This will flush any outstanding rendering
		// work that has already submitted - it cannot be canceled, so we may as well execute it and save the results.
		TransitionToState(EMovieRenderPipelineState::Finalize);

		// Abandon the current frame. When using temporal sampling we may had canceled mid-frame, so the rendering
		// commands were never submitted, thus the output builder will still be expecting a frame to come in.
		//if (CachedOutputState.TemporalSampleCount > 1)
		//{
		//	OutputBuilder->AbandonOutstandingWork();
		//}
	}

	if (PipelineState == EMovieRenderPipelineState::Finalize)
	{
		// We were either in the middle of writing frames to disk, or we have moved to Finalize as a result of the above block.
		// Tick output containers until they report they have finished writing to disk. This is a blocking operation. 
		// Finalize automatically switches our state to Export so no need to manually transition afterwards.
		constexpr bool bForceFinish = true;
		TickFinalizeOutputContainers(bForceFinish);
	}

	if (PipelineState == EMovieRenderPipelineState::Export)
	{
		// All frames have been written to disk but we're doing a post-export step (such as encoding). Flush this operation as well.
		// Export automatically switches our state to Finished so no need to manually transition afterwards.
		constexpr bool bForceFinish = true;
		TickPostFinalizeExport(bForceFinish);
	}

	// Duplicated graphs are part of the transient package, so by default they'll stick around after the pipeline is destroyed. To prevent
	// the graphs from hanging onto references after the render is finished, remove them during pipeline shutdown (so they are GC'd).
	DuplicatedGraphs.Empty();
}

void UMovieGraphPipeline::TransitionToState(const EMovieRenderPipelineState InNewState)
{
	// No re-entrancy. This isn't an error as tearing down a shot may try to move to
	// Finalize on its own, but we don't want that.
	if (bIsTransitioningState)
	{
		return;
	}

	TGuardValue<bool> StateTransitionGuard(bIsTransitioningState, true);

	bool bInvalidTransition = true;
	switch (PipelineState)
	{
	case EMovieRenderPipelineState::Uninitialized:
		PipelineState = InNewState;
		bInvalidTransition = false;
		break;
	case EMovieRenderPipelineState::ProducingFrames:
		if (InNewState == EMovieRenderPipelineState::Finalize)
		{
			bInvalidTransition = false;

			// If we had naturally finished the last shot before doing this transition it will have
			// already been torn down, so this only catches mid-shot transitions to ensure teardown.
			// We also check that the shot was actually initialized to avoid restoring hierarchy state
			// that was never saved. This can happen when PIE ends on the same frame as pipeline
			// initialization, before the first tick runs InitializeShot/SetSoloShot.
			if (bShutdownRequested && (CurrentShotIndex < ActiveShotList.Num())
				&& ActiveShotList[CurrentShotIndex]->ShotInfo.IsInitialized())
			{
				// Ensures all in-flight work for that shot is handled.
				TeardownShot(ActiveShotList[CurrentShotIndex]);
			}

			// Unregister our OnEngineTickEndFrame delegate. We don't unregister BeginFrame as we need
			// to continue to call it to allow ticking the Finalization stage.
			FCoreDelegates::OnEndFrame.RemoveAll(this);

			// Ensure all frames have been processed by the GPU and sent to the Output Merger
			FlushRenderingCommands();

			// And then make sure all frames are sent to the Output Containers before we finalize.
			ProcessOutstandingFinishedFrames();

			// Restore any custom Time Step that may have been set before. We do this here
			// because the TimeStepInstance is only expected to be having to calculate times
			// during ProducingFrames. The instance may be null if the pipeline was shut down
			// before the first TickProducingFrames assigned it.
			if (GetTimeStepInstance())
			{
				GetTimeStepInstance()->Shutdown();
			}

			// We assign it to nullptr here so that post-ProducingFrame code doesn't accidentally 
			// use a stale time step instance.
			GraphTimeStepInstance = nullptr;

			// Shut down our custom timestep which reqstores some world settings we modified.
			GEngine->SetCustomTimeStep(PrevCustomEngineTimeStep);

			// Generate a job-level (sequence) graph that post-render tasks can use. Shot graphs should not be used at this point.
			constexpr bool bForShot = false;
			FString FlattenError;
			PostRenderEvaluatedGraph = GetCurrentJob()->GetGraphPreset()->CreateFlattenedGraph(GetCurrentTraversalContext(bForShot), FlattenError);
			if (!PostRenderEvaluatedGraph)
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "Generating a post-render evaluated graph failed for job '%ls'. Post-render exports will not run. Evaluation error: %ls", *GetCurrentJob()->JobName, *FlattenError);
			}

			// This is called once notifying output containers that all frames that will be submitted have been submitted.
			PipelineState = EMovieRenderPipelineState::Finalize;
			BeginFinalize();
		}
		break;
	case EMovieRenderPipelineState::Finalize:
		if (InNewState == EMovieRenderPipelineState::Export)
		{
			bInvalidTransition = false;

			// This is called once notifying our export step that they can begin the export.
			PipelineState = EMovieRenderPipelineState::Export;

			// Stop playing the data source once export begins. Audio issues may happen if this is not done.
			GetDataSourceInstance()->StopDataSource();

			// Ensure all of our Futures have been converted to the GeneratedOutputData. This has to happen
			// after finalize finishes, because the futures won't be available until actually written to disk.
			ProcessOutstandingFutures();

			BeginExport();

			// Clear out MRQ tick information for external consumers as we are done
			FMovieRenderPipelineCoreModule::SetTickInfo(FMoviePipelineLightweightTickInfo());
		}
		break;
	case EMovieRenderPipelineState::Export:
		if (InNewState == EMovieRenderPipelineState::Finished)
		{
			bInvalidTransition = false;
			PipelineState = EMovieRenderPipelineState::Finished;

			// Uninitialize our primary config settings. Reverse sorted so settings that cached values restore correctly.
			//TArray<UMovieGraphSetting*> Settings = GetPipelinePrimaryConfig()->GetAllSettings();
			//Algo::SortBy(Settings, [](const UMovieGraphSetting* Setting) { return Setting->GetPriority(); }, TLess<int32>());
			//for (UMovieGraphSetting* Setting : Settings)
			//{
			//	Setting->OnMoviePipelineShutdown(this);
			//}

			FMoviePipelineOutputData Params;
			Params.Pipeline = this;
			Params.Job = GetCurrentJob();
			Params.bSuccess = !bShutdownSetErrorFlag;
			Params.GraphData = GeneratedOutputData;

			ExecutePostJobScripts(Params);

			// Ensure our delegates don't get called anymore as we're going to become null soon.
			FCoreDelegates::OnBeginFrame.RemoveAll(this);
			FCoreDelegates::OnEndFrame.RemoveAll(this);

			if (PreviewWidget)
			{
				PreviewWidget->RemoveFromParent();
				PreviewWidget = nullptr;
			}

			// Stop Insights trace
			if (bDidStartInsightsCapture)
			{
				StopUnrealInsightsCapture();
			}

			// Job-level evaluated graph should not be referenced after export has finished
			PostRenderEvaluatedGraph = nullptr;

			//TArray<UMovieGraphOutputBase*> ContainerSettings = GetPipelinePrimaryConfig()->GetOutputContainers();
			//Algo::SortBy(ContainerSettings, [](const UMovieGraphOutputBase* Setting) { return Setting->GetPriority(); });
			//for (UMovieGraphOutputBase* Setting : ContainerSettings)
			//{
			//	Setting->OnPipelineFinished();
			//}
			//
			GraphAudioRendererInstance->TeardownAudioRendering();
			//LevelSequenceActor->GetSequencePlayer()->Stop();
			//RestoreTargetSequenceToOriginalState();
			//
			//if (UGameViewportClient* Viewport = GetWorld()->GetGameViewport())
			//{
			//	Viewport->bDisableWorldRendering = false;
			//}

			// Because the render target pool is shared, if you had a high-resolution render in editor the entire gbuffer
			// has been resized up to match the new maximum extent. This console command will reset the size of the pool
			// and cause it to re-allocate at the currrent size on the next render request, which is likely to be the size
			// of the PIE window (720p) or the Viewport itself.
			//UKismetSystemLibrary::ExecuteConsoleCommand(GetWorld(), TEXT("r.ResetRenderTargetsExtent"), nullptr);
			//CustomTimeStep->RestoreCachedWorldSettings();

			//GAreScreenMessagesEnabled = bPrevGScreenMessagesEnabled;

			//UE_LOGF(LogMovieRenderPipeline, Log, "Movie Pipeline completed. Duration: %ls", *(FDateTime::UtcNow() - InitializationTime).ToString());

			OutputNodesDataSentTo.Reset();

			GraphDataSourceInstance->RestoreCachedDataPostJob();

			OnMoviePipelineFinishedImpl();
			UE_LOGF(LogMovieRenderPipeline, Log, "Movie Graph Render completed. Duration: %ls", *(FDateTime::UtcNow() - GraphInitializationTime).ToString());
		}
		break;
	}

	if (!ensureAlwaysMsgf(!bInvalidTransition, TEXT("MovieGraph An invalid transition was requested (from: %d to: %d), ignoring transition request."),
		EnumToUnderlyingType(PipelineState), EnumToUnderlyingType(InNewState)))
	{
		return;
	}
}

void UMovieGraphPipeline::OnMoviePipelineFinishedImpl()
{
	// Broadcast to both Native and Python/BP
	//
	// Generate a params struct containing the data generated by this job.
	FMoviePipelineOutputData Params;
	Params.Pipeline = this;
	Params.Job = GetCurrentJob();
	Params.bSuccess = !bShutdownSetErrorFlag;
	Params.GraphData = GeneratedOutputData;
	//
	//UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Files written to disk for entire sequence:");
	//PrintVerboseLogForFiles(GeneratedShotOutputData);
	//UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Completed outputting files written to disk.");
	//

	OnMoviePipelineWorkFinishedDelegateNative.Broadcast(Params);
	OnMoviePipelineWorkFinishedDelegate.Broadcast(Params);
}

//void UMovieGraphPipeline::PrintVerboseLogForFiles(const TArray<FMoviePipelineShotOutputData>& InOutputData) const
//{
//	for (const FMoviePipelineShotOutputData& OutputData : InOutputData)
//	{
//		const UMoviePipelineExecutorShot* Shot = OutputData.Shot.Get();
//		if (Shot)
//		{
//			UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Shot: %ls [%ls]", *Shot->OuterName, *Shot->InnerName);
//		}
//		for (const TPair<FMoviePipelinePassIdentifier, FMoviePipelineRenderPassOutputData>& Pair : OutputData.RenderPassData)
//		{
//			UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Render Pass: %ls", *Pair.Key.Name);
//			for (const FString& FilePath : Pair.Value.FilePaths)
//			{
//				UE_LOGF(LogMovieRenderPipelineIO, Verbose, "\t\t%ls", *FilePath);
//			}
//		}
//	}
//}

void UMovieGraphPipeline::ProcessOutstandingFinishedFrames()
{
	while (!OutputMerger->GetFinishedFrames().IsEmpty())
	{
		UE::MovieGraph::FMovieGraphOutputMergerFrame OutputFrame;
		OutputMerger->GetFinishedFrames().Dequeue(OutputFrame);

		UE::MovieGraph::FRenderTimeStatistics* TimeStats = GetRendererInstance()->GetRenderTimeStatistics(OutputFrame.TraversalContext.Time.OutputFrameNumber);
		if (ensure(TimeStats))
		{
			TimeStats->EndTime = FDateTime::UtcNow();

			// Add render time metadata
			FString StartTimeStr = TimeStats->StartTime.ToString();
			FString EndTimeStr = TimeStats->EndTime.ToString();
			FString DurationTimeStr = (TimeStats->EndTime - TimeStats->StartTime).ToString();
			OutputFrame.FileMetadata.Add(TEXT("unreal/frameRenderStartTimeUTC"), StartTimeStr);
			OutputFrame.FileMetadata.Add(TEXT("unreal/frameRenderEndTimeUTC"), EndTimeStr);
			OutputFrame.FileMetadata.Add(TEXT("unreal/frameRenderDuration"), DurationTimeStr);

			// int32 FrameNumber = OutputFrame.TraversalContext.Time.RenderedFrameNumber;
			// UE_LOGF(LogTemp, Log, "Frame: %d Duration: %ls", FrameNumber, *DurationTimeStr);
		}

		// How we choose which image gets sent to which output container is a little tricky. We use the CDO of each node type on purpose
		// as the graph can change node instances every frame. If a output type node is in the "Globals" graph, we assume that we should
		// send all rendered layers to the image type. However we should also allow placing an output of a given type in a render layer,
		// so you could choose to send the layer to only some output types (ie: only send the "beauty" layer to the .jpeg container).
		// Because our data is always treated as a block of all image data (and is non-copyable) we instead generate both a list of
		// output containers for the data in this output frame, and a mask which lets that container know if it should skip a layer.
		TMap<UMovieGraphFileOutputNode*, TSet<FMovieGraphRenderDataIdentifier>> MaskData;

		// For each layer in the output merger frame...
		for (TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& LayerData : OutputFrame.ImageOutputData)
		{
			UE::MovieGraph::FMovieGraphSampleState* LayerPayload = LayerData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
			if (!ensure(LayerPayload))
			{
				continue;
			}

			if (!ensure(OutputFrame.EvaluatedConfig))
			{
				continue;
			}

			// For each File Output Node CDO in the layer, associate the CDO with the layer...
			//
			// Get a list of all output nodes for this particular render layer. We specifically skip CDOs here because we're just trying
			// to find out which output nodes the user _wanted_ to place items onto, we later collect only the CDOs so that we have
			// a central point for actually handling the file writing.
			constexpr bool bIncludeCDOs = false;
			const FMovieGraphRenderDataIdentifier& LayerIdentifier = LayerData.Key;
			const FName& LayerBranchName = LayerIdentifier.RootBranchName;
			TArray<UMovieGraphFileOutputNode*> OutputNodesForLayer = OutputFrame.EvaluatedConfig->GetSettingsForBranch<UMovieGraphFileOutputNode>(LayerBranchName, bIncludeCDOs);
			for (const UMovieGraphFileOutputNode* LayerOutputNode : OutputNodesForLayer)
			{
				UMovieGraphFileOutputNode* CDO = LayerOutputNode->GetClass()->GetDefaultObject<UMovieGraphFileOutputNode>();
				MaskData.FindOrAdd(CDO).Add(LayerIdentifier);
			}
		}

		// Now that we've looped through the above, we have the total list of which output formats are being used by the graph for
		// all of the render layers given. We also have a list of which identifiers should go into each one. So we can loop through
		// the CDO instances and pass the data to them.
		for (const TPair<UMovieGraphFileOutputNode*, TSet<FMovieGraphRenderDataIdentifier>>& Pair : MaskData)
		{
			UMovieGraphFileOutputNode* OutputCDO = Pair.Key;
			const TSet<FMovieGraphRenderDataIdentifier>& LayerIdentifiers = Pair.Value;

			// Tell the CDO that it's getting data from the output merger frame, and specify which layers it should output data for
			Pair.Key->OnReceiveImageData(this, &OutputFrame, LayerIdentifiers);

			// Ensure we keep track of which nodes we actually sent data to during this render
			// so that we can call BeginFinalize/IsFinishedWritingToDisk on them when shutting down.
			OutputNodesDataSentTo.Add(OutputCDO);
		}
	}
}

void UMovieGraphPipeline::AddOutputFuture(TFuture<bool>&& InOutputFuture, const UE::MovieGraph::FMovieGraphOutputFutureData& InData)
{
	OutstandingOutputFutures.Add(
		UE::MovieGraph::FMovieGraphOutputFuture(MoveTemp(InOutputFuture), InData)
	);
}

void UMovieGraphPipeline::ProcessOutstandingFutures(const bool bStallAndProcessAllFutures)
{
	// Check if any frames failed to output
	TArray<int32> CompletedOutputFutures;
	for (int32 Index = 0; Index < OutstandingOutputFutures.Num(); ++Index)
	{
		const UE::MovieGraph::FMovieGraphOutputFuture& OutputFuture = OutstandingOutputFutures[Index];
		
		// Output futures are pushed in order to the OutputFutures array. However they are
		// completed asynchronously, so we don't process any futures after a not-yet-ready one
		// otherwise we push into the GeneratedShotOutputData array out of order.
		//
		// Optionally, we can wait for the future being processed to actually finish before proceeding
		// to the next. This is helpful when we need to guarantee that all futures are processed before
		// moving on to the next step in the pipeline (eg, a step that depends on images being written
		// to disk).
		if (bStallAndProcessAllFutures)
		{
			while (!OutputFuture.Get<0>().IsReady())
			{
				// Sleep for a while to allow the futures to complete
				FPlatformProcess::Sleep(0.1f);
			}
		}
		else
		{
			if (!OutputFuture.Get<0>().IsReady())
			{
				break;
			}
		}

		CompletedOutputFutures.Add(Index);

		const UE::MovieGraph::FMovieGraphOutputFutureData& FutureData = OutputFuture.Get<1>();

		// The future was completed, time to add it to our shot output data.
		FMovieGraphRenderOutputData* ShotOutputData = nullptr;
		for (int32 OutputDataIndex = 0; OutputDataIndex < GeneratedOutputData.Num(); OutputDataIndex++)
		{
			if (FutureData.Shot == GeneratedOutputData[OutputDataIndex].Shot)
			{
				ShotOutputData = &GeneratedOutputData[OutputDataIndex];
			}
		}

		if (!ShotOutputData)
		{
			GeneratedOutputData.Add(FMovieGraphRenderOutputData());
			ShotOutputData = &GeneratedOutputData.Last();
			ShotOutputData->Shot = FutureData.Shot;
		}

		// Add the filepath and other data to the render layer.
		FMovieGraphRenderLayerOutputData& OutputData = ShotOutputData->RenderLayerData.FindOrAdd(FutureData.DataIdentifier);
		OutputData.FilePaths.Add(FutureData.FilePath);
		OutputData.FrameTemplatedFilePaths.AddUnique(FutureData.FrameTemplatedFilePath);
		OutputData.RenderLayerIndex = FutureData.RenderLayerIndex;

		// Also associate this file with a specific origin node class.
		FMovieGraphStringArray& FilesFromNodeClass = OutputData.NodeTypeToFilePaths.FindOrAdd(FutureData.OriginNodeClass);
		FilesFromNodeClass.Array.Add(FutureData.FilePath);

		// Sometime futures can be completed, but will be set to an error state (such as we couldn't write to the specified disk path.)
		if (!OutputFuture.Get<0>().Get())
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Error exporting frame, canceling movie export.");
			RequestShutdown(true);
			break;
		}
	}

	// Remove any output futures that have been completed now.
	for (int32 Index = CompletedOutputFutures.Num() - 1; Index >= 0; --Index)
	{
		OutstandingOutputFutures.RemoveAt(CompletedOutputFutures[Index]);
	}
}

UMovieGraphConfig* UMovieGraphPipeline::GetRootGraphForShot(UMoviePipelineExecutorShot* InShot) const
{
	if (GetCurrentJob())
	{
		for (UMoviePipelineExecutorShot* Shot : GetCurrentJob()->ShotInfo)
		{
			if (Shot == InShot)
			{
				if (Shot->GetGraphPreset())
				{
					return Shot->GetGraphPreset();
				}
			}

		}

		// If the shot hasn't overwritten the preset then we return the root one for the whole job.
		if (GetCurrentJob()->GetGraphPreset())
		{
			return GetCurrentJob()->GetGraphPreset();
		}
	}

	return nullptr;
}

FMovieGraphTraversalContext UMovieGraphPipeline::GetCurrentTraversalContext(const bool bForShot) const
{
	const TObjectPtr<UMoviePipelineExecutorShot> CurrentShot = bForShot ? GetActiveShotList()[GetCurrentShotIndex()] : nullptr;
	
	FMovieGraphTraversalContext CurrentContext;
	CurrentContext.ShotIndex = bForShot ? GetCurrentShotIndex() : -1;
	CurrentContext.ShotCount = GetActiveShotList().Num();
	CurrentContext.Job = GetCurrentJob();
	CurrentContext.Shot = CurrentShot;
	CurrentContext.RootGraph = bForShot ? GetRootGraphForShot(CurrentShot) : CurrentContext.Job->GetGraphPreset();
	CurrentContext.Time = bForShot ? GetTimeStepInstance()->GetCalculatedTimeData() : FMovieGraphTimeStepData();

	return CurrentContext;
}

const TSet<TObjectPtr<UMovieGraphFileOutputNode>> UMovieGraphPipeline::GetOutputNodesUsed() const
{
	return OutputNodesDataSentTo;
}


bool UMovieGraphPipeline::IsPostShotCallbackNeeded() const
{
	bool bAnyScriptNeedsCallbacks = false;
	for (UMovieGraphScriptBase* Script : CurrentScriptInstances)
	{
		if (Script->IsPerShotCallbackNeeded())
		{
			bAnyScriptNeedsCallbacks = true;
			break;
		}
	}

	// If a script instance didn't specify per-shot callbacks are needed, check to see if a script flagged bFlushDiskWritesPerShot on the Global Output
	// Settings node. This accomplishes the same thing as IsPerShotCallbackNeeded() on scripting nodes.
	if (!bAnyScriptNeedsCallbacks)
	{
		if (const TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig = GetTimeStepInstance()->GetCalculatedTimeData().EvaluatedConfig)
		{
			constexpr bool bIncludeCDOs = true;
			constexpr bool bExactMatch = true;
			const UMovieGraphGlobalOutputSettingNode* OutputSettingsNode =
				EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);
		
			bAnyScriptNeedsCallbacks = OutputSettingsNode->bFlushDiskWritesPerShot;
		}
	}

	return bAnyScriptNeedsCallbacks;
}

void UMovieGraphPipeline::BeginDestroy()
{
	// Ask the output merger to clean up. It can hold strong obj pointers, and those should not stick around after the pipeline shuts down. Generally
	// this shouldn't be needed, but could be necessary if pending frames were not flushed for some reason.
	if (OutputMerger.IsValid())
	{
		OutputMerger->AbandonOutstandingWork();
	}
	
	Super::BeginDestroy();
}

void UMovieGraphPipeline::ExecutePreJobScripts()
{
	for (UMovieGraphScriptBase* Script : CurrentScriptInstances)
	{
		// GetCurrentJob returns the duplicated job and not the original.
		Script->OnJobStart(GetCurrentJob());
	}
}

void UMovieGraphPipeline::ExecutePreShotScripts(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	for (UMovieGraphScriptBase* Script : CurrentScriptInstances)
	{
		if (Script->IsPerShotCallbackNeeded())
		{
			Script->OnShotStart(GetCurrentJob(), InShot);
		}
	}
}

void UMovieGraphPipeline::ExecutePostShotScripts(const FMoviePipelineOutputData& InData)
{
	for (UMovieGraphScriptBase* Script : CurrentScriptInstances)
	{
		if (Script->IsPerShotCallbackNeeded())
		{
			Script->OnShotFinished(GetCurrentJob(), ActiveShotList[CurrentShotIndex], InData);
		}
	}

	OnMoviePipelineShotWorkFinishedDelegate.Broadcast(InData);
	OnMoviePipelineShotWorkFinishedImpl().Broadcast(InData);
}

void UMovieGraphPipeline::ExecutePostJobScripts(const FMoviePipelineOutputData& InData)
{
	for (UMovieGraphScriptBase* Script : CurrentScriptInstances)
	{
		Script->OnJobFinished(GetCurrentJob(), InData);
	}
}
