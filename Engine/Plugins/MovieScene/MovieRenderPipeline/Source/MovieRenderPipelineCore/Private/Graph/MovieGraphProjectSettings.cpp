// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphProjectSettings.h"

#include "Algo/Find.h"
#include "Algo/FindLast.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphProjectSettings)

UMovieGraphProjectSettings::UMovieGraphProjectSettings()
{
	DefaultNamedResolutions = {
		{ "720p (HD)", FIntPoint(1280, 720), "High definition" },
		{ "1080p (FHD)", FIntPoint(1920, 1080), "Full high definition" },
		{ "1440p (QHD)", FIntPoint(2560, 1440), "Quad high definition" },
		{ "2160p (4K UHD)", FIntPoint(3840, 2160), "Ultra high definition (4k)" }
	};
}

const FMovieGraphNamedResolution* UMovieGraphProjectSettings::FindNamedResolutionForOption(const FName& InOption) const
{
	return Algo::FindByPredicate(
			DefaultNamedResolutions,
			[&InOption](const FMovieGraphNamedResolution& Other)
			{
				return Other.ProfileName == InOption;
			});
}

#if WITH_EDITOR
void UMovieGraphProjectSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	EnsureUniqueNamesInDefaultNamedResolutions();
}
#endif

void RenameSimilarlyNamedResolutionsLoop(
	TPair<FName, int32>& NameToCountPair, TArray<FMovieGraphNamedResolution>& DefaultNamedResolutions)
{
    // Loop until the count of a given profile name is reduced to 1
	while (NameToCountPair.Value > 1)
	{
		// Find the last occurrence of the profile name in the default named resolutions
		FMovieGraphNamedResolution* Match =
			Algo::FindLastByPredicate(
				DefaultNamedResolutions,
				[&NameToCountPair](const FMovieGraphNamedResolution& Other)
				{
					return Other.ProfileName == NameToCountPair.Key;
				});

		if (ensureAlwaysMsgf(Match, TEXT("%hs: Failed to find FMovieGraphNamedResolution %s!"), __FUNCTION__, *NameToCountPair.Key.ToString()))
		{
			// Generate a new name by appending a number to the existing profile name
			int32 NumberToAppend = NameToCountPair.Value - 1;
			FName NewName = *FString::Printf(TEXT("%s_%d"), *Match->ProfileName.ToString(), NumberToAppend);

			// Check if the new name already exists in the default named resolutions
			while (DefaultNamedResolutions.ContainsByPredicate([&NewName](const FMovieGraphNamedResolution& Other)
			{
				return Other.ProfileName.IsEqual(NewName);
			}))
			{
				// If it exists, increment the number and try again
				NumberToAppend++;
				NewName = *FString::Printf(TEXT("%s_%d"), *Match->ProfileName.ToString(), NumberToAppend);
			}

			// Reduce the count of the profile name and assign the new name to the matching resolution
			NameToCountPair.Value--;
			Match->ProfileName = NewName;
		}
		else
		{
			// If no matching resolution is found, break to avoid an infinite loop
			break;
		}
	}
}

void UMovieGraphProjectSettings::EnsureUniqueNamesInDefaultNamedResolutions()
{
    // Create a map to store the count of each profile name
    TMap<FName, int32> NameToCount;

    // Iterate through each default named resolution
    for (const FMovieGraphNamedResolution& ResolutionName : DefaultNamedResolutions)
    {
        // Check if the profile name exists in the map
        if (int32* Match = NameToCount.Find(ResolutionName.ProfileName))
        {
            // If it exists, increment the count
            *Match += 1;
        }
        else
        {
            // If it doesn't exist, add it to the map with a count of 1
            NameToCount.Add(ResolutionName.ProfileName, 1);
        }
    }

    // Iterate through the map of profile names and their counts, renaming redundant items as we go
    for (TPair<FName, int32>& NameToCountPair : NameToCount)
    {
		RenameSimilarlyNamedResolutionsLoop(NameToCountPair, DefaultNamedResolutions);
    }
}
