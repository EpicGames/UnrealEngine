// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphQuickRenderSettings.h"

#include "Misc/CoreDelegates.h"
#include "Misc/TransactionObjectEvent.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "PackageHelperFunctions.h"
#include "Settings/EditorLoadingSavingSettings.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphQuickRenderSettings)

UMovieGraphQuickRenderModeSettings* UMovieGraphQuickRenderSettings::GetSavedQuickRenderModeSettings(const EMovieGraphQuickRenderMode SettingsMode)
{
	// Some modes share the same settings, so here we map the mode to an internal settings grouping. Each settings group shares the same settings.
	static const TMap<EMovieGraphQuickRenderMode, FName> ModeToInternalName =
	{
		{ EMovieGraphQuickRenderMode::CurrentSequence, FName(TEXT("Sequence"))},
		{ EMovieGraphQuickRenderMode::CurrentViewport, FName(TEXT("CurrentViewport"))},
		{ EMovieGraphQuickRenderMode::SelectedCameras, FName(TEXT("SelectedCameras"))},
		{ EMovieGraphQuickRenderMode::UseViewportCameraInSequence, FName(TEXT("Sequence"))},
	};
	
	UMovieGraphQuickRenderSettings* QuickRenderSettings = nullptr;

	// Use FindPackage first to avoid triggering the async loading path on packages that are already in memory.
	const UPackage* QuickRenderSettingsPackage = FindPackage(nullptr, *QuickRenderSettingsPackagePath);
	if (!QuickRenderSettingsPackage)
	{
		QuickRenderSettingsPackage = LoadPackage(nullptr, *QuickRenderSettingsPackagePath, LOAD_None);
	}

	if (QuickRenderSettingsPackage)
	{
		QuickRenderSettings = Cast<UMovieGraphQuickRenderSettings>(FindObjectWithOuter(QuickRenderSettingsPackage, StaticClass()));
	}

	// The settings asset may not exist on disk yet, or (rarely) the settings object may be from a future version (eg, if the user went back to a
	// previous build). Both of these situations require a new settings asset to be created.
	if (!QuickRenderSettings)
	{
		UPackage* NewSettingsPackage = CreatePackage(*QuickRenderSettingsPackagePath);
		QuickRenderSettings = NewObject<UMovieGraphQuickRenderSettings>(NewSettingsPackage);
	}

	// Get the settings for the mode specified. Create them if they don't exist yet.
	const FName& InternalSettingsGroupName = ModeToInternalName.FindChecked(SettingsMode);
	const TObjectPtr<UMovieGraphQuickRenderModeSettings>* ModeSettings = QuickRenderSettings->ModeSettings.Find(InternalSettingsGroupName);
	if (!ModeSettings || !IsValid(*ModeSettings))
	{
		UMovieGraphQuickRenderModeSettings* NewModeSettings = NewObject<UMovieGraphQuickRenderModeSettings>(QuickRenderSettings, NAME_None, RF_Transactional);
		QuickRenderSettings->ModeSettings.Add(InternalSettingsGroupName, NewModeSettings);

		return NewModeSettings;
	}

	return ModeSettings->Get();
}

#if WITH_EDITOR
void UMovieGraphQuickRenderSettings::SaveSettings(const UMovieGraphQuickRenderSettings* InSettings)
{
	// Settings are saved to a uasset rather than using the ini configuration system. The ini configuration system isn't flexible enough for the
	// needs of Quick Render (for example, it cannot store the variable assignments).
	
	const FString PackageFileName = FPackageName::LongPackageNameToFilename(*QuickRenderSettingsPackagePath, FPackageName::GetAssetPackageExtension());

	// Duplicate the settings into a new package (or the existing one)
	UPackage* NewPackage = CreatePackage(*QuickRenderSettingsPackagePath);
	UMovieGraphQuickRenderSettings* DuplicatedSettings = CastChecked<UMovieGraphQuickRenderSettings>(StaticDuplicateObject(InSettings, NewPackage));
	DuplicatedSettings->SetFlags(RF_Public | RF_Transactional | RF_Standalone);

	// Save the settings out to disk. Turn off the behavior that auto-adds new files to source control.
	UEditorLoadingSavingSettings* SaveSettings = GetMutableDefault<UEditorLoadingSavingSettings>();
	const uint32 bSCCAutoAddNewFiles = SaveSettings->bSCCAutoAddNewFiles;
	SaveSettings->bSCCAutoAddNewFiles = 0;
	const bool bSuccess = SavePackageHelper(NewPackage, *PackageFileName);
	SaveSettings->bSCCAutoAddNewFiles = bSCCAutoAddNewFiles;
	
	if (!bSuccess)
	{
		// SavePackageHelper() will emit warnings if the save was unsuccessful, but log a separate warning for movie pipeline in case warnings are
		// being specifically filtered for LogMovieRenderPipeline.
		UE_LOGF(LogMovieRenderPipeline, Warning, "Unable to save Quick Render settings. Could not save to destination file [%ls].", *PackageFileName);
	}
}

void UMovieGraphQuickRenderSettings::NotifyNeedsSave()
{
	if (OnEnginePreExitHandle.IsValid())
	{
		FCoreDelegates::OnEnginePreExit.Remove(OnEnginePreExitHandle);
	}

	// Queue up saving the settings to a uasset file in the Saved directory. Do this only when the editor is closing, otherwise the package-saving
	// dialog will briefly pop up, which looks glitchy.
	OnEnginePreExitHandle = FCoreDelegates::OnEnginePreExit.AddLambda([this]()
	{
		SaveSettings(this);
	});
}
#endif	// WITH_EDITOR

UMovieGraphQuickRenderModeSettings::UMovieGraphQuickRenderModeSettings()
	: GraphPreset(FSoftObjectPath(TEXT("/MovieRenderPipeline/DefaultQuickRenderGraph.DefaultQuickRenderGraph")))
{
}

void UMovieGraphQuickRenderModeSettings::RefreshVariableAssignments(UMovieGraphQuickRenderModeSettings* InSettings)
{
	if (!InSettings)
	{
		return;
	}
	
	MoviePipeline::RefreshVariableAssignments(InSettings->GraphPreset.LoadSynchronous(), InSettings->GraphVariableAssignments, InSettings);
}

UMovieJobVariableAssignmentContainer* UMovieGraphQuickRenderModeSettings::GetVariableAssignmentsForGraph(const TSoftObjectPtr<UMovieGraphConfig>& InGraphConfigPath) const
{
	for (const TObjectPtr<UMovieJobVariableAssignmentContainer>& VariableAssignments : GraphVariableAssignments)
	{
		if (VariableAssignments->GetGraphConfig() == InGraphConfigPath)
		{
			return VariableAssignments.Get();
		}
	}

	return nullptr;
}

#if WITH_EDITOR
void UMovieGraphQuickRenderModeSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Let the parent settings object know that it is dirty and should queue up a save
	if (UMovieGraphQuickRenderSettings* QuickRenderSettings = GetTypedOuter<UMovieGraphQuickRenderSettings>())
	{
		QuickRenderSettings->NotifyNeedsSave();
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMovieGraphQuickRenderModeSettings::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	// Refresh the variable assignments if the graph preset changes. This is done in PostTransacted() rather than PostEditChangeProperty() so the
	// change to GraphPreset can be picked up when the user changes it directly AND when changed through undo/redo.
	const TArray<FName>& ChangedProperties = TransactionEvent.GetChangedProperties();
	if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UMovieGraphQuickRenderModeSettings, GraphPreset)))
	{
		RefreshVariableAssignments(this);
		OnGraphChangedDelegate.Broadcast();
	}
}
#endif	// WITH_EDITOR
