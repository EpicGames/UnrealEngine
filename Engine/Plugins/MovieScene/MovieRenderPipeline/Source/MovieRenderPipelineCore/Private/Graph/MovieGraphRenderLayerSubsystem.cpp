// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphRenderLayerSubsystem.h"

#include "Components/PrimitiveComponent.h"
#include "Components/SkyAtmosphereComponent.h"
#include "Components/VolumetricCloudComponent.h"
#include "Components/ExponentialHeightFogComponent.h"
#include "Engine/DirectionalLight.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "Engine/PointLight.h"
#include "Engine/RectLight.h"
#include "Engine/SkyLight.h"
#include "Engine/SpotLight.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Materials/MaterialInterface.h"
#include "Modules/ModuleManager.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieRenderPipelineCoreObjectVersion.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/Package.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"

#if WITH_EDITOR
#include "ActorFolderPickingMode.h"
#include "ActorFolderTreeItem.h"
#include "ActorMode.h"
#include "ActorTreeItem.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "ComponentTreeItem.h"
#include "ContentBrowserDataDragDropOp.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserModule.h"
#include "DataLayer/DataLayerDragDropOp.h"
#include "DetailLayoutBuilder.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "DragAndDrop/LevelDragDropOp.h"
#include "Editor.h"
#include "EditorActorFolders.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Graph/MovieGraphSharedWidgets.h"
#include "IContentBrowserSingleton.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerTreeItem.h"
#include "LayersDragDropOp.h"
#include "Layers/LayersSubsystem.h"
#include "MovieGraphUtils.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "SClassViewer.h"
#include "ScopedTransaction.h"
#include "SDropTarget.h"
#include "Selection.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphRenderLayerSubsystem)

#define LOCTEXT_NAMESPACE "MovieGraph"

namespace UE::MovieGraph::Private
{
#if WITH_EDITOR
	/**
	 * A filter that can be used in the class viewer that appears in the Add menu. Filters out specified classes, and optionally filters out classes
	 * that do not have a specific base class.
	 */
	class FClassViewerTypeFilter final : public IClassViewerFilter
	{
	public:
		explicit FClassViewerTypeFilter(TArray<TObjectPtr<UClass>>* InClassesToDisallow, UClass* InRequiredBaseClass = nullptr)
			: ClassesToDisallow(InClassesToDisallow)
			, RequiredBaseClass(InRequiredBaseClass)
		{
			
		}

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return
				InClass &&
				!ClassesToDisallow->Contains(InClass) &&
				(RequiredBaseClass ? InClass->IsChildOf(RequiredBaseClass) : true);
		}
		
		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return false;
		}

	private:
		/** Classes which should be prevented from showing up in the class viewer. */
		TArray<TObjectPtr<UClass>>* ClassesToDisallow;

		/** Classes must have this base class to pass the filter. */
		UClass* RequiredBaseClass = nullptr;
	};

	/** Gets all actors from a scene drag-drop operation (which is assumed to be dragging a folder). */
	void GetActorsFromSceneDragDropOp(const TSharedPtr<FSceneOutlinerDragDropOp> InSceneDragDropOp, TArray<AActor*>& OutActors)
	{
		if (const TSharedPtr<FFolderDragDropOp> FolderOp = InSceneDragDropOp->GetSubOp<FFolderDragDropOp>())
		{
			FActorFolders::GetActorsFromFolders(*FolderOp->World.Get(), FolderOp->Folders, OutActors);
		}
	}
#endif
}

void UMovieGraphMaterialModifier::ApplyModifier(const UWorld* World)
{
	UMaterialInterface* NewMaterial = Material.LoadSynchronous();
	if (!NewMaterial)
	{
		return;
	}

	ModifiedComponents.Empty();
	
	for (const UMovieGraphCollection* Collection : Collections)
	{
		if (!Collection)
		{
			continue;
		}

		FMovieGraphEvaluationResult EvaluationResult = Collection->EvaluateActorsAndComponents(World);

		for (UPrimitiveComponent* PrimitiveComponent : EvaluationResult.GetAllComponentsOfType<UPrimitiveComponent>())
		{
			TArray<FMaterialSlotAssignment>& ModifiedMaterials = ModifiedComponents.FindOrAdd(PrimitiveComponent);
			
			for (int32 Index = 0; Index < PrimitiveComponent->GetNumMaterials(); ++Index)
			{
				ModifiedMaterials.Add(FMaterialSlotAssignment(Index, PrimitiveComponent->GetMaterial(Index)));
			
				PrimitiveComponent->SetMaterial(Index, NewMaterial);
			}
		}
	}
}

void UMovieGraphMaterialModifier::UndoModifier()
{
	for (const FComponentToMaterialMap::ElementType& ModifiedComponent : ModifiedComponents) 
	{
		UPrimitiveComponent* MeshComponent = ModifiedComponent.Key.LoadSynchronous();
		const TArray<FMaterialSlotAssignment>& OldMaterials = ModifiedComponent.Value;

		if (!MeshComponent)
		{
			continue;
		}

		for (const FMaterialSlotAssignment& MaterialPair : OldMaterials)
		{
			UMaterialInterface* MaterialInterface = MaterialPair.Value.LoadSynchronous();
			if (!MaterialInterface)
			{
				continue;
			}

			const int32 ElementIndex = MaterialPair.Key;
			MeshComponent->SetMaterial(ElementIndex, MaterialInterface);
		}
	}

	ModifiedComponents.Empty();
}

FText UMovieGraphMaterialModifier::GetModifierName()
{
	static const FText ModifierName = LOCTEXT("MaterialModifierName", "Material");
	return ModifierName;
}

UMovieGraphRenderPropertyModifier::UMovieGraphRenderPropertyModifier()
	: bIsHidden(false)
	, bCastsShadows(true)
	, bCastShadowWhileHidden(false)
	, bAffectIndirectLightingWhileHidden(false)
	, bHoldout(false)
	, bVisibleInReflections(true)
	, bProcessEditorOnlyActors(false)
{
	// Note: The default modifier values here reflect the defaults on the scene component. If a modifier property is marked as overridden, the
	// override will initially be a no-op due to the defaults being the same.
}

void UMovieGraphRenderPropertyModifier::PostLoad()
{
	Super::PostLoad();

	ValidateProjectSettings();
}

#if WITH_EDITOR
void UMovieGraphRenderPropertyModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphRenderPropertyModifier, bHoldout))
	{
		ValidateProjectSettings();
	}
}
#endif

void UMovieGraphRenderPropertyModifier::ApplyModifier(const UWorld* World)
{
	auto CacheComponentState = [this](USceneComponent* InComponent)
	{
#if WITH_EDITORONLY_DATA
		// Don't bother processing editor-only components (editor billboard icons, text, etc). This behavior can be optionally ignored for niche use cases.
		if (InComponent->IsEditorOnly() && !bProcessEditorOnlyActors)
		{
			return;
		}
#endif // WITH_EDITORONLY_DATA
		
		FComponentState& CachedState = CachedComponentState.AddDefaulted_GetRef();
		CachedState.Component = InComponent;

		// Cache the state
		if (const UPrimitiveComponent* AsPrimitiveComponent = Cast<UPrimitiveComponent>(InComponent))
		{
			CachedState.bCastsShadows = AsPrimitiveComponent->CastShadow;
			CachedState.bCastShadowWhileHidden = AsPrimitiveComponent->bCastHiddenShadow;
			CachedState.bAffectIndirectLightingWhileHidden = AsPrimitiveComponent->bAffectIndirectLightingWhileHidden;
			CachedState.bHoldout = AsPrimitiveComponent->bHoldout;
			CachedState.bVisibleInReflections = AsPrimitiveComponent->bVisibleInReflections;
			CachedState.bIsHidden = AsPrimitiveComponent->bHiddenInGame;
		}
		// Volumetrics are special cases as they don't inherit from UPrimitiveComponent, and don't support all of the flags.
		else if (const UVolumetricCloudComponent* AsVolumetricCloudComponent = Cast<UVolumetricCloudComponent>(InComponent))
		{
			CachedState.bIsHidden = AsVolumetricCloudComponent->bHiddenInGame;
			CachedState.bHoldout = AsVolumetricCloudComponent->bHoldout;
			CachedState.bAffectIndirectLightingWhileHidden = !AsVolumetricCloudComponent->bRenderInMainPass;
		}
		else if (const USkyAtmosphereComponent* AsSkyAtmosphereComponent = Cast<USkyAtmosphereComponent>(InComponent))
		{
			CachedState.bIsHidden = AsSkyAtmosphereComponent->bHiddenInGame;
			CachedState.bHoldout = AsSkyAtmosphereComponent->bHoldout;
			CachedState.bAffectIndirectLightingWhileHidden = !AsSkyAtmosphereComponent->bRenderInMainPass;
		}
		else if (const UExponentialHeightFogComponent* AsExponentialHeightFogComponent = Cast<UExponentialHeightFogComponent>(InComponent))
		{
			CachedState.bIsHidden = AsExponentialHeightFogComponent->bHiddenInGame;
			CachedState.bHoldout = AsExponentialHeightFogComponent->bHoldout;
			CachedState.bAffectIndirectLightingWhileHidden = !AsExponentialHeightFogComponent->bRenderInMainPass;
		}
		else
		{
			// General fallback for other USceneComponent types (e.g., lights, decals) that don't inherit
			// from UPrimitiveComponent. Only bIsHidden is applicable for these component types.
			CachedState.bIsHidden = InComponent->bHiddenInGame;
		}
	};

	auto CacheActorState = [this](AActor* InActor)
	{
		bool bIsActorHidden = InActor->IsHidden();

		// PPV actors use the "enabled" flag for visibility
		if (const APostProcessVolume* PpvActor = Cast<APostProcessVolume>(InActor))
		{
			bIsActorHidden = !PpvActor->bEnabled;
		}
		
		FActorState& CachedState = CachedActorState.AddDefaulted_GetRef();
		CachedState.Actor = InActor;
		CachedState.bIsHidden = bIsActorHidden;
	};
	
	CachedActorState.Empty();
	CachedComponentState.Empty();

	ValidateProjectSettings();
	
	// Cache matching actor/component state for all collections before applying the modifier.
	for (const UMovieGraphCollection* Collection : Collections)
	{
		if (!Collection)
		{
			continue;
		}

		FMovieGraphEvaluationResult EvaluationResult = Collection->EvaluateActorsAndComponents(World);

		CachedActorState.Reserve(EvaluationResult.MatchingActors.Num());
		CachedComponentState.Reserve(EvaluationResult.MatchingComponents.Num());	// Can't really accurately reserve the proper number of elements here

		for (AActor* Actor : EvaluationResult.MatchingActors)
		{
			CacheActorState(Actor);

			constexpr bool bIncludeFromChildActors = true;
			TInlineComponentArray<USceneComponent*> Components(Actor, bIncludeFromChildActors);

			for (USceneComponent* SceneComponent : Components)
			{
				CacheComponentState(SceneComponent);
			}
		}

		for (UActorComponent* Component : EvaluationResult.MatchingComponents)
		{
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
			{
				CacheComponentState(SceneComponent);
			}
		}
	}

	// Actor/component state has been cached. Apply the node's state (settings) to all affected (ie, cached) actors/components.
	constexpr bool bUseStateFromNode = true;
	SetActorAndComponentState({}, {}, bUseStateFromNode);
}

void UMovieGraphRenderPropertyModifier::UndoModifier()
{
	constexpr bool bUseStateFromNode = false;
	SetActorAndComponentState(CachedActorState, CachedComponentState, bUseStateFromNode);

	CachedActorState.Empty();
	CachedComponentState.Empty();
}

FText UMovieGraphRenderPropertyModifier::GetModifierName()
{
	static const FText ModifierName = LOCTEXT("RenderPropertyModifierName", "Visibility");
	return ModifierName;
}

void UMovieGraphRenderPropertyModifier::SetActorAndComponentState(const TArray<FActorState>& InActorState, const TArray<FComponentState>& InComponentState, const bool bUseStateFromNode)
{
	// In most cases, if the hidden state is being modified, the hidden state should be set. However, there is an exception for volumetrics.
	// If volumetrics set the 'Affect Indirect Lighting While Hidden' flag to true, the volumetric component needs to set the 'Render in Main' flag
	// instead, and the 'Hidden' flag should NOT be set on the *actor*. Setting the Hidden flag on the actor in this case will override the behavior
	// of 'Render in Main' and volumetrics will not affect indirect lighting.
	TSet<AActor*> VolumetricComponentsWithAffectIndirect;

	// Volumetrics are a special case and their visibility properties need to be handled separately
	auto SetStateForVolumetrics = [this, &VolumetricComponentsWithAffectIndirect]<typename VolumetricsType>(VolumetricsType& VolumetricsComponent, const FComponentState& NewComponentState)
	{
		if (bOverride_bHoldout)
		{
			VolumetricsComponent->SetHoldout(NewComponentState.bHoldout);
		}

		if (bOverride_bIsHidden)
		{
			VolumetricsComponent->SetRenderInMainPass(!NewComponentState.bIsHidden);
		}

		if (bOverride_bAffectIndirectLightingWhileHidden)
		{
			// If the component should affect indirect while hidden, then we need to use 'Render in Main' instead.
			VolumetricsComponent->SetRenderInMainPass(!NewComponentState.bAffectIndirectLightingWhileHidden);

			// Don't allow the actor to hide itself if this component is not going to be rendered in the main pass. Hiding the actor will
			// negate the effects of setting Render In Main Pass.
			if (NewComponentState.bAffectIndirectLightingWhileHidden)
			{
				VolumetricComponentsWithAffectIndirect.Add(VolumetricsComponent->GetOwner());
			}
		}
	};

	// Sets the state for a component to the specified component state
	auto SetStateForComponent = [this, &SetStateForVolumetrics](UActorComponent* InComponent, const FComponentState& InNewComponentState)
	{
		// TODO: These could potentially cause a large rendering penalty due to dirtying the render state; investigate potential
		// ways to optimize this
		if (UPrimitiveComponent* AsPrimitiveComponent = Cast<UPrimitiveComponent>(InComponent))
		{
			if (bOverride_bCastsShadows)
			{
				AsPrimitiveComponent->SetCastShadow(InNewComponentState.bCastsShadows);
			}

			if (bOverride_bCastShadowWhileHidden)
			{
				AsPrimitiveComponent->SetCastHiddenShadow(InNewComponentState.bCastShadowWhileHidden);
			}

			if (bOverride_bAffectIndirectLightingWhileHidden)
			{
				AsPrimitiveComponent->SetAffectIndirectLightingWhileHidden(InNewComponentState.bAffectIndirectLightingWhileHidden);
			}

			if (bOverride_bHoldout)
			{
				AsPrimitiveComponent->SetHoldout(InNewComponentState.bHoldout);
			}

			if (bOverride_bVisibleInReflections)
			{
				// There's no Set*() method for Visible in Reflections, so MarkRenderStateDirty() needs to be called manually
				if (AsPrimitiveComponent->bVisibleInReflections != InNewComponentState.bVisibleInReflections)
				{
					AsPrimitiveComponent->bVisibleInReflections = InNewComponentState.bVisibleInReflections;
					AsPrimitiveComponent->MarkRenderStateDirty();
				}
			}

			if (bOverride_bIsHidden)
			{
				AsPrimitiveComponent->SetHiddenInGame(InNewComponentState.bIsHidden);
			}
		}
		// Volumetrics are special cases as they don't inherit from UPrimitiveComponent, and don't support all of the flags.
		else if (UVolumetricCloudComponent* AsVolumetricCloudComponent = Cast<UVolumetricCloudComponent>(InComponent))
		{
			SetStateForVolumetrics(AsVolumetricCloudComponent, InNewComponentState);
		}
		else if (USkyAtmosphereComponent* AsSkyAtmosphereComponent = Cast<USkyAtmosphereComponent>(InComponent))
		{
			SetStateForVolumetrics(AsSkyAtmosphereComponent, InNewComponentState);
		}
		else if (UExponentialHeightFogComponent* AsExponentialHeightFogComponent = Cast<UExponentialHeightFogComponent>(InComponent))
		{
			SetStateForVolumetrics(AsExponentialHeightFogComponent, InNewComponentState);
		}
		else if (USceneComponent* AsSceneComponent = Cast<USceneComponent>(InComponent))
		{
			// General fallback for other USceneComponent types (e.g., lights, decals) that don't inherit
			// from UPrimitiveComponent. Only bIsHidden is applicable for these component types.
			if (bOverride_bIsHidden)
			{
				AsSceneComponent->SetHiddenInGame(InNewComponentState.bIsHidden);
			}
		}
	};

	// Sets the state for an actor to the specified actor state
	auto SetStateForActor = [this, &VolumetricComponentsWithAffectIndirect](AActor* InActor, const FActorState& InNewActorState)
	{
		if (VolumetricComponentsWithAffectIndirect.Contains(InActor))
		{
			return;
		}

		// Post-process volumes have an "enabled" flag that we'll use to hide/un-hide them
		if (APostProcessVolume* PpvActor = Cast<APostProcessVolume>(InActor))
		{
			PpvActor->bEnabled = !InNewActorState.bIsHidden;
		}

		InActor->SetActorHiddenInGame(InNewActorState.bIsHidden);
	};

	// If the node's state/settings should be applied, generate a singular FComponentState and apply it to all cached components.
	// Otherwise, apply all of the component states that were provided.
	if (bUseStateFromNode)
	{
		FComponentState NewComponentState;
		NewComponentState.bIsHidden = bIsHidden;
		NewComponentState.bHoldout = bHoldout;
		NewComponentState.bCastsShadows = bCastsShadows;
		NewComponentState.bCastShadowWhileHidden = bCastShadowWhileHidden;
		NewComponentState.bAffectIndirectLightingWhileHidden = bAffectIndirectLightingWhileHidden;
		NewComponentState.bVisibleInReflections = bVisibleInReflections;

		for (const FComponentState& CachedState : CachedComponentState)
		{
			SetStateForComponent(CachedState.Component.Get(), NewComponentState);
		}
	}
	else
	{
		for (const FComponentState& ComponentState : InComponentState)
		{
			if (USceneComponent* Component = ComponentState.Component.Get())
			{
				SetStateForComponent(Component, ComponentState);
			}
		}
	}

	// Do a similar operation for actors; if the node's state should be applied, generate a singular FActorState and apply it to all cached actors.
	// Otherwise, apply all of the actor states that were provided.
	if (bUseStateFromNode)
	{
		FActorState NewActorState;
		NewActorState.bIsHidden = bIsHidden;

		for (const FActorState& CachedState : CachedActorState)
		{
			SetStateForActor(CachedState.Actor.LoadSynchronous(), NewActorState);
		}
	}
	else
	{
		for (const FActorState& ActorState : InActorState)
		{
			if (AActor* Actor = ActorState.Actor.LoadSynchronous())
			{
				SetStateForActor(Actor, ActorState);
			}
		}
	}
}

void UMovieGraphRenderPropertyModifier::ValidateProjectSettings() const
{
	static const FText HoldoutModifierLabel = LOCTEXT("ConditionGroupQueryHoldoutModifier", "Holdout Modifier");

	if (bHoldout)
	{
#if WITH_EDITOR
		UE_CALL_ONCE([]
			{
				constexpr bool bMandatePrimitiveAlphaHoldout = true;
				UE::MovieGraph::ValidateAlphaProjectSettings(HoldoutModifierLabel, bMandatePrimitiveAlphaHoldout);
			}
		);
#else
		UE_LOGF(LogMovieRenderPipeline, Warning, "Both \"Alpha Output\" and \"Support Primitive Alpha Holdout\" project settings must be enabled, otherwise holdout modifiers will not work properly.");
#endif
	}
}

void UMovieGraphCollectionModifier::AddCollection(UMovieGraphCollection* Collection)
{
	// Don't allow adding a duplicate collection
	for (const UMovieGraphCollection* ExistingCollection : Collections)
	{
		if (Collection && ExistingCollection && Collection->GetCollectionName().Equals(ExistingCollection->GetCollectionName()))
		{
			return;
		}
	}
	
	Collections.Add(Collection);
}

void UMovieGraphCollectionModifier::SetCollections(const TArray<UMovieGraphCollection*> InCollections)
{
	Collections = InCollections;
}

TArray<UMovieGraphCollection*> UMovieGraphCollectionModifier::GetCollections() const
{
	return Collections;
}

FText UMovieGraphCollectionModifier::GetModifierName()
{
	// Derived classes should override this method
	static const FText ModifierName = LOCTEXT("CollectionModifierName", "Unknown Collection Modifier");
	return ModifierName;
}

UMovieGraphConditionGroupQueryBase::UMovieGraphConditionGroupQueryBase()
	: OpType(EMovieGraphConditionGroupQueryOpType::Add)
	, bIsEnabled(true)
{
}

void UMovieGraphConditionGroupQueryBase::SetOperationType(const EMovieGraphConditionGroupQueryOpType OperationType)
{
	// Always allow setting the operation type to Union. If not setting to Union, only allow setting the operation type if this is not the first
	// query in the condition group. The first query is always a Union.
	if (OperationType == EMovieGraphConditionGroupQueryOpType::Add)
	{
		OpType = EMovieGraphConditionGroupQueryOpType::Add;
		return;
	}

	const UMovieGraphConditionGroup* ParentConditionGroup = GetTypedOuter<UMovieGraphConditionGroup>();
	if (ensureMsgf(ParentConditionGroup, TEXT("Cannot set the operation type on a condition group query that doesn't have a condition group outer")))
	{
		if (ParentConditionGroup->GetQueries().Find(this) != 0)
		{
			OpType = OperationType;
		}
	}
}

EMovieGraphConditionGroupQueryOpType UMovieGraphConditionGroupQueryBase::GetOperationType() const
{
	return OpType;
}

void UMovieGraphConditionGroupQueryBase::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	// No implementation
}

void UMovieGraphConditionGroupQueryBase::EvaluateActorsAndComponents(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const
{
	// No implementation
}

bool UMovieGraphConditionGroupQueryBase::ShouldHidePropertyNames() const
{
	// Show property names by default; subclassed queries can opt-out if they want a cleaner UI 
	return false;
}

bool UMovieGraphConditionGroupQueryBase::ShouldEvaluateComponents() const
{
	// By default, only actors are evaluated
	return false;
}

const FSlateIcon& UMovieGraphConditionGroupQueryBase::GetIcon() const
{
	static const FSlateIcon EmptyIcon = FSlateIcon();
	return EmptyIcon;
}

const FText& UMovieGraphConditionGroupQueryBase::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName", "Query Base");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQueryBase::GetWidgets()
{
	return TArray<TSharedRef<SWidget>>();
}

bool UMovieGraphConditionGroupQueryBase::HasAddMenu() const
{
	return false;
}

TSharedRef<SWidget> UMovieGraphConditionGroupQueryBase::GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished)
{
	return SNullWidget::NullWidget;
}
#endif

bool UMovieGraphConditionGroupQueryBase::IsEditorOnlyQuery() const
{
	return false;
}

void UMovieGraphConditionGroupQueryBase::SetEnabled(const bool bEnabled)
{
	bIsEnabled = bEnabled;
}

bool UMovieGraphConditionGroupQueryBase::IsEnabled() const
{
	return bIsEnabled;
}

bool UMovieGraphConditionGroupQueryBase::IsFirstConditionGroupQuery() const
{
	const UMovieGraphConditionGroup* ParentConditionGroup = GetTypedOuter<UMovieGraphConditionGroup>();
	if (ensureMsgf(ParentConditionGroup, TEXT("Cannot determine if this is the first condition group query when no parent condition group is present")))
	{
		// GetQueries() returns an array of non-const pointers, so Find() doesn't like having a const pointer passed to it.
		// Find() won't mutate the condition group query though, so the const_cast here is OK.
		return ParentConditionGroup->GetQueries().Find(const_cast<UMovieGraphConditionGroupQueryBase*>(this)) == 0;
	}
	
	return false;
}

/**
 * Finds the counterpart of InSourceActor in InTargetWorld by matching the correct sublevel via
 * outermost package name, then looking up the actor by name within that level.
 *
 * This works more reliably than EditorUtilities::GetSimWorldCounterpartActor / GetEditorWorldCounterpartActor
 * which can return actors from the wrong sublevel when actor names collide across sublevels.
 */
static AActor* FindCounterpartActorInWorld(const AActor* InSourceActor, const UWorld* InTargetWorld)
{
	if (!InSourceActor || !InTargetWorld)
	{
		return nullptr;
	}

	const ULevel* SourceLevel = InSourceActor->GetLevel();
	if (!SourceLevel || !SourceLevel->GetOutermost())
	{
		return nullptr;
	}

	// RemovePIEPrefix is a no-op if the name has no PIE prefix, so this works for both directions.
	const FString SourceLevelPackageName = UWorld::RemovePIEPrefix(SourceLevel->GetOutermost()->GetName());

	for (FConstLevelIterator LevelIt(InTargetWorld->GetLevelIterator()); LevelIt; ++LevelIt)
	{
		if (ULevel* TargetLevel = *LevelIt)
		{
			if (!TargetLevel->GetOutermost())
			{
				continue;
			}

			const FString TargetLevelPackageName = UWorld::RemovePIEPrefix(TargetLevel->GetOutermost()->GetName());
			if (TargetLevelPackageName == SourceLevelPackageName)
			{
				return FindObject<AActor>(TargetLevel, *InSourceActor->GetFName().ToString());
			}
		}
	}

	return nullptr;
}

AActor* UMovieGraphConditionGroupQueryBase::GetActorForCurrentWorld(AActor* InActorToConvert)
{
	if (!InActorToConvert)
	{
		return nullptr;
	}

#if WITH_EDITOR
	// Legacy API: subclasses may still call this without an evaluation world.
	// Guard editor globals because remote MRQ runs editor code without always having an interactive GEditor context.
	if (!GEditor)
	{
		return InActorToConvert;
	}

	const bool bIsPIE = GEditor->IsPlaySessionInProgress();

	const UWorld* ActorWorld = InActorToConvert->GetWorld();
	const bool bIsEditorActor = ActorWorld && ActorWorld->IsEditorWorld();

	// If a PIE session is NOT in progress, make sure that the actor is the editor equivalent
	if (!bIsPIE)
	{
		// Only do PIE -> editor actor conversion when the actor is NOT from the editor
		if (!bIsEditorActor)
		{
			return GetActorForCurrentWorld(InActorToConvert, GEditor->EditorWorld);
		}

		// Just use InActorToConvert as-is if it's not from PIE
		return InActorToConvert;
	}

	// If a PIE session IS active, try to get the PIE equivalent of the editor actor
	else
	{
		// Only do editor -> PIE actor conversion when the actor is from an editor world
		if (bIsEditorActor)
		{
			return GetActorForCurrentWorld(InActorToConvert, GEditor->PlayWorld);
		}

		// Just use InActorToConvert as-is if it's not from an editor actor
		return InActorToConvert;
	}
#else
	return InActorToConvert;
#endif
}

AActor* UMovieGraphConditionGroupQueryBase::GetActorForCurrentWorld(AActor* InActorToConvert, const UWorld* InTargetWorld)
{
	if (!InActorToConvert || !InTargetWorld)
	{
		return nullptr;
	}

	if (InActorToConvert->GetWorld() == InTargetWorld)
	{
		return InActorToConvert;
	}

	// Prefer the explicit evaluation/render world over global PIE/editor state. Remote MRQ may not have usable editor globals.
	return FindCounterpartActorInWorld(InActorToConvert, InTargetWorld);
}

bool FMovieGraphActorQueryEntry::operator==(const FMovieGraphActorQueryEntry& InOther) const
{
	return (InOther.ActorToMatch == ActorToMatch) && (InOther.ComponentsToMatch == ComponentsToMatch);
}

void UMovieGraphConditionGroupQuery_Actor::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	TSet<UActorComponent*> UnusedComponentsArray;
	EvaluateActorsAndComponents(InActorsToQuery, InWorld, OutMatchingActors, UnusedComponentsArray);
}

void UMovieGraphConditionGroupQuery_Actor::EvaluateActorsAndComponents(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphConditionGroupQuery_Actor::EvaluateActorsAndComponents);

	// Note: This query does not need to search through InActorsToQuery because it has direct references to actors/components
	for (const FMovieGraphActorQueryEntry& Entry : ActorsAndComponentsToMatch)
	{
		if (AActor* ConvertedActor = GetActorForCurrentWorld(Entry.ActorToMatch.Get(), InWorld))
		{
			if (!Entry.ComponentsToMatch.IsEmpty())
			{
				// Instead of matching the actor, get the components requested from the actor in the correct world.
				// Since the number of components here is typically going to be very small, an array should be more efficient vs. a set when used in the matching loop below.
				TArray<FString> ComponentNamesToMatch;
				ComponentNamesToMatch.Reserve(Entry.ComponentsToMatch.Num());
				Algo::Transform(Entry.ComponentsToMatch, ComponentNamesToMatch, [](const TSoftObjectPtr<UActorComponent>& InComponent) { return InComponent.IsValid() ? InComponent->GetName() : FString(); });

				for (UActorComponent* ActorComponent : ConvertedActor->GetComponents())
				{
					if (ActorComponent && ComponentNamesToMatch.Contains(ActorComponent->GetName()))
					{
						OutMatchingComponents.Add(ActorComponent);
					}
				}
			}
			else
			{
				OutMatchingActors.Add(ConvertedActor);
			}
		}
	}
}

bool UMovieGraphConditionGroupQuery_Actor::ShouldEvaluateComponents() const
{
	return true;
}

const FSlateIcon& UMovieGraphConditionGroupQuery_Actor::GetIcon() const
{
	static const FSlateIcon ActorIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Actor");
	return ActorIcon;
}

const FText& UMovieGraphConditionGroupQuery_Actor::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_Actor", "Actor");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_Actor::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	// Create the data source for the list view
	RefreshListDataSource();

	auto GetValidActorsFromOperation = [](TSharedPtr<FDragDropOperation> InOperation, TArray<AActor*>& OutActors, bool& bHadTransient) {

		// Support dragging both actors and folders from the Outliner (dragging a folder will add all actors in the folder)
		if (InOperation->IsOfType<FActorDragDropGraphEdOp>())
		{
			const TSharedPtr<FActorDragDropGraphEdOp> ActorOperation = StaticCastSharedPtr<FActorDragDropGraphEdOp>(InOperation);
			Algo::Transform(ActorOperation->Actors, OutActors, [](const TWeakObjectPtr<AActor>& Actor) { return Actor.IsValid() ? Actor.Get() : nullptr; });
		}

		if (InOperation->IsOfType<FSceneOutlinerDragDropOp>())
		{
			const TSharedPtr<FSceneOutlinerDragDropOp> SceneOperation = StaticCastSharedPtr<FSceneOutlinerDragDropOp>(InOperation);
			UE::MovieGraph::Private::GetActorsFromSceneDragDropOp(SceneOperation, OutActors);
		}

		// Prevent any transient actors (ie: spawnables) from being added as they won't exist later
		bHadTransient = false;
		for (int32 Index = OutActors.Num() - 1; Index >= 0; Index--)
		{
			if(OutActors[Index]->HasAnyFlags(RF_Transient))
			{
				OutActors.RemoveAt(Index);
				bHadTransient = true;
			}
		}
	};

	// Generate the header row for the list widget
	const TSharedPtr<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		.Visibility(EVisibility::Visible);
	auto AddColumnToHeader = [HeaderRow](const FName& InColumnId, const FText& InColumnText)
	{
		HeaderRow->AddColumn(
			SHeaderRow::Column(InColumnId)
			.HAlignCell(HAlign_Left)
			.HAlignHeader(HAlign_Left)
			.VAlignCell(VAlign_Fill)
			.HeaderContentPadding(FMargin(4.0f))
			.HeaderContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(InColumnText)
			]
		);
	};
	AddColumnToHeader(ColumnID_ActorName, LOCTEXT("ActorConditionGroupQuery_ActorColumnName", "Actor Name"));
	AddColumnToHeader(ColumnID_ActorType, LOCTEXT("ActorConditionGroupQuery_ActorTypeColumnName", "Actor Type"));
	AddColumnToHeader(ColumnID_Components, LOCTEXT("ActorConditionGroupQuery_ComponentsColumnName", "Components"));

	Widgets.Add(
		SNew(SDropTarget)
		.OnAllowDrop_Lambda([GetValidActorsFromOperation](TSharedPtr<FDragDropOperation> InDragOperation)
		{
				TArray<AActor*> DroppedActors;
				bool bHadTransient;
				GetValidActorsFromOperation(InDragOperation, DroppedActors, bHadTransient);

				return DroppedActors.Num() > 0;
		})
		.OnDropped_Lambda([this, GetValidActorsFromOperation](const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
		{
			TArray<AActor*> DroppedActors;
			bool bHadTransient;

			GetValidActorsFromOperation(DragDropEvent.GetOperation(), DroppedActors, bHadTransient);
			if (bHadTransient)
			{
				// If we go this far, we have some non-spawnables that will actually get added to the list,
				// but apparently they also had a transient actor selected, so we'll toast notify them that
				// that one in particular won't be added.
				FNotificationInfo Info(LOCTEXT("TransientActorsUnsupported_Notification", "Actor Conditions do not support Spawnable (Transient) actors"));
				Info.SubText = LOCTEXT("TransientActorsUnsupported_NotificationSubtext", "Use the \"Actor Name\" Condition to add Spawnable actors to Collections.");
				Info.Image = FAppStyle::GetBrush(TEXT("Icons.Warning"));

				//Set a default expire duration
				Info.ExpireDuration = 5.0f;

				FSlateNotificationManager::Get().AddNotification(Info);
			}

			const FMovieGraphConditionGroupQueryContentsChanged OnAddFinished = nullptr;
			AddActors(DroppedActors, OnAddFinished);

			return FReply::Handled();
		})
		[
			SAssignNew(ActorsList, SMovieGraphSimpleList<TSharedPtr<FMovieGraphActorQueryEntry>>)
			.DataSource(&ListDataSource)
			.HeaderRow(HeaderRow)
			.OnGetCustomRow_Lambda([this](const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FMovieGraphActorQueryEntry> InEntry)
			{
				return SNew(SActorListRow, InOwnerTable, InEntry);
			})
			.CustomSummaryWidget_Lambda([this]()
			{
				return SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text_Lambda([this]()
					{
						int32 NumActors = 0;
						int32 NumComponents = 0;
						
						for (const FMovieGraphActorQueryEntry& Entry : ActorsAndComponentsToMatch)
						{
							if (!Entry.ComponentsToMatch.IsEmpty())
							{
								NumComponents += Entry.ComponentsToMatch.Num();
							}
							else
							{
								NumActors++;
							}
						}

						return FText::Format(LOCTEXT("ActorQuery_SummaryRowText", "{0} Actor(s) and {1} Component(s)"), NumActors, NumComponents);
					});
			})
			.OnDelete_Lambda([this](const TArray<TSharedPtr<FMovieGraphActorQueryEntry>> InEntries)
			{
				TArray<FMovieGraphActorQueryEntry> ActorsToRemove;
				for (const TSharedPtr<FMovieGraphActorQueryEntry>& Entry : InEntries)
				{
					if (Entry.IsValid())
					{
						ActorsToRemove.Add(*Entry.Get());
					}
				}
				
				RemoveEntries(ActorsToRemove);
			})
			.OnGetContextMenuContent_UObject(this, &UMovieGraphConditionGroupQuery_Actor::GetListContextMenu)
			.OnRefreshDataSourceRequested_UObject(this, &UMovieGraphConditionGroupQuery_Actor::RefreshListDataSource)
		]
	);

	return Widgets;
}

bool UMovieGraphConditionGroupQuery_Actor::HasAddMenu() const
{
	return true;
}

TSharedRef<SWidget> UMovieGraphConditionGroupQuery_Actor::GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished)
{	
	FSceneOutlinerInitializationOptions SceneOutlinerInitOptions;
	SceneOutlinerInitOptions.bShowHeaderRow = true;
	SceneOutlinerInitOptions.bShowSearchBox = true;
	SceneOutlinerInitOptions.bShowCreateNewFolder = false;
	SceneOutlinerInitOptions.bFocusSearchBoxWhenOpened = true;


	// Show the custom "Add" column, as well as the built-in name/label/type columns
	SceneOutlinerInitOptions.ColumnMap.Add(
		FActorSelectionColumn::GetID(),
		FSceneOutlinerColumnInfo(
			ESceneOutlinerColumnVisibility::Visible,
			0,
			FCreateSceneOutlinerColumn::CreateLambda([this](ISceneOutliner& InSceneOutliner)
			{
				return MakeShared<FActorSelectionColumn>(MakeWeakObjectPtr(this));
			}),
			false,
			TOptional<float>()));
	SceneOutlinerInitOptions.ColumnMap.Add(
		FSceneOutlinerBuiltInColumnTypes::Label(),
		FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 1, FCreateSceneOutlinerColumn(), false, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Label_Localized()));
	SceneOutlinerInitOptions.ColumnMap.Add(
		FSceneOutlinerBuiltInColumnTypes::ActorInfo(),
		FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10,  FCreateSceneOutlinerColumn(), false, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::ActorInfo_Localized()));
	
	const FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	FMenuBuilder MenuBuilder(false, MakeShared<FUICommandList>());

	auto CanExecuteAction = []() -> bool {
		// Assume we have only transient actors until we prove we don't.
		bool bHasNonTransientActorsSelected = false;
		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

		for (AActor* SelectedActor : SelectedActors)
		{
			if (!SelectedActor->HasAnyFlags(RF_Transient))
			{
				bHasNonTransientActorsSelected = true;
				break;
			}
		}
		return bHasNonTransientActorsSelected;
	};
	
	MenuBuilder.BeginSection("AddActor", LOCTEXT("AddActor", "Add Actor"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AddSelectedInOutliner", "Add Selected In Outliner"),
			TAttribute<FText>::CreateLambda([CanExecuteAction]()
				{
					bool bCanExecute = CanExecuteAction();
					if (bCanExecute)
					{
						return LOCTEXT("AddSelectedInOutlinerTooltip_Success", "Add actors currently selected in the level editor's scene outliner.");
					}
					else
					{
						return LOCTEXT("AddSelectedInOutlinerTooltip_Failure", "Actor Conditions do not support Spawnable (Transient) actors.  Select one or more non-transient actors or use the \"Actor Name\" Condition to add Spawnable actors to Collections.");
					}
				}),
			FSlateIcon(FAppStyle::GetAppStyleSetName(),"FoliageEditMode.SetSelect"),
			FUIAction(
				FExecuteAction::CreateLambda([this, OnAddFinished]()
				{
					const FScopedTransaction Transaction(LOCTEXT("AddSelectedActorsToCollection", "Add Selected Actors to Collection"));
					
					TArray<AActor*> SelectedActors;
					GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

					for (int32 Index = SelectedActors.Num() - 1; Index >= 0; Index--)
					{
						if(SelectedActors[Index]->HasAnyFlags(RF_Transient))
						{
							SelectedActors.RemoveAt(Index);
						}
					}

					AddActors(SelectedActors, OnAddFinished);
				}),
				FCanExecuteAction::CreateLambda([CanExecuteAction]()
				{
					return CanExecuteAction();
				})
			)
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Browse", LOCTEXT("Browse", "Browse"));
	{
		ActorPickerWidget = SceneOutlinerModule.CreateActorPicker(
			SceneOutlinerInitOptions,
			FOnActorPicked::CreateLambda([this, OnAddFinished](AActor* InActor)
			{
				AddActors({InActor}, OnAddFinished);
			}));

		const TSharedRef<SBox> ActorPickerWidgetBox =
			SNew(SBox)
			.WidthOverride(400.f)
			.HeightOverride(300.f)
			[
				ActorPickerWidget.ToSharedRef()
			];

		MenuBuilder.AddWidget(ActorPickerWidgetBox, FText());
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

#endif

void UMovieGraphConditionGroupQuery_Actor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Upgrade the old ActorsToMatch (Actors-only) to ActorsAndComponentsToMatch (Actors + Components)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!ActorsToMatch.IsEmpty())
	{
		for (const TSoftObjectPtr<AActor>& Actor : ActorsToMatch)
		{
			FMovieGraphActorQueryEntry NewEntry;
			NewEntry.ActorToMatch = Actor;
			
			ActorsAndComponentsToMatch.Add(MoveTemp(NewEntry));
		}

		ActorsToMatch.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif	// WITH_EDITOR
}

#if WITH_EDITOR
void UMovieGraphConditionGroupQuery_Actor::AddActors(const TArray<AActor*>& InActors, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished, const bool bCloseAddMenu)
{
	const FScopedTransaction Transaction(LOCTEXT("AddActorsToCollection", "Add Actors to Collection"));
	Modify();
	
	for (AActor* Actor : InActors)
	{
		if (Actor && !ActorsAndComponentsToMatch.ContainsByPredicate([Actor](const FMovieGraphActorQueryEntry& InEntry) { return InEntry.ActorToMatch == Actor; }))
		{
			FMovieGraphActorQueryEntry NewEntry;
			NewEntry.ActorToMatch = Actor;
			ActorsAndComponentsToMatch.Add(MoveTemp(NewEntry));
			ListDataSource.Add(MakeShared<FMovieGraphActorQueryEntry>(ActorsAndComponentsToMatch.Last()));
			
			InOnAddFinished.ExecuteIfBound();
		}
	}

	// Ensure that the actor picker filter runs again so duplicate actors cannot be selected
	if (ActorPickerWidget.IsValid())
	{
		ActorPickerWidget->FullRefresh();
	}
	
	ActorsList->Refresh();

	if (bCloseAddMenu)
	{
		FSlateApplication::Get().DismissAllMenus();
	}
}

void UMovieGraphConditionGroupQuery_Actor::RemoveEntries(const TArray<FMovieGraphActorQueryEntry>& InEntries)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveActorsFromCollection", "Remove Actors from Collection"));
	Modify();
	
	for (const FMovieGraphActorQueryEntry& Entry : InEntries)
	{
		ListDataSource.RemoveAll([&Entry](const TSharedPtr<FMovieGraphActorQueryEntry>& ListEntry)
		{
			if (const FMovieGraphActorQueryEntry* ListEntryPtr = ListEntry.Get())
			{
				return *ListEntryPtr == Entry;
			}
			return false;
		});
		
		ActorsAndComponentsToMatch.Remove(Entry);
	}
	
	ActorsList->Refresh();
}

void UMovieGraphConditionGroupQuery_Actor::GetListContextMenu(FMenuBuilder& InMenuBuilder, TArray<TSharedPtr<FMovieGraphActorQueryEntry>> SelectedEntries)
{
	if (SelectedEntries.IsEmpty() || (SelectedEntries.Num() > 1))
	{
		return;
	}
	
	const TSharedPtr<FMovieGraphActorQueryEntry> SelectedEntry = SelectedEntries[0];
	if (!SelectedEntry.IsValid())
	{
		return;
	}


	// Get the associated actor for this entry
	const AActor* AssociatedActor = SelectedEntry.Get()->ActorToMatch.Get();
	if (!AssociatedActor)
	{
		return;
	}

	// Find the entry in the query associated with this actor
	FMovieGraphActorQueryEntry* Entry = ActorsAndComponentsToMatch.FindByPredicate([AssociatedActor](const FMovieGraphActorQueryEntry& InEntry) { return InEntry.ActorToMatch == AssociatedActor; });
	if (!Entry)
	{
		return;
	}

	InMenuBuilder.BeginSection(NAME_None, LOCTEXT("ActorQuery_MatchTypeSection", "Only Match"));

	TInlineComponentArray<UActorComponent*> ActorComponents;
	AssociatedActor->GetComponents(ActorComponents);

	if (!ActorComponents.IsEmpty())
	{
		const FUIAction ToggleSelectedActor(
			FExecuteAction::CreateLambda([this, Entry]()
			{
				if (!Entry->ComponentsToMatch.IsEmpty())
				{
					const FScopedTransaction Transaction(LOCTEXT("ToggleSelectedActorInCollection", "Toggle Selected Actor in Collection"));
					Modify();
					
					Entry->ComponentsToMatch.Empty();
					ActorsList->Refresh();
				}
			}),
			FCanExecuteAction::CreateLambda([Entry]() { return !Entry->ComponentsToMatch.IsEmpty(); }),
			FIsActionChecked::CreateLambda([Entry]()
			{
				return Entry->ComponentsToMatch.IsEmpty();
			})
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ActorQuery_ToggleSelectedActor", "Selected Actor"),
			LOCTEXT("ActorQuery_ToggleSelectedActorTooltip", "Match the selected actor, or some component(s) contained in it."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Actor"),
			ToggleSelectedActor,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddSeparator();
	}
	
	for (UActorComponent* ActorComponent : ActorComponents)
	{
		// Exclude editor-only components (editor billboard icons, text, etc)
		if (ActorComponent->IsEditorOnly())
		{
			continue;
		}
		
		FUIAction AddRemoveComponentAction(
			FExecuteAction::CreateLambda([this, Entry, ActorComponent]()
			{
				const FScopedTransaction Transaction(LOCTEXT("ToggleComponentInCollection", "Toggle Component in Collection"));
				Modify();
				
				if (Entry->ComponentsToMatch.Contains(ActorComponent))
				{
					Entry->ComponentsToMatch.Remove(ActorComponent);
				}
				else
				{
					Entry->ComponentsToMatch.Add(ActorComponent);
				}

				ActorsList->Refresh();
			}),
			FCanExecuteAction::CreateLambda([]() { return true; }),
			FIsActionChecked::CreateLambda([Entry, ActorComponent]() {
				return Entry->ComponentsToMatch.Contains(ActorComponent);
			})
		);
		
		InMenuBuilder.AddMenuEntry(
			FText::FromString(ActorComponent->GetName()),
			LOCTEXT("ActorQuery_AddComponentTooltip", "Restrict to only matching the selected component types."),
			FSlateIconFinder::FindIconForClass(ActorComponent->GetClass()),
			AddRemoveComponentAction,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	
	InMenuBuilder.EndSection();
}

void UMovieGraphConditionGroupQuery_Actor::RefreshListDataSource()
{
	ListDataSource.Empty();
	for (FMovieGraphActorQueryEntry& Entry : ActorsAndComponentsToMatch)
	{
		ListDataSource.Add(MakeShared<FMovieGraphActorQueryEntry>(Entry));
	}
}

FName UMovieGraphConditionGroupQuery_Actor::FActorSelectionColumn::GetID()
{
	static const FName ColumnId = FName("ActorSelection");
	return ColumnId;
}

FName UMovieGraphConditionGroupQuery_Actor::FActorSelectionColumn::GetColumnID()
{
	return GetID();
}

SHeaderRow::FColumn::FArguments UMovieGraphConditionGroupQuery_Actor::FActorSelectionColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(25.f)
		[
			SNew(SSpacer)
		];
}

const TSharedRef<SWidget> UMovieGraphConditionGroupQuery_Actor::FActorSelectionColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	const FActorTreeItem* ActorTreeItem = TreeItem->CastTo<FActorTreeItem>();
	if (!ActorTreeItem)
	{
		return SNew(SSpacer);
	}
	
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FActorSelectionColumn::IsRowChecked, ActorTreeItem)
			.OnCheckStateChanged(this, &FActorSelectionColumn::OnCheckStateChanged, ActorTreeItem)
		];
}

ECheckBoxState UMovieGraphConditionGroupQuery_Actor::FActorSelectionColumn::IsRowChecked(const FActorTreeItem* InActorTreeItem) const
{
	if (!WeakActorQuery.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	const TStrongObjectPtr<UMovieGraphConditionGroupQuery_Actor> ActorQuery = WeakActorQuery.Pin();
	for (FMovieGraphActorQueryEntry& Entry : ActorQuery->ActorsAndComponentsToMatch)
	{
		if (Entry.ActorToMatch.Get() == InActorTreeItem->Actor.Get())
		{
			return ECheckBoxState::Checked;
		}
	}
				
	return ECheckBoxState::Unchecked;
}

void UMovieGraphConditionGroupQuery_Actor::FActorSelectionColumn::OnCheckStateChanged(const ECheckBoxState NewState, const FActorTreeItem* InActorTreeItem) const
{
	if (!WeakActorQuery.IsValid())
	{
		return;
	}

	const TStrongObjectPtr<UMovieGraphConditionGroupQuery_Actor> ActorQuery = WeakActorQuery.Pin();
	if (InActorTreeItem->Actor.IsValid())
	{
		if (NewState == ECheckBoxState::Unchecked)
		{
			FMovieGraphActorQueryEntry TempEntry;
			TempEntry.ActorToMatch = InActorTreeItem->Actor.Get();
			
			ActorQuery->RemoveEntries({TempEntry});
		}
		else
		{
			const FMovieGraphConditionGroupQueryContentsChanged OnAddFinished = nullptr;
			constexpr bool bCloseAddMenu = false;
			ActorQuery->AddActors({InActorTreeItem->Actor.Get()}, OnAddFinished, bCloseAddMenu);
		}
	}
}

void UMovieGraphConditionGroupQuery_Actor::SActorListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FMovieGraphActorQueryEntry> InEntry)
{
	WeakEntry = InEntry;

	SMultiColumnTableRow<TSharedPtr<FMovieGraphActorQueryEntry>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> UMovieGraphConditionGroupQuery_Actor::SActorListRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	const TSharedPtr<FMovieGraphActorQueryEntry> EntryPin = WeakEntry.Pin();
	if (!EntryPin.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const TWeakObjectPtr<const AActor> WeakActor = MakeWeakObjectPtr(EntryPin->ActorToMatch.Get());
	
	if (InColumnName == ColumnID_ActorName)
	{
		const FSlateBrush* ActorIcon = WeakActor.IsValid()
			? FSlateIconFinder::FindIconForClass(WeakActor.Get()->GetClass()).GetIcon()
			: FSlateIconFinder::FindIconForClass(AActor::StaticClass()).GetIcon();
		
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 5.0f, 2.0f, 5.0f)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(ActorIcon)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 5.0f)
			[
				SNew(STextBlock)
				.Text_Lambda([WeakActor]()
				{
					return WeakActor.IsValid()
						? FText::FromString(WeakActor->GetActorNameOrLabel())
						: LOCTEXT("MovieGraphActorConditionGroupQuery_InvalidActor", "(invalid)");
				})
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	if (InColumnName == ColumnID_ActorType)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 5.0f)
			[
				SNew(STextBlock)
				.Text_Lambda([WeakActor]()
				{
					return WeakActor.IsValid() ? FText::FromString(WeakActor->GetClass()->GetName()) : FText::GetEmpty();
				})
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	if (InColumnName == ColumnID_Components)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(5.0f, 5.0f)
			[
				SNew(STextBlock)
				.Text_Lambda([EntryPin]()
				{
					FText ComponentText;
					if (EntryPin->ComponentsToMatch.Num() == 1)
					{
						if (const UActorComponent* ActorComponent = EntryPin->ComponentsToMatch[0].Get())
						{
							ComponentText = FText::FromString(ActorComponent->GetName());
						}
					}
					else if (EntryPin->ComponentsToMatch.Num() > 1)
					{
						ComponentText = FText::Format(LOCTEXT("ActorQuery_ComponentColumnComponentCount", "{0} Components"), EntryPin->ComponentsToMatch.Num());
					}

					return ComponentText;
				})
				.ToolTipText_Lambda([EntryPin]()
				{
					TArray<FString> ComponentNames;
					for (const TSoftObjectPtr<UActorComponent>& ActorComponent : EntryPin->ComponentsToMatch)
					{
						if (ActorComponent.IsValid())
						{
							ComponentNames.Add(ActorComponent->GetName());
						}
					}

					return FText::Format(LOCTEXT("ActorQuery_SelectedComponentsTooltip", "Selected components: {0}"),
						FText::FromString(FString::Join(ComponentNames, TEXT(", "))));
				})
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	return SNullWidget::NullWidget;
}
#endif	// WITH_EDITOR

void UMovieGraphConditionGroupQuery_ActorTagName::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphConditionGroupQuery_ActorTag::Evaluate);
	
	// Quick early-out if "*" is used as the wildcard. Faster than doing the wildcard matching.
	if (TagsToMatch == TEXT("*"))
	{
		OutMatchingActors.Append(InActorsToQuery);
		return;
	}

	// Actor tags can be specified on multiple lines
	TArray<FString> AllTagNameStrings;
	TagsToMatch.ParseIntoArrayLines(AllTagNameStrings);

	for (AActor* Actor : InActorsToQuery)
	{
		for (const FString& TagToMatch : AllTagNameStrings)
		{
			bool bMatchedTag = false;
			
			for (const FName& ActorTag : Actor->Tags)
			{
				if (ActorTag.ToString().MatchesWildcard(TagToMatch))
				{
					OutMatchingActors.Add(Actor);
					bMatchedTag = true;
					break;
				}
			}

			// Skip comparing the rest of the tags if one tag already matched 
			if (bMatchedTag)
			{
				break;
			}
		}
	}
}

const FSlateIcon& UMovieGraphConditionGroupQuery_ActorTagName::GetIcon() const
{
	static const FSlateIcon ActorTagIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.OpenIssueTracker");
	return ActorTagIcon;
}

const FText& UMovieGraphConditionGroupQuery_ActorTagName::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_ActorTagName", "Actor Tag Name");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_ActorTagName::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	Widgets.Add(
		SNew(SBox)
		.HAlign(HAlign_Fill)
		.Padding(7.f, 2.f)
		[
			SNew(SMultiLineEditableTextBox)
			.Text_Lambda([this]() { return FText::FromString(TagsToMatch); })
			.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type TextCommitType)
			{
				const FScopedTransaction Transaction(LOCTEXT("UpdateActorTagNamesInCollection", "Update Actor Tag Names in Collection"));
				Modify();
				
				TagsToMatch = InText.ToString();
			})
			.HintText(LOCTEXT("MovieGraphActorTagNameQueryHintText", "The actor must match one or more tags. Wildcards allowed.\nEnter each tag on a separate line."))
		]
	);

	return Widgets;
}
#endif

void UMovieGraphConditionGroupQuery_ActorName::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphConditionGroupQuery_ActorName::Evaluate);
	
	// Quick early-out if "*" is used as the wildcard. Faster than doing the wildcard matching.
	if (WildcardSearch == TEXT("*"))
	{
		OutMatchingActors.Append(InActorsToQuery);
		return;
	}

	// Actor names can be specified on multiple lines
	TArray<FString> AllActorNames;
	WildcardSearch.ParseIntoArrayLines(AllActorNames);

	for (AActor* Actor : InActorsToQuery)
	{
#if WITH_EDITOR
		for (const FString& ActorName : AllActorNames)
		{
			if (Actor->GetActorLabel().MatchesWildcard(ActorName))
			{
				OutMatchingActors.Add(Actor);
			}
		}
#endif
	}
}

const FSlateIcon& UMovieGraphConditionGroupQuery_ActorName::GetIcon() const
{
	static const FSlateIcon ActorTagIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.TextRenderActor");
	return ActorTagIcon;
}

const FText& UMovieGraphConditionGroupQuery_ActorName::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_ActorName", "Actor Name");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_ActorName::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	Widgets.Add(
		SNew(SDropTarget)
		.OnAllowDrop_Lambda([](TSharedPtr<FDragDropOperation> InDragOperation)
		{
			// Support dragging both actors and folders from the Outliner (dragging a folder will add all actors in the folder)
			return InDragOperation->IsOfType<FActorDragDropGraphEdOp>() || InDragOperation->IsOfType<FSceneOutlinerDragDropOp>();
		})
		.OnDropped_Lambda([this](const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
		{
			TArray<AActor*> DroppedActors;
			
			if (const TSharedPtr<FActorDragDropGraphEdOp> ActorOperation = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>())
			{
				Algo::Transform(ActorOperation->Actors, DroppedActors, [](const TWeakObjectPtr<AActor>& Actor) { return Actor.IsValid() ? Actor.Get() : nullptr; });
			}

			if (const TSharedPtr<FSceneOutlinerDragDropOp> SceneOperation = DragDropEvent.GetOperationAs<FSceneOutlinerDragDropOp>())
			{
				UE::MovieGraph::Private::GetActorsFromSceneDragDropOp(SceneOperation, DroppedActors);
			}
			
			const FScopedTransaction Transaction(LOCTEXT("UpdateActorNamesInCollection", "Update Actor Names in Collection"));
			Modify();

			for (const AActor* DroppedActor : DroppedActors)
			{
				TArray<FString> ActorStrings;
				WildcardSearch.ParseIntoArrayLines(ActorStrings);
				
				// Only add the actor if it's not in the list already
				if (!ActorStrings.Contains(DroppedActor->GetActorLabel()))
				{
					const FString LineSeparator = WildcardSearch.IsEmpty() ? FString() : LINE_TERMINATOR;
					WildcardSearch += LineSeparator + DroppedActor->GetActorLabel();
				}
			}

			return FReply::Handled();
		})
		[
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.Padding(7.f, 2.f)
			[
				SNew(SMultiLineEditableTextBox)
				.Text_Lambda([this]() { return FText::FromString(WildcardSearch); })
				.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type TextCommitType)
				{
					const FScopedTransaction Transaction(LOCTEXT("UpdateActorNamesInCollection", "Update Actor Names in Collection"));
					Modify();

					WildcardSearch = InText.ToString();
				})
				.HintText(LOCTEXT("MovieGraphActorNameQueryHintText", "Actor names to query. Wildcards allowed.\nEnter each actor name on a separate line."))
			]
		]
	);

	return Widgets;
}
#endif

bool UMovieGraphConditionGroupQuery_ActorName::IsEditorOnly() const
{
	// GetActorLabel() is editor-only
	return true;
}

void UMovieGraphConditionGroupQuery_ActorType::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphConditionGroupQuery_ActorType::Evaluate);
	
	for (AActor* Actor : InActorsToQuery)
	{
		if (ActorTypes.Contains(Actor->GetClass()))
		{
			OutMatchingActors.Add(Actor);
		}
	}
}

const FSlateIcon& UMovieGraphConditionGroupQuery_ActorType::GetIcon() const
{
	static const FSlateIcon ActorTagIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.ActorComponent");
	return ActorTagIcon;
}

const FText& UMovieGraphConditionGroupQuery_ActorType::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_ActorType", "Actor Type");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_ActorType::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	Widgets.Add(
		SNew(SDropTarget)
		.OnAllowDrop_Lambda([](TSharedPtr<FDragDropOperation> InDragOperation)
		{
			// Support dragging both actors and folders from the Outliner (dragging a folder will add all actor types in the folder)
			return InDragOperation->IsOfType<FActorDragDropGraphEdOp>() || InDragOperation->IsOfType<FSceneOutlinerDragDropOp>();
		})
		.OnDropped_Lambda([this](const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
		{
			TArray<UClass*> DroppedActorClasses;
			
			if (const TSharedPtr<FActorDragDropGraphEdOp> ActorOperation = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>())
			{
				Algo::Transform(ActorOperation->Actors, DroppedActorClasses, [](const TWeakObjectPtr<AActor>& Actor) { return Actor.IsValid() ? Actor.Get()->GetClass() : nullptr; });
			}

			if (const TSharedPtr<FSceneOutlinerDragDropOp> SceneOperation = DragDropEvent.GetOperationAs<FSceneOutlinerDragDropOp>())
			{
				TArray<AActor*> DroppedActors;
				UE::MovieGraph::Private::GetActorsFromSceneDragDropOp(SceneOperation, DroppedActors);

				Algo::Transform(DroppedActors, DroppedActorClasses, [](const AActor* Actor) { return Actor ? Actor->GetClass() : nullptr; });
			}
			
			const FMovieGraphConditionGroupQueryContentsChanged OnAddFinished = nullptr;
			AddActorTypes(DroppedActorClasses, OnAddFinished);

			return FReply::Handled();
		})
		[
			SAssignNew(ActorTypesList, SMovieGraphSimpleList<TObjectPtr<UClass>>)
			.DataSource(&ActorTypes)
			.DataType(FText::FromString("Actor Type"))
			.DataTypePlural(FText::FromString("Actor Types"))
			.OnGetRowText_Static(&GetRowText)
			.OnGetRowIcon_Static(&GetRowIcon)
			.OnDelete_Lambda([this](const TArray<UClass*> InActorClasses)
			{
				const FScopedTransaction Transaction(LOCTEXT("RemoveActorTypesFromCollection", "Remove Actor Types from Collection"));
				Modify();

				for (UClass* ActorClass : InActorClasses)
				{
					ActorTypes.Remove(ActorClass);
				}
				
				ActorTypesList->Refresh();
			})
		]
	);

	return Widgets;
}

bool UMovieGraphConditionGroupQuery_ActorType::HasAddMenu() const
{
	return true;
}

TSharedRef<SWidget> UMovieGraphConditionGroupQuery_ActorType::GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished)
{
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.bShowNoneOption = false;
	Options.bIsActorsOnly = true;
	Options.bShowUnloadedBlueprints = false;

	// Add a class filter to disallow adding duplicates of actor types that were already picked
	Options.ClassFilters.Add(MakeShared<UE::MovieGraph::Private::FClassViewerTypeFilter>(&ActorTypes));

	const TSharedRef<SWidget> ClassViewer = ClassViewerModule.CreateClassViewer(
		Options,
		FOnClassPicked::CreateLambda([this, OnAddFinished](UClass* InNewClass)
		{
			AddActorTypes({InNewClass}, OnAddFinished);
		}));

	ClassViewerWidget = StaticCastSharedPtr<SClassViewer>(ClassViewer.ToSharedPtr()); 
	
	return SNew(SBox)
		.WidthOverride(300.f)
		.HeightOverride(300.f)
		[
			ClassViewerWidget.ToSharedRef()
		];
}

const FSlateBrush* UMovieGraphConditionGroupQuery_ActorType::GetRowIcon(TObjectPtr<UClass> InActorType)
{
	return FSlateIconFinder::FindIconForClass(InActorType).GetIcon();
}

FText UMovieGraphConditionGroupQuery_ActorType::GetRowText(TObjectPtr<UClass> InActorType)
{
	if (InActorType)
	{
		return InActorType->GetDisplayNameText();
	}

	return LOCTEXT("MovieGraphActorTypeConditionGroupQuery_Invalid", "(invalid)");
}

void UMovieGraphConditionGroupQuery_ActorType::AddActorTypes(const TArray<UClass*>& InActorTypes, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished)
{
	const FScopedTransaction Transaction(LOCTEXT("AddActorTypesToCollection", "Add Actor Types to Collection"));
	Modify();
	
	FSlateApplication::Get().DismissAllMenus();

	for (UClass* ActorType : InActorTypes)
	{
		ActorTypes.AddUnique(ActorType);
	}
	
	InOnAddFinished.ExecuteIfBound();

	// Ensure that the class filters run again so duplicate actor types cannot be selected
	if (ClassViewerWidget.IsValid())
	{
		ClassViewerWidget->Refresh();
	}

	ActorTypesList->Refresh();
}
#endif

UMovieGraphConditionGroupQuery_LightActorType::UMovieGraphConditionGroupQuery_LightActorType()
{
	// All light actor types are enabled by default.
	EnabledLightActorTypes = *GetSupportedLightActorTypes();
}

void UMovieGraphConditionGroupQuery_LightActorType::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphConditionGroupQuery_LightActorType::Evaluate);

	for (AActor* Actor : InActorsToQuery)
	{
		if (EnabledLightActorTypes.Contains(Actor->GetClass()))
		{
			OutMatchingActors.Add(Actor);
		}
	}
}

const FSlateIcon& UMovieGraphConditionGroupQuery_LightActorType::GetIcon() const
{
	static const FSlateIcon LightActorTypeIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PointLightComponent");
	return LightActorTypeIcon;
}

const FText& UMovieGraphConditionGroupQuery_LightActorType::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_LightActorType", "Light Actor Type");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_LightActorType::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	Widgets.Add(
		SAssignNew(LightActorTypesList, SMovieGraphSimpleList<UClass*>)
		.DataSource(GetSupportedLightActorTypes())
		.CustomSummaryWidget(SNullWidget::NullWidget.ToSharedPtr())
		.OnGetRowText_Static(&GetRowText)
		.OnGetRowIcon_Static(&GetRowIcon)
		.ShowEnableDisable(true)
		.OnGetRowEnableState_Lambda([this](UClass* InLightActorType)
		{
			return GetEnableStateForLightType(InLightActorType);
		})
		.OnSetRowEnableState_Lambda([this](UClass* InLightActorType, bool bNewEnableState)
		{
			SetEnableStateForLightType(InLightActorType, bNewEnableState);
		})
	);

	return Widgets;
}
#endif	// WITH_EDITOR

void UMovieGraphConditionGroupQuery_LightActorType::SetEnableStateForLightType(UClass* InLightActorType, const bool bNewEnableState)
{
	if (GetSupportedLightActorTypes()->Contains(InLightActorType))
	{
#if WITH_EDITOR
		const FScopedTransaction Transaction(LOCTEXT("UpdateLightActorTypesInCollection", "Update Light Actor Types in Collection"));
		Modify();
#endif

		if (bNewEnableState)
		{
			EnabledLightActorTypes.AddUnique(InLightActorType);
		}
		else
		{
			EnabledLightActorTypes.Remove(InLightActorType);
		}
	}
}

bool UMovieGraphConditionGroupQuery_LightActorType::GetEnableStateForLightType(UClass* InLightActorType)
{
	return EnabledLightActorTypes.Contains(InLightActorType);
}

TArray<UClass*>* UMovieGraphConditionGroupQuery_LightActorType::GetSupportedLightActorTypes()
{
	// Unfortunately this cannot be const because the list widget doesn't expect a const array
	static TArray<UClass*> SupportedLightActorTypes = {
		ADirectionalLight::StaticClass(),
		APointLight::StaticClass(),
		ARectLight::StaticClass(),
		ASkyLight::StaticClass(),
		ASpotLight::StaticClass()
	};

	return &SupportedLightActorTypes;
}

#if WITH_EDITOR
const FSlateBrush* UMovieGraphConditionGroupQuery_LightActorType::GetRowIcon(UClass* InLightActorType)
{
	return FSlateIconFinder::FindIconForClass(InLightActorType).GetIcon();
}

FText UMovieGraphConditionGroupQuery_LightActorType::GetRowText(UClass* InLightActorType)
{
	if (InLightActorType)
	{
		return InLightActorType->GetDisplayNameText();
	}

	return LOCTEXT("MovieGraphLightActorTypeConditionGroupQuery_Invalid", "(invalid light type)");
}
#endif	// WITH_EDITOR

UMovieGraphConditionGroupQuery_ComponentTagName::UMovieGraphConditionGroupQuery_ComponentTagName()
	: bOnlyMatchComponents(true)
{
	
}

void UMovieGraphConditionGroupQuery_ComponentTagName::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	if (bOnlyMatchComponents)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Evaluate() called on the Component Tag Name condition, but the condition only matches components. No results will be returned; EvaluateActorsAndComponents() should be used instead.");
	}
	
	TSet<UActorComponent*> UnusedComponentSet;
	EvaluateActorsAndComponents(InActorsToQuery, InWorld, OutMatchingActors, UnusedComponentSet);
}

void UMovieGraphConditionGroupQuery_ComponentTagName::EvaluateActorsAndComponents(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphConditionGroupQuery_ComponentTagName::Evaluate);
	
	// Quick early-out if "*" is used as the wildcard. Faster than doing the wildcard matching.
	if (TagsToMatch == TEXT("*"))
	{
		if (bOnlyMatchComponents)
		{
			TInlineComponentArray<UActorComponent*> MatchingComponents;
			
			// This will be EXTREMELY slow if there are a lot of actors in the scene with many components
			for (const AActor* ActorToQuery : InActorsToQuery)
			{
				constexpr bool bIncludeFromChildActors = true;
				ActorToQuery->GetComponents(MatchingComponents, bIncludeFromChildActors);

				OutMatchingComponents.Append(MatchingComponents);
			}
		}
		else
		{
			OutMatchingActors.Append(InActorsToQuery);
		}
		
		return;
	}

	// Component tags can be specified on multiple lines
	TArray<FString> AllTagNameStrings;
	TagsToMatch.ParseIntoArrayLines(AllTagNameStrings);
	
	TInlineComponentArray<UActorComponent*> ActorComponents;

	for (AActor* Actor : InActorsToQuery)
	{
		// Include child components so components inside of Blueprints can be found
		constexpr bool bIncludeFromChildActors = false;
		Actor->GetComponents<UActorComponent*>(ActorComponents, bIncludeFromChildActors);
		
		for (UActorComponent* Component : ActorComponents)
		{
			bool bMatchedTag = false;
			
			for (const FString& TagToMatch : AllTagNameStrings)
			{			
				for (const FName& ComponentTag : Component->ComponentTags)
				{
					if (ComponentTag.ToString().MatchesWildcard(TagToMatch))
					{
						if (bOnlyMatchComponents)
						{
							OutMatchingComponents.Add(Component);
						}
						else
						{
							OutMatchingActors.Add(Actor);
						}

						bMatchedTag = true;
						break;
					}
				}

				// Skip comparing the rest of the tags if one tag already matched 
				if (bMatchedTag)
				{
					break;
				}
			}

			// Skip comparing the rest of the components if one component already matched. If matching components instead of actors,
			// we still have to continue processing all components.
			if (bMatchedTag && !bOnlyMatchComponents)
			{
				break;
			}
		}

		ActorComponents.Reset();
	}
}

bool UMovieGraphConditionGroupQuery_ComponentTagName::ShouldEvaluateComponents() const
{
	return bOnlyMatchComponents;
}

const FSlateIcon& UMovieGraphConditionGroupQuery_ComponentTagName::GetIcon() const
{
	static const FSlateIcon ActorTagIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.OpenIssueTracker");
	return ActorTagIcon;
}

const FText& UMovieGraphConditionGroupQuery_ComponentTagName::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_ComponentTagName", "Component Tag Name");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_ComponentTagName::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	Widgets.Add(
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		.Padding(7.f, 2.f)
		[
			SNew(SMultiLineEditableTextBox)
			.Text_Lambda([this]() { return FText::FromString(TagsToMatch); })
			.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type TextCommitType)
			{
				const FScopedTransaction Transaction(LOCTEXT("UpdateComponentTagNamesInCollection", "Update Component Tag Names in Collection"));
				Modify();
				
				TagsToMatch = InText.ToString();
			})
			.HintText(LOCTEXT("MovieGraphComponentTagNameQueryHintText", "A component on the actor must match one or more component tags.\nWildcards allowed. Enter each tag on a separate line."))
		]

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoHeight()
		.Padding(8, 0, 8, 2)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
			{
				const FScopedTransaction Transaction(LOCTEXT("ConditionGroupQueryComponentTagName_ToggleOnlyMatchComponents", "Toggle Only Match Components"));
				Modify();
					
				bOnlyMatchComponents = (NewState == ECheckBoxState::Checked);
			})
			.IsChecked_Lambda([this]() { return bOnlyMatchComponents ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("ConditionGroupQueryComponentTagName_OnlyMatchComponents", "Only Match Components"))
			]
			.ToolTipText(LOCTEXT("ConditionGroupQueryComponentTagName_OnlyMatchComponentsTooltip", "If checked, only components with the specified tag(s) will be matched. If unchecked, actors that contain components with the specified tag(s) will be matched."))
		]
	);

	return Widgets;
}
#endif

void UMovieGraphConditionGroupQuery_ComponentTagName::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FMovieRenderPipelineCoreObjectVersion::GUID);

	if (Ar.CustomVer(FMovieRenderPipelineCoreObjectVersion::GUID) < FMovieRenderPipelineCoreObjectVersion::OnlyMatchComponentsAdded)
	{
		// Default this to false if this query existed before OnlyMatchComponents was added
		bOnlyMatchComponents = false;
	}
}

UMovieGraphConditionGroupQuery_ComponentType::UMovieGraphConditionGroupQuery_ComponentType()
	: bOnlyMatchComponents(true)
{
	
}

void UMovieGraphConditionGroupQuery_ComponentType::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	if (bOnlyMatchComponents)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Evaluate() called on the Component Type condition, but the condition only matches components. No results will be returned; EvaluateActorsAndComponents() should be used instead.");
	}
	
	TSet<UActorComponent*> UnusedComponentSet;
	EvaluateActorsAndComponents(InActorsToQuery, InWorld, OutMatchingActors, UnusedComponentSet);
}

void UMovieGraphConditionGroupQuery_ComponentType::EvaluateActorsAndComponents(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphConditionGroupQuery_ComponentType::EvaluateActorsAndComponents);
	
	TInlineComponentArray<UActorComponent*> ActorComponents;

	for (AActor* Actor : InActorsToQuery)
	{
		// Include child components so components inside of Blueprints can be found
		constexpr bool bIncludeFromChildActors = false;
		Actor->GetComponents<UActorComponent*>(ActorComponents, bIncludeFromChildActors);
		
		for (UActorComponent* Component : ActorComponents)
		{
			if (ComponentTypes.Contains(Component->GetClass()))
			{
				if (bOnlyMatchComponents)
				{
					OutMatchingComponents.Add(Component);
				}
				else
				{
					OutMatchingActors.Add(Actor);
				}
			}
		}

		ActorComponents.Reset();
	}
}

bool UMovieGraphConditionGroupQuery_ComponentType::ShouldEvaluateComponents() const
{
	return bOnlyMatchComponents;
}

const FSlateIcon& UMovieGraphConditionGroupQuery_ComponentType::GetIcon() const
{
	static const FSlateIcon ActorTagIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.ActorComponent");
	return ActorTagIcon;
}

const FText& UMovieGraphConditionGroupQuery_ComponentType::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_ComponentType", "Component Type");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_ComponentType::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	Widgets.Add(
		SAssignNew(ComponentTypesList, SMovieGraphSimpleList<TObjectPtr<UClass>>)
			.DataSource(&ComponentTypes)
			.OnGetRowText_Static(&GetRowText)
			.OnGetRowIcon_Static(&GetRowIcon)
			.OnDelete_Lambda([this](const TArray<UClass*> InComponentTypes)
			{
				const FScopedTransaction Transaction(LOCTEXT("RemoveComponentTypesFromCollection", "Remove Component Types from Collection"));
				Modify();

				for (UClass* ComponentType : InComponentTypes)
				{
					ComponentTypes.Remove(ComponentType);
				}
				
				ComponentTypesList->Refresh();
			})
			.CustomSummaryWidget_Lambda([this]()
			{
				return SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						.Text_Lambda([this]()
						{
							return FText::Format(LOCTEXT("ConditionGroupQueryComponentType_ComponentCount", "{0} Component(s)"), FText::FromString(LexToString(ComponentTypes.Num())));
						})
					]

					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
						{
							const FScopedTransaction Transaction(LOCTEXT("ConditionGroupQueryComponentType_ToggleOnlyMatchComponents", "Toggle Only Match Components"));
							Modify();
							
							bOnlyMatchComponents = (NewState == ECheckBoxState::Checked);
						})
						.IsChecked_Lambda([this]() { return bOnlyMatchComponents ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
							.Text(LOCTEXT("ConditionGroupQueryComponentType_OnlyMatchComponents", "Only Match Components"))
						]
						.ToolTipText(LOCTEXT("ConditionGroupQueryComponentType_OnlyMatchComponentsTooltip", "If checked, only components with the specified tag(s) will be matched. If unchecked, actors that contain components with the specified tags(s) will be matched."))
					];
			})
	);

	return Widgets;
}

bool UMovieGraphConditionGroupQuery_ComponentType::HasAddMenu() const
{
	return true;
}

TSharedRef<SWidget> UMovieGraphConditionGroupQuery_ComponentType::GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished)
{
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.bShowNoneOption = false;
	Options.bIsActorsOnly = false;
	Options.bShowUnloadedBlueprints = false;

	// Add a class filter to disallow adding duplicates of component types that were already picked, as well as restrict the types of classes displayed
	// to only show component classes
	Options.ClassFilters.Add(MakeShared<UE::MovieGraph::Private::FClassViewerTypeFilter>(&ComponentTypes, UActorComponent::StaticClass()));

	const TSharedRef<SWidget> ClassViewer = ClassViewerModule.CreateClassViewer(
		Options,
		FOnClassPicked::CreateLambda([this, OnAddFinished](UClass* InNewClass)
		{
			const FScopedTransaction Transaction(LOCTEXT("AddComponentTypeToCollection", "Add Component Type to Collection"));
			Modify();
			
			FSlateApplication::Get().DismissAllMenus();
			
			ComponentTypes.Add(InNewClass);
			OnAddFinished.ExecuteIfBound();

			// Ensure that the class filters run again so duplicate actor types cannot be selected
			if (ClassViewerWidget.IsValid())
			{
				ClassViewerWidget->Refresh();
				ComponentTypesList->Refresh();
			}
		}));

	ClassViewerWidget = StaticCastSharedPtr<SClassViewer>(ClassViewer.ToSharedPtr()); 
	
	return SNew(SBox)
		.WidthOverride(300.f)
		.HeightOverride(300.f)
		[
			ClassViewerWidget.ToSharedRef()
		];
}

const FSlateBrush* UMovieGraphConditionGroupQuery_ComponentType::GetRowIcon(TObjectPtr<UClass> InComponentType)
{
	return FSlateIconFinder::FindIconForClass(InComponentType).GetIcon();
}

FText UMovieGraphConditionGroupQuery_ComponentType::GetRowText(TObjectPtr<UClass> InComponentType)
{
	if (InComponentType)
	{
		return InComponentType->GetDisplayNameText();
	}
	
	return LOCTEXT("MovieGraphComponentTypeConditionGroupQuery_Invalid", "(invalid)");
}
#endif	// WITH_EDITOR

void UMovieGraphConditionGroupQuery_ComponentType::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FMovieRenderPipelineCoreObjectVersion::GUID);

	if (Ar.CustomVer(FMovieRenderPipelineCoreObjectVersion::GUID) < FMovieRenderPipelineCoreObjectVersion::OnlyMatchComponentsAdded)
	{
		// Default this to false if this query existed before OnlyMatchComponents was added
		bOnlyMatchComponents = false;
	}
}

UMovieGraphConditionGroupQuery_ComponentName::UMovieGraphConditionGroupQuery_ComponentName()
	: bOnlyMatchComponents(true)
{
	
}

void UMovieGraphConditionGroupQuery_ComponentName::Evaluate(
	const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	if (bOnlyMatchComponents)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Evaluate() called on the Component Name condition, but the condition only matches components. No results will be returned; EvaluateActorsAndComponents() should be used instead.");
	}

	TSet<UActorComponent*> UnusedComponentSet;
	EvaluateActorsAndComponents(InActorsToQuery, InWorld, OutMatchingActors, UnusedComponentSet);
}

void UMovieGraphConditionGroupQuery_ComponentName::EvaluateActorsAndComponents(
	const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphConditionGroupQuery_ComponentName::EvaluateActorsAndComponents);

	// Quick early-out if "*" is used as the wildcard. Faster than doing the wildcard matching.
	if (ComponentNames == TEXT("*"))
	{
		if (bOnlyMatchComponents)
		{
			TInlineComponentArray<UActorComponent*> MatchingComponents;

			// This will be EXTREMELY slow if there are a lot of actors in the scene with many components
			for (const AActor* ActorToQuery : InActorsToQuery)
			{
				constexpr bool bIncludeFromChildActors = true;
				ActorToQuery->GetComponents(MatchingComponents, bIncludeFromChildActors);

				OutMatchingComponents.Append(MatchingComponents);
			}
		}
		else
		{
			OutMatchingActors.Append(InActorsToQuery);
		}
		
		return;
	}

	// Component names can be specified on multiple lines
	TArray<FString> AllComponentNames;
	ComponentNames.ParseIntoArrayLines(AllComponentNames);

	TInlineComponentArray<UActorComponent*> ActorComponents;

	for (AActor* Actor : InActorsToQuery)
	{
		// Include child components so components inside of Blueprints can be found
		constexpr bool bIncludeFromChildActors = true;
		Actor->GetComponents<UActorComponent*>(ActorComponents, bIncludeFromChildActors);

		for (UActorComponent* Component : ActorComponents)
		{
			for (const FString& ComponentName : AllComponentNames)
			{
				if (Component->GetName().MatchesWildcard(ComponentName))
				{
					if (bOnlyMatchComponents)
					{
						OutMatchingComponents.Add(Component);
					}
					else
					{
						OutMatchingActors.Add(Actor);
					}
				}
			}
		}

		ActorComponents.Reset();
	}
}

bool UMovieGraphConditionGroupQuery_ComponentName::ShouldEvaluateComponents() const
{
	return bOnlyMatchComponents;
}

const FSlateIcon& UMovieGraphConditionGroupQuery_ComponentName::GetIcon() const
{
	static const FSlateIcon ComponentNameIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.TextRenderActor");
	return ComponentNameIcon;
}

const FText& UMovieGraphConditionGroupQuery_ComponentName::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_ComponentName", "Component Name");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_ComponentName::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	Widgets.Add(
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		.Padding(7.f, 2.f)
		[
			SNew(SMultiLineEditableTextBox)
			.Text_Lambda([this]() { return FText::FromString(ComponentNames); })
			.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type TextCommitType)
			{
				const FScopedTransaction Transaction(LOCTEXT("UpdateComponentNamesInCollection", "Update Component Names in Collection"));
				Modify();

				ComponentNames = InText.ToString();
			})
			.HintText(LOCTEXT("MovieGraphComponentNameQueryHintText", "A component on the actor must match one or more component names.\nWildcards allowed. Enter each component name on a separate line."))
		]

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoHeight()
		.Padding(8, 0, 8, 2)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
			{
				const FScopedTransaction Transaction(LOCTEXT("ConditionGroupQueryComponentName_ToggleOnlyMatchComponents", "Toggle Only Match Components"));
				Modify();

				bOnlyMatchComponents = (NewState == ECheckBoxState::Checked);
			})
			.IsChecked_Lambda([this]() { return bOnlyMatchComponents ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("ConditionGroupQueryComponentName_OnlyMatchComponents", "Only Match Components"))
			]
			.ToolTipText(LOCTEXT("ConditionGroupQueryComponentName_OnlyMatchComponentsTooltip", "If checked, only components with the specified name(s) will be matched. If unchecked, actors that contain components with the specified name(s) will be matched."))
		]
	);

	return Widgets;
}
#endif	// WITH_EDITOR

void UMovieGraphConditionGroupQuery_EditorFolder::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
#if WITH_EDITOR
	// This const cast is unfortunate, but should be harmless
	const FFolder::FRootObject FolderRootObject = FFolder::GetWorldRootFolder(const_cast<UWorld*>(InWorld)).GetRootObject();

	for (AActor* Actor : InActorsToQuery)
	{
		if (Actor->GetFolderRootObject() != FolderRootObject)
		{
			continue;
		}
		
		const FName ActorFolderPath = Actor->GetFolderPath();
		if (ActorFolderPath.IsNone())
		{
			continue;
		}
		
		const FString ActorFolderPathString = ActorFolderPath.ToString();
		const int32 ActorFolderLen = ActorFolderPath.GetStringLength();

		for (const FName& ParentFolderPath : FolderPaths)
		{
			const int32 ParentFolderLen = ParentFolderPath.GetStringLength();
			
			// We shouldn't be looking at an empty folder path, but just in case.
			if (ParentFolderLen == 0)
			{
				continue;
			}

			// The actor is a match if it's in a folder that's an exact match.
			if (ActorFolderPath == ParentFolderPath)
			{
				OutMatchingActors.Add(Actor);
				break;
			}

			// The actor is also a match if it's in a matching subfolder.
			if ((ActorFolderLen > ParentFolderLen) &&
				(ActorFolderPathString[ParentFolderLen] == '/') &&
				(ActorFolderPathString.Left(ParentFolderLen) == ParentFolderPath))
			{
				OutMatchingActors.Add(Actor);
				break;
			}
		}
	}
#endif	// WITH_EDITOR
}

const FSlateIcon& UMovieGraphConditionGroupQuery_EditorFolder::GetIcon() const
{
	static const FSlateIcon EditorFolderIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderOpen");
	return EditorFolderIcon;
}

const FText& UMovieGraphConditionGroupQuery_EditorFolder::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_EditorFolder", "Editor Folder");
	return DisplayName;
}

bool UMovieGraphConditionGroupQuery_EditorFolder::IsEditorOnlyQuery() const
{
	// This query is editor-only because folders do not exist outside of the editor
	return true;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_EditorFolder::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	auto GetFolderDragOp = [](FDragDropOperation* InDragDropOp) -> TSharedPtr<FFolderDragDropOp>
	{
		if (InDragDropOp->IsOfType<FSceneOutlinerDragDropOp>())
		{
			FSceneOutlinerDragDropOp* SceneOutlinerOp = static_cast<FSceneOutlinerDragDropOp*>(InDragDropOp);
			return SceneOutlinerOp->GetSubOp<FFolderDragDropOp>();
		}

		return nullptr;
	};
	
    Widgets.Add(
		SNew(SDropTarget)
		.OnAllowDrop_Lambda([GetFolderDragOp](TSharedPtr<FDragDropOperation> InDragOperation)
		{
			return GetFolderDragOp(InDragOperation.Get()) != nullptr;
		})
		.OnDropped_Lambda([this, GetFolderDragOp](const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
		{
			if (const TSharedPtr<FFolderDragDropOp> FolderDragDropOp = GetFolderDragOp(DragDropEvent.GetOperation().Get()))
			{
				const FMovieGraphConditionGroupQueryContentsChanged OnAddFinished = nullptr;
				AddFolders(FolderDragDropOp->Folders, OnAddFinished);
			}

			return FReply::Handled();
		})
		[
			SAssignNew(FolderPathsList, SMovieGraphSimpleList<FName>)
			.DataSource(&FolderPaths)
			.DataType(FText::FromString("Folder"))
			.DataTypePlural(FText::FromString("Folders"))
			.OnGetRowText_Static(&GetRowText)
			.OnGetRowIcon_Static(&GetRowIcon)
			.OnDelete_Lambda([this](const TArray<FName> InFolderPaths)
			{
				const FScopedTransaction Transaction(LOCTEXT("RemoveEditorFoldersFromCollection", "Remove Editor Folders from Collection"));
				Modify();

				for (const FName& FolderPath : InFolderPaths)
				{
					FolderPaths.Remove(FolderPath);
				}
				
				FolderPathsList->Refresh();

				if (FolderPickerWidget.IsValid())
				{
					FolderPickerWidget->FullRefresh();
				}
			})
		]
    );

    return Widgets;
}

bool UMovieGraphConditionGroupQuery_EditorFolder::HasAddMenu() const
{
	return true;
}

TSharedRef<SWidget> UMovieGraphConditionGroupQuery_EditorFolder::GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished)
{
	auto OnItemPicked = FOnSceneOutlinerItemPicked::CreateLambda([this, OnAddFinished](TSharedRef<ISceneOutlinerTreeItem> Item)
	{
		if (const FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>())
		{
			if (FolderItem->IsValid())
			{
				const FName& FolderPath = FolderItem->GetPath();
				
				AddFolders({FolderPath}, OnAddFinished);
			}
		}
	});

	const FCreateSceneOutlinerMode ModeFactory = FCreateSceneOutlinerMode::CreateLambda([&OnItemPicked](SSceneOutliner* Outliner)
	{
		return new FActorFolderPickingMode(Outliner, OnItemPicked);
	});
	
	FSceneOutlinerInitializationOptions SceneOutlinerInitOptions;
	SceneOutlinerInitOptions.bShowCreateNewFolder = false;
	SceneOutlinerInitOptions.bFocusSearchBoxWhenOpened = true;
	SceneOutlinerInitOptions.ModeFactory = ModeFactory;

	// Don't show folders which have already been picked
	SceneOutlinerInitOptions.Filters->AddFilterPredicate<FActorFolderTreeItem>(
		FActorFolderTreeItem::FFilterPredicate::CreateLambda([this](const FFolder& InFolder)
		{
			return !FolderPaths.Contains(InFolder.GetPath());
		}));

	// Only show the name/label column, that's the only column relevant to folders
	SceneOutlinerInitOptions.ColumnMap.Add(
		FSceneOutlinerBuiltInColumnTypes::Label(),
		FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn(), false, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Label_Localized()));

	FolderPickerWidget = SNew(SSceneOutliner, SceneOutlinerInitOptions)
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());
	
	return
		SNew(SBox)
		.WidthOverride(400.f)
		.HeightOverride(300.f)
		[
			FolderPickerWidget.ToSharedRef()
		];
}

const FSlateBrush* UMovieGraphConditionGroupQuery_EditorFolder::GetRowIcon(FName InFolderPath)
{
	return FAppStyle::Get().GetBrush("Icons.FolderOpen");
}

FText UMovieGraphConditionGroupQuery_EditorFolder::GetRowText(FName InFolderPath)
{
	return FText::FromString(InFolderPath.ToString());
}

void UMovieGraphConditionGroupQuery_EditorFolder::AddFolders(const TArray<FName>& InFolderPaths, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished)
{
	const FScopedTransaction Transaction(LOCTEXT("AddEditorFolderToCollection", "Add Editor Folder to Collection"));
	Modify();
	
	for (const FName& FolderPath : InFolderPaths)
	{
		FolderPaths.AddUnique(FolderPath);
	}
	
	InOnAddFinished.ExecuteIfBound();
				
	if (FolderPickerWidget.IsValid())
	{
		FolderPickerWidget->FullRefresh();
	}

	FolderPathsList->Refresh();
}
#endif	// WITH_EDITOR

void UMovieGraphConditionGroupQuery_Sublevel::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	for (const TSoftObjectPtr<UWorld>& World : Sublevels)
	{
		// Don't load the level, only use levels which are already loaded
		const UWorld* LoadedWorld = World.Get();
		if (!LoadedWorld)
		{
			const UMovieGraphCollection* ParentCollection = GetTypedOuter<UMovieGraphCollection>();
			const FString CollectionName = ParentCollection ? ParentCollection->GetCollectionName() : TEXT("<unknown>");
			
			UE_LOGF(LogMovieRenderPipeline, Warning, "Sublevel query in collection '%ls' is excluding level (%ls) because it is not loaded.", *CollectionName, *World.ToString())
			continue;
		}

		ULevel* CurrentLevel = LoadedWorld->GetCurrentLevel();
		if (!CurrentLevel)
		{
			continue;
		}

		for (const TObjectPtr<AActor>& LevelActor : CurrentLevel->Actors)
		{
			// The actors accessed directly from the level may need to be converted into the current world (most likely editor -> PIE)
			if (AActor* ConvertedActor = GetActorForCurrentWorld(LevelActor.Get(), InWorld))
			{
				OutMatchingActors.Add(ConvertedActor);
			}
		}
	}
}

const FSlateIcon& UMovieGraphConditionGroupQuery_Sublevel::GetIcon() const
{
	static const FSlateIcon SublevelIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Level");
	return SublevelIcon;
}

const FText& UMovieGraphConditionGroupQuery_Sublevel::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_Sublevel", "Sublevel");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_Sublevel::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	// Create the data source for the list view
	RefreshListDataSource();

	Widgets.Add(
		SNew(SDropTarget)
		.OnAllowDrop_Lambda([](TSharedPtr<FDragDropOperation> InDragOperation)
		{
			// Support drag-n-drop from the Content browser
			if (InDragOperation->IsOfType<FContentBrowserDataDragDropOp>())
			{
				const FContentBrowserDataDragDropOp* ContentBrowserOp = static_cast<FContentBrowserDataDragDropOp*>(InDragOperation.Get());
				for (const FAssetData& DraggedAsset : ContentBrowserOp->GetAssets())
				{
					if (DraggedAsset.AssetClassPath == UWorld::StaticClass()->GetClassPathName())
					{
						return true;
					}
				}
			}

			// Support drag-n-drop from the Levels editor
			if (InDragOperation->IsOfType<FLevelDragDropOp>())
			{
				return true;
			}

			return false;
		})
		.OnDropped_Lambda([this](const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
		{
			if (const TSharedPtr<FContentBrowserDataDragDropOp> ContentBrowserOp = DragDropEvent.GetOperationAs<FContentBrowserDataDragDropOp>())
			{
				TArray<UWorld*> DroppedLevels;
				
				for (const FAssetData& DraggedAsset : ContentBrowserOp->GetAssets())
				{
					if (DraggedAsset.AssetClassPath == UWorld::StaticClass()->GetClassPathName())
					{
						DroppedLevels.Add(Cast<UWorld>(DraggedAsset.GetAsset()));
					}
				}

				const FMovieGraphConditionGroupQueryContentsChanged OnAddFinished = nullptr;
				AddLevels(DroppedLevels, OnAddFinished);
			}

			if (const TSharedPtr<FLevelDragDropOp> LevelEditorOp = DragDropEvent.GetOperationAs<FLevelDragDropOp>())
			{
				TArray<UWorld*> Levels;
				Algo::Transform(LevelEditorOp->LevelsToDrop, Levels, [](const TWeakObjectPtr<ULevel>& Level) { return Level.IsValid() ? Level->GetWorld() : nullptr; });

				TArray<UWorld*> StreamingLevels;
				Algo::Transform(LevelEditorOp->StreamingLevelsToDrop, StreamingLevels, [](const TWeakObjectPtr<ULevelStreaming>& LevelStreaming) { return LevelStreaming.IsValid() ? LevelStreaming->GetWorldAsset().Get() : nullptr; });
				
				const FMovieGraphConditionGroupQueryContentsChanged OnAddFinished = nullptr;
				AddLevels(Levels, OnAddFinished);
				AddLevels(StreamingLevels, OnAddFinished);
			}

			return FReply::Handled();
		})
		[
			SAssignNew(SublevelsList, SMovieGraphSimpleList<TSharedPtr<TSoftObjectPtr<UWorld>>>)
			.DataSource(&ListDataSource)
			.DataType(FText::FromString("Sublevel"))
			.DataTypePlural(FText::FromString("Sublevels"))
			.OnGetRowText_Static(&GetRowText)
			.OnGetRowIcon_Static(&GetRowIcon)
			.OnDelete_Lambda([this](const TArray<TSharedPtr<TSoftObjectPtr<UWorld>>> InSublevels)
			{
				const FScopedTransaction Transaction(LOCTEXT("RemoveSublevelsFromCollection", "Remove Sublevels from Collection"));
				Modify();

				for (TSharedPtr<TSoftObjectPtr<UWorld>> Sublevel : InSublevels)
				{
					ListDataSource.Remove(Sublevel);
					Sublevels.Remove(*Sublevel.Get());
				}
				
				SublevelsList->Refresh();
				
				constexpr bool bUpdateSources = true;
				RefreshLevelPicker.ExecuteIfBound(bUpdateSources);
			})
			.OnRefreshDataSourceRequested_UObject(this, &UMovieGraphConditionGroupQuery_Sublevel::RefreshListDataSource)
		]
	);

	return Widgets;
}

bool UMovieGraphConditionGroupQuery_Sublevel::HasAddMenu() const
{
	return true;
}

TSharedRef<SWidget> UMovieGraphConditionGroupQuery_Sublevel::GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished)
{
	FAssetPickerConfig SublevelPickerConfig;
	{
		SublevelPickerConfig.SelectionMode = ESelectionMode::Single;
		SublevelPickerConfig.SaveSettingsName = TEXT("MovieRenderGraphSublevelPicker");
		SublevelPickerConfig.RefreshAssetViewDelegates.Add(&RefreshLevelPicker);
		SublevelPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		SublevelPickerConfig.bFocusSearchBoxWhenOpened = true;
		SublevelPickerConfig.bAllowNullSelection = false;
		SublevelPickerConfig.bShowBottomToolbar = true;
		SublevelPickerConfig.bAutohideSearchBar = false;
		SublevelPickerConfig.bAllowDragging = false;
		SublevelPickerConfig.bCanShowClasses = false;
		SublevelPickerConfig.bShowPathInColumnView = true;
		SublevelPickerConfig.bShowTypeInColumnView = false;
		SublevelPickerConfig.bSortByPathInColumnView = false;
		SublevelPickerConfig.HiddenColumnNames = {
			ContentBrowserItemAttributes::ItemDiskSize.ToString(),
			ContentBrowserItemAttributes::VirtualizedData.ToString(),
			TEXT("PrimaryAssetType"),
			TEXT("PrimaryAssetName")
		};
		SublevelPickerConfig.AssetShowWarningText = LOCTEXT("ConditionGroupQuery_NoSublevelsFound", "No Sublevels Found");
		SublevelPickerConfig.Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
		SublevelPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([this, OnAddFinished](const FAssetData& InLevelAsset)
		{
			AddLevels({Cast<UWorld>(InLevelAsset.GetAsset())}, OnAddFinished);
		});
		SublevelPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([this](const FAssetData& InLevelAsset)
		{
			if (bOnlyShowSublevelsFromActivePersistentLevel && !IsSublevelInActivePersistentLevel(InLevelAsset.PackageName))
			{
				return true;
			}

			// Don't show sublevels which have already been picked
			for (const TSoftObjectPtr<UWorld>& Sublevel : Sublevels)
			{
				if (Sublevel.ToSoftObjectPath() == InLevelAsset.GetSoftObjectPath())
				{
					return true;
				}
			}

			return false;
		});
	}

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	return
		SNew(SBox)
		.Padding(0, 10.f, 0, 0)
		.WidthOverride(400.f)
		.HeightOverride(330.f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			.Padding(6.f, 0, 0, 6.f)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("ConditionGroupQuerySublevel_OnlyShowActivePersistentLevelSublevelsTooltip", "Filter out sublevels not from the active persistent level."))
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
				{
					const FScopedTransaction Transaction(LOCTEXT("ConditionGroupQuerySublevel_ToggleOnlyShowActivePersistentLevelSublevels", "Toggle Only Show Active Persistent Level Sublevels"));
					Modify();

					bOnlyShowSublevelsFromActivePersistentLevel = (NewState == ECheckBoxState::Checked);

					constexpr bool bUpdateSources = true;
					RefreshLevelPicker.ExecuteIfBound(bUpdateSources);
				})
				.IsChecked_Lambda([this]()
				{
					return bOnlyShowSublevelsFromActivePersistentLevel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				[
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(LOCTEXT("ConditionGroupQuerySublevel_OnlyShowActivePersistentLevelSublevels", "Only show sublevels from the active persistent level"))
				]
			]

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.FillHeight(1.f)
			[
				ContentBrowser.CreateAssetPicker(SublevelPickerConfig)
			]
		];
}

const FSlateBrush* UMovieGraphConditionGroupQuery_Sublevel::GetRowIcon(TSharedPtr<TSoftObjectPtr<UWorld>> InSublevel)
{
	return FAppStyle::Get().GetBrush("Icons.Level");
}

FText UMovieGraphConditionGroupQuery_Sublevel::GetRowText(TSharedPtr<TSoftObjectPtr<UWorld>> InSublevel)
{
	if (InSublevel.IsValid())
	{
		if (InSublevel.Get()->IsValid())
		{
			// The first Get() returns the TSoftObjectPtr, the second Get() dereferences the TSoftObjectPtr
			return FText::FromString(InSublevel.Get()->Get()->GetName());
		}
	}

	return LOCTEXT("MovieGraphSublevelConditionGroupQuery_InvalidLevel", "(invalid)");
}

bool UMovieGraphConditionGroupQuery_Sublevel::IsSublevelInActivePersistentLevel(FName InSublevelPackageName)
{
	if (InSublevelPackageName.IsNone() || !GEditor)
	{
		return false;
	}

	const UWorld* ActiveWorld = GEditor->GetEditorWorldContext().World();
	if (!ActiveWorld)
	{
		return false;
	}

	for (const ULevelStreaming* StreamingLevel : ActiveWorld->GetStreamingLevels())
	{
		if (StreamingLevel && StreamingLevel->GetWorldAssetPackageFName() == InSublevelPackageName)
		{
			return true;
		}
	}

	return false;
}

void UMovieGraphConditionGroupQuery_Sublevel::AddLevels(const TArray<UWorld*>& InLevels, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished)
{
	const FScopedTransaction Transaction(LOCTEXT("AddSublevelsToCollection", "Add Sublevels to Collection"));
	Modify();
	
	FSlateApplication::Get().DismissAllMenus();

	for (UWorld* Level : InLevels)
	{
		if (Level && !Sublevels.Contains(Level))
		{
			Sublevels.Add(Level);
			ListDataSource.Add(MakeShared<TSoftObjectPtr<UWorld>>(Level));
		}
	}
	
	InOnAddFinished.ExecuteIfBound();

	if (SublevelsList.IsValid())
	{
		SublevelsList->Refresh();
	}

	constexpr bool bUpdateSources = false;
	RefreshLevelPicker.ExecuteIfBound(bUpdateSources);
}

void UMovieGraphConditionGroupQuery_Sublevel::RefreshListDataSource()
{
	ListDataSource.Empty();
	for (TSoftObjectPtr<UWorld>& Sublevel : Sublevels)
	{
		ListDataSource.Add(MakeShared<TSoftObjectPtr<UWorld>>(Sublevel));
	}
}
#endif	// WITH_EDITOR

void UMovieGraphConditionGroupQuery_ActorLayer::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	for (AActor* Actor : InActorsToQuery)
	{
		for (const FName& LayerName : LayerNames)
		{
			if (Actor->Layers.Contains(LayerName))
			{
				OutMatchingActors.Add(Actor);
				break;
			}
		}
	}
}

const FSlateIcon& UMovieGraphConditionGroupQuery_ActorLayer::GetIcon() const
{
	static const FSlateIcon ActorLayerIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.DataLayers");
	return ActorLayerIcon;
}

const FText& UMovieGraphConditionGroupQuery_ActorLayer::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_ActorLayer", "Actor Layer");
	return DisplayName;
}

bool UMovieGraphConditionGroupQuery_ActorLayer::IsEditorOnly() const
{
	// Actor Layers are only available in the editor
	return true;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_ActorLayer::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	Widgets.Add(
		SNew(SDropTarget)
		.OnAllowDrop_Lambda([](TSharedPtr<FDragDropOperation> InDragOperation)
		{
			return InDragOperation->IsOfType<FLayersDragDropOp>();
		})
		.OnDropped_Lambda([this](const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
		{
			if (const TSharedPtr<FLayersDragDropOp> LayerOperation = DragDropEvent.GetOperationAs<FLayersDragDropOp>())
			{
				const FMovieGraphConditionGroupQueryContentsChanged OnAddFinished = nullptr;
				AddActorLayers(LayerOperation->Layers, OnAddFinished);
			}
			
			return FReply::Handled();
		})
		[
			SAssignNew(LayerNamesList, SMovieGraphSimpleList<FName>)
			.DataSource(&LayerNames)
			.DataType(FText::FromString("Actor Layer"))
			.DataTypePlural(FText::FromString("Actor Layers"))
			.OnGetRowText_Static(&GetRowText)
			.OnGetRowIcon_Static(&GetRowIcon)
			.OnDelete_Lambda([this](const TArray<FName> InLayerNames)
			{
				const FScopedTransaction Transaction(LOCTEXT("RemoveActorLayerFromCollection", "Remove Actor Layers from Collection"));
				Modify();

				for (const FName& LayerName : InLayerNames)
				{
					LayerNames.Remove(LayerName);
				}
				
				LayerNamesList->Refresh();
			})
		]
	);

	return Widgets;
}

bool UMovieGraphConditionGroupQuery_ActorLayer::HasAddMenu() const
{
	return true;
}

TSharedRef<SWidget> UMovieGraphConditionGroupQuery_ActorLayer::GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished)
{
	// Refresh the list's data source
	LayerPickerDataSource.Reset();
	if (const ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>())
	{
		LayersSubsystem->AddAllLayerNamesTo(LayerPickerDataSource);

		// Don't include layers that have already been picked
		LayerPickerDataSource.RemoveAll([this](const FName InLayerName) { return LayerNames.Contains(InLayerName); });
	}
	
	return
		SNew(SMovieGraphSimplePicker<FName>)
		.Title(LOCTEXT("PickActorLayerHelpText", "Pick an Actor Layer"))
		.DataSourceEmptyMessage(LOCTEXT("NoActorLayersFoundWarning", "No actor layers found."))
		.DataSource(LayerPickerDataSource)
		.OnGetRowIcon_Lambda([](FName ListItem)
		{
			return FAppStyle::Get().GetBrush("Layer.Icon16x");
		})
		.OnGetRowText_Lambda([](FName ListItem)
		{
			return FText::FromName(ListItem);
		})
		.OnItemPicked_Lambda([this, OnAddFinished](FName InLayerName)
		{
			AddActorLayers({InLayerName}, OnAddFinished);
		});
}

const FSlateBrush* UMovieGraphConditionGroupQuery_ActorLayer::GetRowIcon(FName InLayerName)
{
	return FAppStyle::Get().GetBrush("Layer.Icon16x");
}

FText UMovieGraphConditionGroupQuery_ActorLayer::GetRowText(FName InLayerName)
{
	if (const ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>())
	{
		if (LayersSubsystem->GetLayer(InLayerName))
		{
			return FText::FromName(InLayerName);
		}
	}

	return FText::Format(LOCTEXT("InvalidActorLayer", "{0} (invalid)"), FText::FromName(InLayerName));
}

void UMovieGraphConditionGroupQuery_ActorLayer::AddActorLayers(const TArray<FName>& InActorLayers, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished)
{
	const FScopedTransaction Transaction(LOCTEXT("AddActorLayersToCollection", "Add Actor Layers to Collection"));
	Modify();

	for (const FName& ActorLayer : InActorLayers)
	{
		LayerNames.AddUnique(ActorLayer);
	}
	
	LayerNamesList->Refresh();

	InOnAddFinished.ExecuteIfBound();
}
#endif	// WITH_EDITOR

void UMovieGraphConditionGroupQuery_DataLayer::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	for (AActor* InActor : InActorsToQuery)
	{
		for (const TSoftObjectPtr<UDataLayerAsset>& DataLayer : DataLayers)
		{
			if (InActor->ContainsDataLayer(DataLayer.Get()))
			{
				OutMatchingActors.Add(InActor);
				break;
			}
		}
	}
}

const FSlateIcon& UMovieGraphConditionGroupQuery_DataLayer::GetIcon() const
{
	static const FSlateIcon DataLayerIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "DataLayer.Editor");
	return DataLayerIcon;
}

const FText& UMovieGraphConditionGroupQuery_DataLayer::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_DataLayer", "Data Layer");
	return DisplayName;
}

#if WITH_EDITOR
TArray<TSharedRef<SWidget>> UMovieGraphConditionGroupQuery_DataLayer::GetWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets;

	// Create the data source for the list view
	RefreshListDataSource();

	Widgets.Add(
		SNew(SDropTarget)
		.OnAllowDrop_Lambda([](TSharedPtr<FDragDropOperation> InDragOperation)
		{
			return InDragOperation->IsOfType<FDataLayerDragDropOp>() || InDragOperation->IsOfType<FAssetDragDropOp>();
		})
		.OnDropped_Lambda([this](const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
		{
			TArray<const UDataLayerAsset*> DroppedLayers;

			// Drag-n-drop from the Data Layers editor
			if (const TSharedPtr<FDataLayerDragDropOp> LayerOperation = DragDropEvent.GetOperationAs<FDataLayerDragDropOp>())
			{
				for (const TWeakObjectPtr<UDataLayerInstance>& DroppedLayer : LayerOperation->DataLayerInstances)
				{
					if (const UDataLayerAsset* DroppedLayerAsset = DroppedLayer->GetAsset())
					{
						DroppedLayers.AddUnique(DroppedLayerAsset);
					}
				}
			}

			// Drag-n-drop from the Content Browser
			else if (const TSharedPtr<FAssetDragDropOp> AssetOperation = DragDropEvent.GetOperationAs<FAssetDragDropOp>())
			{
				for (const FAssetData& AssetData : AssetOperation->GetAssets())
				{
					if (const UDataLayerAsset* DataLayer = Cast<const UDataLayerAsset>(AssetData.GetAsset()))
					{
						DroppedLayers.AddUnique(DataLayer);
					}
				}
			}
			
			if (!DroppedLayers.IsEmpty())
			{
				const FMovieGraphConditionGroupQueryContentsChanged OnAddFinished = nullptr;
				AddDataLayers(DroppedLayers, OnAddFinished);
			}
			
			return FReply::Handled();
		})
		[
			SAssignNew(DataLayersList, SMovieGraphSimpleList<TSharedPtr<TSoftObjectPtr<UDataLayerAsset>>>)
			.DataSource(&ListDataSource)
			.DataType(FText::FromString("Data Layer"))
			.DataTypePlural(FText::FromString("Data Layers"))
			.OnGetRowText_Static(&GetRowText)
			.OnGetRowIcon_Static(&GetRowIcon)
			.OnDelete_Lambda([this](const TArray<TSharedPtr<TSoftObjectPtr<UDataLayerAsset>>> InLayers)
			{
				const FScopedTransaction Transaction(LOCTEXT("RemoveDataLayerFromCollection", "Remove Data Layers from Collection"));
				Modify();

				for (TSharedPtr<TSoftObjectPtr<UDataLayerAsset>> Layer : InLayers)
				{
					ListDataSource.Remove(Layer);
					DataLayers.Remove(*Layer.Get());
				}
				
				DataLayersList->Refresh();

				constexpr bool bUpdateSources = true;
				RefreshDataLayerPicker.ExecuteIfBound(bUpdateSources);
			})
			.OnRefreshDataSourceRequested_UObject(this, &UMovieGraphConditionGroupQuery_DataLayer::RefreshListDataSource)
		]
	);

	return Widgets;
}

bool UMovieGraphConditionGroupQuery_DataLayer::HasAddMenu() const
{
	return true;
}

TSharedRef<SWidget> UMovieGraphConditionGroupQuery_DataLayer::GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished)
{
	FAssetPickerConfig DataLayerPickerConfig;
	{
		DataLayerPickerConfig.SelectionMode = ESelectionMode::Single;
		DataLayerPickerConfig.SaveSettingsName = TEXT("MovieRenderGraphDataLayerPicker");
		DataLayerPickerConfig.RefreshAssetViewDelegates.Add(&RefreshDataLayerPicker);
		DataLayerPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		DataLayerPickerConfig.bFocusSearchBoxWhenOpened = true;
		DataLayerPickerConfig.bAllowNullSelection = false;
		DataLayerPickerConfig.bShowBottomToolbar = true;
		DataLayerPickerConfig.bAutohideSearchBar = false;
		DataLayerPickerConfig.bAllowDragging = false;
		DataLayerPickerConfig.bCanShowClasses = false;
		DataLayerPickerConfig.bShowPathInColumnView = true;
		DataLayerPickerConfig.bShowTypeInColumnView = false;
		DataLayerPickerConfig.bSortByPathInColumnView = false;
		DataLayerPickerConfig.HiddenColumnNames = {
			ContentBrowserItemAttributes::ItemDiskSize.ToString(),
			ContentBrowserItemAttributes::VirtualizedData.ToString(),
			TEXT("PrimaryAssetType"),
			TEXT("PrimaryAssetName")
		};
		DataLayerPickerConfig.AssetShowWarningText = LOCTEXT("ConditionGroupQuery_NoDataLayersFound", "No Data Layers Found");
		DataLayerPickerConfig.Filter.ClassPaths.Add(UDataLayerAsset::StaticClass()->GetClassPathName());
		DataLayerPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([this, OnAddFinished](const FAssetData& InDataLayerAsset)
		{
			AddDataLayers({Cast<UDataLayerAsset>(InDataLayerAsset.GetAsset())}, OnAddFinished);
		});
		DataLayerPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([this](const FAssetData& InDataLayerAsset)
		{
			// Don't show data layers which have already been picked
			UDataLayerAsset* DataLayer = Cast<UDataLayerAsset>(InDataLayerAsset.GetAsset());
			return !DataLayer || DataLayers.Contains(DataLayer);
		});
	}

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	return
		SNew(SBox)
		.Padding(0, 10.f, 0, 0)
		.WidthOverride(400.f)
		.HeightOverride(300.f)
		[
			ContentBrowser.CreateAssetPicker(DataLayerPickerConfig)
		];
}

const FSlateBrush* UMovieGraphConditionGroupQuery_DataLayer::GetRowIcon(TSharedPtr<TSoftObjectPtr<UDataLayerAsset>> InDataLayer)
{
	return FAppStyle::Get().GetBrush("DataLayer.Editor");
}

FText UMovieGraphConditionGroupQuery_DataLayer::GetRowText(TSharedPtr<TSoftObjectPtr<UDataLayerAsset>> InDataLayer)
{
	if (InDataLayer.IsValid())
	{
		if (InDataLayer.Get()->IsValid())
		{
			// The first Get() returns the TSoftObjectPtr, the second Get() dereferences the TSoftObjectPtr
			return FText::FromString(InDataLayer.Get()->Get()->GetName());
		}
	}

	return LOCTEXT("MovieGraphDataLayerConditionGroupQuery_InvalidDataLayer", "(invalid or unloaded)");
}

void UMovieGraphConditionGroupQuery_DataLayer::AddDataLayers(const TArray<const UDataLayerAsset*>& InDataLayers, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished)
{
	const FScopedTransaction Transaction(LOCTEXT("AddDataLayersToCollection", "Add Data Layers to Collection"));
	Modify();
	
	for (const UDataLayerAsset* DataLayer : InDataLayers)
	{
		if (DataLayer && !DataLayers.Contains(DataLayer))
		{
			DataLayers.Add(const_cast<UDataLayerAsset*>(DataLayer));
			ListDataSource.Add(MakeShared<TSoftObjectPtr<UDataLayerAsset>>(const_cast<UDataLayerAsset*>(DataLayer)));
		}
	}
	
	DataLayersList->Refresh();
	FSlateApplication::Get().DismissAllMenus();

	InOnAddFinished.ExecuteIfBound();

	constexpr bool bUpdateSources = false;
	RefreshDataLayerPicker.ExecuteIfBound(bUpdateSources);
}

void UMovieGraphConditionGroupQuery_DataLayer::RefreshListDataSource()
{
	ListDataSource.Empty();
	for (TSoftObjectPtr<UDataLayerAsset>& DataLayer : DataLayers)
	{
		ListDataSource.Add(MakeShared<TSoftObjectPtr<UDataLayerAsset>>(DataLayer));
	}
}
#endif	// WITH_EDITOR

void UMovieGraphConditionGroupQuery_IsSpawnable::Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const
{
	for (AActor* InActor : InActorsToQuery)
	{
		const TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(InActor);
		if (Spawnable.IsSet() == bIsSpawnable)
		{
			OutMatchingActors.Add(InActor);
		}
	}
}

const FSlateIcon& UMovieGraphConditionGroupQuery_IsSpawnable::GetIcon() const
{
	static const FSlateIcon SpawnableIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.SpawnActor_16x");
	return SpawnableIcon;
}

const FText& UMovieGraphConditionGroupQuery_IsSpawnable::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("ConditionGroupQueryDisplayName_IsSpawnable", "Is Spawnable");
	return DisplayName;
}

FMovieGraphEvaluationResult::FMovieGraphEvaluationResult(TSet<TObjectPtr<AActor>>&& InActors, TSet<TObjectPtr<UActorComponent>>&& InComponents)
	: MatchingActors(MoveTemp(InActors))
	, MatchingComponents(MoveTemp(InComponents))
{
}

void FMovieGraphEvaluationResult::Reset()
{
	MatchingActors.Reset();
	MatchingComponents.Reset();
}

void FMovieGraphEvaluationResult::Append(const FMovieGraphEvaluationResult& InOther)
{
	MatchingActors.Append(InOther.MatchingActors);
	MatchingComponents.Append(InOther.MatchingComponents);
}

FMovieGraphEvaluationResult FMovieGraphEvaluationResult::Union(const FMovieGraphEvaluationResult& InOther) const
{
	return FMovieGraphEvaluationResult(
		MatchingActors.Union(InOther.MatchingActors),
		MatchingComponents.Union(InOther.MatchingComponents));
}

FMovieGraphEvaluationResult FMovieGraphEvaluationResult::Intersect(const FMovieGraphEvaluationResult& InOther) const
{
	FMovieGraphEvaluationResult Result;

	// Actor AND Actor is simple: the actors just need to be the same
	for (const TObjectPtr<AActor>& MatchingActor : MatchingActors)
	{
		if (MatchingActor && InOther.MatchingActors.Contains(MatchingActor))
		{
			Result.MatchingActors.Add(MatchingActor);
		}
	}

	// Component AND Component is also simple: the components just need to be the same
	for (const TObjectPtr<UActorComponent>& MatchingComponent : MatchingComponents)
	{
		if (MatchingComponent && InOther.MatchingComponents.Contains(MatchingComponent))
		{
			Result.MatchingComponents.Add(MatchingComponent);
		}
	}

	// Actor AND Component is more complex: there's a match if the component is contained *within* an actor
	auto FindMatchingComponents = [&Result](const TSet<TObjectPtr<UActorComponent>>& InComponents, const TSet<TObjectPtr<AActor>>& InActors)
	{
		for (const TObjectPtr<UActorComponent>& MatchingComponent : InComponents)
		{
			for (const TObjectPtr<AActor>& OtherActor : InActors)
			{
				if (MatchingComponent && OtherActor->GetComponents().Contains(MatchingComponent.Get()))
				{
					Result.MatchingComponents.Add(MatchingComponent);
				}
			}
		}
	};
	FindMatchingComponents(MatchingComponents, InOther.MatchingActors);
	FindMatchingComponents(InOther.MatchingComponents, MatchingActors);
	
	return Result;
}

FMovieGraphEvaluationResult FMovieGraphEvaluationResult::Difference(const FMovieGraphEvaluationResult& InOther) const
{
	FMovieGraphEvaluationResult Result;

	// Actor MINUS Actor is simple: the actor cannot be in the other result set
	for (const TObjectPtr<AActor>& MatchingActor : MatchingActors)
	{
		if (MatchingActor && !InOther.MatchingActors.Contains(MatchingActor))
		{
			Result.MatchingActors.Add(MatchingActor);
		}
	}

	// Component MINUS Component is also simple: the component cannot be in the other result set
	for (const TObjectPtr<UActorComponent>& MatchingComponent : MatchingComponents)
	{
		if (MatchingComponent && !InOther.MatchingComponents.Contains(MatchingComponent))
		{
			Result.MatchingComponents.Add(MatchingComponent);
		}
	}

	// Actor MINUS Component is more complex: an actor is "deconstructed" into all of its components, and component(s) are subtracted out
	TArray<TObjectPtr<UActorComponent>, TInlineAllocator<24>> ActorComponents;
	TArray<AActor*, TInlineAllocator<10>> DeconstructedActorsToRemove;
	for (const TObjectPtr<AActor>& MatchingActor : Result.MatchingActors)
	{
		constexpr bool bIncludeFromChildActors = true;
		MatchingActor->GetComponents(ActorComponents, bIncludeFromChildActors);
		
		for (const TObjectPtr<UActorComponent>& OtherComponent : InOther.MatchingComponents)
		{
			// If a matching actor's components contains one of the components to subtract, deconstruct the actor into its components and match all of
			// its components except the component that is being subtracted out.
			if (ActorComponents.Contains(OtherComponent.Get()))
			{
				// Deconstruct the actor if needed. We can't remove the actor from MatchingActors yet because we're iterating it, but we'll remove it
				// after the loop finishes.
				TObjectPtr<AActor> ComponentActor = OtherComponent->GetOwner();
				if (!DeconstructedActorsToRemove.Contains(ComponentActor) && Result.MatchingActors.Contains(ComponentActor))
				{
					DeconstructedActorsToRemove.Add(ComponentActor.Get());
					Result.MatchingComponents.Append(ActorComponents);
				}
				
				// Remove the component to be subtracted out
				Result.MatchingComponents.Remove(OtherComponent);
			}

			// Reuse the array
			ActorComponents.Reset();
		}
	}

	// Remove all of the deconstructed actors found above
	for (AActor* ActorToRemove : DeconstructedActorsToRemove)
	{
		Result.MatchingActors.Remove(ActorToRemove);
	}

	// Component MINUS Actor is a little more straightforward: if a component is contained in the actor, remove the component
	TArray<TObjectPtr<UActorComponent>, TInlineAllocator<24>> ComponentsToRemove;
	for (const TObjectPtr<AActor>& OtherActor : InOther.MatchingActors)
	{
		constexpr bool bIncludeFromChildActors = true;
		OtherActor->GetComponents(ActorComponents, bIncludeFromChildActors);
		
		for (const TObjectPtr<UActorComponent>& MatchingComponent : Result.MatchingComponents)
		{
			if (ActorComponents.Contains(MatchingComponent))
			{
				ComponentsToRemove.Add(MatchingComponent);
				Result.MatchingActors.Remove(OtherActor);
			}
		}
	}

	// Remove all of the components found above
	for (UActorComponent* ComponentToRemove : ComponentsToRemove)
	{
		Result.MatchingComponents.Remove(ComponentToRemove);
	}

	return Result;
}

template<typename T>
TArray<T*> FMovieGraphEvaluationResult::GetAllComponentsOfType() const
{
	TArray<T*> ComponentsOfType;

	for (const AActor* MatchedActor : MatchingActors)
	{
		constexpr bool bIncludeFromChildActors = true;
		TInlineComponentArray<T*> PrimitiveComponents(MatchedActor, bIncludeFromChildActors);

		ComponentsOfType.Append(MoveTemp(PrimitiveComponents));
	}

	for (UActorComponent* ActorComponent : MatchingComponents)
	{
		if (T* PrimitiveComponent = Cast<T>(ActorComponent))
		{
			ComponentsOfType.Add(PrimitiveComponent);
		}
	}

	return ComponentsOfType;
}

UMovieGraphConditionGroup::UMovieGraphConditionGroup()
	: OpType(EMovieGraphConditionGroupOpType::Add)
{
	// The CDO will always have the default GUID
	if (!HasAllFlags(RF_ClassDefaultObject))
	{
		Id = FGuid::NewGuid();
	}
	else
	{
		Id = FGuid();
	}
}

void UMovieGraphConditionGroup::SetOperationType(const EMovieGraphConditionGroupOpType OperationType)
{
	// Always allow setting the operation type to Union. If not setting to Union, only allow setting the operation type if this is not the first
	// condition group in the collection. The first condition group is always a Union.
	if (OperationType == EMovieGraphConditionGroupOpType::Add)
	{
		OpType = EMovieGraphConditionGroupOpType::Add;
		return;
	}

	const UMovieGraphCollection* ParentCollection = GetTypedOuter<UMovieGraphCollection>();
	if (ensureMsgf(ParentCollection, TEXT("Cannot set the operation type on a condition group that doesn't have a collection outer")))
	{
		if (ParentCollection->GetConditionGroups().Find(this) != 0)
		{
			OpType = OperationType;
		}
	}
}

EMovieGraphConditionGroupOpType UMovieGraphConditionGroup::GetOperationType() const
{
	return OpType;
}

TSet<AActor*> UMovieGraphConditionGroup::Evaluate(const UWorld* InWorld) const
{
	return ObjectPtrDecay(EvaluateActorsAndComponents(InWorld).MatchingActors);
}

FMovieGraphEvaluationResult UMovieGraphConditionGroup::EvaluateActorsAndComponents(const UWorld* InWorld) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphConditionGroup::EvaluateActorsAndComponents);

	// Reset the TSet for evaluation results; it is persisted across frames to prevent constantly re-allocating it
	EvaluationResult.Reset();

	// Generate a list of actors that can be fed to the queries once, rather than having all queries perform this
	TArray<AActor*> AllActors;
	for (TActorIterator<AActor> ActorItr(InWorld); ActorItr; ++ActorItr)
	{
		if (AActor* Actor = *ActorItr)
		{
			AllActors.Add(Actor);
		}
	}

	for (int32 QueryIndex = 0; QueryIndex < Queries.Num(); ++QueryIndex)
	{
		const TObjectPtr<UMovieGraphConditionGroupQueryBase>& Query = Queries[QueryIndex];
		if (!Query || !Query->IsEnabled())
		{
			continue;
		}
		
		if (QueryIndex == 0)
		{
			// The first query should always be a Union
			ensure(Query->GetOperationType() == EMovieGraphConditionGroupQueryOpType::Add);
		}

		// Similar to EvaluationResult, QueryResult is persisted+reset to prevent constantly re-allocating it
		QueryResult.Reset();

		if (!Query->ShouldEvaluateComponents())
		{
			Query->Evaluate(AllActors, InWorld, MutableView(QueryResult.MatchingActors));
		}
		else
		{
			Query->EvaluateActorsAndComponents(AllActors, InWorld, MutableView(QueryResult.MatchingActors), MutableView(QueryResult.MatchingComponents));
		}
		
		switch (Query->GetOperationType())
		{
		case EMovieGraphConditionGroupQueryOpType::Add:
			// Append() is faster than Union() because we don't need to allocate a new set
			EvaluationResult.Append(QueryResult);
			break;

		case EMovieGraphConditionGroupQueryOpType::And:
			EvaluationResult = EvaluationResult.Intersect(QueryResult);
			break;

		case EMovieGraphConditionGroupQueryOpType::Subtract:
			EvaluationResult = EvaluationResult.Difference(QueryResult);
			break;
		}
	}
	
	return ObjectPtrDecay(EvaluationResult);
}

UMovieGraphConditionGroupQueryBase* UMovieGraphConditionGroup::AddQuery(const TSubclassOf<UMovieGraphConditionGroupQueryBase>& InQueryType, const int32 InsertIndex)
{
	UMovieGraphConditionGroupQueryBase* NewQueryObj = NewObject<UMovieGraphConditionGroupQueryBase>(this, InQueryType.Get(), NAME_None, RF_Transactional);

#if WITH_EDITOR
	Modify();
#endif

	if (InsertIndex < 0)
	{
		Queries.Add(NewQueryObj);
	}
	else
	{
		// Clamp the insert index to a valid range in case an invalid one is provided
		Queries.Insert(NewQueryObj, FMath::Clamp(InsertIndex, 0, Queries.Num()));
	}
	
	return NewQueryObj;
}

const TArray<UMovieGraphConditionGroupQueryBase*>& UMovieGraphConditionGroup::GetQueries() const
{
	return Queries;
}

bool UMovieGraphConditionGroup::RemoveQuery(UMovieGraphConditionGroupQueryBase* InQuery)
{
#if WITH_EDITOR
	Modify();
#endif
	
	const bool bRemoved = Queries.RemoveSingle(InQuery) == 1;

	// Always make sure the first query is an "add"
	if (!Queries.IsEmpty())
	{
		const TObjectPtr<UMovieGraphConditionGroupQueryBase> FirstQuery = Queries[0];
		if (FirstQuery->GetOperationType() != EMovieGraphConditionGroupQueryOpType::Add)
		{
			FirstQuery->Modify();
			FirstQuery->SetOperationType(EMovieGraphConditionGroupQueryOpType::Add);
		}
	}
	
	return bRemoved;
}

UMovieGraphConditionGroupQueryBase* UMovieGraphConditionGroup::DuplicateQuery(const int32 QueryIndex)
{
	if (!Queries.IsValidIndex(QueryIndex))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Invalid query index provided to DuplicateQuery().");
		return nullptr;
	}
	
	const UMovieGraphConditionGroupQueryBase* SourceQuery = Queries[QueryIndex];

	FObjectDuplicationParameters DuplicationParameters = InitStaticDuplicateObjectParams(SourceQuery, this);
	UMovieGraphConditionGroupQueryBase* DuplicateQuery = Cast<UMovieGraphConditionGroupQueryBase>(StaticDuplicateObjectEx(DuplicationParameters));

	if (DuplicateQuery)
	{
		Modify();
		Queries.Add(DuplicateQuery);
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to duplicate condition group query.");
	}

	return DuplicateQuery;
}

bool UMovieGraphConditionGroup::IsFirstConditionGroup() const
{
	const UMovieGraphCollection* ParentCollection = GetTypedOuter<UMovieGraphCollection>();
	if (ensureMsgf(ParentCollection, TEXT("Cannot determine if this is the first condition group when no parent collection is present")))
	{
		// GetConditionGroups() returns an array of non-const pointers, so Find() doesn't like having a const pointer passed to it.
		// Find() won't mutate the condition group though, so the const_cast here is OK.
		return ParentCollection->GetConditionGroups().Find(const_cast<UMovieGraphConditionGroup*>(this)) == 0;
	}
	
	return false;
}

bool UMovieGraphConditionGroup::MoveQueryToIndex(UMovieGraphConditionGroupQueryBase* InQuery, const int32 NewIndex)
{
#if WITH_EDITOR
	Modify();
#endif

	if (!InQuery)
	{
		return false;
	}

	const int32 ExistingIndex = Queries.Find(InQuery);
	if (ExistingIndex == INDEX_NONE)
	{
		return false;
	}

	// If the new index is greater than the current index, then decrement the destination index so it remains valid after the removal below
	int32 DestinationIndex = NewIndex;
	if (DestinationIndex > ExistingIndex)
	{
		--DestinationIndex;
	}

	Queries.Remove(InQuery);
	Queries.Insert(InQuery, DestinationIndex);

	// Enforce that the first query is set to Union
	InQuery->SetOperationType(EMovieGraphConditionGroupQueryOpType::Add);

	return true;
}

const FGuid& UMovieGraphConditionGroup::GetId() const
{
	return Id;
}

TSet<AActor*> UMovieGraphCollection::Evaluate(const UWorld* InWorld) const
{
	return ObjectPtrDecay(EvaluateActorsAndComponents(InWorld).MatchingActors);
}

FMovieGraphEvaluationResult UMovieGraphCollection::EvaluateActorsAndComponents(const UWorld* InWorld) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovieGraphCollection::EvaluateActorsAndComponents);
	
	FMovieGraphEvaluationResult FinalResult;

	for (int32 ConditionGroupIndex = 0; ConditionGroupIndex < ConditionGroups.Num(); ++ConditionGroupIndex)
	{
		const TObjectPtr<UMovieGraphConditionGroup>& ConditionGroup = ConditionGroups[ConditionGroupIndex];
		if (!ConditionGroup)
		{
			continue;
		}
		
		if (ConditionGroupIndex == 0)
		{
			// The first condition group should always be a Union
			ensure(ConditionGroup->GetOperationType() == EMovieGraphConditionGroupOpType::Add);
		}

		const FMovieGraphEvaluationResult QueryResult = ConditionGroup->EvaluateActorsAndComponents(InWorld);
		
		switch (ConditionGroup->GetOperationType())
		{
		case EMovieGraphConditionGroupOpType::Add:
			FinalResult = FinalResult.Union(QueryResult);
			break;

		case EMovieGraphConditionGroupOpType::And:
			FinalResult = FinalResult.Intersect(QueryResult);
			break;

		case EMovieGraphConditionGroupOpType::Subtract:
			FinalResult = FinalResult.Difference(QueryResult);
			break;
		}
	}
	
	return FinalResult;
}

UMovieGraphConditionGroup* UMovieGraphCollection::AddConditionGroup()
{
	UMovieGraphConditionGroup* NewConditionGroup = NewObject<UMovieGraphConditionGroup>(this, NAME_None, RF_Transactional);

#if WITH_EDITOR
	Modify();
#endif
	
	ConditionGroups.Add(NewConditionGroup);
	return NewConditionGroup;
}

const TArray<UMovieGraphConditionGroup*>& UMovieGraphCollection::GetConditionGroups() const
{
	return ConditionGroups;
}

bool UMovieGraphCollection::RemoveConditionGroup(UMovieGraphConditionGroup* InConditionGroup)
{
#if WITH_EDITOR
	Modify();
#endif
	
	const bool bRemoved = ConditionGroups.RemoveSingle(InConditionGroup) == 1;
	
	// Always make sure the first condition group is an "add"
	if (!ConditionGroups.IsEmpty())
	{
		const TObjectPtr<UMovieGraphConditionGroup> FirstConditionGroup = ConditionGroups[0];
		if (FirstConditionGroup->GetOperationType() != EMovieGraphConditionGroupOpType::Add)
		{
			FirstConditionGroup->Modify();
			FirstConditionGroup->SetOperationType(EMovieGraphConditionGroupOpType::Add);
		}
	}
	
	return bRemoved;
}

bool UMovieGraphCollection::MoveConditionGroupToIndex(UMovieGraphConditionGroup* InConditionGroup, const int32 NewIndex)
{
#if WITH_EDITOR
	Modify();
#endif

	if (!InConditionGroup)
	{
		return false;
	}

	const int32 ExistingIndex = ConditionGroups.Find(InConditionGroup);
	if (ExistingIndex == INDEX_NONE)
	{
		return false;
	}

	// If the new index is greater than the current index, then decrement the destination index so it remains valid after the removal below
	int32 DestinationIndex = NewIndex;
	if (DestinationIndex > ExistingIndex)
	{
		--DestinationIndex;
	}

	ConditionGroups.Remove(InConditionGroup);
	ConditionGroups.Insert(InConditionGroup, DestinationIndex);

	// Enforce that the first condition group is set to Union
	InConditionGroup->SetOperationType(EMovieGraphConditionGroupOpType::Add);

	return true;
}

#if WITH_EDITOR
void UMovieGraphCollection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Name change delegate is broadcast here so it catches both SetCollectionName() and a direct change of the property via the details panel
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphCollection, CollectionName))
	{
		OnCollectionNameChangedDelegate.Broadcast(this);
	}
}
#endif	// WITH_EDITOR

void UMovieGraphCollection::SetCollectionName(const FString& InName)
{
	CollectionName = InName;
}

const FString& UMovieGraphCollection::GetCollectionName() const
{
	return CollectionName;
}

UMovieGraphCollection* UMovieGraphRenderLayer::GetCollectionByName(const FString& Name) const
{
	for (const UMovieGraphModifierBase* Modifier : Modifiers)
	{
		const UMovieGraphCollectionModifier* CollectionModifier = Cast<UMovieGraphCollectionModifier>(Modifier);
		if (!CollectionModifier)
		{
			continue;
		}
		
		for (UMovieGraphCollection* Collection : CollectionModifier->GetCollections())
		{
			if (Collection && Collection->GetCollectionName().Equals(Name))
			{
				return Collection;
			}
		}
	}

	return nullptr;
}

void UMovieGraphRenderLayer::AddModifier(UMovieGraphCollectionModifier* Modifier)
{
	if (!Modifiers.Contains(Modifier))
	{
		Modifiers.Add(Modifier);
	}
}

void UMovieGraphRenderLayer::AddLayerModifier(UMovieGraphModifierBase* InModifier)
{
	if (!Modifiers.Contains(InModifier))
	{
		Modifiers.Add(InModifier);
	}
}

TArray<UMovieGraphCollectionModifier*> UMovieGraphRenderLayer::GetModifiers() const
{
	TArray<UMovieGraphCollectionModifier*> ConvertedModifiers;
	for (const TObjectPtr<UMovieGraphModifierBase>& Modifier : Modifiers)
	{
		if (UMovieGraphCollectionModifier* CollectionModifier = Cast<UMovieGraphCollectionModifier>(Modifier))
		{
			ConvertedModifiers.Add(CollectionModifier);
		}
	}
		
	return ConvertedModifiers;
}

TArray<UMovieGraphModifierBase*> UMovieGraphRenderLayer::GetLayerModifiers() const
{
	return Modifiers;
}

void UMovieGraphRenderLayer::RemoveModifier(UMovieGraphCollectionModifier* Modifier)
{
	Modifiers.Remove(Modifier);
}

void UMovieGraphRenderLayer::RemoveLayerModifier(UMovieGraphModifierBase* Modifier)
{
	Modifiers.Remove(Modifier);
}

void UMovieGraphRenderLayer::Apply(const UWorld* World)
{
	if (!World)
	{
		return;
	}
	
	// Apply all modifiers
	for (UMovieGraphModifierBase* Modifier : Modifiers)
	{
		Modifier->ApplyModifier(World);
	}
}

void UMovieGraphRenderLayer::Revert()
{
	// Undo actions performed by all modifiers. Do this in the reverse order that they were applied, since the undo
	// state of one modifier may depend on modifiers that were previously applied.
	for (int32 Index = Modifiers.Num() - 1; Index >= 0; Index--)
	{
		if (UMovieGraphModifierBase* Modifier = Modifiers[Index])
		{
			Modifier->UndoModifier();
		}
	}
}

UMovieGraphRenderLayerSubsystem* UMovieGraphRenderLayerSubsystem::GetFromWorld(const UWorld* World)
{
	if (World)
	{
		return UWorld::GetSubsystem<UMovieGraphRenderLayerSubsystem>(World);
	}

	return nullptr;
}

void UMovieGraphRenderLayerSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
}

void UMovieGraphRenderLayerSubsystem::Deinitialize()
{
}

void UMovieGraphRenderLayerSubsystem::Reset()
{
	RevertAndClearActiveRenderLayer();
	RenderLayers.Empty();
}

bool UMovieGraphRenderLayerSubsystem::AddRenderLayer(UMovieGraphRenderLayer* RenderLayer)
{
	if (!RenderLayer)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Invalid render layer provided to AddRenderLayer().");
		return false;
	}
	
	const bool bRenderLayerExists = RenderLayers.ContainsByPredicate([RenderLayer](const UMovieGraphRenderLayer* RL)
	{
		return RL && (RenderLayer->GetRenderLayerName() == RL->GetName());
	});

	if (bRenderLayerExists)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Render layer '%ls' already exists in the render layer subsystem; it will not be added again.", *RenderLayer->GetRenderLayerName().ToString());
		return false;
	}

	RenderLayers.Add(RenderLayer);
	return true;
}

void UMovieGraphRenderLayerSubsystem::RemoveRenderLayer(const FString& RenderLayerName)
{
	if (ActiveRenderLayer && (ActiveRenderLayer->GetName() == RenderLayerName))
	{
		RevertAndClearActiveRenderLayer();
	}
	
	const uint32 Index = RenderLayers.IndexOfByPredicate([&RenderLayerName](const UMovieGraphRenderLayer* RenderLayer)
	{
		return RenderLayer->GetRenderLayerName() == RenderLayerName;
	});

	if (Index != INDEX_NONE)
	{
		RenderLayers.RemoveAt(Index);
	}
}

void UMovieGraphRenderLayerSubsystem::SetActiveRenderLayerByObj(UMovieGraphRenderLayer* RenderLayer)
{
	if (!RenderLayer)
	{
		return;
	}
	
	RevertAndClearActiveRenderLayer();
	SetAndApplyRenderLayer(RenderLayer);
}

void UMovieGraphRenderLayerSubsystem::SetActiveRenderLayerByName(const FName& RenderLayerName)
{
	const uint32 Index = RenderLayers.IndexOfByPredicate([&RenderLayerName](const UMovieGraphRenderLayer* RenderLayer)
	{
		return RenderLayer->GetRenderLayerName() == RenderLayerName;
	});

	if (Index != INDEX_NONE)
	{
		SetActiveRenderLayerByObj(RenderLayers[Index]);
	}
}

void UMovieGraphRenderLayerSubsystem::ClearActiveRenderLayer()
{
	RevertAndClearActiveRenderLayer();
}

void UMovieGraphRenderLayerSubsystem::RevertAndClearActiveRenderLayer()
{
	if (ActiveRenderLayer)
	{
		ActiveRenderLayer->Revert();
	}

	ActiveRenderLayer = nullptr;
}

void UMovieGraphRenderLayerSubsystem::SetAndApplyRenderLayer(UMovieGraphRenderLayer* RenderLayer)
{
	ActiveRenderLayer = RenderLayer;
	ActiveRenderLayer->Apply(GetWorld());
}

#undef LOCTEXT_NAMESPACE
