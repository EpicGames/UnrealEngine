// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphSequenceDataSource.h"
#include "Graph/MovieGraphPipeline.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelineUtils.h"
#include "MoviePipelineQueue.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "MovieSceneCommonHelpers.h"
#include "CoreGlobals.h"
#include "EngineUtils.h"
#include "Engine/GameViewportClient.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphSequenceDataSource)

UMovieGraphSequenceDataSource::UMovieGraphSequenceDataSource()
{
	CustomSequenceTimeController = MakeShared<UE::MovieGraph::FMovieGraphSequenceTimeController>();
}
 
void UMovieGraphSequenceDataSource::CacheDataPreJob(const FMovieGraphInitConfig& InInitConfig)
{
	// Turn off screen messages as some forms are drawn directly to final render targets
	// which will polute final frames.
	GAreScreenMessagesEnabled = false;

	// Turn off player viewport rendering to avoid overhead of an extra render.
	if (UGameViewportClient* Viewport = GetWorld()->GetGameViewport())
	{
		Viewport->bDisableWorldRendering = !InInitConfig.bRenderViewport;
	}

	ULevelSequence* RootSequence = Cast<ULevelSequence>(GetOwningGraph()->GetCurrentJob()->Sequence.TryLoad());
	if (RootSequence)
	{
		CacheLevelSequenceData(RootSequence);
	}
}

void UMovieGraphSequenceDataSource::RestoreCachedDataPostJob()
{
	GAreScreenMessagesEnabled = true;
	if (UGameViewportClient* Viewport = GetWorld()->GetGameViewport())
	{
		Viewport->bDisableWorldRendering = false;
	}

	ULevelSequence* RootSequence = Cast<ULevelSequence>(GetOwningGraph()->GetCurrentJob()->Sequence.TryLoad());
	MoviePipeline::RestoreCompleteSequenceHierarchy(RootSequence, CachedSequenceHierarchyRoot);
}

void UMovieGraphSequenceDataSource::UpdateShotList()
{
	ULevelSequence* RootSequence = Cast<ULevelSequence>(GetOwningGraph()->GetCurrentJob()->Sequence.TryLoad());
	if (RootSequence)
	{
		bool bShotsChanged = false;
		UMoviePipelineBlueprintLibrary::UpdateJobShotListFromSequence(RootSequence, GetOwningGraph()->GetCurrentJob(), bShotsChanged);
	}
}

void UMovieGraphSequenceDataSource::OverrideSequencePlaybackRangeFromGlobalOutputSettings(ULevelSequence* InSequence)
{
	FMovieGraphTraversalContext TraversalContext;
	UMoviePipelineExecutorJob* CurrentJob = GetOwningGraph()->GetCurrentJob();
	TraversalContext.Job = CurrentJob;

	FString OutErrorMessage;
	UMovieGraphEvaluatedConfig* EvaluatedGraph = CurrentJob->GetGraphPreset()->CreateFlattenedGraph(TraversalContext, OutErrorMessage);
	if (!EvaluatedGraph)
	{
		return;
	}

	constexpr bool bIncludeCDOs = true;
	constexpr bool bExactMatch = false;
	UMovieGraphGlobalOutputSettingNode* OutputSetting =
		EvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);

	TRange<FFrameNumber> CurrentPlaybackRange = InSequence->GetMovieScene()->GetPlaybackRange();

	FFrameNumber StartFrameTickResolution = CurrentPlaybackRange.GetLowerBound().GetValue();
	FFrameNumber EndFrameTickResolution = CurrentPlaybackRange.GetUpperBound().GetValue();
	
	if (OutputSetting->bOverride_CustomPlaybackRangeStart && OutputSetting->CustomPlaybackRangeStart.Type == EMovieGraphSequenceRangeType::Custom)
	{
		StartFrameTickResolution = FFrameRate::TransformTime(FFrameTime(FFrameNumber(OutputSetting->CustomPlaybackRangeStart.Value)), InSequence->GetMovieScene()->GetDisplayRate(), InSequence->GetMovieScene()->GetTickResolution()).FloorToFrame();
	}
	if (OutputSetting->bOverride_CustomPlaybackRangeEnd && OutputSetting->CustomPlaybackRangeEnd.Type == EMovieGraphSequenceRangeType::Custom)
	{
		EndFrameTickResolution = FFrameRate::TransformTime(FFrameTime(FFrameNumber(OutputSetting->CustomPlaybackRangeEnd.Value)), InSequence->GetMovieScene()->GetDisplayRate(), InSequence->GetMovieScene()->GetTickResolution()).CeilToFrame();
	}

	TRange<FFrameNumber> NewPlaybackRange = TRange<FFrameNumber>(StartFrameTickResolution, EndFrameTickResolution);
	
#if WITH_EDITOR
	InSequence->GetMovieScene()->SetPlaybackRangeLocked(false);
	InSequence->GetMovieScene()->SetReadOnly(false);
#endif
	InSequence->GetMovieScene()->SetPlaybackRange(NewPlaybackRange);

	// Warn about zero length playback ranges, often happens because they set the Start/End frame to the same frame.
	if (InSequence->GetMovieScene()->GetPlaybackRange().IsEmpty())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Playback Range was zero. End Frames are exclusive, did you mean [n, n+1]?");
	}
}

void UMovieGraphSequenceDataSource::CacheLevelSequenceData(ULevelSequence* InSequence)
{
	// There is a reasonable chance that there exists a Level Sequence Actor in the world already set up to play this sequence.
	ALevelSequenceActor* ExistingActor = nullptr;

	for (auto It = TActorIterator<ALevelSequenceActor>(GetWorld()); It; ++It)
	{
		// Iterate through all of them in the event someone has multiple copies in the world on accident.
		if (It->GetSequence() == InSequence)
		{
			// Found it!
			ExistingActor = *It;

			// Stop it from playing if it's already playing.
			if (ExistingActor->GetSequencePlayer())
			{
				ExistingActor->GetSequencePlayer()->Stop();
			}
		}
	}

	LevelSequenceActor = ExistingActor;
	if (!LevelSequenceActor)
	{
		// Spawn a new level sequence
		LevelSequenceActor = GetWorld()->SpawnActor<ALevelSequenceActor>();
		check(LevelSequenceActor);
	}

	// Enforce settings.
	LevelSequenceActor->PlaybackSettings.LoopCount.Value = 0;
	LevelSequenceActor->PlaybackSettings.bAutoPlay = false;
	LevelSequenceActor->PlaybackSettings.bPauseAtEnd = true;
	LevelSequenceActor->PlaybackSettings.FinishCompletionStateOverride = EMovieSceneCompletionModeOverride::ForceRestoreState;

	// Cache the sequence data to be restored later
	CachedSequenceHierarchyRoot = MakeShared<MoviePipeline::FCameraCutSubSectionHierarchyNode>();
	MoviePipeline::CacheCompleteSequenceHierarchy(InSequence, CachedSequenceHierarchyRoot);
	
	// Override the frame range on the target sequence if needed first before anyone has a chance to modify it.
	OverrideSequencePlaybackRangeFromGlobalOutputSettings(InSequence);

	// Ensure the (possibly new) Level Sequence Actor uses our sequence
	LevelSequenceActor->SetSequence(InSequence);

	LevelSequenceActor->GetSequencePlayer()->SetTimeController(CustomSequenceTimeController);
	LevelSequenceActor->GetSequencePlayer()->Stop();

	LevelSequenceActor->GetSequencePlayer()->OnSequenceUpdated().AddUObject(this, &UMovieGraphSequenceDataSource::OnSequenceEvaluated);

}

void UMovieGraphSequenceDataSource::OnSequenceEvaluated(const UMovieSceneSequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime)
{
	// This callback exists for logging purposes. DO NOT HINGE LOGIC ON THIS CALLBACK
	// because this may get called multiple times per frame and may be the result of
	// a seek operation which is reverted before a frame is even rendered.
	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "Sequence Evaluated. CurrentTime: %ls PreviousTime: %ls", *LexToString(CurrentTime), *LexToString(PreviousTime));
}

FFrameRate UMovieGraphSequenceDataSource::GetTickResolution() const
{
	if (ensure(LevelSequenceActor && LevelSequenceActor->GetSequence()))
	{
		return LevelSequenceActor->GetSequence()->GetMovieScene()->GetTickResolution();
	}

	return FFrameRate(24000, 1);
}

FFrameRate UMovieGraphSequenceDataSource::GetDisplayRate() const
{
	if (ensure(LevelSequenceActor && LevelSequenceActor->GetSequence()))
	{
		return LevelSequenceActor->GetSequence()->GetMovieScene()->GetDisplayRate();
	}

	return FFrameRate(24, 1);
}

void UMovieGraphSequenceDataSource::CacheHierarchyForShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	// Save the whole hierarchy for this shot, then set it to be inactive.
	TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> Node = InShot->ShotInfo.SubSectionHierarchy;
	const bool bSaveSettings = true;
	MoviePipeline::SaveOrRestoreSubSectionHierarchy(Node, bSaveSettings);
}

void UMovieGraphSequenceDataSource::RestoreHierarchyForShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> Node = InShot->ShotInfo.SubSectionHierarchy;

	const bool bSaveSettings = false;
	MoviePipeline::SaveOrRestoreSubSectionHierarchy(Node, bSaveSettings);
}

void UMovieGraphSequenceDataSource::MuteShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	const bool bActive = false;
	MoviePipeline::SetSubSectionHierarchyActive(InShot->ShotInfo.SubSectionHierarchy, bActive);
}

void UMovieGraphSequenceDataSource::UnmuteShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot)
{
	const bool bActive = true;
	MoviePipeline::SetSubSectionHierarchyActive(InShot->ShotInfo.SubSectionHierarchy, bActive);
}

void UMovieGraphSequenceDataSource::ExpandShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot, const int32 InLeftDeltaFrames, const int32 InLeftDeltaFramesUserPoV, 
	const int32 InRightDeltaFrames, const bool bInPrepass)
{
	const FFrameRate& DisplayRate = GetDisplayRate();
	const FFrameRate& TickResolution = GetTickResolution();

	TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> Node = InShot->ShotInfo.SubSectionHierarchy;
	while (Node)
	{
		// We need to expand the inner playback bounds to cover three features:
		// 1) Temporal Sampling (+1 frame left side)
		// 2) Handle frames (+n frames both sides)
		// 3) Non-emulated Warm Up (+n frames left side)
		// To keep the inner movie scene and outer sequencer section in sync we can calculate the tick delta
		// to each side and simply expand both sections like that - ignoring all start frame offsets, etc.
		FFrameNumber LeftDeltaTicks = FFrameRate::TransformTime(FFrameTime(InLeftDeltaFrames), DisplayRate, TickResolution).CeilToFrame();
		FFrameNumber RightDeltaTicks = FFrameRate::TransformTime(FFrameTime(InRightDeltaFrames), DisplayRate, TickResolution).CeilToFrame();
		FFrameTime LeftDeltaTimeUserPoV = FFrameRate::TransformTime(FFrameTime(InLeftDeltaFramesUserPoV), DisplayRate, TickResolution);

		// During pre-pass we cache which items we'd like to auto-expand later, and we print a warning
		// for anything we can't automatically expand that is now getting partial evaluation.
		if (bInPrepass)
		{
			if (Node->MovieScene.IsValid())
			{
				for (UMovieSceneSection* Section : Node->MovieScene->GetAllSections())
				{
					if (!Section)
					{
						continue;
					}

					// Their data is already cached for restore elsewhere.
					if (Section == Node->Section || Section == Node->CameraCutSection)
					{
						continue;
					}

					if (Section->GetSupportsInfiniteRange())
					{
						Node->AdditionalSectionsToExpand.Add(MakeTuple(Section, Section->GetRange()));
					}
				}
			}

			// We only do our warnings during the pre-pass
			// Check for sections that start in the expanded evaluation range and warn user. 
			// Only check the frames user expects to (handle + temporal, no need for warm up frames to get checked as well)
			MoviePipeline::CheckPartialSectionEvaluationAndWarn(LeftDeltaTimeUserPoV.CeilToFrame(), Node, InShot, DisplayRate);
		}
		else
		{
			if (Node->CameraCutSection.IsValid())
			{
				// Expand the camera cut section because there's no harm in doing it.
				Node->CameraCutSection->SetRange(UE::MovieScene::DilateRange(Node->CameraCutSection->GetRange(), -LeftDeltaTicks, RightDeltaTicks));
				Node->CameraCutSection->MarkAsChanged();
			}

			if (Node->Section.IsValid())
			{
				// Expand the MovieSceneSubSequenceSection
				Node->Section->SetRange(UE::MovieScene::DilateRange(Node->Section->GetRange(), -LeftDeltaTicks, RightDeltaTicks));
				Node->Section->MarkAsChanged();
			}

			if (Node->MovieScene.IsValid())
			{
				// Expand the Playback Range of the movie scene as well. Expanding this at the same time as expanding the 
				// SubSequenceSection will result in no apparent change to the evaluated time. ToDo: This doesn't work if
				// sub-sequences have different tick resolutions?
				Node->MovieScene->SetPlaybackRange(UE::MovieScene::DilateRange(Node->MovieScene->GetPlaybackRange(), -LeftDeltaTicks, RightDeltaTicks));
				Node->MovieScene->MarkAsChanged();
			}

			FFrameNumber LowerCheckBound = InShot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue() - LeftDeltaTimeUserPoV.CeilToFrame();
			FFrameNumber UpperCheckBound = InShot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue();

			TRange<FFrameNumber> CheckRange = TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Exclusive(LowerCheckBound), TRangeBound<FFrameNumber>::Inclusive(UpperCheckBound));

			for (const TTuple<UMovieSceneSection*, TRange<FFrameNumber>>& Pair : Node->AdditionalSectionsToExpand)
			{
				// Expand the section. Because it's an infinite range, we know the contents won't get shifted.
				TRange<FFrameNumber> NewRange = TRange<FFrameNumber>::Hull(Pair.Key->GetRange(), CheckRange);
				Pair.Key->SetRange(NewRange);
				Pair.Key->MarkAsChanged();
			}
		}

		// Travel up to the parent and expand it too.
		Node = Node->GetParent();
	}
}


void UMovieGraphSequenceDataSource::SyncDataSourceTime(const FFrameTime& InTime)
{
	if(LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{
		FFrameRate TickResolution = LevelSequenceActor->GetSequence()->GetMovieScene()->GetTickResolution();
		CustomSequenceTimeController->SetCachedFrameTiming(FQualifiedFrameTime(InTime, TickResolution));
	}
}
void UMovieGraphSequenceDataSource::PlayDataSource()
{
	if(LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{
		LevelSequenceActor->GetSequencePlayer()->Play();
	}
}

void UMovieGraphSequenceDataSource::PauseDataSource() 
{
	if(LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{
		LevelSequenceActor->GetSequencePlayer()->Pause();
	}
}

void UMovieGraphSequenceDataSource::StopDataSource()
{
	if (LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{
		LevelSequenceActor->GetSequencePlayer()->Stop();
	}
}

void UMovieGraphSequenceDataSource::JumpDataSource(const FFrameTime& InTimeToJumpTo) 
{
	if(LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{

		// SetPlaybackPosition takes time in display rate and not tick resolution, so we convert
		// it from tick resolution to display rate. JumpDataSource provides it in tick resolution
		// to be consistent with other functions in this class. This doesn't use the "Effective"
		// framerate of the render because we never modified the Sequence, so we need to use original
		// DisplayRate to jump to the correct frame.
		FFrameRate TickResolution = LevelSequenceActor->GetSequence()->GetMovieScene()->GetTickResolution();
		FFrameRate DisplayRate = LevelSequenceActor->GetSequence()->GetMovieScene()->GetDisplayRate();
		FFrameTime RequestTime = FFrameRate::TransformTime(InTimeToJumpTo, TickResolution, DisplayRate);

		LevelSequenceActor->GetSequencePlayer()->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(RequestTime, EUpdatePositionMethod::Jump));
	}
}

void UMovieGraphSequenceDataSource::InitializeShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot, const FFrameTime& InEvalTime)
{
	Super::InitializeShot(InShot, InEvalTime);
	
	SyncDataSourceTime(InEvalTime);
	JumpDataSource(InEvalTime);
}

TArray<UE::MovieGraph::FMinimalCameraInfo> UMovieGraphSequenceDataSource::GetCameraInformation(UMoviePipelineExecutorShot* InShot, bool bIncludeSidecar) const
{
	TArray<UE::MovieGraph::FMinimalCameraInfo> OutInfo;

	if (!bIncludeSidecar)
	{
		if (const APlayerController* LocalPlayerController = GetOwningGraph()->GetWorld()->GetFirstPlayerController())
		{
			FMinimalViewInfo ViewInfo = LocalPlayerController->PlayerCameraManager->GetCameraCacheView();

			UE::MovieGraph::FMinimalCameraInfo MinimalInfo;
			MinimalInfo.ViewActor = LocalPlayerController->GetViewTarget();
			MinimalInfo.ViewInfo = ViewInfo;

			OutInfo.Add(MinimalInfo);
		}
	}
	else
	{
		// Keep one result per sidecar so caller indices match SidecarCameras.
		OutInfo.SetNum(InShot->SidecarCameras.Num());

		for (int32 Index = 0; Index < InShot->SidecarCameras.Num(); Index++)
		{
			const FGuid CameraBindingRef = InShot->SidecarCameras[Index].BindingId;
			const FMovieSceneSequenceID CameraSequenceRef = InShot->SidecarCameras[Index].SequenceId;

			TArrayView<TWeakObjectPtr<UObject>> BoundCameras = LevelSequenceActor->GetSequencePlayer()->FindBoundObjects(CameraBindingRef, CameraSequenceRef);
			if (BoundCameras.Num() > 0)
			{
				UCameraComponent* BoundCamera = MovieSceneHelpers::CameraComponentFromRuntimeObject(BoundCameras[0].Get());
				if (BoundCamera)
				{

					FMinimalViewInfo ViewInfo;
					BoundCamera->GetCameraView(GetWorld()->GetDeltaSeconds(), ViewInfo);

					UE::MovieGraph::FMinimalCameraInfo MinimalInfo;
					MinimalInfo.ViewActor = BoundCamera->GetTypedOuter<AActor>();
					MinimalInfo.ViewInfo = ViewInfo;
					OutInfo[Index] = MinimalInfo;
				}
			}
		}
	}

	return OutInfo;
}

UMovieSceneSequencePlayer* UMovieGraphSequenceDataSource::GetSequencePlayer() const
{
	if (LevelSequenceActor)
	{
		return LevelSequenceActor->GetSequencePlayer();
	}

	return nullptr;
}

namespace UE::MovieGraph
{
	FFrameTime FMovieGraphSequenceTimeController::OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate)
	{
		FFrameTime RequestTime = FFrameRate::TransformTime(TimeCache.Time, TimeCache.Rate, InCurrentTime.Rate);
		UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "[%lld] OnRequestCurrentTime: %d %f", GFrameCounter, RequestTime.FloorToFrame().Value, RequestTime.GetSubFrame());

		return RequestTime;
	}
}
