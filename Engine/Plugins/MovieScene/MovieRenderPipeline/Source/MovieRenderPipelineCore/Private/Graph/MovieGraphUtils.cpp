// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphUtils.h"

#include "AudioMixerDevice.h"
#include "AudioThread.h"
#include "Engine/Engine.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "HAL/PlatformFileManager.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"

#if WITH_EDITOR
#include "Engine/RendererSettings.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ISettingsEditorModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "MovieGraphUtils"

namespace UE::MovieGraph
{
	FString GetUniqueName(const TArray<FString>& InExistingNames, const FString& InBaseName)
	{
		int32 Postfix = 0;
		FString NewName = InBaseName;

		while (InExistingNames.Contains(NewName))
		{
			Postfix++;
			NewName = FString::Format(TEXT("{0} {1}"), {InBaseName, Postfix});
		}

		return NewName;
	}

	bool ResolveConnectedPinValue(UMovieGraphPin* InFromPin, const TArray<UMovieGraphPin*>& InConnectedPins, const FMovieGraphEvaluationContext& InEvaluationContext, FString& OutResolvedValue)
	{
		TArray<UMovieGraphPin*> ConnectionPath;

		// Iterate up the connection chain and find all pins which might have a value that can be resolved.
		FMovieGraphEvaluationContext ValueConnectionContext;
		ValueConnectionContext.PinBeingFollowed = InFromPin;
		ValueConnectionContext.SubgraphStack = InEvaluationContext.SubgraphStack;

		TArray<UMovieGraphPin*> ConnectedValuePins = InConnectedPins;
		while (!ConnectedValuePins.IsEmpty())
		{
			UMovieGraphPin* ConnectedValuePin = ConnectedValuePins[0];
			if (!ensureMsgf(ConnectedValuePin, TEXT("Found an invalid pin on node '%s'."), *InFromPin->Node->GetName()))
			{
				// Can't continue following the connection chain if an invalid pin was found
				break;
			}

			if (ConnectionPath.Contains(ConnectedValuePin))
			{
				// Recursive connection found
				UE_LOGF(LogMovieRenderPipeline, Warning, "Found a cycle when following the data connection on pin '%ls' for node '%ls'. Value will not be resolved.",
					*ConnectedValuePin->Properties.Label.ToString(), *InFromPin->Node->GetName());
				break;
			}

			// For the connected value to be used, the types must be compatible and the node the value is originating from must be enabled
			if (InFromPin->IsTypeCompatibleWith(ConnectedValuePin) && ConnectedValuePin->Node && !ConnectedValuePin->Node->IsDisabled())
			{
				ConnectionPath.Add(ConnectedValuePin);
			}

			ValueConnectionContext.PinBeingFollowed = ConnectedValuePin;
			ConnectedValuePins = ConnectedValuePin->Node->EvaluatePinsToFollow(ValueConnectionContext);
		}

		// Work backwards and use the most upstream value that can be resolved. The most upstream value wins. For example, if a node has an
		// exposed pin, that pin is connected to a subgraph's input, and that input is then connected to a variable node in the parent graph,
		// then the variable node's value should be used if it can be resolved, not the subgraph's input value.
		for (int32 Index = ConnectionPath.Num() - 1; Index >= 0; --Index)
		{
			const UMovieGraphPin* ConnectedPin = ConnectionPath[Index];

			const FString ResolvedValue = ConnectedPin->Node->GetResolvedValueForOutputPin(ConnectedPin->Properties.Label, &InEvaluationContext.UserContext);
			if (ResolvedValue.IsEmpty())
			{
				continue;
			}

			OutResolvedValue = ResolvedValue;
			return true;
		}

		return false;
	}

	namespace Audio
	{
		FAudioDevice* GetAudioDeviceFromWorldContext(const UObject* InWorldContextObject)
		{
			const UWorld* ThisWorld = GEngine->GetWorldFromContextObject(InWorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
			if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || (ThisWorld->GetNetMode() == NM_DedicatedServer))
			{
				return nullptr;
			}

			return ThisWorld->GetAudioDeviceRaw();
		}

		::Audio::FMixerDevice* GetAudioMixerDeviceFromWorldContext(const UObject* InWorldContextObject)
		{
			if (FAudioDevice* AudioDevice = GetAudioDeviceFromWorldContext(InWorldContextObject))
			{
				return static_cast<::Audio::FMixerDevice*>(AudioDevice);
			}

			return nullptr;
		}

		bool IsMoviePipelineAudioOutputSupported(const UObject* InWorldContextObject)
		{
			const ::Audio::FMixerDevice* MixerDevice = GetAudioMixerDeviceFromWorldContext(InWorldContextObject);
			const ::Audio::IAudioMixerPlatformInterface* AudioMixerPlatform = MixerDevice ? MixerDevice->GetAudioMixerPlatform() : nullptr;

			// If the current audio mixer is non-realtime, audio output is supported
			if (AudioMixerPlatform && AudioMixerPlatform->IsNonRealtime())
			{
				return true;
			}

			// If there is no async audio processing (e.g. we're in the editor), it's possible to create a new non-realtime audio mixer
			if (!FAudioThread::IsUsingThreadedAudio())
			{
				return true;
			}

			// Otherwise, we can't support audio output
			return false;
		}
	}

	namespace Capture
	{
		FString PrepareCapturePath(
			const UMovieGraphPipeline* InPipeline,
			const TObjectPtr<UMovieGraphEvaluatedConfig>& InEvaluatedConfig,
			const FString& InPathFormat, const FString& InExt, bool InForShot)
		{
			check(InEvaluatedConfig);
			check(InPipeline);

			constexpr bool bIncludeCDOs = true;
			constexpr bool bExactMatch = true;
			const UMovieGraphGlobalOutputSettingNode* OutputSetting =
				InEvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphSettingNode::GlobalsPinName, bIncludeCDOs, bExactMatch);
			FString FileNameFormatString = OutputSetting->OutputDirectory.Path / InPathFormat;

			const bool bOverwriteExistingOutput = OutputSetting->bOverwriteExistingOutput;

			// Generate a filename for this encoded file
			TMap<FString, FString> FormatOverrides;
			FormatOverrides.Add(TEXT("ext"), InExt);

			FMovieGraphRenderDataIdentifier TempRenderDataIdentifier;
			TempRenderDataIdentifier.RootBranchName = UMovieGraphSettingNode::GlobalsPinName;

			FMovieGraphFilenameResolveParams Params = FMovieGraphFilenameResolveParams::MakeResolveParams(
				TempRenderDataIdentifier, InPipeline, InEvaluatedConfig, InPipeline->GetCurrentTraversalContext(InForShot), FormatOverrides);

			FMovieGraphResolveArgs FinalFormatArgs;
			constexpr bool bIncludeRenderPass = false;
			const bool bTestFrameNumber = InForShot;
			constexpr bool bIncludeCameraName = false;
			UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber, bIncludeCameraName);
			FString FinalPath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FileNameFormatString, Params, FinalFormatArgs);

			if (FPaths::IsRelative(FinalPath))
			{
				FinalPath = FPaths::ConvertRelativePathToFull(FinalPath);
			}

			if (bOverwriteExistingOutput)
			{
				if (FPlatformFileManager::Get().GetPlatformFile().DirectoryExists(*FinalPath))
				{
					FPlatformFileManager::Get().GetPlatformFile().DeleteDirectoryRecursively(*FinalPath);
				} 
				else if (!UE::MoviePipeline::CanWriteToFile(*FinalPath, false)) // false otherwise it will always return true
				{
					FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*FinalPath);
				}
			}

			return FinalPath;
		}
	}

#if WITH_EDITOR
	void UpdateDependentPropertyInConfigFile(URendererSettings* RendererSettings, FProperty* RendererProperty)
	{
		FString RelativePath = RendererSettings->GetDefaultConfigFilename();
		FString FullPath = FPaths::ConvertRelativePathToFull(RelativePath);

		const bool bIsWriteable = !FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*FullPath);

		if (!bIsWriteable)
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*FullPath, false);
		}

		RendererSettings->UpdateSinglePropertyInConfigFile(RendererProperty, RendererSettings->GetDefaultConfigFilename());

		// Restore original state for source control
		if (!bIsWriteable)
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*FullPath, true);
		}
	}

	void ValidateAlphaProjectSettings(const FText& InRequestingFeatureName, bool bMandatePrimitiveAlphaHoldout)
	{
		URendererSettings* RendererSettings = GetMutableDefault<URendererSettings>();
		check(RendererSettings);

		const bool bAlphaOutputMissing = !RendererSettings->bEnableAlphaChannelInPostProcessing;
		const bool bPrimitiveHoldoutMissing = bMandatePrimitiveAlphaHoldout ? !RendererSettings->bDeferredSupportPrimitiveAlphaHoldout : false;

		if (bAlphaOutputMissing || bPrimitiveHoldoutMissing)
		{
			static TWeakPtr<SNotificationItem> NotificationItem;

			const FText AlphaOutputSettingOption = LOCTEXT("MovieAlphaSetting_AlphaOutput", "\n- Alpha Output");
			const FText PrimitiveHoldoutSettingOption = LOCTEXT("MovieAlphaSetting_PrimitiveHoldout", "\n- Support Primitive Alpha Holdout");
			const FText MovieAlphaText = FText::Format(LOCTEXT("MovieAlphaSettingPrompt", "The 'Support Primitive Alpha Holdout' project setting w/ 'Alpha Ouput' must be enabled for Movie Render Graph's {0}.\n\nWarning: update can add renderer performance costs."), InRequestingFeatureName);
			const FText MovieAlphaConfirmText = LOCTEXT("MovieAlphaSettingConfirm", "Enable (DefaultEngine.ini)");
			const FText MovieAlphaCancelText = LOCTEXT("MovieAlphaSettingCancel", "Not Now");

			/** Utility functions for notifications */
			struct FSuppressDialogOptions
			{
				static bool ShouldSuppressModal()
				{
					bool bSuppressNotification = false;
					GConfig->GetBool(TEXT("MovieRenderPipeline"), TEXT("SuppressMovieRenderPipelineAlphaPromptNotification"), bSuppressNotification, GEditorPerProjectIni);
					return bSuppressNotification;
				}

				static ECheckBoxState GetDontAskAgainCheckBoxState()
				{
					return ShouldSuppressModal() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}

				static void OnDontAskAgainCheckBoxStateChanged(ECheckBoxState NewState)
				{
					// If the user selects to not show this again, set that in the config so we know about it in between sessions
					const bool bSuppressNotification = (NewState == ECheckBoxState::Checked);
					GConfig->SetBool(TEXT("MovieRenderPipeline"), TEXT("SuppressMovieRenderPipelineAlphaPromptNotification"), bSuppressNotification, GEditorPerProjectIni);
				}
			};

			// If the user has specified to suppress this pop up, then just early out and exit
			if (FSuppressDialogOptions::ShouldSuppressModal())
			{
				return;
			}

			FSimpleDelegate OnConfirmDelegate = FSimpleDelegate::CreateLambda(
				[RendererSettings, bAlphaOutputMissing, bPrimitiveHoldoutMissing]()
				{
					if (IsValid(RendererSettings))
					{
						if (bAlphaOutputMissing)
						{
							FProperty* Property = RendererSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableAlphaChannelInPostProcessing));
							RendererSettings->PreEditChange(Property);

							RendererSettings->bEnableAlphaChannelInPostProcessing = true;

							FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet, { RendererSettings });
							RendererSettings->PostEditChangeProperty(PropertyChangedEvent);
							UpdateDependentPropertyInConfigFile(RendererSettings, Property);
						}

						if (bPrimitiveHoldoutMissing)
						{
							FProperty* Property = RendererSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, bDeferredSupportPrimitiveAlphaHoldout));
							RendererSettings->PreEditChange(Property);

							RendererSettings->bDeferredSupportPrimitiveAlphaHoldout = true;

							FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet, { RendererSettings });
							RendererSettings->PostEditChangeProperty(PropertyChangedEvent);
							UpdateDependentPropertyInConfigFile(RendererSettings, Property);

							// SupportPrimitiveAlphaHoldout requires shader recompilation, ask for a restart.
							FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor").OnApplicationRestartRequired();
						}

						if (TSharedPtr<SNotificationItem> Item = NotificationItem.Pin())
						{
							Item->SetCompletionState(SNotificationItem::CS_Success);
							Item->ExpireAndFadeout();
						}

						NotificationItem.Reset();
					}
				}
			);

			FSimpleDelegate OnCancelDelegate = FSimpleDelegate::CreateLambda([]
				{
					if (TSharedPtr<SNotificationItem> Item = NotificationItem.Pin())
					{
						Item->SetCompletionState(SNotificationItem::CS_None);
						Item->ExpireAndFadeout();
					}

					NotificationItem.Reset();
				}
			);

			FNotificationInfo Info(MovieAlphaText);
			Info.bFireAndForget = false;
			Info.bUseLargeFont = false;
			Info.bUseThrobber = false;
			Info.bUseSuccessFailIcons = false;
			Info.ButtonDetails.Add(FNotificationButtonInfo(MovieAlphaConfirmText, FText(), OnConfirmDelegate));
			Info.ButtonDetails.Add(FNotificationButtonInfo(MovieAlphaCancelText, FText(), OnCancelDelegate));

			// Add a "Don't show this again" option
			Info.CheckBoxState = TAttribute<ECheckBoxState>::Create(&FSuppressDialogOptions::GetDontAskAgainCheckBoxState);
			Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(&FSuppressDialogOptions::OnDontAskAgainCheckBoxStateChanged);
			Info.CheckBoxText = LOCTEXT("DefaultCheckBoxMessage", "Don't show this again");

			if (TSharedPtr<SNotificationItem> Item = NotificationItem.Pin())
			{
				Item->ExpireAndFadeout();
			}

			NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

			if (TSharedPtr<SNotificationItem> Item = NotificationItem.Pin())
			{
				Item->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
	}
#endif
}

#undef LOCTEXT_NAMESPACE // "MovieGraphUtils"
