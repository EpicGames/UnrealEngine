// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Forward declares
class FAudioDevice;
class UMovieGraphPin;
class UMovieGraphPipeline;
class UMovieGraphEvaluatedConfig;
struct FMovieGraphEvaluationContext;
namespace Audio
{
	class FMixerDevice;
}

namespace UE::MovieGraph
{
	/**
	 * Generate a unique name given a set of existing names and the desired base name. The base name will
	 * be given a postfix value if it conflicts with an existing name (eg, if the base name is "Foo" but
	 * there's already an existing name "Foo", the generated name would be "Foo 1").
	 */
	FString GetUniqueName(const TArray<FString>& InExistingNames, const FString& InBaseName);

	/**
	 * Gets the resolved value of an input pin (InFromPin), given all of its connections (InConnectedPins). Providing an evaluation context is needed,
	 * mostly to ensure that recursion doesn't occur. The resolved value is provided in OutResolvedValue as the serialized representation.
	 * Returns true if a value could be resolved, else false.
	 *
	 * Generally the normal graph evaluation process should resolve values, and using this method might indicate you're doing the wrong thing.
	 * However, for some non-setting nodes, it may be necessary to resolve values manually since non-setting nodes are not evaluated like normal
	 * setting nodes.
	 */
	bool ResolveConnectedPinValue(UMovieGraphPin* InFromPin, const TArray<UMovieGraphPin*>& InConnectedPins, const FMovieGraphEvaluationContext& InEvaluationContext, FString& OutResolvedValue);

	namespace Audio
	{
		/** Gets the audio device from the supplied world context (or nullptr if it could not be determined). */
		FAudioDevice* GetAudioDeviceFromWorldContext(const UObject* InWorldContextObject);

		/** Gets the audio mixer from the supplied world context (or nullptr if it could not be determined). */
		::Audio::FMixerDevice* GetAudioMixerDeviceFromWorldContext(const UObject* InWorldContextObject);

		/** Determines if the pipeline can generate audio. */
		bool IsMoviePipelineAudioOutputSupported(const UObject* InWorldContextObject);
	}

	namespace Capture
	{
		FString PrepareCapturePath(
			const UMovieGraphPipeline* InPipeline,
			const TObjectPtr<UMovieGraphEvaluatedConfig>& InEvaluatedConfig,
			const FString& InPathFormat, const FString& InExt, bool InForShot);
	}

#if WITH_EDITOR
	void ValidateAlphaProjectSettings(const FText& InRequestingFeatureLabel, bool bMandatePrimitiveAlphaHoldout = false);
#endif
}
