// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphValueUtils.h"

#include "UObject/EnumProperty.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

bool UE::MovieGraph::ValueUtils::GetTypesForFProperty(
	const FProperty* InProperty, EMovieGraphValueType& OutValueType, TObjectPtr<UObject>& OutValueTypeObject)
{
	OutValueTypeObject = nullptr;
	OutValueType = EMovieGraphValueType::None;

	if (!InProperty)
	{
		return false;
	}

	if (InProperty->IsA<FBoolProperty>())
	{
		OutValueType = EMovieGraphValueType::Bool;
	}
	else if (InProperty->IsA<FByteProperty>())
	{
		OutValueType = EMovieGraphValueType::Byte;

		if (const TObjectPtr<UEnum> ByteAsEnum = CastField<FByteProperty>(InProperty)->Enum)
		{
			OutValueType = EMovieGraphValueType::Enum;
			OutValueTypeObject = ByteAsEnum;
		}
	}
	else if (InProperty->IsA<FIntProperty>())
	{
		OutValueType = EMovieGraphValueType::Int32;
	}
	else if (InProperty->IsA<FInt64Property>())
	{
		OutValueType = EMovieGraphValueType::Int64;
	}
	else if (InProperty->IsA<FFloatProperty>())
	{
		OutValueType = EMovieGraphValueType::Float;
	}
	else if (InProperty->IsA<FDoubleProperty>())
	{
		OutValueType = EMovieGraphValueType::Double;
	}
	else if (InProperty->IsA<FNameProperty>())
	{
		OutValueType = EMovieGraphValueType::Name;
	}
	else if (InProperty->IsA<FStrProperty>())
	{
		OutValueType = EMovieGraphValueType::String;
	}
	else if (InProperty->IsA<FTextProperty>())
	{
		OutValueType = EMovieGraphValueType::Text;
	}
	else if (InProperty->IsA<FEnumProperty>())
	{
		OutValueType = EMovieGraphValueType::Enum;
		OutValueTypeObject = CastField<FEnumProperty>(InProperty)->GetEnum();
	}
	else if (InProperty->IsA<FStructProperty>())
	{
		OutValueType = EMovieGraphValueType::Struct;
		OutValueTypeObject = CastField<FStructProperty>(InProperty)->Struct.Get();
	}
	else if (InProperty->IsA<FObjectPropertyBase>())
	{
		UClass* PropertyClass = CastField<FObjectPropertyBase>(InProperty)->PropertyClass.Get();
		
		if (InProperty->IsA<FObjectProperty>())
		{
			if (PropertyClass == UClass::StaticClass())
			{
				OutValueType = EMovieGraphValueType::Class;
				OutValueTypeObject = UObject::StaticClass();
			}
			else
			{
				OutValueType = EMovieGraphValueType::Object;
				OutValueTypeObject = PropertyClass;
			}
		}
		else if (InProperty->IsA<FSoftObjectProperty>())
		{
			OutValueType = EMovieGraphValueType::SoftObject;
			OutValueTypeObject = PropertyClass;
		}
		else if (InProperty->IsA<FClassProperty>())
		{
			OutValueType = EMovieGraphValueType::Class;
			OutValueTypeObject = CastField<FClassProperty>(InProperty)->MetaClass;
		}
		else if (InProperty->IsA<FSoftClassProperty>())
		{
			OutValueType = EMovieGraphValueType::SoftClass;
			OutValueTypeObject = CastField<FSoftClassProperty>(InProperty)->MetaClass;
		}
	}
	else if (InProperty->IsA<FInt8Property>())
	{
		OutValueType = EMovieGraphValueType::Int8;
	}
	else if (InProperty->IsA<FInt16Property>())
	{
		OutValueType = EMovieGraphValueType::Int16;
	}
	else if (InProperty->IsA<FUInt16Property>())
	{
		OutValueType = EMovieGraphValueType::UInt16;
	}
	else if (InProperty->IsA<FUInt32Property>())
	{
		OutValueType = EMovieGraphValueType::UInt32;
	}
	else if (InProperty->IsA<FUInt64Property>())
	{
		OutValueType = EMovieGraphValueType::UInt64;
	}
	else
	{
		// Couldn't find the value type and/or value type object for the property.
		return false;
	}

	return true;
}
