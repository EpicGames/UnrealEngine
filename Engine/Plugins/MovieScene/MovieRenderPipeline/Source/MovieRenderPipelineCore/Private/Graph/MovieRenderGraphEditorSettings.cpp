// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieRenderGraphEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieRenderGraphEditorSettings)

UMovieRenderGraphEditorSettings::UMovieRenderGraphEditorSettings()
	: PostRenderSettings(FMovieGraphPostRenderSettings())
{
	// Seed default visible output types for Basic Config mode.
	// These defaults are used on first launch before the user modifies via the gear menu.
	VisibleBasicConfigOutputTypes.Add(FSoftClassPath(TEXT("/Script/MovieRenderPipelineRenderPasses.MovieGraphImageSequenceOutputNode_MultiLayerEXR")));
	VisibleBasicConfigOutputTypes.Add(FSoftClassPath(TEXT("/Script/MovieRenderPipelineRenderPasses.MovieGraphImageSequenceOutputNode_JPG")));
	VisibleBasicConfigOutputTypes.Add(FSoftClassPath(TEXT("/Script/MovieRenderPipelineMP4Encoder.MovieGraphMP4EncoderNode")));
	VisibleBasicConfigOutputTypes.Add(FSoftClassPath(TEXT("/Script/MovieRenderPipelineRenderPasses.MovieGraphImageSequenceOutputNode_PNG")));
	VisibleBasicConfigOutputTypes.Add(FSoftClassPath(TEXT("/Script/AppleProResMedia.MovieGraphAppleProResNode")));
}

FName UMovieRenderGraphEditorSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}

#if WITH_EDITOR
void UMovieRenderGraphEditorSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	static FProperty* ImageSequenceOptions =
		FMovieGraphPostRenderSettings::StaticStruct()->FindPropertyByName(
			GET_MEMBER_NAME_STRING_CHECKED(FMovieGraphPostRenderSettings, PostRenderImageSequencePlayOptions));

	static FProperty* ProResOptions =
		FMovieGraphPostRenderSettings::StaticStruct()->FindPropertyByName(
			GET_MEMBER_NAME_STRING_CHECKED(FMovieGraphPostRenderSettings, PostRenderAppleProResPlayOptions));

	static FProperty* AvidOptions =
		FMovieGraphPostRenderSettings::StaticStruct()->FindPropertyByName(
			GET_MEMBER_NAME_STRING_CHECKED(FMovieGraphPostRenderSettings, PostRenderAvidDNxHRPlayOptions));

	static FProperty* MP4Options =
		FMovieGraphPostRenderSettings::StaticStruct()->FindPropertyByName(
			GET_MEMBER_NAME_STRING_CHECKED(FMovieGraphPostRenderSettings, PostRenderMP4PlayOptions));

	auto UpdatePlayOptions = [](FMovieGraphPostRenderVideoPlayOptions* InPlayOptions)
	{
		if (InPlayOptions->PlaybackMethod == EMovieGraphPlaybackMethod::CustomViewer)
		{
			InPlayOptions->JobPlayback = EMovieGraphJobPlaybackRange::AllJobs;
			InPlayOptions->RenderLayerPlayback = EMovieGraphRenderLayerPlaybackRange::AllRenderLayers;
		}
		else
		{
			InPlayOptions->JobPlayback = EMovieGraphJobPlaybackRange::FirstJobOnly;
			InPlayOptions->RenderLayerPlayback = EMovieGraphRenderLayerPlaybackRange::FirstRenderLayerOnly;
		}
	};

	// If the Playback Method changes, update some other properties to reflect the most likely use case of this playback method.
	if ((PropertyChangedEvent.Property != nullptr) && (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FMovieGraphPostRenderVideoPlayOptions, PlaybackMethod)))
	{
		if (PropertyChangedEvent.PropertyChain.Contains(ImageSequenceOptions))
		{
			UpdatePlayOptions(&PostRenderSettings.PostRenderImageSequencePlayOptions);

			// Image sequences additionally have a Playback Range option that needs to be updated
			const EMovieGraphPlaybackMethod PlaybackMethod = PostRenderSettings.PostRenderImageSequencePlayOptions.PlaybackMethod;
			PostRenderSettings.PostRenderImageSequencePlayOptions.PlaybackRange = (PlaybackMethod == EMovieGraphPlaybackMethod::CustomViewer)
				? EMovieGraphImageSequencePlaybackRange::FullRange
				: EMovieGraphImageSequencePlaybackRange::FirstFrameOnly;
		}
		else if (PropertyChangedEvent.PropertyChain.Contains(ProResOptions))
		{
			UpdatePlayOptions(&PostRenderSettings.PostRenderAppleProResPlayOptions);
		}
		else if (PropertyChangedEvent.PropertyChain.Contains(AvidOptions))
		{
			UpdatePlayOptions(&PostRenderSettings.PostRenderAvidDNxHRPlayOptions);
		}
		else if (PropertyChangedEvent.PropertyChain.Contains(MP4Options))
		{
			UpdatePlayOptions(&PostRenderSettings.PostRenderMP4PlayOptions);
		}
	}
	
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif	// WITH_EDITOR
