// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphApplyCVarPresetNode.h"

#include "Graph/MovieGraphConfig.h"
#include "MoviePipelineTelemetry.h"
#include "Sections/MovieSceneConsoleVariableTrackInterface.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphApplyCVarPresetNode)

FString UMovieGraphApplyCVarPresetNode::GetNodeInstanceName() const
{
	if (ConsoleVariablePreset)
	{
		return ConsoleVariablePreset.GetObject()->GetName();
	}

	return FString();
}

EMovieGraphBranchRestriction UMovieGraphApplyCVarPresetNode::GetBranchRestriction() const
{
	return EMovieGraphBranchRestriction::Globals;
}

void UMovieGraphApplyCVarPresetNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	if (ConsoleVariablePreset)
	{
		InTelemetry->bUsesConsoleVariablePreset = true;
	}
}

TArray<FMovieGraphPropertyInfo> UMovieGraphApplyCVarPresetNode::GetOverrideablePropertyInfo() const
{
	TArray<FMovieGraphPropertyInfo> PropertyInfo = Super::GetOverrideablePropertyInfo();

	for (FMovieGraphPropertyInfo& Info : PropertyInfo)
	{
		if (!Info.bIsDynamicProperty)
		{
			continue;
		}

		Info.ContextMenuName = FText::Format(NSLOCTEXT("MovieGraphNodes", "ApplyCVarPresetNode_ValuePromotionContextMenuName", "Value ({0})"), FText::FromName(Info.Name));
		Info.PromotionName = Info.Name;
	}

	return PropertyInfo;
}

TArray<FPropertyBagPropertyDesc> UMovieGraphApplyCVarPresetNode::GetDynamicPropertyDescriptions() const
{
	static const FName HideInActiveRenderSettingsMetaDataKey(TEXT("HideInActiveRenderSettings"));
	
	TArray<FPropertyBagPropertyDesc> CvarDescs;

	// No dynamic properties if the console variable preset has not yet been chosen or is invalid
	if (!ConsoleVariablePreset)
	{
		return CvarDescs;
	}

	constexpr bool bOnlyIncludeChecked = true;
	TArray<TTuple<FString, FString>> ConsoleVariables;
	ConsoleVariablePreset->GetConsoleVariablesForTrack(bOnlyIncludeChecked, ConsoleVariables);

	// Add in properties which correspond to the cvars within the preset
	for (const TTuple<FString, FString>& CVarPair : ConsoleVariables)
	{
		const FName CVarName = FName(CVarPair.Key);

		// Hide these dynamic properties in the Active Render Settings. They will not show correct values if they're not connected to a variable.
		// Cvars still need to be fully resolved, and these properties are not the place for that.
		FPropertyBagPropertyDesc NewDesc(CVarName, EPropertyBagPropertyType::Float);
#if WITH_EDITOR
		NewDesc.MetaData.Add(FPropertyBagPropertyDescMetaData(HideInActiveRenderSettingsMetaDataKey, FString()));
#endif
		CvarDescs.Add(MoveTemp(NewDesc));

		// Need a bOverride_* property in order for this to be picked up properly by graph evaluation
		const FString OverrideName = FString::Printf(TEXT("bOverride_%s"), *CVarName.ToString());
		FPropertyBagPropertyDesc OverrideDesc(FName(OverrideName), EPropertyBagPropertyType::Bool);
		CvarDescs.Add(MoveTemp(OverrideDesc));
	}

	return CvarDescs;
}

bool UMovieGraphApplyCVarPresetNode::GetDynamicPropertyValue(const FName PropertyName, FString& OutValue)
{
	// This node uses dynamic properties to enable cvars within the preset to be promoted to pins. However, the DynamicProperties property bag does
	// not track the value of the cvars within the preset (that would be a syncing nightmare). Instead, resolve the value of the cvar on-demand here.

	constexpr bool bOnlyIncludeChecked = true;
	TArray<TTuple<FString, FString>> ConsoleVariables;
	ConsoleVariablePreset->GetConsoleVariablesForTrack(bOnlyIncludeChecked, ConsoleVariables);

	for (const TPair<FString, FString>& CVarPair : ConsoleVariables)
	{
		if (CVarPair.Key == PropertyName)
		{
			OutValue = CVarPair.Value;
			return true;
		}
	}

	return false;
}

void UMovieGraphApplyCVarPresetNode::TogglePromotePropertyToPin(const FName& PropertyName)
{
	Super::TogglePromotePropertyToPin(PropertyName);
	
	UpdateDynamicProperties();
}

void UMovieGraphApplyCVarPresetNode::PrepareForFlattening(const UMovieGraphSettingNode* InSourceNode)
{
	Super::PrepareForFlattening(InSourceNode);

	// This node's dynamic properties rely on the cvar preset being valid and correct in order to be properly initialized.
	if (const UMovieGraphApplyCVarPresetNode* SourcePresetNode = Cast<UMovieGraphApplyCVarPresetNode>(InSourceNode))
	{
		ConsoleVariablePreset = SourcePresetNode->ConsoleVariablePreset;
	}
}

TArray<TPair<FString, float>> UMovieGraphApplyCVarPresetNode::GetConsoleVariableOverrides() const
{
	TArray<TPair<FString, float>> OverrideValues;
	
	for (const FPropertyBagPropertyDesc& Desc : GetDynamicPropertyDescriptions())
	{
		if (IsDynamicPropertyOverridden(Desc.Name))
		{
			TPair<FString, float> NewOverride(Desc.Name.ToString(), DynamicProperties.GetValueFloat(Desc.Name).GetValue());
			OverrideValues.Add(MoveTemp(NewOverride));
		}
	}

	return OverrideValues;
}

#if WITH_EDITOR
FText UMovieGraphApplyCVarPresetNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText ApplyCVarPresetNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_ApplyCVarPreset", "Apply Console Variable Preset");
	static const FText ApplyCVarPresetNodeDescription = NSLOCTEXT("MovieGraphNodes", "NodeDescription_ApplyCVarPreset", "Apply Console Variable Preset\n{0} ({1} CVars)");

	const FString InstanceName = GetNodeInstanceName();
	if (bGetDescriptive && !InstanceName.IsEmpty())
	{
		constexpr bool bOnlyIncludeChecked = true;
		TArray<TTuple<FString, FString>> ConsoleVariables;
		ConsoleVariablePreset->GetConsoleVariablesForTrack(bOnlyIncludeChecked, ConsoleVariables);
		
		return FText::Format(ApplyCVarPresetNodeDescription, FText::FromString(InstanceName), FText::FromString(LexToString(ConsoleVariables.Num())));
	}
	
	return ApplyCVarPresetNodeName;
}

FText UMovieGraphApplyCVarPresetNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "ApplyCVarPresetGraphNode_Category", "Utility");
}

FText UMovieGraphApplyCVarPresetNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "ApplyCVarPresetGraphNode_Keywords", "cvar console variable preset");
	return Keywords;
}

FLinearColor UMovieGraphApplyCVarPresetNode::GetNodeTitleColor() const
{
	static const FLinearColor ApplyCVarPresetNodeColor = FLinearColor(0.04f, 0.22f, 0.36f);
	return ApplyCVarPresetNodeColor;
}

FSlateIcon UMovieGraphApplyCVarPresetNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon ApplyCVarPresetIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.BrowseCVars");

	OutColor = FLinearColor::White;
	return ApplyCVarPresetIcon;
}

void UMovieGraphApplyCVarPresetNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphApplyCVarPresetNode, ConsoleVariablePreset))
	{
		// Update the available cvars to override based on the preset that is chosen
		UpdateDynamicProperties();
		
		OnNodeChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR
