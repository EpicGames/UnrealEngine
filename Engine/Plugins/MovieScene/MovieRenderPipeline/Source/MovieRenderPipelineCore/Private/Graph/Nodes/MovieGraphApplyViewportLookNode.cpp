// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphApplyViewportLookNode.h"

#include "MeshEdges.h"
#include "SceneView.h"
#include "Styling/AppStyle.h"
#include "UnrealClient.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphApplyViewportLookNode)

EMovieGraphBranchRestriction UMovieGraphApplyViewportLookNode::GetBranchRestriction() const
{
	return EMovieGraphBranchRestriction::Globals;
}

#if WITH_EDITOR
FLevelEditorViewportClient* UMovieGraphApplyViewportLookNode::GetViewportClient()
{
	// Get the currently active viewport that's in use (both perspective and orthographic viewports are supported)
	if (const FViewport* ActiveViewport = GEditor->GetActiveViewport())
	{
		for (FLevelEditorViewportClient* LevelViewportClient : GEditor->GetLevelViewportClients())
		{
			if (LevelViewportClient->Viewport == ActiveViewport)
			{
				return LevelViewportClient;
			}
		}
	}

	return nullptr;
}

bool UMovieGraphApplyViewportLookNode::GetViewportInfo(FEngineShowFlags& OutShowFlags, EViewModeIndex& OutViewModeIndex) const
{
	if (const FEditorViewportClient* ViewportClient = GetViewportClient())
	{
		OutShowFlags = ViewportClient->EngineShowFlags;
		OutViewModeIndex = ViewportClient->GetViewMode();

		return true;
	}

	return false;
}

void UMovieGraphApplyViewportLookNode::UpdateSceneView(FSceneView* InSceneView) const
{
	const FEditorViewportClient* ViewportClient = GetViewportClient();
	
	if (InSceneView && ViewportClient)
	{
		// Note: Most of the scene view is set up in the deferred pass; these are the settings that are not handled there.
		
		InSceneView->CurrentBufferVisualizationMode = ViewportClient->CurrentBufferVisualizationMode;
		InSceneView->CurrentNaniteVisualizationMode = ViewportClient->CurrentNaniteVisualizationMode;
		InSceneView->CurrentLumenVisualizationMode = ViewportClient->CurrentLumenVisualizationMode;
		InSceneView->CurrentMegaLightsVisualizationMode = ViewportClient->CurrentMegaLightsVisualizationMode;
		InSceneView->CurrentSubstrateVisualizationMode = ViewportClient->CurrentSubstrateVisualizationMode;
		InSceneView->CurrentGroomVisualizationMode = ViewportClient->CurrentGroomVisualizationMode;
		InSceneView->CurrentVirtualShadowMapVisualizationMode = ViewportClient->CurrentVirtualShadowMapVisualizationMode;
		InSceneView->CurrentGPUSkinCacheVisualizationMode = ViewportClient->CurrentGPUSkinCacheVisualizationMode;
		InSceneView->CurrentRayTracingDebugVisualizationMode = ViewportClient->CurrentRayTracingDebugVisualizationMode;

		// Wireframe opacity
		GetMeshEdgesViewSettings(*InSceneView).Opacity = ViewportClient->WireframeOpacity;
	}
}

FText UMovieGraphApplyViewportLookNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText NodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_ApplyViewportLook", "Apply Viewport Look");
	return NodeName;
}

FText UMovieGraphApplyViewportLookNode::GetMenuCategory() const
{
	static const FText NodeCategory_Globals = NSLOCTEXT("MovieGraphNodes", "NodeCategory_Globals", "Globals");
	return NodeCategory_Globals;
}

FLinearColor UMovieGraphApplyViewportLookNode::GetNodeTitleColor() const
{
	static const FLinearColor NodeColor = FLinearColor(0.04f, 0.22f, 0.36f);
	return NodeColor;
}

FSlateIcon UMovieGraphApplyViewportLookNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports");

	OutColor = FLinearColor::White;
	return Icon;
}
#endif // WITH_EDITOR
