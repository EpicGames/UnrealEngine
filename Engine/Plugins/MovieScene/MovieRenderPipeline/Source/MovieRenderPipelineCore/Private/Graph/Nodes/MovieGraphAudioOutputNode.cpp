// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphAudioOutputNode.h"

#include "AudioMixerDevice.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphUtils.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphAudioOutputNode)

static TAutoConsoleVariable<float> CVarWaveOutputDelay(
	TEXT("MovieRenderPipeline.WaveOutput.WriteDelay"),
	0.5f,
	TEXT("How long (in seconds) should the .wav writer stall the main thread to wait for the async write to finish\n")
	TEXT("before moving on? If your .wav files take a long time to write and they're not finished by the time the\n")
	TEXT("encoder runs, the encoder may fail.\n"),
	ECVF_Default);

UMovieGraphAudioOutputNode::UMovieGraphAudioOutputNode()
{
	FileNameFormat = TEXT("{sequence_name}");
}

void UMovieGraphAudioOutputNode::BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const
{
	// Always add this so that audio is muted, it'll never line up during preview anyways.
	InOutCommandLineArgs.Add("-deterministicaudio");
}

void UMovieGraphAudioOutputNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesAudio = true;
}

#if WITH_EDITOR
FText UMovieGraphAudioOutputNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText AudioOutputNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_AudioOutput", ".wav Output");
	return AudioOutputNodeName;
}

FText UMovieGraphAudioOutputNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "AudioOutputNode_Keywords", "wav audio");
	return Keywords;
}

FLinearColor UMovieGraphAudioOutputNode::GetNodeTitleColor() const
{
	static const FLinearColor AudioOutputNodeColor = FLinearColor(0.04f, 0.22f, 0.36f);
	return AudioOutputNodeColor;
}

FSlateIcon UMovieGraphAudioOutputNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon AudioOutputPresetIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericPlay");

	OutColor = FLinearColor::White;
	return AudioOutputPresetIcon;
}

FText UMovieGraphAudioOutputNode::GetBasicConfigShortDisplayName() const
{
	return NSLOCTEXT("MovieGraphNodes", "BasicConfigShortName_WAV", "WAV");
}
#endif // WITH_EDITOR

EMovieGraphBranchRestriction UMovieGraphAudioOutputNode::GetBranchRestriction() const
{
	return EMovieGraphBranchRestriction::Globals;
}

void UMovieGraphAudioOutputNode::OnAllFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph)
{
	CachedPipeline = InPipeline;
	EvaluatedGraph = InPrimaryJobEvaluatedGraph;

	// It's possible that the pipeline is being shut down early, in which case the shot index will be invalid. Do not begin generating audio
	// if a shutdown is being requested.
	const int32 ShotIndex = CachedPipeline->GetCurrentShotIndex();
	if (!CachedPipeline->GetActiveShotList().IsValidIndex(ShotIndex))
	{
		return;
	}

	// No need to do work if per-shot audio has already been exported
	if (NeedsPerShotFlushing())
	{
		return;
	}

	StartAudioExport();
}

void UMovieGraphAudioOutputNode::OnAllShotFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, const UMoviePipelineExecutorShot* InShot, TObjectPtr<UMovieGraphEvaluatedConfig>& InShotEvaluatedGraph, const bool bFlushToDisk)
{
	CachedPipeline = InPipeline;
	EvaluatedGraph = InPipeline->GetTimeStepInstance()->GetCalculatedTimeData().EvaluatedConfig;

	// Don't do per-shot exports if a sequence-wide export was requested
	if (!NeedsPerShotFlushing())
	{
		return;
	}

	StartAudioExport();
}

bool UMovieGraphAudioOutputNode::IsFinishedWritingToDiskImpl() const
{
	for (const TUniquePtr<Audio::FSoundWavePCMWriter>& ActiveWriter : ActiveWriters)
	{
		if (ActiveWriter && !ActiveWriter->IsDone())
		{
			return false;
		}
	}

	return true;
}

FString UMovieGraphAudioOutputNode::GenerateOutputPath(const FMovieGraphRenderDataIdentifier& InRenderIdentifier, const TObjectPtr<UMoviePipelineExecutorShot>& InShot) const
{
	// Note that we fetch the audio node from the evaluated graph because this method is executing on the CDO (and we want the evaluated FileNameFormat)
	constexpr bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	const UMovieGraphGlobalOutputSettingNode* OutputNode = EvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);
	const UMovieGraphAudioOutputNode* AudioNode = EvaluatedGraph->GetSettingForBranch<UMovieGraphAudioOutputNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);
	FString FileNameFormatString = OutputNode->OutputDirectory.Path / AudioNode->FileNameFormat;

	constexpr bool bIncludeRenderPass = false;
	constexpr bool bTestFrameNumber = false;
	UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber);
	
	// Strip any frame number tags so we don't get one audio file per frame.
	UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);

	FMovieGraphFilenameResolveParams ResolveParams;
	ResolveParams.InitializationTime = CachedPipeline->GetInitializationTime();
	ResolveParams.InitializationTimeOffset = CachedPipeline->GetInitializationTimeOffset();
	ResolveParams.Job = CachedPipeline->GetCurrentJob();
	ResolveParams.Shot = InShot;
	ResolveParams.FileNameFormatOverrides = {
		{TEXT("render_pass"), RendererName},
		{TEXT("ext"), OutputExtension}
	};
	ResolveParams.EvaluatedConfig = EvaluatedGraph;
	ResolveParams.RenderDataIdentifier = InRenderIdentifier;
	ResolveParams.Version = InShot->ShotInfo.VersionNumber;

	// Generate a filename for this output file
	FMovieGraphResolveArgs FinalFormatArgs;
	FString FinalFilePath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FileNameFormatString, ResolveParams, FinalFormatArgs);

	if (FPaths::IsRelative(FinalFilePath))
	{
		FinalFilePath = FPaths::ConvertRelativePathToFull(FinalFilePath);
	}

	return FinalFilePath;
}

void UMovieGraphAudioOutputNode::GenerateFinalAudioData(TArray<FFinalAudioData>& OutFinalAudioData)
{
	// Generate the "render" identifier for the audio
	const int32 ShotIndex = CachedPipeline->GetCurrentShotIndex();
	const TObjectPtr<UMoviePipelineExecutorShot>& Shot = CachedPipeline->GetActiveShotList()[ShotIndex];
	const FString SubResourceName = FString();
	const FString LayerName = GlobalsPinNameString;
	const FString CameraName = Shot->InnerName;
	const FMovieGraphRenderDataIdentifier RenderDataIdentifier(GlobalsPinName, GlobalsPinNameString, RendererName, SubResourceName, CameraName);
	
	const FString FinalFilePath = GenerateOutputPath(RenderDataIdentifier, Shot);
	
	// Move each finished audio segment to its associated final audio data struct (the one with a matching file name)
	const TArray<MoviePipeline::FAudioState::FAudioSegment>& FinishedSegments = CachedPipeline->GetAudioRendererInstance()->GetAudioState().FinishedSegments;
	for (int32 SegmentIndex = 0; SegmentIndex < FinishedSegments.Num(); SegmentIndex++)
	{
		const MoviePipeline::FAudioState::FAudioSegment& Segment = FinishedSegments[SegmentIndex];
		if (AlreadyWrittenSegments.Contains(Segment.Id))
		{
			continue;
		}

		// This function may get called many times over the course of a rendering (if flushing to disk). But because we don't own the audio
		// data we can't remove it after we've written it to disk. So we keep track of which segments we've already written to disk for and
		// skip them. This avoids us generating extra output futures for previous shots when we render a new shot.
		AlreadyWrittenSegments.Add(Segment.Id);

		// Look to see if we already have a output file to append this to.
		FFinalAudioData* OutputSegment = nullptr;
		for (int32 Index = 0; Index < OutFinalAudioData.Num(); Index++)
		{
			if (OutFinalAudioData[Index].FilePath == FinalFilePath)
			{
				OutputSegment = &OutFinalAudioData[Index];
				break;
			}
		}

		if (!OutputSegment)
		{
			OutFinalAudioData.AddDefaulted();
			OutputSegment = &OutFinalAudioData[OutFinalAudioData.Num() - 1];
			OutputSegment->FilePath = FinalFilePath;
			OutputSegment->ShotIndex = CachedPipeline->GetCurrentShotIndex();
			OutputSegment->RenderIdentifier = RenderDataIdentifier;
		}

		// Convert our samples and append them to the existing array
		const double StartTime = FPlatformTime::Seconds();
		Audio::TSampleBuffer<int16> SampleBuffer = Audio::TSampleBuffer<int16>(Segment.SegmentData.GetData(), Segment.SegmentData.Num(), Segment.NumChannels, Segment.SampleRate);
		OutputSegment->SampleBuffer.Append(SampleBuffer.GetData(), Segment.SegmentData.Num(), Segment.NumChannels, Segment.SampleRate);
		UE_LOGF(LogMovieRenderPipeline, Log, "Audio Segment took %f seconds to convert to a sample buffer.", (FPlatformTime::Seconds() - StartTime));
	}
}

void UMovieGraphAudioOutputNode::StartAudioExport()
{
	if (!UE::MovieGraph::Audio::IsMoviePipelineAudioOutputSupported(CachedPipeline.Get()))
	{
		return;
	}

	ActiveWriters.Reset();

	const MoviePipeline::FAudioState& AudioState = CachedPipeline->GetAudioRendererInstance()->GetAudioState();
		
	// There should be no active submixes by the time we finalize - they should all have been converted to recorded samples.
	check(AudioState.ActiveSubmixes.IsEmpty());

	// If we didn't end up recording audio, don't try outputting any files.
	if (AudioState.FinishedSegments.IsEmpty())
	{
		return;
	}

	TArray<FFinalAudioData> FinalAudioData;
	GenerateFinalAudioData(FinalAudioData);

	// Write audio files to disk
	for (FFinalAudioData& AudioData : FinalAudioData)
	{
		UE::MovieGraph::FMovieGraphOutputFutureData OutputData;
		OutputData.Shot = CachedPipeline->GetActiveShotList()[AudioData.ShotIndex];
		OutputData.DataIdentifier = AudioData.RenderIdentifier;
		OutputData.OriginNodeClass = GetClass();

		// Do this before we start manipulating the filepath for the audio API
		OutputData.FilePath = AudioData.FilePath;

		const FString FileName = FPaths::GetBaseFilename(AudioData.FilePath);
		FString FileFolder = FPaths::GetPath(AudioData.FilePath);

		TPromise<bool> Completed;
		CachedPipeline->AddOutputFuture(Completed.GetFuture(), OutputData);

		TUniquePtr<Audio::FSoundWavePCMWriter> Writer = MakeUnique<Audio::FSoundWavePCMWriter>();
		const bool bSuccess = Writer->BeginWriteToWavFile(AudioData.SampleBuffer, FileName, FileFolder);

		Completed.SetValue(bSuccess);
		ActiveWriters.Add(MoveTemp(Writer));
	}

	// The FSoundWavePCMWriter is unfortunately async, and the completion callbacks don't work unless the main thread
	// can be spun (as it enqueues a callback onto the main thread). We're going to just cheat here and stall the main thread
	// for 0.5s to give it a chance to write to disk. It'll only potentially be an issue with command line encoding if it takes
	// longer than 0.5s to write to disk.
	if (const TConsoleVariableData<float>* CVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("MovieRenderPipeline.WaveOutput.WriteDelay")))
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Delaying main thread for %f seconds while audio writes to disk.", CVar->GetValueOnGameThread());
		FPlatformProcess::Sleep(CVar->GetValueOnGameThread());
	}
}

bool UMovieGraphAudioOutputNode::NeedsPerShotFlushing() const
{
	constexpr bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSettingNode =
		EvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);
	const UMovieGraphAudioOutputNode* AudioNode =
		EvaluatedGraph->GetSettingForBranch<UMovieGraphAudioOutputNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);
	
	const FString FullPath = OutputSettingNode->OutputDirectory.Path / AudioNode->FileNameFormat;
	
	if (FullPath.Contains(TEXT("{shot_name}")) || FullPath.Contains(TEXT("{camera_name}")))
	{
		return true;
	}

	return false;
}
