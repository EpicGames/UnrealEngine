// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphBranchNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphEdge.h"
#include "Graph/MovieGraphUtils.h"
#include "MovieRenderPipelineCoreModule.h"

#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphBranchNode)

#define LOCTEXT_NAMESPACE "MovieGraphNode"

namespace UE::MovieGraph::BranchNode
{
	static const FName TrueBranch("True");
	static const FName FalseBranch("False");
	static const FName Condition("Condition");
}

TArray<UMovieGraphPin*> UMovieGraphBranchNode::EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const
{
	TArray<UMovieGraphPin*> PinsToFollow;

	// If the node is disabled, follow the first connected pin.
	if (IsDisabled())
	{
		if (UMovieGraphPin* GraphPin = GetFirstConnectedInputPin())
		{
			PinsToFollow.Add(GraphPin);
		}
		
		return PinsToFollow;
	}

	// The branch node has two branches that could be followed, True or False. To figure out which one we're actually going
	// to follow, we need to evaluate the Conditional pin. 
	UMovieGraphPin* ConditionalPin = GetInputPin(UE::MovieGraph::BranchNode::Condition);
	if (!ensure(ConditionalPin))
	{
		return PinsToFollow;
	}

	UMovieGraphPin* OtherPin = nullptr;
	
	// We only support a single connection to this pin type anyways.
	if (ConditionalPin->Edges.Num() > 0)
	{
		UMovieGraphEdge* Edge = ConditionalPin->Edges[0];
		constexpr bool bFollowRerouteConnections = true;
		OtherPin = Edge->GetOtherPin(ConditionalPin, bFollowRerouteConnections);
	}

	// There may not be a node actually connected. We don't know what to do in this case (as our nodes don't have default values)
	// so for now we choose to follow neither branch.
	if (!OtherPin)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Unconnected conditional pin has no default value. Node: %ls", *GetName());
		return PinsToFollow;
	}

	UMovieGraphNode* ConnectedNode = OtherPin->Node;
	if (!ensure(ConnectedNode))
	{
		return PinsToFollow;
	}

	FString ResolvedValue;
	UE::MovieGraph::ResolveConnectedPinValue(ConditionalPin, {OtherPin}, InContext, ResolvedValue);

	// TODO: Comparing against a string like this is not ideal. This could use GetPropertyValueContainerForPin() instead,
	// however that may not work in a general case. This fits variables nicely since they already have a
	// UMovieGraphValueContainer -- other nodes in the future may not, though, and generating them on every frame may be expensive.
	// Still an area that needs to be researched.
	static const FString TrueValue = FString("true");
	if (ResolvedValue == TrueValue)
	{
		PinsToFollow.Add(GetInputPin(UE::MovieGraph::BranchNode::TrueBranch));
	}
	else
	{
		PinsToFollow.Add(GetInputPin(UE::MovieGraph::BranchNode::FalseBranch));
	}

	return PinsToFollow;
}

TArray<FMovieGraphPinProperties> UMovieGraphBranchNode::GetInputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;
	
	const TObjectPtr<UObject> ValueTypeObject = nullptr;
	Properties.Add(FMovieGraphPinProperties::MakeBranchProperties(UE::MovieGraph::BranchNode::TrueBranch));
	Properties.Add(FMovieGraphPinProperties::MakeBranchProperties(UE::MovieGraph::BranchNode::FalseBranch));
	Properties.Add(FMovieGraphPinProperties(UE::MovieGraph::BranchNode::Condition, EMovieGraphValueType::Bool, ValueTypeObject, false));
	return Properties;
}

TArray<FMovieGraphPinProperties> UMovieGraphBranchNode::GetOutputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;
	Properties.Add(FMovieGraphPinProperties::MakeBranchProperties());
	return Properties;
}

#if WITH_EDITOR
FText UMovieGraphBranchNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText BranchNodeName = LOCTEXT("NodeName_Branch", "Branch");
	return BranchNodeName;
}

FText UMovieGraphBranchNode::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory_Conditionals", "Conditionals");
}

FText UMovieGraphBranchNode::GetKeywords() const
{
	static const FText Keywords = LOCTEXT("BranchNode_Keywords", "branch if logic conditional");
    return Keywords;
}

FLinearColor UMovieGraphBranchNode::GetNodeTitleColor() const
{
	static const FLinearColor BranchNodeColor = FLinearColor(0.266f, 0.266f, 0.266f);
	return BranchNodeColor;
}

FSlateIcon UMovieGraphBranchNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon BranchIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Merge");

	OutColor = FLinearColor::White;
	return BranchIcon;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
