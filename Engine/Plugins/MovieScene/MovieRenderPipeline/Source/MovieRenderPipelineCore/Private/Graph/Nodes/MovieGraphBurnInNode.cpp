// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphBurnInNode.h"

#include "Framework/Application/SlateApplication.h"
#include "Graph/MovieGraphBurnInWidget.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphPipeline.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Styling/AppStyle.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphBurnInNode)

#define LOCTEXT_NAMESPACE "MovieGraphNodes"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FString UMovieGraphBurnInNode::RendererName = FString("BurnIn");
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const FString UMovieGraphBurnInNode::DefaultBurnInWidgetAsset = TEXT("/MovieRenderPipeline/Blueprints/Graph/DefaultGraphBurnIn.DefaultGraphBurnIn_C");

UMovieGraphBurnInNode::UMovieGraphBurnInNode()
	: BurnInClass(DefaultBurnInWidgetAsset)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Super::RendererName = RendererName;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	RendererSubName = TEXT("burnin");
}

#if WITH_EDITOR
FText UMovieGraphBurnInNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return LOCTEXT("BurnInGraphNode_Description", "Burn In [DEPRECATED]\nUse burn-in settings on output nodes.");
}

FLinearColor UMovieGraphBurnInNode::GetNodeTitleColor() const
{
	return FLinearColor(0.572f, 0.274f, 1.f);
}

FSlateIcon UMovieGraphBurnInNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon BurnInIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "SequenceRecorder.TabIcon");

	OutColor = FLinearColor::White;
	return BurnInIcon;
}
#endif	// WITH_EDITOR

TUniquePtr<UMovieGraphWidgetRendererBaseNode::FMovieGraphWidgetPass> UMovieGraphBurnInNode::GeneratePass()
{
	return MakeUnique<FMovieGraphBurnInPass>();
}

void UMovieGraphBurnInNode::GatherOutputPassesImpl(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const
{
	for (const TUniquePtr<FMovieGraphWidgetPass>& Instance : CurrentInstances)
	{
		// Only generate passes if there's a valid burn-in class
		if (StaticCast<FMovieGraphBurnInPass*>(Instance.Get())->GetBurnInClass())
		{
			Instance->GatherOutputPasses(OutExpectedPasses);
		}
	}
}

void UMovieGraphBurnInNode::TeardownImpl()
{
	Super::TeardownImpl();
	
	BurnInWidgetInstances.Empty();
}

FString UMovieGraphOutputBurnInNode::GetNodeInstanceName() const
{
	return OutputName;
}

#if WITH_EDITOR
FText UMovieGraphOutputBurnInNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return LOCTEXT("BurnInOutputGraphNode_Description", "Burn In");
}
#endif	// WITH_EDITOR

TArray<FSoftClassPath> UMovieGraphOutputBurnInNode::GetOutputTypeRestrictionsImpl() const
{
	return { OutputRestriction };
}

FString UMovieGraphOutputBurnInNode::GetRendererNameImpl() const
{
	return FString::Format(TEXT("BurnIn_{0}"), { OutputName });
}

FString UMovieGraphOutputBurnInNode::GetFileNameFormatOverride() const
{
	return FileNameFormat;
}

FString UMovieGraphOutputBurnInNode::GetLayerNameOverride() const
{
	return LayerNameFormat;
}

bool UMovieGraphOutputBurnInNode::GetNodeValidationErrors(const FName& InBranchName, const UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<FText>& OutValidationErrors) const
{
	if (Super::GetNodeValidationErrors(InBranchName, InEvaluatedConfig, OutValidationErrors))
	{
		return false;
	}

	// The old burn-in node (deprecated) cannot be used with the output-type-specific burn-in node (this node). If the old node appears in globals,
	// or appears in the same branch as this node, that is an error and should halt the render.
	for (const FName& BranchName : InEvaluatedConfig->GetBranchNames())
	{
		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = true;
		if (InEvaluatedConfig->GetSettingForBranch(UMovieGraphBurnInNode::StaticClass(), BranchName, bIncludeCDOs, bExactMatch))
		{
			if ((BranchName == GlobalsPinName) || (BranchName == InBranchName))
			{
				const FText OutputClassName = OutputRestriction.ResolveClass()
					? FText::FromString(OutputRestriction.ResolveClass()->GetName())
					: LOCTEXT("BurnInOutputNode_UnknownClass", "Unknown");

				OutValidationErrors.Add(FText::Format(LOCTEXT("BurnInOutputNode_Incompatibility", "An output-specific ({0}) burn-in was added to a graph that also uses the deprecated Burn In node. These two nodes are incompatible; only the Burn In node or the output-specific burn-ins can be used."), OutputClassName));
				return true;
			}
		}
	}

	return false;
}

TObjectPtr<UMovieGraphBurnInWidget> UMovieGraphBurnInNode::GetOrCreateBurnInWidget(UClass* InWidgetClass, UWorld* InOwner)
{
	if (const TObjectPtr<UMovieGraphBurnInWidget>* ExistingBurnInWidget = BurnInWidgetInstances.Find(InWidgetClass))
	{
		return *ExistingBurnInWidget;
	}

	TObjectPtr<UMovieGraphBurnInWidget> BurnInWidget = CreateWidget<UMovieGraphBurnInWidget>(InOwner, InWidgetClass);
	if (BurnInWidget)
	{
		BurnInWidgetInstances.Emplace(InWidgetClass, BurnInWidget);
	}

	return BurnInWidget;
}

void UMovieGraphBurnInNode::FMovieGraphBurnInPass::Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, const FMovieGraphRenderPassLayerData& InLayer)
{
	FMovieGraphWidgetPass::Setup(InRenderer, InLayer);
	
	RenderDataIdentifier.SubResourceName = InLayer.RenderPassNode->GetRendererSubName();

	UMovieGraphBurnInNode* ParentNodeOnInitialization = Cast<UMovieGraphBurnInNode>(InLayer.RenderPassNode);
	if (!ensureMsgf(ParentNodeOnInitialization, TEXT("FMovieGraphBurnInPass shouldn't exist without a parent node in the graph.")))
	{
		return;
	}
	CachedBurnInWidgetClass = ParentNodeOnInitialization->BurnInClass.TryLoadClass<UMovieGraphBurnInWidget>();

	// Output-specific burn-ins keep track of what their parent node's instance name is. There can be multiple output-specific burn-ins
	// on any given branch, so this is needed in order to disambiguate.
	if (ParentNodeOnInitialization->IsA<UMovieGraphOutputBurnInNode>())
	{
		RendererNodeInstanceName = ParentNodeOnInitialization->GetNodeInstanceName();
	}
}

TSharedPtr<SWidget> UMovieGraphBurnInNode::FMovieGraphBurnInPass::GetWidget(UMovieGraphWidgetRendererBaseNode* InNodeThisFrame)
{
	if (const TObjectPtr<UMovieGraphBurnInWidget> BurnInWidget = GetBurnInWidget(InNodeThisFrame))
	{
		return BurnInWidget->TakeWidget();
	}
	
	return nullptr;
}

TObjectPtr<UMovieGraphBurnInWidget> UMovieGraphBurnInNode::FMovieGraphBurnInPass::GetBurnInWidget(UMovieGraphWidgetRendererBaseNode* InNodeThisFrame) const
{
	const UMovieGraphPipeline* Pipeline = Renderer->GetOwningGraph();

	UClass* LoadedBurnInClass = GetBurnInClass();
	if (!LoadedBurnInClass)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "The burn-in widget provided in layer '%ls' for renderer '%ls' is not valid.", *LayerData.BranchName.ToString(), *Renderer->GetClass()->GetName());
		return nullptr;
	}
	
	// The CDO contains the resources which are shared with all FMovieGraphBurnInPass instances
	UMovieGraphBurnInNode* BurnInCDO = InNodeThisFrame->GetClass()->GetDefaultObject<UMovieGraphBurnInNode>();
	const TObjectPtr<UMovieGraphBurnInWidget> BurnInWidget = BurnInCDO->GetOrCreateBurnInWidget(LoadedBurnInClass, Pipeline->GetWorld());
	if (!BurnInWidget)
	{
		UClass* BurnInWidgetClass = GetBurnInClass();
		if (BurnInWidgetClass)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Unable to load burn-in widget at path: %ls", *BurnInWidgetClass->GetPathName());
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Unable to load burn-in widget (nullptr)");
		}
		return nullptr;
	}

	return BurnInWidget;
}

void UMovieGraphBurnInNode::FMovieGraphBurnInPass::Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData)
{
	UMovieGraphWidgetRendererBaseNode* ParentNodeThisFrame = GetParentNode(InTimeData.EvaluatedConfig);

	// Update the widget with the latest frame information
	if (const TObjectPtr<UMovieGraphBurnInWidget> BurnInWidget = GetBurnInWidget(ParentNodeThisFrame))
	{
		UMovieGraphPipeline* Pipeline = Renderer->GetOwningGraph();
		BurnInWidget->UpdateForGraph(Pipeline, Pipeline->GetTimeStepInstance()->GetCalculatedTimeData().EvaluatedConfig, LayerData.CameraIndex, LayerData.CameraName);
	}
	 
	FMovieGraphWidgetPass::Render(InFrameTraversalContext, InTimeData);
}

int32 UMovieGraphBurnInNode::FMovieGraphBurnInPass::GetCompositingSortOrder() const
{
	// Burn-ins should always appear over all other passes
	return 100;
}

UMovieGraphWidgetRendererBaseNode* UMovieGraphBurnInNode::FMovieGraphBurnInPass::GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const
{
	// If there's no instance name, fall back to the default behavior.
	if (RendererNodeInstanceName.IsEmpty())
	{
		return FMovieGraphWidgetPass::GetParentNode(InConfig);
	}
	
	constexpr bool bIncludeCDOs = false;
	TArray<UMovieGraphOutputBurnInNode*> BurnInNodes = InConfig->GetSettingsForBranch<UMovieGraphOutputBurnInNode>(RenderDataIdentifier.RootBranchName, bIncludeCDOs);

	// Find the output-specific burn-in node assigned to this pass.
	for (UMovieGraphOutputBurnInNode* BurnInNode : BurnInNodes)
	{
		if (BurnInNode->GetNodeInstanceName() == RendererNodeInstanceName)
		{
			return BurnInNode;
		}
	}

	ensureMsgf(false, TEXT("FMovieGraphBurnInPass should not exist without parent node in graph."));
	return nullptr;
}

UClass* UMovieGraphBurnInNode::FMovieGraphBurnInPass::GetBurnInClass() const
{
	return CachedBurnInWidgetClass.Get();
}

#undef LOCTEXT_NAMESPACE // "MovieGraphNodes"