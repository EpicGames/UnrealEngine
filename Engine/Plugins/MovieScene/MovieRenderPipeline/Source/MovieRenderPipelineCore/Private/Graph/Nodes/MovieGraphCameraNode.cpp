// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphCameraNode.h"

#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphCameraNode)

void UMovieGraphCameraSettingNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	OutMergedFormatArgs.FilenameArguments.Add(TEXT("shutter_timing"), StaticEnum<EMoviePipelineShutterTiming>()->GetNameStringByValue(static_cast<int64>(ShutterTiming)));
	OutMergedFormatArgs.FilenameArguments.Add(TEXT("overscan_percentage"), FString::SanitizeFloat(OverscanPercentage));
	
	OutMergedFormatArgs.FileMetadata.Add(TEXT("unreal/camera/shutterTiming"), StaticEnum<EMoviePipelineShutterTiming>()->GetNameStringByValue(static_cast<int64>(ShutterTiming)));
	OutMergedFormatArgs.FileMetadata.Add(TEXT("unreal/camera/overscanPercentage"), FString::SanitizeFloat(OverscanPercentage));
}

#if WITH_EDITOR
FText UMovieGraphCameraSettingNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText NodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_CameraSettings", "Camera Settings");
	return NodeName;
}

FText UMovieGraphCameraSettingNode::GetMenuCategory() const
{
	static const FText NodeCategory_Globals = NSLOCTEXT("MovieGraphNodes", "NodeCategory_Globals", "Globals");
	return NodeCategory_Globals;
}

FLinearColor UMovieGraphCameraSettingNode::GetNodeTitleColor() const
{
	static const FLinearColor NodeColor = FLinearColor(0.65f, 0.60f, 0.75f);
	return NodeColor;
}

FSlateIcon UMovieGraphCameraSettingNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CameraComponent");

	OutColor = FLinearColor::White;
	return Icon;
}
#endif // WITH_EDITOR
