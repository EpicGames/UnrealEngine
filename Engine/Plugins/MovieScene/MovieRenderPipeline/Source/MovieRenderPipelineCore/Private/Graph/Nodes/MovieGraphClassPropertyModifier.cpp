// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphClassPropertyModifier.h"

#include "Engine/World.h"
#include "Graph/MovieGraphValueUtils.h"
#include "Graph/Nodes/MovieGraphLightModifierNode.h"
#include "Kismet/GameplayStatics.h"
#include "MovieRenderPipelineCoreModule.h"

FMovieGraphPropertyReference::FMovieGraphPropertyReference(
	const TObjectPtr<UClass>& InPropertyClass,
	const TSubclassOf<UActorComponent>& InComponentClass,
	const FName& InPropertyName,
	FProperty* InProperty)
	: PropertyClass(InPropertyClass)
	, ComponentClass(InComponentClass)
	, PropertyName(InPropertyName)
	, Property(InProperty)
{
	PopulateDerivedData();
}

EMovieGraphValueType FMovieGraphPropertyReference::GetValueType() const
{
	return ValueType;
}

UObject* FMovieGraphPropertyReference::GetValueTypeObject() const
{
	return ValueTypeObject.Get();
}

FProperty* FMovieGraphPropertyReference::GetProperty() const
{
	return Property;
}

FName FMovieGraphPropertyReference::GetPropertyName() const
{
	return PropertyName;
}

TSubclassOf<AActor> FMovieGraphPropertyReference::GetPropertyClass() const
{
	return PropertyClass;
}

TSubclassOf<UActorComponent> FMovieGraphPropertyReference::GetComponentClass() const
{
	return ComponentClass;
}

bool FMovieGraphPropertyReference::IsValidProperty() const
{
	return !!Property && (PropertyClass.Get() || ComponentClass.Get()) && !PropertyName.IsNone();
}

void FMovieGraphPropertyReference::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		// FProperty cannot be serialized, so we re-derive it upon loading (and any other calculated members)
		PopulateDerivedData();
	}
}

void FMovieGraphPropertyReference::PopulateDerivedData()
{
	// If the property is part of a component, use the component as the container property; otherwise, use the class.
	if (!Property)
	{
		if (ComponentClass.Get())
		{
			Property = FindFProperty<FProperty>(ComponentClass.Get(), PropertyName);
		}
		else if (PropertyClass.Get())
		{
			Property = FindFProperty<FProperty>(PropertyClass.Get(), PropertyName);
		}
	}

	if (!Property)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Could not find property with name [%ls]. Functionality relating to this property may be broken.", *PropertyName.ToString());
	}

	if (Property)
	{
		UE::MovieGraph::ValueUtils::GetTypesForFProperty(Property, ValueType, ValueTypeObject);
	}
}

UMovieGraphPropertyModifier::UMovieGraphPropertyModifier()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UpdatedPropertyValuesView = NewObject<UMovieGraphMutableValueView>(this, TEXT("UpdatedPropertyValuesView"));
		UpdatedPropertyValuesView->ViewProperties(&UpdatedPropertyValues);

		PreviousPropertyValuesView = NewObject<UMovieGraphMutableValueView>(this, TEXT("PreviousPropertyValuesView"));
		PreviousPropertyValuesView->ViewProperties(&PreviousPropertyValues);
	}
}

void UMovieGraphPropertyModifier::ApplyModifier(const UWorld* World)
{
	// Set the value of the given property (with the value that was set for it on the modifier), and also cache its current value so it can be
	// restored when the modifier is undone. The property is modified within the specified container (an actor or component).
	auto CacheAndSetPropertyValue = [this](const FMovieGraphPropertyReference& InPropertyReference, UObject* InPropertyContainer)
	{
		// TODO: Ideally the below export/import aren't string based, but that would require a lot of specialized code in order to
		// properly handle every type. This will do for now, but could potentially be optimized in the future if this is a bottleneck
		// or causes issues.

		const FProperty* TargetProperty = InPropertyReference.GetProperty();

		// Cache the current value so it can be restored later
		FString CurrentValue;
		TargetProperty->ExportTextItem_InContainer(CurrentValue, InPropertyContainer, nullptr, nullptr, PPF_None);
		PreviousPropertyValuesView->SetValueSerializedString(InPropertyReference.AuxPropertyName, CurrentValue);
		ModifiedProperties.Add({ InPropertyContainer, InPropertyReference });

		// Set the property to its new value
		const FString PropertyValue = UpdatedPropertyValuesView->GetValueSerializedString(InPropertyReference.AuxPropertyName);
		TargetProperty->ImportText_InContainer(*PropertyValue, InPropertyContainer, InPropertyContainer, 0);
	};

	TArray<TSubclassOf<AActor>> TargetActorClasses;
	PropertiesToModifyByActorClass.GetKeys(TargetActorClasses);

	// Keep track of the components that were updated; they need their render state marked as dirty so the render thread gets the update(s).
	TArray<UActorComponent*> DirtiedComponents;

	for (const TObjectPtr<UMovieGraphCollection>& Collection : Collections)
	{
		if (!Collection)
		{
			continue;
		}

		FMovieGraphEvaluationResult EvaluationResult = Collection->EvaluateActorsAndComponents(World);

		// Set any actor properties on actors the collection contains.
		for (const TObjectPtr<AActor>& MatchingActor : EvaluationResult.MatchingActors)
		{
			UClass* ActorClass = MatchingActor->GetClass();
			bool bOnlyMatchComponent = false;

			// The modifier only applies to specific actor types. If this modifier isn't targeting any of the actor types in the collection, the
			// modifier *may* indicate that some properties should be applied to all components of a specific type within an actor (which is indicated
			// by a null target actor class).
			if (!TargetActorClasses.Contains(ActorClass))
			{
				bOnlyMatchComponent = true;
				ActorClass = nullptr;

				// However, there may be component-specific matches that don't target distinct actor classes
				if (!TargetActorClasses.Contains(nullptr))
				{
					continue;
				}
			}

			for (const FMovieGraphPropertyReference& PropertyReference : PropertiesToModifyByActorClass[ActorClass])
			{
				// If the property is part of a component, use the component as the property container; otherwise, use the actor itself.
				UObject* PropertyContainer = nullptr;
				if (PropertyReference.GetComponentClass())
				{
					PropertyContainer = MatchingActor->GetComponentByClass(PropertyReference.GetComponentClass());
					
					if (PropertyContainer)
					{
						DirtiedComponents.Add(Cast<UActorComponent>(PropertyContainer));
					}
				}
				else if (!bOnlyMatchComponent)
				{
					PropertyContainer = MatchingActor;
				}

				if (PropertyContainer)
				{
					CacheAndSetPropertyValue(PropertyReference, PropertyContainer);
				}
			}
		}

		// Set any component properties on components the collection contains.
		for (const TObjectPtr<UActorComponent>& MatchingComponent : EvaluationResult.MatchingComponents)
		{
			// The properties to modify are grouped by actor type. For the purposes of components, we need to inspect all actor types since any
			// actor type could potentially contain a component of the type(s) that are contained in the evaluated collection.
			for (const TPair<TSubclassOf<AActor>, TArray<FMovieGraphPropertyReference>>& PropertyPair : PropertiesToModifyByActorClass)
			{
				const UClass* ActorType = PropertyPair.Key.Get();
				const TArray<FMovieGraphPropertyReference>& PropertyReferences = PropertyPair.Value;

				for (const FMovieGraphPropertyReference& PropertyReference : PropertyReferences)
				{
					UObject* PropertyContainer = nullptr;
					if (PropertyReference.GetComponentClass() == MatchingComponent->GetClass())
					{
						// The property reference may indicate that the component AND class types need to match
						if (PropertyReference.GetPropertyClass())
						{
							if (MatchingComponent->GetOwner()->GetClass() == ActorType)
							{
								PropertyContainer = MatchingComponent;
							}
						}
						else
						{
							PropertyContainer = MatchingComponent;
						}

						DirtiedComponents.Add(Cast<UActorComponent>(PropertyContainer));
					}

					if (PropertyContainer)
					{
						CacheAndSetPropertyValue(PropertyReference, PropertyContainer);
					}
				}
			}
		}
	}

	// The render thread needs the updates that were made here.
	for (UActorComponent* DirtyComponent : DirtiedComponents)
	{
		if (DirtyComponent)
		{
			DirtyComponent->MarkRenderStateDirty();
		}
	}
}

void UMovieGraphPropertyModifier::UndoModifier()
{
	// Undo property modifications in reverse. There's a possibility that a property could be modified multiple times on the same object;
	// to ensure that the original property value is correctly restored, the first modification to it must be restored *last*. This is due to how
	// property values are set + cached at the same time (see CacheAndSetPropertyValue()).
	for (int32 Index = ModifiedProperties.Num() - 1; Index >= 0; --Index)
	{
		const TPair<UObject*, FMovieGraphPropertyReference>& ModifiedPropertyPair = ModifiedProperties[Index];

		UObject* ModifiedObject = ModifiedPropertyPair.Key;
		const FMovieGraphPropertyReference& ModifiedPropertyReference = ModifiedPropertyPair.Value;
		const FProperty* ModifiedProperty = ModifiedPropertyPair.Value.GetProperty();

		if (ModifiedProperty && ModifiedObject)
		{
			// Restore the property value to what it was prior to being updated by the modifier
			const FString PreviousValue = PreviousPropertyValuesView->GetValueSerializedString(ModifiedPropertyReference.AuxPropertyName);
			ModifiedProperty->ImportText_InContainer(*PreviousValue, ModifiedObject, ModifiedObject, 0);

			// Ensure the render thread gets this update
			if (UActorComponent* ActorComponent = Cast<UActorComponent>(ModifiedObject))
			{
				ActorComponent->MarkRenderStateDirty();
			}
		}
	}

	ModifiedProperties.Reset();
}

FText UMovieGraphPropertyModifier::GetModifierName()
{
	static const FText ModifierName = NSLOCTEXT("MovieGraph", "ClassPropertyModifierName", "Class Property");
	return ModifierName;
}

FMovieGraphPropertyReference UMovieGraphPropertyModifier::MakePropertyReference(
	UClass* InPropertyClass,
	const TSubclassOf<UActorComponent>& InComponentClass,
	const FName& InPropertyName)
{
	return FMovieGraphPropertyReference(InPropertyClass, InComponentClass, InPropertyName);
}

void UMovieGraphPropertyModifier::AddProperties(const TArray<FMovieGraphPropertyReference>& InProperties)
{
	TArray<FMovieGraphValueViewProperty> PropertiesToAdd;

	PropertiesToAdd.Reserve(InProperties.Num());

	// Intentionally iterated by value instead of reference. The AuxPropertyName needs to be added to the reference, and the reference will be
	// copied anyway when added to the PropertiesToModify array.
	for (FMovieGraphPropertyReference PropertyReference : InProperties)
	{
		if (!PropertyReference.IsValidProperty())
		{
			continue;
		}

		// Populate the AuxPropertyName; this is the "internal" name of the property within the modifier which is used in the property bags.
		PropertyReference.AuxPropertyName = GetInternalPropertyName(PropertyReference);

		// Batch up this property to be added to the internal property bags.
		PropertiesToAdd.Emplace(FMovieGraphValueViewProperty(
			PropertyReference.AuxPropertyName, PropertyReference.GetValueType(), PropertyReference.GetValueTypeObject(), {}));

		PropertiesToModifyByActorClass.FindOrAdd(PropertyReference.GetPropertyClass()).Add(PropertyReference);
	}

	// With all the data about the properties found, add all of them in one batch for efficiency.
	UpdatedPropertyValuesView->AddProperties(PropertiesToAdd);
	PreviousPropertyValuesView->AddProperties(PropertiesToAdd);
}

FName UMovieGraphPropertyModifier::GetPropertyNameForSettingCustomValue(const FMovieGraphPropertyReference& InPropertyReference)
{
	return GetInternalPropertyName(InPropertyReference);
}

UMovieGraphFixedValueView* UMovieGraphPropertyModifier::GetPropertiesForSettingValues()
{
	if (!UserFacingPropertiesView)
	{
		UserFacingPropertiesView = NewObject<UMovieGraphFixedValueView>(this, TEXT("UserFacingPropertiesView"));
		UserFacingPropertiesView->ViewProperties(&UpdatedPropertyValues);
	}

	return UserFacingPropertiesView;
}

FName UMovieGraphPropertyModifier::GetInternalPropertyName(const FMovieGraphPropertyReference& InPropertyReference)
{
	// If the property reference includes a component, that needs to be included in the property name in order to properly disambiguate it.
	// Otherwise, using the class name is sufficient.

	if (InPropertyReference.GetComponentClass().Get() && InPropertyReference.GetPropertyClass())
	{
		return FName(FString::Format(TEXT("{0}_{1}_{2}"), {
			InPropertyReference.GetPropertyClass()->GetName(),
			InPropertyReference.GetComponentClass()->GetName(),
			InPropertyReference.GetPropertyName().ToString() }));
	}

	if (InPropertyReference.GetPropertyClass())
	{
		return FName(FString::Format(TEXT("{0}_{1}"), {
			InPropertyReference.GetPropertyClass()->GetName(),
			InPropertyReference.GetPropertyName().ToString() }));
	}

	if (InPropertyReference.GetComponentClass())
	{
		return FName(FString::Format(TEXT("{0}_{1}"), {
			InPropertyReference.GetComponentClass()->GetName(),
			InPropertyReference.GetPropertyName().ToString() }));
	}

	UE_LOGF(LogMovieRenderPipeline, Warning, "Attempting to modify a property via the Property Modifier with an invalid actor class and/or component class.");

	return FName(TEXT("INVALID"));
}
