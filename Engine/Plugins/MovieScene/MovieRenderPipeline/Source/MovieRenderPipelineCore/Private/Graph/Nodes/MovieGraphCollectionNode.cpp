// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphCollectionNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphCollectionNode)

#define LOCTEXT_NAMESPACE "MovieGraph"

UMovieGraphCollectionNode::UMovieGraphCollectionNode()
{
	Collection = CreateDefaultSubobject<UMovieGraphCollection>(TEXT("Collection"));

	RegisterDelegates();
}

#if WITH_EDITOR
FText UMovieGraphCollectionNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText CollectionNodeName = LOCTEXT("NodeName_Collection", "Collection");
	static const FText CollectionNodeDescription = LOCTEXT("NodeDescription_Collection", "Collection\n{0}");

	const FString CollectionDisplayName = (Collection && !Collection->GetCollectionName().IsEmpty())
		? Collection->GetCollectionName()
		: LOCTEXT("NodeNoNameWarning_Collection", "NO NAME").ToString();

	if (bGetDescriptive)
	{
		return FText::Format(CollectionNodeDescription, FText::FromString(CollectionDisplayName));
	}

	return CollectionNodeName;
}

FText UMovieGraphCollectionNode::GetMenuCategory() const
{
	return LOCTEXT("CollectionNode_Category", "Utility");
}

FLinearColor UMovieGraphCollectionNode::GetNodeTitleColor() const
{
	static const FLinearColor CollectionNodeColor = FLinearColor(0.047f, 0.501f, 0.654f);
	return CollectionNodeColor;
}

FSlateIcon UMovieGraphCollectionNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon CollectionIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "SceneOutliner.NewFolderIcon");

	OutColor = FLinearColor::White;
	return CollectionIcon;
}
#endif // WITH_EDITOR

FString UMovieGraphCollectionNode::GetNodeInstanceName() const
{
	return Collection ? Collection->GetCollectionName() : FString();
}

void UMovieGraphCollectionNode::RegisterDelegates()
{
	Super::RegisterDelegates();

#if WITH_EDITOR
	if (Collection)
	{
		Collection->OnCollectionNameChangedDelegate.AddWeakLambda(this, [this](UMovieGraphCollection* ChangedCollection)
		{
			OnNodeChangedDelegate.Broadcast(this);
		});
	}
#endif
}

#undef LOCTEXT_NAMESPACE // "MovieGraph"
