// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphCommandLineEncoderNode.h"

#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphAudioOutputNode.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "MoviePipelineCommandLineEncoderSettings.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphCommandLineEncoderNode)

#define LOCTEXT_NAMESPACE "MovieGraph"

UMovieGraphCommandLineEncoderNode::UMovieGraphCommandLineEncoderNode()
	: bDeleteSourceFiles(false)
	, bSkipEncodeOnRenderCanceled(true)
	, bRetainInputTextFiles(false)
{
	FileNameFormat = TEXT("{sequence_name}");
		
	// Sensible defaults for cmdline arguments
	CommandLineFormat = TEXT("-hide_banner -y -loglevel error {VideoInputs} {AudioInputs} -acodec {AudioCodec} -vcodec {VideoCodec} {Quality} \"{OutputPath}\"");
	VideoInputStringFormat = TEXT("-f concat -safe 0 -i \"{InputFile}\" -r {FrameRate}");
	AudioInputStringFormat = TEXT("-f concat -safe 0 -i \"{InputFile}\"");
	EncodeSettings = TEXT("-crf 20");

	// Fill in some defaults from the project settings
	if (const UMoviePipelineCommandLineEncoderSettings* EncoderSettings = GetDefault<UMoviePipelineCommandLineEncoderSettings>())
	{
		AudioCodec = EncoderSettings->AudioCodec;
		VideoCodec = EncoderSettings->VideoCodec;
		OutputFileExtension = EncoderSettings->OutputFileExtension;
	}
}

#if WITH_EDITOR
FText UMovieGraphCommandLineEncoderNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText CommandLineEncoderNodeName = LOCTEXT("NodeName_CommandLineEncoder", "Command Line Encoder");
	return CommandLineEncoderNodeName;
}

FText UMovieGraphCommandLineEncoderNode::GetMenuCategory() const
{
	return LOCTEXT("CommandLineEncoder_Category", "Utility");
}

FLinearColor UMovieGraphCommandLineEncoderNode::GetNodeTitleColor() const
{
	static const FLinearColor ModifierNodeColor = FLinearColor(0.92f, 0.42f, 0.2f);
	return ModifierNodeColor;
}

FSlateIcon UMovieGraphCommandLineEncoderNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon ModifierIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ObjectBrowser.TabIcon");

	OutColor = FLinearColor::White;
	return ModifierIcon;
}
#endif // WITH_EDITOR

EMovieGraphBranchRestriction UMovieGraphCommandLineEncoderNode::GetBranchRestriction() const
{
	return EMovieGraphBranchRestriction::Globals;
}

void UMovieGraphCommandLineEncoderNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesCommandLineEncoder = true;
}

void UMovieGraphCommandLineEncoderNode::StartEncodingProcess(TArray<FMovieGraphRenderOutputData>& InGeneratedData, const bool bInIsShotEncode)
{
	// If the format string isn't split per shot, we can't start encoding now.
	if (bInIsShotEncode && !NeedsPerShotFlushing())
	{
		return;
	}

	// Early out if there are any issues with settings. This will log and shutdown the pipeline if there are errors.
	TArray<FText> ValidationErrors;
	if (!AreSettingsValid(ValidationErrors))
	{
		for (const FText& Error : ValidationErrors)
		{
			UE_LOGF(LogMovieRenderPipelineIO, Error, "%ls", *Error.ToString());
		}
		
		CachedPipeline->RequestShutdown(true);
		
		return;
	}

	// Early out if there isn't any data to feed to the encoder. Could happen if it's running on a branch with no nodes that generate output (eg, JPG).
	if (InGeneratedData.IsEmpty())
	{
		UE_LOGF(LogMovieRenderPipelineIO, Error, "The Command Line Encoder cannot run because no input files were generated for it to use.");
		CachedPipeline->RequestShutdown(true);
		
		return;
	}

	// Get the base output file path; it still needs to be validated and resolved at this point
	const UMovieGraphGlobalOutputSettingNode* OutputSettingNode = GetSettingOnBranch<UMovieGraphGlobalOutputSettingNode>();
	FString FilePathFormatString = OutputSettingNode->OutputDirectory.Path / FileNameFormat;
	
	constexpr bool bTestRenderPass = false;
	constexpr bool bTestFrameNumber = false;
	UE::MoviePipeline::ValidateOutputFormatString(FilePathFormatString, bTestRenderPass, bTestFrameNumber);
	UE::MoviePipeline::RemoveFrameNumberFormatStrings(FilePathFormatString, true);

	// Launch the encoder, once per encoder param (one for a sequence encode, or multiple for shot encodes)
	for (TTuple<FMovieGraphRenderDataIdentifier, FEncoderParams>& RenderLayerEncoderParams : GenerateRenderLayerEncoderParams(InGeneratedData, bInIsShotEncode))
	{
		FMovieGraphRenderDataIdentifier& RenderIdentifier = RenderLayerEncoderParams.Key;
		FEncoderParams& EncoderParams = RenderLayerEncoderParams.Value;
		
		FString FinalFilePath = GetResolvedOutputFilename(RenderIdentifier, EncoderParams.Shot, FilePathFormatString);

		// Manipulate the in/out data in case scripting tries to get access to the files. It's not a perfect solution
		// because the encoding won't be finished by the time the scripting layer is called. Need to manipulate the original
		// and not the copy that we're currently iterating through.
		FMovieGraphRenderDataIdentifier CommandLineEncoderIdentifier;
		CommandLineEncoderIdentifier.RootBranchName = GlobalsPinName;
		CommandLineEncoderIdentifier.LayerName = GlobalsPinName.ToString();
		CommandLineEncoderIdentifier.RendererName = FString(TEXT("CommandLineEncoder"));

		// Add the Command Line Encoder output data to the last shot if this is a sequence encode, or to the matching set of output
		// data if this is a shot encode.
		if (!bInIsShotEncode)
		{
			InGeneratedData.Last().RenderLayerData.FindOrAdd(CommandLineEncoderIdentifier).FilePaths.Add(FinalFilePath);
		}
		else
		{
			for (FMovieGraphRenderOutputData& OutputData : InGeneratedData)
			{
				if (OutputData.Shot != EncoderParams.Shot)
				{
					continue;
				}

				OutputData.RenderLayerData.FindOrAdd(CommandLineEncoderIdentifier).FilePaths.Add(FinalFilePath);
			}
		}
		
		EncoderParams.NamedArguments.Add(TEXT("OutputPath"), FinalFilePath);
		
		LaunchEncoder(EncoderParams);
	}
}

void UMovieGraphCommandLineEncoderNode::BeginExport(UMovieGraphPipeline* InMoviePipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph)
{
	CachedPipeline = InMoviePipeline;
	PrimaryJobEvaluatedGraph = InPrimaryJobEvaluatedGraph;

	// This is called after all shots have finished rendering. If we were starting encode jobs per shot, then there's already
	// an encode job going for all shots, so we early out.
	if (NeedsPerShotFlushing())
	{
		// Don't warn here about a primary job graph wanting to generate a shot-level encode. It is not an error. If there are no shot-level
		// graphs defined, it's valid that the primary graph requests that shot-level encodes are completed. The shot-level encodes will be picked
		// up from the primary graph via BeginShotExport() and not started here.
		return;
	}

	// However, if they didn't want a per-shot flush (ie: rendering all shots into one file) then we start now.
	TArray<FMovieGraphRenderOutputData>& OutputData = CachedPipeline->GetGeneratedOutputData();
	constexpr bool bIsShotEncode = false;
	StartEncodingProcess(OutputData, bIsShotEncode);

	// Should not be used after a primary job export finishes
	PrimaryJobEvaluatedGraph = nullptr;
}

void UMovieGraphCommandLineEncoderNode::BeginShotExport(UMovieGraphPipeline* InMoviePipeline)
{
	CachedPipeline = InMoviePipeline;
	PrimaryJobEvaluatedGraph = nullptr;
	
	if (!NeedsPerShotFlushing())
	{
		// If the CLE node didn't request per-shot flushing, skip encoding at this point. When the job-level export is done, the CLE will
		// run at that point. No need to warn about this.
		return;
	}
	
	TArray<FMovieGraphRenderOutputData>& OutputData = CachedPipeline->GetGeneratedOutputData();
	TArray<FMovieGraphRenderOutputData> LatestShotData;
	if (!OutputData.IsEmpty())
	{
		LatestShotData.Add(OutputData.Last());
	}
	
	constexpr bool bIsShotEncode = true;
	StartEncodingProcess(LatestShotData, bIsShotEncode);
}

bool UMovieGraphCommandLineEncoderNode::HasFinishedExporting()
{
	// Manually tick the output (which cleans up ActiveEncodeJobs). Once a render finishes, the engine is not ticking frames, so the pipeline
	// will call this method repeatedly until the encode has completed.
	OnTick();
	
	const bool bIsFinished = ActiveEncodeJobs.IsEmpty();

	// Clean up persisted data if the encode is finished
	if (bIsFinished)
	{
		CachedPipeline = nullptr;
		PrimaryJobEvaluatedGraph = nullptr;
	}
	
	return bIsFinished;
}

bool UMovieGraphCommandLineEncoderNode::AreSettingsValid(TArray<FText>& OutErrors) const
{
	OutErrors.Empty();
	
	const UMoviePipelineCommandLineEncoderSettings* EncoderSettings = GetDefault<UMoviePipelineCommandLineEncoderSettings>();

	// Validate project settings
	{
		const FString EncoderPath = EncoderSettings->ExecutablePath;
		if (EncoderPath.IsEmpty())
		{
			OutErrors.Add(LOCTEXT("CommandLineEncode_MissingExecutable", "No encoder executable has been specified in the Project Settings. Please set an encoder executable in Project Settings > Movie Pipeline CLI Encoder."));
		}
	}

	// Validate node settings
	{
		if (VideoCodec.IsEmpty())
		{
			OutErrors.Add(LOCTEXT("CommandLineEncode_MissingVideoCodec", "No video codec has been specified on the Command Line Encoder node."));
		}

		if (AudioCodec.IsEmpty() && CommandLineFormat.Contains("{AudioCodec}"))
		{
			OutErrors.Add(LOCTEXT("CommandLineEncode_MissingAudioCodec", "The {AudioCodec} token is in the Command Line Format property on the Command Line Encoder node, but no audio codec has been specified."));
		}

		if (OutputFileExtension.IsEmpty())
		{
			OutErrors.Add(LOCTEXT("CommandLineEncode_MissingFileExtension", "No file extension has been specified on the Command Line Encoder node."));
		}
	}

	return OutErrors.IsEmpty();
}

void UMovieGraphCommandLineEncoderNode::OnTick()
{
	for (int32 Index = ActiveEncodeJobs.Num() - 1; Index >= 0; Index--)
	{
		FActiveJob& Job = ActiveEncodeJobs[Index];
		TArray<uint8> ProcessOutput;
		FString Results = FPlatformProcess::ReadPipe(Job.ReadPipe);
		if (Results.Len() > 0)
		{
			// The default global arguments suppress everything but errors. We unfortunately can't tell stdout from stderr
			// using a non-blocking platform process launch, so we'll just promote all the output as an error log.
			UE_LOGF(LogMovieRenderPipeline, Error, "Command Line Encoder: %ls", *Results);
		}

		// If they hit escape during a render, (potentially) cancel the encode job
		bool bCancelEncode = false;
		if (bSkipEncodeOnRenderCanceled && CachedPipeline->IsShutdownRequested())
		{
			bCancelEncode = true;

			// We have to specifically terminate the process instead of just closing it
			constexpr bool bKillTree = true;
			FPlatformProcess::TerminateProc(Job.ProcessHandle, bKillTree);
			FPlatformProcess::WaitForProc(Job.ProcessHandle);
		}

		// If the process has finished we'll clean up
		if (!FPlatformProcess::IsProcRunning(Job.ProcessHandle) || bCancelEncode)
		{
			FPlatformProcess::ClosePipe(Job.ReadPipe, Job.WritePipe);
			FPlatformProcess::CloseProc(Job.ProcessHandle);

			IFileManager& FileManager = IFileManager::Get();
			for (const FString& FilePath : Job.FilesToDelete)
			{
				constexpr bool bRequireExist = false;
				constexpr bool bEvenReadOnly = false;
				constexpr bool bQuiet = false;
				FileManager.Delete(*FilePath, bRequireExist, bEvenReadOnly, bQuiet);
			}

			ActiveEncodeJobs.RemoveAt(Index);
		}
	}
}

bool UMovieGraphCommandLineEncoderNode::NeedsPerShotFlushing() const
{
	const UMovieGraphGlobalOutputSettingNode* OutputSettingNode = GetSettingOnBranch<UMovieGraphGlobalOutputSettingNode>();
	
	const FString FullPath = OutputSettingNode->OutputDirectory.Path / FileNameFormat;
	
	if (FullPath.Contains(TEXT("{shot_name}")) || FullPath.Contains(TEXT("{camera_name}")))
	{
		return true;
	}

	return false;
}

TMap<FMovieGraphRenderDataIdentifier, UMovieGraphCommandLineEncoderNode::FEncoderParams> UMovieGraphCommandLineEncoderNode::GenerateRenderLayerEncoderParams(TArray<FMovieGraphRenderOutputData>& InGeneratedData, const bool bInIsShotEncode) const
{
	// We produce one set of encoder parameters (ie, one file) per render layer
	TMap<FMovieGraphRenderDataIdentifier, FEncoderParams> RenderLayerEncoderParams;

	// The path shouldn't have quotes on it as it's already kept as a separate argument right up until creating the process, at which point
	// the platform puts quotes around the FString if needed.
	const UMoviePipelineCommandLineEncoderSettings* EncoderSettings = GetDefault<UMoviePipelineCommandLineEncoderSettings>();
	FString ExecutablePathNoQuotes = EncoderSettings->ExecutablePath.Replace(TEXT("\""), TEXT(""));
	FPaths::NormalizeFilename(ExecutablePathNoQuotes);

	UMovieGraphGlobalOutputSettingNode* OutputSettingNode = GetSettingOnBranch<UMovieGraphGlobalOutputSettingNode>();
	const FFrameRate SourceFrameRate = CachedPipeline->GetDataSourceInstance()->GetDisplayRate();
	const FFrameRate EffectiveFrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputSettingNode, SourceFrameRate);

	const FString SanitizedEncodeSettings = EncodeSettings.Replace(LINE_TERMINATOR, TEXT(" "));
	FStringFormatNamedArguments SharedArguments;
	SharedArguments.Add(TEXT("Executable"), ExecutablePathNoQuotes);
	SharedArguments.Add(TEXT("AudioCodec"), AudioCodec);
	SharedArguments.Add(TEXT("VideoCodec"), VideoCodec);
	SharedArguments.Add(TEXT("FrameRate"), EffectiveFrameRate.AsDecimal());
	SharedArguments.Add(TEXT("Quality"), SanitizedEncodeSettings);

	// If this is a sequence encode, use the first render identifier that's found as the encoder params key. This key will be re-used for all
	// shots that are included in the sequence encode.
	FMovieGraphRenderDataIdentifier SequenceEncoderParamsKey;
	if (!bInIsShotEncode)
	{
		TArray<FMovieGraphRenderDataIdentifier> Identifiers;
		InGeneratedData[0].RenderLayerData.GetKeys(Identifiers);

		SequenceEncoderParamsKey = Identifiers[0];
	}
	
	for (FMovieGraphRenderOutputData& GeneratedRenderData : InGeneratedData)
	{
		// Search for audio to attach
		TMap<FString, TArray<FString>> AudioPathsByExtension;
		for (const TPair<FMovieGraphRenderDataIdentifier, FMovieGraphRenderLayerOutputData>& InnerRenderLayer : GeneratedRenderData.RenderLayerData)
		{
			if (InnerRenderLayer.Key.RendererName == UMovieGraphAudioOutputNode::RendererName)
			{
				for (const FString& FilePath : InnerRenderLayer.Value.FilePaths)
				{
					FString Extension = FPaths::GetExtension(FilePath);
					AudioPathsByExtension.FindOrAdd(Extension).Add(FilePath);
				}
			}
		}

		TArray<FMovieGraphRenderDataIdentifier> RenderDataToRemove;
		
		for (const TPair<FMovieGraphRenderDataIdentifier, FMovieGraphRenderLayerOutputData>& RenderLayer : GeneratedRenderData.RenderLayerData)
		{
			const FMovieGraphRenderDataIdentifier& RenderIdentifier = RenderLayer.Key;
			const FMovieGraphRenderLayerOutputData& RenderOutputData = RenderLayer.Value;
			
			if (RenderIdentifier.RendererName == UMovieGraphAudioOutputNode::RendererName)
			{
				continue;
			}

			FMovieGraphRenderDataIdentifier EncoderParamsKey = !bInIsShotEncode ? SequenceEncoderParamsKey : RenderIdentifier;
			FEncoderParams& EncoderParams = RenderLayerEncoderParams.FindOrAdd(EncoderParamsKey);
			EncoderParams.Shot = GeneratedRenderData.Shot;
			EncoderParams.RenderDataIdentifier = EncoderParamsKey;
			EncoderParams.NamedArguments = SharedArguments;
			
			for (const FString& FilePath : RenderOutputData.FilePaths)
			{
				FString Extension = FPaths::GetExtension(FilePath);
				EncoderParams.FilesByExtensionType.FindOrAdd(Extension).Add(FilePath);
			}

			// Attach audio.
			// If this is a sequence encode, only add the audio file if it doesn't exist already (because there's only one encoder params for the
			// sequence, vs multiple for shot encodes). In most cases, the audio provided will be the audio for the entire sequence, so there's no
			// need to add it again.
			if (!bInIsShotEncode)
			{
				for (const TPair<FString, TArray<FString>>& AudioPair : AudioPathsByExtension)
				{
					TArray<FString>& ExistingAudioFilesForExtension = EncoderParams.FilesByExtensionType.FindOrAdd(AudioPair.Key);
					for (const FString& NewAudioFile : AudioPair.Value)
					{
						ExistingAudioFilesForExtension.AddUnique(NewAudioFile);
					}
				}
			}
			else
			{
				EncoderParams.FilesByExtensionType.Append(AudioPathsByExtension);
			}

			RenderDataToRemove.Add(RenderIdentifier);

			// Data was found to include in the encode job; do not add additional data from other output types or branches
			break;
		}
		
		// If we're going to delete the source files, don't make them available to the scripting layer callback
		// because the files will be deleted out from underneath at a random point, so don't want the scripting
		// layer to think they can rely on them actually existing. If scripting layer really needs source files,
		// they need to not use the Command Line Encoder setting and instead roll their own.
		if (bDeleteSourceFiles)
		{
			for (const FMovieGraphRenderDataIdentifier& RenderIdentifier : RenderDataToRemove)
			{
				GeneratedRenderData.RenderLayerData.Remove(RenderIdentifier);
			}
		}
	}

	return RenderLayerEncoderParams;
}

FString UMovieGraphCommandLineEncoderNode::GetResolvedOutputFilename(const FMovieGraphRenderDataIdentifier& RenderIdentifier, const TWeakObjectPtr<UMoviePipelineExecutorShot>& Shot, const FString& FileNameFormatString) const
{
	TMap<FString, FString> FormatOverrides;
	FormatOverrides.Add(TEXT("render_pass"), RenderIdentifier.RendererName);
	FormatOverrides.Add(TEXT("ext"), OutputFileExtension);
	if (Shot.IsValid())
	{
		FormatOverrides.Add(TEXT("shot_name"), Shot->OuterName);
		FormatOverrides.Add(TEXT("camera_name"), Shot->InnerName);
	}

	// We must manually resolve {version} tokens because they're intended to be per-shot/global (ie: individual file names
	// don't have versions when using image sequences).
	FMovieGraphFilenameResolveParams ResolveParams;
	ResolveParams.InitializationTime = CachedPipeline->GetInitializationTime();
	ResolveParams.InitializationTimeOffset = CachedPipeline->GetInitializationTimeOffset();
	ResolveParams.Job = CachedPipeline->GetCurrentJob();
	ResolveParams.Shot = Shot.Get();
	ResolveParams.FileNameFormatOverrides = FormatOverrides;
	ResolveParams.FileNameOverride = FileNameFormat;
	ResolveParams.EvaluatedConfig = GetEvaluatedConfig();
	ResolveParams.RenderDataIdentifier = RenderIdentifier;
	ResolveParams.Version = Shot.IsValid() ? Shot->ShotInfo.VersionNumber : UMovieGraphBlueprintLibrary::ResolveVersionNumber(ResolveParams);

	FMovieGraphResolveArgs FinalFormatArgs;
	FString FinalFilePath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FileNameFormatString, ResolveParams, FinalFormatArgs);

	if (FPaths::IsRelative(FinalFilePath))
	{
		FinalFilePath = FPaths::ConvertRelativePathToFull(FinalFilePath);
	}

	FPaths::NormalizeFilename(FinalFilePath);
	FPaths::CollapseRelativeDirectories(FinalFilePath);

	const FString FinalFileDirectory = FPaths::GetPath(FinalFilePath);

	// Ensure the output directory is created
	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
	if (!FileManager.CreateDirectoryTree(*FinalFileDirectory))
	{
		UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to create directory for output path '%ls'", *FinalFileDirectory);
	}

	return FinalFilePath;
}

void UMovieGraphCommandLineEncoderNode::GenerateTemporaryEncoderInputFiles(const FEncoderParams& InParams, TArray<FString>& OutVideoInputFilePaths, TArray<FString>& OutAudioInputFilePaths) const
{
	UMovieGraphGlobalOutputSettingNode* OutputSettingNode = GetSettingOnBranch<UMovieGraphGlobalOutputSettingNode>();

	// Generate a text file for each input type which lists the files for that input type. We generate a FGuid in case there are
	// multiple encode jobs going at once.
	TStringBuilder<64> StringBuilder;

	double InFrameRate = InParams.NamedArguments[TEXT("FrameRate")].DoubleValue;
	double FrameRateAsDuration = 1.0 / InFrameRate;

	for (const TTuple<FString, TArray<FString>>& Pair : InParams.FilesByExtensionType)
	{
		FGuid FileGuid = FGuid::NewGuid();
		FString FilePath = OutputSettingNode->OutputDirectory.Path / FileGuid.ToString() + TEXT("_input");

		TMap<FString, FString> FormatOverrides;
		FormatOverrides.Add(TEXT("ext"), TEXT("txt"));

		FMovieGraphFilenameResolveParams ResolveParams;
		ResolveParams.InitializationTime = CachedPipeline->GetInitializationTime();
		ResolveParams.InitializationTimeOffset = CachedPipeline->GetInitializationTimeOffset();
		ResolveParams.Job = CachedPipeline->GetCurrentJob();
		ResolveParams.Shot = InParams.Shot.Get();
		ResolveParams.FileNameFormatOverrides = FormatOverrides;
		ResolveParams.EvaluatedConfig = GetEvaluatedConfig();
		ResolveParams.RenderDataIdentifier = InParams.RenderDataIdentifier;

		FMovieGraphResolveArgs FinalFormatArgs;
		FString FinalFilePath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FilePath, ResolveParams, FinalFormatArgs);

		UE_LOGF(LogMovieRenderPipelineIO, Log, "Generated Path '%ls' for input data.", *FinalFilePath);
		StringBuilder.Reset();
		for (const FString& Path : Pair.Value)
		{
			StringBuilder.Appendf(TEXT("file 'file:%s'%s"), *Path, LINE_TERMINATOR);

			// Some video encoders require the duration of each file to be listed after the file. Write duration for all
			// video encoders; providing the extra information doesn't hurt.
			if (Pair.Key != UMovieGraphAudioOutputNode::OutputExtension)
			{
				StringBuilder.Appendf(TEXT("duration %f%s"), FrameRateAsDuration, LINE_TERMINATOR);
			}
		}

		// Save this to disk.
		FFileHelper::SaveStringToFile(StringBuilder.ToString(), *FinalFilePath);

		// Separate audio and video files
		if (Pair.Key == UMovieGraphAudioOutputNode::OutputExtension)
		{
			OutAudioInputFilePaths.Add(FinalFilePath);
		}
		else
		{
			OutVideoInputFilePaths.Add(FinalFilePath);
		}
	}
}

FString UMovieGraphCommandLineEncoderNode::BuildEncoderCommand(const FEncoderParams& InParams, TArray<FString>& InVideoInputFilePaths, TArray<FString>& InAudioInputFilePaths) const
{
	// Build our final command line arguments
	FStringFormatNamedArguments FinalNamedArgs = InParams.NamedArguments;
	
	const FString SanitizedVideoInputStringFormat = VideoInputStringFormat.Replace(LINE_TERMINATOR, TEXT(" "));
	const FString SanitizedAudioInputStringFormat = AudioInputStringFormat.Replace(LINE_TERMINATOR, TEXT(" "));
	const FString SanitizedCommandLineFormat = CommandLineFormat.Replace(LINE_TERMINATOR, TEXT(" "));

	FString VideoInputArg;
	FString AudioInputArg;

	for (const FString& FilePath : InVideoInputFilePaths)
	{
		FStringFormatNamedArguments NamedArgs;
		NamedArgs.Add(TEXT("InputFile"), *FilePath);
		NamedArgs.Add(TEXT("FrameRate"), InParams.NamedArguments[TEXT("FrameRate")]);

		VideoInputArg += TEXT(" ") + FString::Format(*SanitizedVideoInputStringFormat, NamedArgs);
	}

	for (const FString& FilePath : InAudioInputFilePaths)
	{
		FStringFormatNamedArguments NamedArgs;
		NamedArgs.Add(TEXT("InputFile"), *FilePath);

		AudioInputArg += TEXT(" ") + FString::Format(*SanitizedAudioInputStringFormat, NamedArgs);
	}

	FinalNamedArgs.Add(TEXT("VideoInputs"), VideoInputArg);
	FinalNamedArgs.Add(TEXT("AudioInputs"), AudioInputArg);

	return FString::Format(*SanitizedCommandLineFormat, FinalNamedArgs);;
}

void UMovieGraphCommandLineEncoderNode::LaunchEncoder(const FEncoderParams& InParams)
{
	// Clear out any stale jobs (shouldn't be needed, but just in case)
	ActiveEncodeJobs.Reset();
	
	TArray<FString> VideoInputs;
	TArray<FString> AudioInputs;
	GenerateTemporaryEncoderInputFiles(InParams, VideoInputs, AudioInputs);
	
	const FString CommandToRun = BuildEncoderCommand(InParams, VideoInputs, AudioInputs);
	
	UE_LOGF(LogMovieRenderPipelineIO, Log, "Final Command Line Arguments: %ls", *CommandToRun);

	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;
	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	constexpr bool bLaunchDetached = false;
	constexpr bool bLaunchHidden = true;
	constexpr bool bLaunchReallyHidden = bLaunchHidden;
	const FString ExecutableArg = FString::Format(TEXT("{Executable}"), InParams.NamedArguments);
	const FProcHandle ProcessHandle = FPlatformProcess::CreateProc(*ExecutableArg, *CommandToRun, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, nullptr, PipeWrite);
	if (ProcessHandle.IsValid())
	{
		FActiveJob& NewJob = ActiveEncodeJobs.AddDefaulted_GetRef();
		NewJob.ProcessHandle = ProcessHandle;
		NewJob.ReadPipe = PipeRead;
		NewJob.WritePipe = PipeWrite;

		// Delete the auto-generated input files (unless they have been flagged to be retained)
		if (!bRetainInputTextFiles)
		{
			NewJob.FilesToDelete.Append(VideoInputs);
			NewJob.FilesToDelete.Append(AudioInputs);
		}

		// Delete source files if requested
		if (bDeleteSourceFiles)
		{
			for (const TTuple<FString, TArray<FString>>& Pair : InParams.FilesByExtensionType)
			{
				NewJob.FilesToDelete.Append(Pair.Value);
			}
		}
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to launch encoder process, see output log for more details.");
		constexpr bool bIsError = true;
		CachedPipeline->RequestShutdown(bIsError);
	}
}

TObjectPtr<UMovieGraphEvaluatedConfig> UMovieGraphCommandLineEncoderNode::GetEvaluatedConfig() const
{
	// If a shot is being exported, the primary evaluated graph will be null. Use the pipeline's evaluated graph for the current shot otherwise.
	return PrimaryJobEvaluatedGraph
		? PrimaryJobEvaluatedGraph
		: CachedPipeline->GetTimeStepInstance()->GetCalculatedTimeData().EvaluatedConfig;
}

template<typename T>
T* UMovieGraphCommandLineEncoderNode::GetSettingOnBranch(const bool bIncludeCDOs, const bool bExactMatch) const
{
	return GetEvaluatedConfig()->GetSettingForBranch<T>(GlobalsPinName, bIncludeCDOs, bExactMatch);
}

#undef LOCTEXT_NAMESPACE // "MovieGraph"
