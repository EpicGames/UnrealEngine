// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphDebugNode.h"

#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphDebugNode)

#if WITH_EDITOR
FText UMovieGraphDebugSettingNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText NodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_DebugNode", "Debug Settings");
	return NodeName;
}

FText UMovieGraphDebugSettingNode::GetMenuCategory() const
{
	static const FText NodeCategory_Globals = NSLOCTEXT("MovieGraphNodes", "NodeCategory_Globals", "Globals");
	return NodeCategory_Globals;
}

FLinearColor UMovieGraphDebugSettingNode::GetNodeTitleColor() const
{
	static const FLinearColor NodeColor = FLinearColor(0.849f, 0.f, 0.350f);
	return NodeColor;
}

FSlateIcon UMovieGraphDebugSettingNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Debug");

	OutColor = FLinearColor::White;
	return Icon;
}
#endif // WITH_EDITOR
