// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphExecuteScriptNode.h"

#include "MoviePipelineTelemetry.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Styling/AppStyle.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphExecuteScriptNode)

#if WITH_EDITOR
FText UMovieGraphExecuteScriptNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("MovieGraphNodes", "ExecuteScriptNode_Description", "Execute Script");
}

FText UMovieGraphExecuteScriptNode::GetMenuCategory() const
{
	static const FText NodeCategory_Globals = NSLOCTEXT("MovieGraphNodes", "NodeCategory_Globals", "Globals");
	return NodeCategory_Globals;
}

FLinearColor UMovieGraphExecuteScriptNode::GetNodeTitleColor() const
{
	return FLinearColor(0.1f, 0.1f, 0.85f);
}

FSlateIcon UMovieGraphExecuteScriptNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon ExecuteScriptIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Log.TabIcon");

	OutColor = FLinearColor::White;
	return ExecuteScriptIcon;
}
#endif	// WITH_EDITOR

UMovieGraphScriptBase* UMovieGraphExecuteScriptNode::AllocateScriptInstance() const
{
	const UClass* NewInstanceType = nullptr;

	if (Mode == EMovieGraphExecuteScriptMode::EditorOnly)
	{
#if WITH_EDITORONLY_DATA
		NewInstanceType = EditorOnlyScript.TryLoadClass<UMovieGraphScriptBase>();
#else
		UE_LOGF(LogMovieRenderPipeline, Warning, "Trying to use an editor-only script in a non-editor build. Script will not execute.");
#endif
	}
	else if (Mode == EMovieGraphExecuteScriptMode::EditorAndRuntime)
	{
		NewInstanceType = EditorAndRuntimeScript.TryLoadClass<UMovieGraphScriptBase>();
	}
	else
	{
		// Invalid mode
		check(false);
	}

	if (NewInstanceType)
	{
		UMovieGraphScriptBase* NewInstance = NewObject<UMovieGraphScriptBase>(GetTransientPackage(), NewInstanceType);
		return NewInstance;
	}

	return nullptr;
}

void UMovieGraphExecuteScriptNode::PostLoad()
{
	Super::PostLoad();

	// If loading an older Execute Script node which uses the Script property, migrate it to the EditorAndRuntimeScript property.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Script.IsValid())
	{
		Mode = EMovieGraphExecuteScriptMode::EditorAndRuntime;
		EditorAndRuntimeScript = Script;
		bOverride_EditorAndRuntimeScript = bOverride_Script;
		Script = FSoftClassPath();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR
void UMovieGraphExecuteScriptNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// If switching modes, clear out the script property for the mode being switched from. This is particularly important when switching
	// from Editor + Runtime mode to Editor-Only mode. If the user is intentionally flagging the script as Editor-Only, but previously
	// had an Editor + Runtime script set, the EditorAndRuntimeScript *will still have a reference to the script*, which can trip up
	// the Asset Referencing Restrictions plugin. Clearing out the script property when switching modes fixes this issue.
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphExecuteScriptNode, Mode))
	{
		if (Mode == EMovieGraphExecuteScriptMode::EditorOnly)
		{
			EditorAndRuntimeScript = FSoftClassPath();
		}
		else if (Mode == EMovieGraphExecuteScriptMode::EditorAndRuntime)
		{
			EditorOnlyScript = FSoftClassPath();
		}
		else
		{
			// Invalid mode
			check(false);
		}
	}
}
#endif	// WITH_EDITOR

void UMovieGraphExecuteScriptNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesScripting = true;
}
