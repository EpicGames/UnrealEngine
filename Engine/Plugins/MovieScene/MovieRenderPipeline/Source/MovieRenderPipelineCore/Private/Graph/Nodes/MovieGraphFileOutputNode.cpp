// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphFileOutputNode.h"

#include "Algo/Count.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "MoviePipelineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphFileOutputNode)

namespace UE::MovieGraphFileOutputNode
{
	/**
	 * If {camera_name} is absent, injects the best disambiguating token: custom override tokens from the
	 * payload, or {camera_name} when multiple cameras are active.
	 *
	 * @param InOutFilenameFormatString Format string to modify in place.
	 * @param InNodeName                Name of the file output node, used for token injection.
	 * @param InRenderData              Pass data identifying the camera being resolved.
	 * @param InValidationInfo          Validation info indicating how many cameras are active.
	 */
	static void InjectCameraDisambiguationToken(
		FString& InOutFilenameFormatString,
		const FName& InNodeName,
		const FMovieGraphPassData& InRenderData,
		const UE::MovieGraph::FMovieGraphRenderDataValidationInfo& InValidationInfo)
	{
		// If {camera_name} is already in the format string the user explicitly placed it, so leave it as-is.
		// Otherwise inject whatever token best disambiguates cameras for this pass.
		if (InOutFilenameFormatString.Contains(TEXT("{camera_name}")))
		{
			return;
		}

		/**
		 * FMovieGraphSampleState::CameraNameTokenOverrides:
		 * If set, and {camera_name} is not already in the filename format string, these tokens are always injected instead of {camera_name},
		 * even when only one camera is active. Tokens already present in the format string are skipped.
		 * See CameraNameTokenOverrides for an example.
		 */
		const UE::MovieGraph::FMovieGraphSampleState* ThisPayload = InRenderData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
		if (ThisPayload && !ThisPayload->CameraNameTokenOverrides.IsEmpty())
		{
			for (const FString& TokenIt : ThisPayload->CameraNameTokenOverrides)
			{
				UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, *TokenIt, InNodeName, InRenderData.Key.RootBranchName);
			}
		}
		// We only add the camera name token if there are more than one cameras for the active branch
		else if (InValidationInfo.ActiveCameraCount > 1)
		{
			UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, TEXT("{camera_name}"), InNodeName, InRenderData.Key.RootBranchName);
		}
	}
}

#if WITH_EDITOR
FText UMovieGraphFileOutputNode::GetBasicConfigShortDisplayName() const
{
	return FText::GetEmpty();
}
#endif	// WITH_EDITOR

int32 UMovieGraphFileOutputNode::GetNumFileOutputNodes(const UMovieGraphEvaluatedConfig& InEvaluatedConfig, const FName& InBranchName)
{
	return InEvaluatedConfig.GetSettingsForBranch(UMovieGraphFileOutputNode::StaticClass(), InBranchName, false /*bIncludeCDOs*/, false /*bExactMatch*/).Num();
}

void UMovieGraphFileOutputNode::DisambiguateFilename(FString& InOutFilenameFormatString, const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const FName& InNodeName, const FMovieGraphPassData& InRenderData)
{
	// ToDo: This is overly protective and could be relaxed later, for instance
	// if different file write nodes have chosen a separate filepath entirely.
	const UE::MovieGraph::FMovieGraphRenderDataValidationInfo ValidationInfo = InRawFrameData->GetValidationInfo(InRenderData.Key);

	// Since there can only be one layer per branch, we restrain layer/branch validation to multi-branch graphs.
	if (ValidationInfo.BranchCount > 1)
	{
		// We can run into the scenario where the users have given layers the same name, so layer_name token won't help differentiate.
		// To resolve this, we look to see if there's multiple branches with the same layer name, and if so we force the branch name into the token too.
		if (ValidationInfo.LayerCount < ValidationInfo.BranchCount)
		{
			UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, TEXT("{branch_name}"), InNodeName, InRenderData.Key.RootBranchName);
		}
		else
		{
			// Otherwise, we separate each branch by its unique layer name.
			UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, TEXT("{layer_name}"), InNodeName, InRenderData.Key.RootBranchName);
		}
	}

	// We only add the renderer name token if multiple (non-composited) renderers are present on the active branch (eg, in the case of optional PPMs).
	if (ValidationInfo.ActiveBranchRendererCount > 1)
	{
		UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, TEXT("{renderer_name}"), InNodeName, InRenderData.Key.RootBranchName);
	}

	// We only add the subresource token if a (non-composited) renderer on the active branch is producing more than one subresource (eg, in the case of optional PPMs).
	if (ValidationInfo.ActiveRendererSubresourceCount > 1)
	{
		UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, TEXT("{renderer_sub_name}"), InNodeName, InRenderData.Key.RootBranchName);
	}

	// Inject {camera_name} or payload override tokens when multiple cameras require disambiguation.
	UE::MovieGraphFileOutputNode::InjectCameraDisambiguationToken(InOutFilenameFormatString, InNodeName, InRenderData, ValidationInfo);
}

void UMovieGraphFileOutputNode::DisambiguateFilename(FString& InOutFilenameFormatString, const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const UMovieGraphFileOutputNode* InParentNode, const FMovieGraphPassData& InRenderData)
{
	constexpr bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSettingNode =
		InRawFrameData->EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);

	// Generates the final format string for a given payload, taking into account the file name format from the Output node, any overrides on the
	// payload, and the output directory from the Global Output Settings node.
	auto GetFinalFormatString = [InParentNode, OutputSettingNode](const UE::MovieGraph::FMovieGraphSampleState* InPayload) -> FString
	{
		if (!InPayload)
		{
			return InParentNode->FileNameFormat;
		}

		// The file name format usually comes from the output node directly, but the payload has a chance to override it.
		const FString FileNameFormat = !InPayload->FilenameFormatOverride.IsEmpty() ? InPayload->FilenameFormatOverride : InParentNode->FileNameFormat;

		// Generate one string that puts the directory combined with the filename format.
		FString FileNameFormatString = OutputSettingNode->OutputDirectory.Path / FileNameFormat;

		return FileNameFormatString;
	};

	const TArray<UE::MovieGraph::FMovieGraphSampleState*>* AllPayloadsForNode = InRawFrameData->NodeInstanceToPayloads.Find(InParentNode);
	if (!AllPayloadsForNode)
	{
		// Shouldn't happen
		return;
	}
	
	TArray<UE::MovieGraph::FMovieGraphSampleState*> AllPayloadsForRenderer;
	for (const FMovieGraphPassData& RenderData : InRawFrameData->ImageOutputData)
	{
		if (RenderData.Key.RendererName == InRenderData.Key.RendererName)
		{
			if (UE::MovieGraph::FMovieGraphSampleState* Payload = RenderData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>())
			{
				AllPayloadsForRenderer.Add(Payload);
			}
		}
	}

	const UE::MovieGraph::FMovieGraphSampleState* ThisPayload = nullptr;
	if (const TUniquePtr<FImagePixelData>* ImagePixelData = InRawFrameData->ImageOutputData.Find(InRenderData.Key))
	{
		ThisPayload = (*ImagePixelData)->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
	}

	TArray<FString> AllFilePaths;
	const FString ThisFinalFormatString = GetFinalFormatString(ThisPayload);
	for (const UE::MovieGraph::FMovieGraphSampleState* NodePayload : *AllPayloadsForNode)
	{
		AllFilePaths.Add(GetFinalFormatString(NodePayload));
	}
	
	const FName NodeName = InParentNode->GetFName();
	const UE::MovieGraph::FMovieGraphRenderDataValidationInfo ValidationInfo = InRawFrameData->GetValidationInfo(InRenderData.Key);

	// Since there can only be one layer per branch, we restrain layer/branch validation to multi-branch graphs.
	if (ValidationInfo.BranchCount > 1)
	{
		// We can run into the scenario where the users have given layers the same name, so layer_name token won't help differentiate.
		// To resolve this, we look to see if there's multiple branches with the same layer name, and if so we force the branch name into the token too.
		if (ValidationInfo.LayerCount < ValidationInfo.BranchCount)
		{
			UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, TEXT("{branch_name}"), NodeName, InRenderData.Key.RootBranchName);
		}
		else
		{
			// Otherwise, we separate each branch by its unique layer name.
			UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, TEXT("{layer_name}"), NodeName, InRenderData.Key.RootBranchName);
		}
	}

	// We may add the renderer name token if multiple (non-composited) renderers are present on the active branch (eg, in the case of optional PPMs).
	if (ValidationInfo.ActiveBranchRendererCount > 1)
	{
		int32 NumBeautyRenders = 0;
		for (const UE::MovieGraph::FMovieGraphSampleState* PayloadInstance : *AllPayloadsForNode)
		{
			if (PayloadInstance->bIsBeautyPass)
			{
				++NumBeautyRenders;
			}
		}

		// Determines if the given beauty pass's payload needs the {renderer_name} token.
		auto BeautyPassNeedsRendererName = [&GetFinalFormatString, &AllFilePaths, &NumBeautyRenders](const UE::MovieGraph::FMovieGraphSampleState* InPayload)
		{
			const FString BeautyPassFinalFormatString = GetFinalFormatString(InPayload);
			const int32 NumPathsWithSameFormatString = Algo::Count(AllFilePaths, BeautyPassFinalFormatString);

			return (NumBeautyRenders > 1) && (NumPathsWithSameFormatString > 1);
		};

		// * If there is only one "beauty" render in the branch, then don't force on {renderer_name}. Beauty renders should be given priority for
		//   "opting out" of this token if it's not already specified in the format string.
		// * If there's more than one "beauty" render in the branch, then {renderer_name} is forced on the beauty passes to disambiguate.
		// * For subresource passes, generally the {renderer_sub_name} token is enough to disambiguate. However, if its parent beauty pass needs
		//   {renderer_name}, then the subresource also needs {renderer_name}. 
		bool bAddRendererName = false;
		if (ThisPayload && ThisPayload->bIsBeautyPass && BeautyPassNeedsRendererName(ThisPayload))
		{
			bAddRendererName = true;
		}
		else if (ThisPayload && !ThisPayload->bIsBeautyPass)
		{
			// Determine if the parent beauty pass needs {renderer_name}. If so, the subresource also needs it.
			for (const UE::MovieGraph::FMovieGraphSampleState* RendererPayload : AllPayloadsForRenderer)
			{
				if (RendererPayload->bIsBeautyPass && BeautyPassNeedsRendererName(RendererPayload))
				{
					bAddRendererName = true;
					break;
				}
			}
		}

		if (bAddRendererName)
		{
			UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, TEXT("{renderer_name}"), NodeName, InRenderData.Key.RootBranchName);
		}
	}

	// We only add the subresource token if a (non-composited) renderer on the active branch is producing more than one subresource (eg, in the case of optional PPMs).
	if (ValidationInfo.ActiveRendererSubresourceCount > 1)
	{
		const bool bIsBeautyPass = ThisPayload && ThisPayload->bIsBeautyPass;

		// However, don't force this token on for beauty passes.
		if (!bIsBeautyPass)
		{
			UE::MoviePipeline::ConformOutputFormatStringToken(InOutFilenameFormatString, TEXT("{renderer_sub_name}"), NodeName, InRenderData.Key.RootBranchName);
		}
	}

	// Inject {camera_name} or payload override tokens when multiple cameras require disambiguation.
	UE::MovieGraphFileOutputNode::InjectCameraDisambiguationToken(InOutFilenameFormatString, NodeName, InRenderData, ValidationInfo);
}

TArray<FMovieGraphPassData> UMovieGraphFileOutputNode::GetCompositedPasses(UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData)
{
	// Gather the passes that need to be composited
	TArray<FMovieGraphPassData> CompositedPasses;

	for (const FMovieGraphPassData& RenderData : InRawFrameData->ImageOutputData)
	{
		const UE::MovieGraph::FMovieGraphSampleState* Payload = RenderData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
		check(Payload);
		if (!Payload->bCompositeOnOtherRenders)
		{
			continue;
		}

		FMovieGraphPassData CompositePass;
		CompositePass.Key = RenderData.Key;
		CompositePass.Value = RenderData.Value->CopyImageData();
		CompositedPasses.Add(MoveTemp(CompositePass));
	}

	// Sort composited passes if multiple were found. Passes with a higher sort order go to the end of the array so they
	// get composited on top of passes with a lower sort order.
	CompositedPasses.Sort([](const FMovieGraphPassData& PassA, const FMovieGraphPassData& PassB)
	{
		const UE::MovieGraph::FMovieGraphSampleState* PayloadA = PassA.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
		const UE::MovieGraph::FMovieGraphSampleState* PayloadB = PassB.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
		check(PayloadA);
		check(PayloadB);

		return PayloadA->CompositingSortOrder < PayloadB->CompositingSortOrder;
	});

	return CompositedPasses;
}
