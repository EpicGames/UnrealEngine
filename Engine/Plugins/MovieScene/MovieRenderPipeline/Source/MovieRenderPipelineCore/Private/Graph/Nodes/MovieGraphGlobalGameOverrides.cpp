// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphGlobalGameOverrides.h"

#include "Graph/MovieGraphPipeline.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphGlobalGameOverrides)

UMovieGraphGlobalGameOverridesNode::UMovieGraphGlobalGameOverridesNode()
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: GameModeOverride(nullptr)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, SoftGameModeOverride(AMoviePipelineGameMode::StaticClass())
	, ScalabilityQualityLevel(EMovieGraphScalabilityQualityLevel::Cinematic)
	, bDisableTextureStreaming(false)
	, bDisableLODs(false)
	, bDisableHLODs(false)
	, bFlushLevelStreaming(false)
	, bFlushAssetCompiler(false)
	, bFlushShaderCompiler(false)
	, bFlushGrassStreaming(false)
	, bFlushStreamingManagers(false)
	, VirtualTextureFeedbackFactor(1)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, bRebuildLumenSceneBetweenRenderLayers(false)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
}

void UMovieGraphGlobalGameOverridesNode::BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const
{
	// We don't provide the GameMode on the command line argument as we expect NewProcess to boot into an empty map and then it will
	// transition into the correct map which will then use the GameModeOverride setting.
	
	{
		Scalability::FQualityLevels QualityLevels;
		QualityLevels.SetFromSingleQualityLevel(static_cast<int32>(ScalabilityQualityLevel));

		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.ViewDistanceQuality={0}"), {QualityLevels.ViewDistanceQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.AntiAliasingQuality={0}"), {QualityLevels.AntiAliasingQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.ShadowQuality={0}"), {QualityLevels.ShadowQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.GlobalIlluminationQuality={0}"), {QualityLevels.GlobalIlluminationQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.ReflectionQuality={0}"), {QualityLevels.ReflectionQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.PostProcessQuality={0}"), {QualityLevels.PostProcessQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.TextureQuality={0}"), {QualityLevels.TextureQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.EffectsQuality={0}"), {QualityLevels.EffectsQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.FoliageQuality={0}"), {QualityLevels.FoliageQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.ShadingQuality={0}"), {QualityLevels.ShadingQuality}));
		InOutDeviceProfileCvars.AddUnique(FString::Format(TEXT("sg.LandscapeQuality={0}"), {QualityLevels.LandscapeQuality}));
	}

	if (bDisableTextureStreaming)
	{
		InOutDeviceProfileCvars.AddUnique(TEXT("r.TextureStreaming=0"));
	}

	if (bDisableLODs)
	{
		InOutDeviceProfileCvars.AddUnique(TEXT("r.ForceLOD=0"));
		InOutDeviceProfileCvars.AddUnique(TEXT("r.SkeletalMeshLODBias=-10"));
		InOutDeviceProfileCvars.AddUnique(TEXT("r.ParticleLODBias=-10"));
		InOutDeviceProfileCvars.AddUnique(TEXT("foliage.DitheredLOD=0"));
		InOutDeviceProfileCvars.AddUnique(TEXT("foliage.ForceLOD=0"));
	}

	if (bDisableHLODs)
	{
		// It's a command and not an integer cvar (despite taking 1/0)
		InOutExecCmds.AddUnique(TEXT("r.HLOD 0"));
	}

	if (bFlushStreamingManagers)
	{
		InOutDeviceProfileCvars.AddUnique(TEXT("r.Streaming.SyncStatesWhenBlocking=1"));
	}

	// Like the extra cvars applied in ApplySettings(), the below are applied to allow MRQ to function correctly.

#if WITH_EDITOR
	{
		InOutDeviceProfileCvars.AddUnique(TEXT("GeometryCache.Streamer.BlockTillFinishStreaming=1"));
		InOutDeviceProfileCvars.AddUnique(TEXT("GeometryCache.Streamer.ShowNotification=0"));
	}
#endif

	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("a.URO.Enable=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.SkyLight.RealTimeReflectionCapture.TimeSlice=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.VolumetricRenderTarget=%d"), 1));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.VolumetricRenderTarget.Mode=%d"), 3));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("wp.Runtime.BlockOnSlowStreaming=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("p.Chaos.ImmPhys.MinStepTime=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("p.ChaosCloth.UseTimeStepSmoothing=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.Water.SkipWaterInfoTextureRenderWhenWorldRenderingDisabled=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("Slate.bAllowThrottling=%d"), 0));

	IConsoleVariable* VTInvalidateCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("MoviePipeline.EnableVTInvalidateOnNaniteLOD"));
	if (VTInvalidateCvar && VTInvalidateCvar->GetInt() != 0)
	{
		InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.Nanite.VSMInvalidateOnLODDelta=%d"), 1));
	}

	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.PostProcessing.PropagateAlpha=%d"), UE::MoviePipeline::GetAlphaOutputOverride() ? 1 : 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.LUT.Size=%d"), 64));
}

void UMovieGraphGlobalGameOverridesNode::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (const UClass* GameModeOverrideClass = GameModeOverride.Get())
	{
		SoftGameModeOverride = GameModeOverrideClass;
		bOverride_SoftGameModeOverride = bOverride_GameModeOverride;
		
		GameModeOverride = nullptr;
		bOverride_GameModeOverride = false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UMovieGraphGlobalGameOverridesNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->ScalabilityQualityLevel = StaticEnum<EMovieGraphScalabilityQualityLevel>()->GetNameStringByValue(static_cast<int64>(ScalabilityQualityLevel));
}

void UMovieGraphGlobalGameOverridesNode::ApplySettings(const bool bOverrideValues, UWorld* InWorld)
{
	// We use a different node instance at the start/end of the shot, so we need to store the cached values on the CDO.
	// It's okay to use the CDO here because these are unserialized properties (which means it won't affect delta-diff
	// serialization with the CDO). There is potentially an issue where if the user changed a property on the node
	// between the start of the shot and the end that we don't restore correctly, but we don't currently provide a way
	// to do that, and even if we did, we have multiple scenarios where changing values mid-render won't work.
	UMovieGraphGlobalGameOverridesNode* NodeCDO = GetMutableDefault<UMovieGraphGlobalGameOverridesNode>();

	// Apply the scalability settings
	if (bOverrideValues)
	{
		// Store the current scalability settings so we can revert back to them
		NodeCDO->PreviousQualityLevels = Scalability::GetQualityLevels();

		// Set to the chosen scalability quality level
		Scalability::FQualityLevels QualityLevels;
		QualityLevels.SetFromSingleQualityLevel(static_cast<int32>(ScalabilityQualityLevel));

		// Apply
		Scalability::SetQualityLevels(QualityLevels);
	}
	else
	{
		// We re-apply old scalability settings at the end of the function during teardown
		// so that any values that are also specified in Scalability don't get overwritten
		// with the wrong values from the ones below restoring.
	}

	if (bDisableTextureStreaming)
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousTextureStreaming, TEXT("r.TextureStreaming"), 0, bOverrideValues);
	}
	
	if (bDisableLODs)
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousForceLOD, TEXT("r.ForceLOD"), 0, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousSkeletalMeshBias, TEXT("r.SkeletalMeshLODBias"), -10, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousParticleLODBias, TEXT("r.ParticleLODBias"), -10, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousFoliageDitheredLOD, TEXT("foliage.DitheredLOD"), 0, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousFoliageForceLOD, TEXT("foliage.ForceLOD"), 0, bOverrideValues);
	}

	if (bDisableHLODs)
	{
		// It's a command and not an integer cvar (despite taking 1/0), so we can't cache it 
		if (GEngine && InWorld)
		{
			// ToDo: We don't restore this right now, because we don't have a way to fetch the current value to cache it.
			GEngine->Exec(InWorld, TEXT("r.HLOD 0"));
		}
	}

	if (bFlushStreamingManagers)
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousStreamingManagerSyncState, TEXT("r.Streaming.SyncStatesWhenBlocking"), 1, bOverrideValues);
	}

	// Set cvars that allow MRQ to run correctly. These are not exposed as properties on the node, but the chance of a
	// user needing to customize these (or be aware of them) is very low.
	{
#if WITH_EDITOR
		// To make sure the GeometryCache streamer doesn't skip frames and doesn't pop up notification during rendering
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT_IF_EXIST(NodeCDO->PreviousGeoCacheStreamerBlockTillFinish, TEXT("GeometryCache.Streamer.BlockTillFinishStreaming"), 1, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT_IF_EXIST(NodeCDO->PreviousGeoCacheStreamerShowNotification, TEXT("GeometryCache.Streamer.ShowNotification"), 0, bOverrideValues);
#endif

		// Disable systems that try to preserve performance in runtime games.
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousAnimationUROEnabled, TEXT("a.URO.Enable"), 0, bOverrideValues);

		// To make sure that the skylight is always valid and consistent across capture sessions, we enforce a full capture each frame, accepting a small GPU cost.
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousSkyLightRealTimeReflectionCaptureTimeSlice, TEXT("r.SkyLight.RealTimeReflectionCapture.TimeSlice"), 0, bOverrideValues);

		// Cloud are rendered using high quality volumetric render target mode 3: per pixel tracing and composition on screen, while supporting cloud on translucent.
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousVolumetricRenderTarget, TEXT("r.VolumetricRenderTarget"), 1, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousVolumetricRenderTargetMode, TEXT("r.VolumetricRenderTarget.Mode"), 3, bOverrideValues);

		// To make sure that the world partition streaming doesn't end up in critical streaming performances and stops streaming low priority cells.
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousIgnoreStreamingPerformance, TEXT("wp.Runtime.BlockOnSlowStreaming"), 0, bOverrideValues);

		// Remove any minimum delta time requirements from Chaos Physics to ensure accuracy at high Temporal Sample counts
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_FLOAT(NodeCDO->PreviousChaosImmPhysicsMinStepTime, TEXT("p.Chaos.ImmPhys.MinStepTime"), 0, bOverrideValues);

		// Cloth's time step smoothing messes up the change in number of simulation substeps that fixes the cloth simulation behavior when using Temporal Samples.
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousChaosClothUseTimeStepSmoothing, TEXT("p.ChaosCloth.UseTimeStepSmoothing"), 0, bOverrideValues);

		// Water skips water info texture when the world's game viewport rendering is disabled so we need to prevent this from happening.
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT_IF_EXIST(NodeCDO->PreviousSkipWaterInfoTextureRenderWhenWorldRenderingDisabled, TEXT("r.Water.SkipWaterInfoTextureRenderWhenWorldRenderingDisabled"), 0, bOverrideValues);

		// Slate throttling can cause the world to not tick, leading to hitching/pausing in the rendered frames. Notifications/toast popping up can trigger
		// throttling, so force-disable it during a render.
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousAllowSlateThrottling, TEXT("Slate.bAllowThrottling"), 0, bOverrideValues);

		// This is only a temporary cvar while it's experimental so it's not exposed to the UI, but exposed as a cvar
		// so that users can turn it off in the event that it causes issues.
		IConsoleVariable* VTInvalidateCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("MoviePipeline.EnableVTInvalidateOnNaniteLOD"));
		if (VTInvalidateCvar && VTInvalidateCvar->GetInt() != 0)
		{
			MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousNaniteVSMInvalidateOnLODDelta, TEXT("r.Nanite.VSMInvalidateOnLODDelta"), 1, bOverrideValues);
		}

		// Enable the alpha channel throughout the renderer
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_BOOL(NodeCDO->PreviousAlphaOutput, TEXT("r.PostProcessing.PropagateAlpha"), UE::MoviePipeline::GetAlphaOutputOverride(), bOverrideValues);

		// Improve (combined) LUT precision to avoid quantization errors, especially with multi-sample accumulation
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(NodeCDO->PreviousLUTSize, TEXT("r.LUT.Size"), 64, bOverrideValues);
	}
	
	// Must come after the above cvars so that if one of those cvars is also specified by the scalability level, then we
	// restore to the value in the original scalability level, not the value we cached in the scalability level (if applied).
	if (!bOverrideValues)
	{
		Scalability::SetQualityLevels(NodeCDO->PreviousQualityLevels);
	}
}

TSubclassOf<AGameModeBase> UMovieGraphGlobalGameOverridesNode::GetGameModeOverride(const UMoviePipelineExecutorJob* InJob)
{
	UMovieGraphConfig* GraphConfig = InJob->GetGraphPreset();
	if (InJob->IsUsingGraphConfiguration() && GraphConfig)
	{
		// There is most likely no pipeline to fetch a traversal context from at this point, so a temporary context
		// is generated instead (which will not be fully filled in, but should be enough to generate a correctly flattened
		// Globals branch).
		FMovieGraphTraversalContext TraversalContext;
		TraversalContext.Job = const_cast<UMoviePipelineExecutorJob*>(InJob);
		
		FString OutTraversalError;
		if (const UMovieGraphEvaluatedConfig* EvaluatedGraph = GraphConfig->CreateFlattenedGraph(TraversalContext, OutTraversalError))
		{
			// Note that the CDO is not fetched here. Users need to explicitly include the Global Game Overrides node and specify the game mode override.
			constexpr bool bIncludeCDOs = false;
			constexpr bool bExactMatch = true;
			UMovieGraphGlobalGameOverridesNode* GameOverridesNode =
				EvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalGameOverridesNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);

			return GameOverridesNode ? GameOverridesNode->SoftGameModeOverride.LoadSynchronous() : nullptr;
		}
	}
	else if (InJob->IsUsingBasicConfiguration())
	{
		// Basic configuration does not support game mode overrides.
		return nullptr;
	}
	else
	{
		TArray<UMoviePipelineSetting*> AllSettings = InJob->GetConfiguration()->GetAllSettings();
		UMoviePipelineSetting** GameOverridesPtr = AllSettings.FindByPredicate([](const UMoviePipelineSetting* InSetting)
		{
			return InSetting->GetClass() == UMoviePipelineGameOverrideSetting::StaticClass();
		});

		if (GameOverridesPtr)
		{
			if (UMoviePipelineSetting* Setting = *GameOverridesPtr)
			{
				return CastChecked<UMoviePipelineGameOverrideSetting>(Setting)->SoftGameModeOverride.LoadSynchronous();
			}
		}
	}

	return nullptr;
}

#if WITH_EDITOR
FText UMovieGraphGlobalGameOverridesNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText GlobalGameOverridesNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_GlobalGameOverrides", "Global Game Overrides");
	return GlobalGameOverridesNodeName;
}

FText UMovieGraphGlobalGameOverridesNode::GetMenuCategory() const
{
	static const FText NodeCategory_Globals = NSLOCTEXT("MovieGraphNodes", "NodeCategory_Globals", "Globals");
	return NodeCategory_Globals;
}

FLinearColor UMovieGraphGlobalGameOverridesNode::GetNodeTitleColor() const
{
	static const FLinearColor GlobalGameOverridesNodeColor = FLinearColor(0.549f, 0.f, 0.250f);
	return GlobalGameOverridesNodeColor;
}

FSlateIcon UMovieGraphGlobalGameOverridesNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon GlobalGameOverridesIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Launcher.TabIcon");

	OutColor = FLinearColor::White;
	return GlobalGameOverridesIcon;
}
#endif // WITH_EDITOR
