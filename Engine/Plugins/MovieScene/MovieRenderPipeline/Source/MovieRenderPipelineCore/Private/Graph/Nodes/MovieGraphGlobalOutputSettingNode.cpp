// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"

#include "Algo/Find.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphProjectSettings.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "MoviePipelineTelemetry.h"
#include "Styling/AppStyle.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphGlobalOutputSettingNode)

UMovieGraphGlobalOutputSettingNode::UMovieGraphGlobalOutputSettingNode()
	: OutputFrameRate(FFrameRate(24, 1))
	, bOverwriteExistingOutput(true)
	, ZeroPadFrameNumbers(4)
	, FrameNumberOffset(0)
	, HandleFrameCount(0)
	, bDropFrameTimecode(true)	// Defaults to true because most 29.97 FPS content uses this
	, bFlushDiskWritesPerShot(false)
	, CustomPlaybackRangeStartFrame(0)
	, CustomPlaybackRangeEndFrame(0)
{
	OutputDirectory.Path = TEXT("{project_dir}/Saved/MovieRenders/");
}

void UMovieGraphGlobalOutputSettingNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	const FString ResolvedProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
	OutMergedFormatArgs.FilenameArguments.Add(TEXT("project_dir"), ResolvedProjectDir);
	OutMergedFormatArgs.FileMetadata.Add(TEXT("unreal/project_dir"), ResolvedProjectDir);

	// We need to look at the Project Settings for the latest value for a given profile
	FMovieGraphNamedResolution NamedResolution;
	if (UMovieGraphBlueprintLibrary::IsNamedResolutionValid(OutputResolution.ProfileName))
	{
		NamedResolution = UMovieGraphBlueprintLibrary::NamedResolutionFromProfile(OutputResolution.ProfileName);
	}
	else
	{
		// Otherwise if it's not in the output settings as a valid profile, we use our internally stored one.
		NamedResolution = OutputResolution;
	}
	
	// Resolution Arguments
	{
		FString Resolution = FString::Printf(TEXT("%d_%d"), NamedResolution.Resolution.X, NamedResolution.Resolution.Y);
		OutMergedFormatArgs.FilenameArguments.Add(TEXT("output_resolution"), Resolution);
		OutMergedFormatArgs.FilenameArguments.Add(TEXT("output_width"), FString::FromInt(NamedResolution.Resolution.X));
		OutMergedFormatArgs.FilenameArguments.Add(TEXT("output_height"), FString::FromInt(NamedResolution.Resolution.Y));
	}

	// We don't resolve the version here because that's handled on a per-file/shot basis
}

void UMovieGraphGlobalOutputSettingNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->HandleFrameCount = HandleFrameCount;
}

#if WITH_EDITOR
FText UMovieGraphGlobalOutputSettingNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText OutputSettingsNodeName = NSLOCTEXT("MoviePipelineGraph", "NodeName_GlobalOutputSettings", "Global Output Settings");
	return OutputSettingsNodeName;
}

FText UMovieGraphGlobalOutputSettingNode::GetMenuCategory() const 
{
	return NSLOCTEXT("MoviePipelineGraph", "Settings_Category", "Settings");
}

FLinearColor UMovieGraphGlobalOutputSettingNode::GetNodeTitleColor() const 
{
	static const FLinearColor OutputSettingsColor = FLinearColor(0.854f, 0.509f, 0.039f);
	return OutputSettingsColor;
}

FSlateIcon UMovieGraphGlobalOutputSettingNode::GetIconAndTint(FLinearColor& OutColor) const 
{
	static const FSlateIcon SettingsIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Settings");

	OutColor = FLinearColor::White;
	return SettingsIcon;
}

EMovieGraphBranchRestriction UMovieGraphGlobalOutputSettingNode::GetBranchRestriction() const
{
	return EMovieGraphBranchRestriction::Globals;
}
#endif // WITH_EDITOR

bool UMovieGraphGlobalOutputSettingNode::GetCustomPromotionDefaultValue(const FMovieGraphPropertyInfo& InPropertyInfo, FString& OutSerializedValue) const
{
	// For the CustomPlaybackRangeStart/End properties, if the property is NOT overridden,
	// the promoted variable should default to SequenceDefault (rather than Custom) so that
	// the variable reflects "use the sequence's own range" semantics.
	static const FName PlaybackRangeStartName = GET_MEMBER_NAME_CHECKED(UMovieGraphGlobalOutputSettingNode, CustomPlaybackRangeStart);
	static const FName PlaybackRangeEndName = GET_MEMBER_NAME_CHECKED(UMovieGraphGlobalOutputSettingNode, CustomPlaybackRangeEnd);

	if (InPropertyInfo.Name != PlaybackRangeStartName && InPropertyInfo.Name != PlaybackRangeEndName)
	{
		return false;
	}

	const FProperty* RealProperty = FindFProperty<FProperty>(GetClass(), InPropertyInfo.Name);
	if (!RealProperty)
	{
		return false;
	}

	const FBoolProperty* OverrideProp = UMovieGraphConfig::FindOverridePropertyForRealProperty(GetClass(), RealProperty);
	if (!OverrideProp || OverrideProp->GetPropertyValue_InContainer(this))
	{
		return false;
	}

	const FStructProperty* StructProp = CastField<FStructProperty>(RealProperty);
	if (!StructProp)
	{
		return false;
	}

	FMovieGraphSequencePlaybackRangeBound DesiredValue;
	DesiredValue.Type = EMovieGraphSequenceRangeType::SequenceDefault;

	// Pass the desired value as both Value and Defaults so ExportText exports all properties
	// (no diff-against-default skipping). This ensures the full struct is serialized.
	StructProp->Struct->ExportText(OutSerializedValue, &DesiredValue, &DesiredValue, nullptr, PPF_None, nullptr);
	return true;
}

void UMovieGraphGlobalOutputSettingNode::PostLoad()
{
	Super::PostLoad();

	// We don't emit a warning here because old assets that are just
	// upgrading will update the properties and be fixed on next save.
	// The ability to warn when running ApplyPostLoadPropertyConversions 
	// is meant to catch scripting which changes it after load, but before render.
	constexpr bool bEmitWarning = false;
	ApplyPostLoadPropertyConversions(bEmitWarning);
}

void UMovieGraphGlobalOutputSettingNode::PostFlatten()
{
	Super::PostFlatten();
	const bool bEmitWarning = true;
	ApplyPostLoadPropertyConversions(bEmitWarning);
}

void UMovieGraphGlobalOutputSettingNode::ApplyPostLoadPropertyConversions(bool bEmitWarning)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// If they had previously stated that they wanted to use a custom Start Frame
	// then we transfer that override to the new Playback Start, and then clear
	// the override flag so that the upgrade doesn't get performed again on next load.
	if (bOverride_CustomPlaybackRangeStartFrame)
	{
		bOverride_CustomPlaybackRangeStart = true;
		CustomPlaybackRangeStart.Type = EMovieGraphSequenceRangeType::Custom;
		CustomPlaybackRangeStart.Value = CustomPlaybackRangeStartFrame;

		// Clear the original override the user had so that we don't upgrade again next load.
		bOverride_CustomPlaybackRangeStartFrame = false;
		CustomPlaybackRangeStartFrame = 0;

		if (bEmitWarning)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "CustomPlaybackRangeStartFrame is deprecated, but it was changed after the asset was loaded. Please update your scripts/pipeline to use CustomPlaybackRangeStart instead!");
		}
	}

	if (bOverride_CustomPlaybackRangeEndFrame)
	{
		bOverride_CustomPlaybackRangeEnd = true;
		CustomPlaybackRangeEnd.Type = EMovieGraphSequenceRangeType::Custom;
		CustomPlaybackRangeEnd.Value = CustomPlaybackRangeEndFrame;

		// Clear the original override the user had so that we don't upgrade again next load.
		bOverride_CustomPlaybackRangeEndFrame = false;
		CustomPlaybackRangeEndFrame = 0;

		if (bEmitWarning)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "CustomPlaybackRangeEndFrame is deprecated, but it was changed after the asset was loaded. Please update your scripts/pipeline to use CustomPlaybackRangeStart instead!");
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

