// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphInputNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphSubgraphNode.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphInputNode)

UMovieGraphInputNode::UMovieGraphInputNode()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		RegisterDelegates();
	}
}

TArray<FMovieGraphPinProperties> UMovieGraphInputNode::GetOutputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;

	if (const UMovieGraphConfig* ParentGraph = GetGraph())
	{
		for (const UMovieGraphInput* Input : ParentGraph->GetInputs())
		{
			FMovieGraphPinProperties PinProperties(FName(Input->GetMemberName()), Input->GetValueType(), Input->GetValueTypeObject(), false);
			PinProperties.bIsBranch = Input->bIsBranch;
			Properties.Add(MoveTemp(PinProperties));
		}
	}
	
	return Properties;
}

TArray<UMovieGraphPin*> UMovieGraphInputNode::EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const
{
	TArray<UMovieGraphPin*> PinsToFollow;
	
	if (!ensure(InContext.PinBeingFollowed))
	{
		return PinsToFollow;
	}

	// If this input node occurs in a subgraph (ie, the subgraph stack is not empty), follow the pin to the input pin on
	// the subgraph node in the parent graph.
	//
	//    Subgraph
	//    +------------------
	//    | Inputs node
	//    | +--------+ 
	// In1| |    In1 | <--- Pin being followed ---
	// In2| |    In2 | 
	// In3| |    In3 | 
	//    | +--------+   
	//    +------------------

	if (InContext.SubgraphStack.IsEmpty())
	{
		// This was not an input node in a subgraph; nothing to continue following
		return PinsToFollow;
	}

	// Pop the subgraph node off the stack and continue traversal in the parent graph
	const TObjectPtr<const UMovieGraphSubgraphNode> ParentSubgraphNode = InContext.SubgraphStack.Pop();
	if (!ensure(ParentSubgraphNode.Get()))
	{
		return PinsToFollow;
	}

	if (UMovieGraphPin* InputPin = ParentSubgraphNode->GetInputPin(InContext.PinBeingFollowed->Properties.Label))
	{
		PinsToFollow.Add(InputPin);
	}
	
	return PinsToFollow;
}

FString UMovieGraphInputNode::GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext) const
{
	if (const UMovieGraphConfig* Graph = GetGraph())
	{
		for (UMovieGraphInput* Input : Graph->GetInputs())
		{
			if (Input && (FName(Input->GetMemberName()) == InPinName))
			{
				return Input->GetValueSerializedString();
			}
		}
	}

	return FString();
}

bool UMovieGraphInputNode::CanBeDisabled() const
{
	return false;
}

#if WITH_EDITOR
FText UMovieGraphInputNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("MovieGraphNodes", "InputNode_Description", "Input");
}

FText UMovieGraphInputNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "InputNode_Category", "Input/Output");
}

FLinearColor UMovieGraphInputNode::GetNodeTitleColor() const
{
	return FLinearColor::Black;
}

FSlateIcon UMovieGraphInputNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon InputIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeBufferMode");

	OutColor = FLinearColor::White;
	return InputIcon;
}
#endif // WITH_EDITOR

void UMovieGraphInputNode::RegisterDelegates()
{
	Super::RegisterDelegates();
	
	if (UMovieGraphConfig* Graph = GetGraph())
	{
#if WITH_EDITOR
		// Register delegates for new inputs when they're added to the graph
		Graph->OnGraphInputAddedDelegate.RemoveAll(this);
		Graph->OnGraphInputAddedDelegate.AddUObject(this, &UMovieGraphInputNode::RegisterInputDelegates);
#endif

		// Register delegates for existing inputs
		for (UMovieGraphInput* InputMember : Graph->GetInputs())
		{
			RegisterInputDelegates(InputMember);
		}
	}
}

void UMovieGraphInputNode::RegisterInputDelegates(UMovieGraphInput* Input)
{
#if WITH_EDITOR
	if (Input)
	{
		Input->OnMovieGraphInputChangedDelegate.RemoveAll(this);
		Input->OnMovieGraphInputChangedDelegate.AddUObject(this, &UMovieGraphInputNode::UpdateExistingPins);
	}
#endif
}

void UMovieGraphInputNode::UpdateExistingPins(UMovieGraphMember* ChangedInput) const
{
	if (const UMovieGraphConfig* Graph = GetGraph())
	{
		const TArray<UMovieGraphInput*> InputMembers = Graph->GetInputs();
		if (InputMembers.Num() == OutputPins.Num())
		{
			for (int32 Index = 0; Index < InputMembers.Num(); ++Index)
			{
				OutputPins[Index]->Properties.Label = FName(InputMembers[Index]->GetMemberName());
				OutputPins[Index]->Properties.Type = InputMembers[Index]->GetValueType();
				OutputPins[Index]->Properties.TypeObject = InputMembers[Index]->GetValueTypeObject();
				OutputPins[Index]->Properties.bIsBranch = InputMembers[Index]->bIsBranch;
			}
		}
		
		OnNodeChangedDelegate.Broadcast(this);
	}
}
