// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphLightModifierNode.h"

#include "Components/DirectionalLightComponent.h"
#include "Components/LightComponent.h"
#include "Components/LightComponentBase.h"
#include "Components/LocalLightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "GameFramework/Actor.h"
#include "Graph/MovieGraphValueUtils.h"
#include "Graph/Nodes/MovieGraphClassPropertyModifier.h"
#include "MoviePipelineTelemetry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphLightModifierNode)

#define LOCTEXT_NAMESPACE "MovieGraph"

UMovieGraphLightModifierNode::UMovieGraphLightModifierNode()
{
	Modifier = CreateDefaultSubobject<UMovieGraphPropertyModifier>(TEXT("LightingModifier"));
}

#if WITH_EDITOR
FText UMovieGraphLightModifierNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText NodeName = LOCTEXT("NodeName_LightModifier", "Light Modifier");
	static const FText NodeDescription = LOCTEXT("NodeDescription_LightModifier", "Light Modifier\n{0}");

	const FString ModifierNameDisp = ModifierName.IsEmpty() ? LOCTEXT("NodeNoNameWarning_LightModifier", "NO NAME").ToString() : ModifierName;

	if (bGetDescriptive)
	{
		return FText::Format(NodeDescription, FText::FromString(ModifierNameDisp));
	}

	return NodeName;
}

FText UMovieGraphLightModifierNode::GetMenuCategory() const
{
	return LOCTEXT("LightModifierNode_Category", "Utility");
}

FLinearColor UMovieGraphLightModifierNode::GetNodeTitleColor() const
{
	static const FLinearColor ModifierNodeColor = FLinearColor(0.6f, 0.113f, 0.113f);
	return ModifierNodeColor;
}

FSlateIcon UMovieGraphLightModifierNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon ModifierIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PointLightComponent");

	OutColor = FLinearColor::White;
	return ModifierIcon;
}

void UMovieGraphLightModifierNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Broadcast a node-changed delegate so that the node title's UI gets updated.
	if ((PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphModifierNode, ModifierName)) ||
		(PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphModifierNode, bOverride_ModifierName)))
	{
		OnNodeChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR

TArray<FPropertyBagPropertyDesc> UMovieGraphLightModifierNode::GetDynamicPropertyDescriptions() const
{
	// Most of these should really be defined somewhere outside of MRG, but they don't appear to be.
	static const FName EditConditionMetadataKey = TEXT("EditCondition");
	static const FName CategoryMetadataKey = TEXT("Category");
	static const FName EnableCategoriesMetadataKey = TEXT("EnableCategories");
	static const FName LightingClassMetadataKey = TEXT("LightingClass");
	static const FName DisplayNameMetadataKey = TEXT("DisplayName");
	static const FName InlineEditConditionToggleMetadataKey = TEXT("InlineEditConditionToggle");

	TArray<FPropertyBagPropertyDesc> PropertyDescs;

	for (const TPair<FName, FMovieGraphPropertyReference>& CustomPropertyPair : CustomPropertyInfo)
	{
		const FMovieGraphPropertyReference& CustomPropertyRef = CustomPropertyPair.Value;
		const FProperty* CustomProperty = CustomPropertyRef.GetProperty();
		const FName OverridePropertyName = GetLightingOverridePropertyName(CustomPropertyRef.AuxPropertyName);

		EMovieGraphValueType ValueType = EMovieGraphValueType::None;
		TObjectPtr<UObject> ValueTypeObject = nullptr;
		if (!UE::MovieGraph::ValueUtils::GetTypesForFProperty(CustomProperty, ValueType, ValueTypeObject))
		{
			continue;
		}

		FPropertyBagPropertyDesc MainDesc(CustomPropertyRef.AuxPropertyName, static_cast<EPropertyBagPropertyType>(ValueType), ValueTypeObject);

#if WITH_EDITOR
		// Use all of the metadata associated with the property
		for (const TPair<FName, FString>& MetaDataPair : *CustomProperty->GetMetaDataMap())
		{
			MainDesc.MetaData.Add({ MetaDataPair.Key, MetaDataPair.Value });
		}

		// Enable categories so that they're grouped by light type. The category also needs to be changed to the class associated with the property.
		if (const TSubclassOf<UActorComponent> PropertyClass = CustomPropertyRef.GetComponentClass())
		{
			const FString LightingClassDisplayName = PropertyClass->GetDisplayNameText().ToString();

			MainDesc.MetaData.Add({ CategoryMetadataKey, LightingClassDisplayName });
			MainDesc.MetaData.Add({ EnableCategoriesMetadataKey, FString(TEXT("true")) });

			// Parts of the UI may need to access the name of the lighting class; this is not officially sanctioned metadata and is MRG-specific.
			MainDesc.MetaData.Add({ LightingClassMetadataKey, LightingClassDisplayName });
		}

		// Multiple light classes could use the same property, so the display name needs to be set (since the property name is prefixed by the light class).
		MainDesc.MetaData.Add({ DisplayNameMetadataKey, CustomProperty->GetDisplayNameText().ToString() });

		// Like other MRG properties, this can be selectively enabled/disabled via an override property.
		MainDesc.MetaData.Add({ EditConditionMetadataKey, OverridePropertyName.ToString() });

		// Some lighting properties may be used as inline Edit Condition toggles. For the purposes of Custom properties, we *don't* want it behaving that
		// way, since MRG adds its own override toggle for properties.
		MainDesc.MetaData.RemoveAll([](const FPropertyBagPropertyDescMetaData& MetaDataEntry)
		{
			return MetaDataEntry.Key == InlineEditConditionToggleMetadataKey;
		});
#endif	// WITH_EDITOR

		// Add the main property
		PropertyDescs.Add(MainDesc);

		// Add the override property (EditCondition checkbox next to the property).
		FPropertyBagPropertyDesc OverrideDesc(OverridePropertyName, EPropertyBagPropertyType::Bool);
#if WITH_EDITOR
		OverrideDesc.MetaData.Add({ InlineEditConditionToggleMetadataKey, TEXT("true") });
#endif
		PropertyDescs.Add(MoveTemp(OverrideDesc));
	}

	return PropertyDescs;
}

TArray<FMovieGraphPropertyInfo> UMovieGraphLightModifierNode::GetOverrideablePropertyInfo() const
{
	TArray<FMovieGraphPropertyInfo> PropertyInfo = Super::GetOverrideablePropertyInfo();

	// Exclude the "Dynamic Properties" property from being shown. All of the properties *within* that property bag will be
	// shown instead.
	PropertyInfo.RemoveAll([](const FMovieGraphPropertyInfo& InPropertyInfo)
	{
		// GET_MEMBER_NAME_CHECKED can't be used because DynamicProperties is protected
		return InPropertyInfo.Name == TEXT("DynamicProperties");
	});

	const UMovieGraphMutableValueView* LightingValues = GetDynamicPropertiesView();

	// The PropertyInfo array will include all of the "Custom" properties since it already takes into account the DynamicProperties property bag.
	// However, these entries should have better display names in the context menu.
	for (FMovieGraphPropertyInfo& Info : PropertyInfo)
	{
		if (!Info.bIsDynamicProperty)
		{
			continue;
		}

#if WITH_EDITOR
		const TMap<FName, FString> PropertyMetadata = LightingValues->GetPropertyMetadata(Info.Name);
		const FString* DisplayNameMetadata = PropertyMetadata.Find(TEXT("DisplayName"));
		const FString* LightingClassMetadata = PropertyMetadata.Find(TEXT("LightingClass"));
		if (DisplayNameMetadata && LightingClassMetadata)
		{
			Info.ContextMenuName =
				FText::FromString(FString::Format(TEXT("[{0}] {1}"), { *LightingClassMetadata, *DisplayNameMetadata }));
		}
#endif	// WITH_EDITOR
	}

	return PropertyInfo;
}

void UMovieGraphLightModifierNode::PrepareForFlattening(const UMovieGraphSettingNode* InSourceNode)
{
	Super::PrepareForFlattening(InSourceNode);

	// This node's dynamic properties rely on the CustomPropertyInfo being present (since GetDynamicPropertyDescriptions() iterates it)
	CustomPropertyInfo = Cast<UMovieGraphLightModifierNode>(InSourceNode)->CustomPropertyInfo;
}

TArray<UMovieGraphModifierBase*> UMovieGraphLightModifierNode::GetAllModifiers() const
{
	// Lighting component property names
	static const FName LightColorName = GET_MEMBER_NAME_CHECKED(ULightComponentBase, LightColor);
	static const FName AffectsWorldName = GET_MEMBER_NAME_CHECKED(ULightComponentBase, bAffectsWorld);
	static const FName CastShadowsName = GET_MEMBER_NAME_CHECKED(ULightComponentBase, CastShadows);
	static const FName IndirectLightingIntensityName = GET_MEMBER_NAME_CHECKED(ULightComponentBase, IndirectLightingIntensity);
	static const FName VolumetricScatteringIntensityName = GET_MEMBER_NAME_CHECKED(ULightComponentBase, VolumetricScatteringIntensity);
	static const FName LightingChannelsName = GET_MEMBER_NAME_CHECKED(ULightComponent, LightingChannels);
	static const FName SamplesPerPixelName = GET_MEMBER_NAME_CHECKED(ULightComponentBase, SamplesPerPixel);
	static const FName IntensityName = GET_MEMBER_NAME_CHECKED(ULightComponentBase, Intensity);
	static const FName IntensityUnitsName = GET_MEMBER_NAME_CHECKED(ULocalLightComponent, IntensityUnits);

	// Node property names (the ones which differ from the corresponding component property names above)
	static const FName DirectionalLightIntensityName = GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, DirectionalLightIntensity);
	static const FName SkyLightIntensityName = GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SkyLightIntensityScale);
	static const FName PointLightIntensityName = GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensity);
	static const FName RectLightIntensityName = GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensity);
	static const FName SpotLightIntensityName = GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensity);
	static const FName PointLightIntensityUnitsName = GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensityUnits);
	static const FName RectLightIntensityUnitsName = GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensityUnits);
	static const FName SpotLightIntensityUnitsName = GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensityUnits);

	UMovieGraphFixedValueView* ModifierValues = Modifier->GetPropertiesForSettingValues();

	// Properties are added to the modifier in one batch in order to avoid re-creating the underlying property bag constantly.
	TArray<FMovieGraphPropertyReference> PropertiesToAdd;

	// Because of the batch property creation, we have to defer setting values in the property bag until the properties have actually been created.
	TArray<TFunction<void()>> DeferredValueSetters;

	auto AddClassPredefinedLightProperty = [this, &ModifierValues, &PropertiesToAdd, &DeferredValueSetters]
		(const bool& InOverrideProperty, const FName& InComponentPropertyName, const FName& InNodePropertyName, UClass* InComponentClass) -> void
	{
		// Nothing to do if this lighting property wasn't marked as overridden.
		if (!InOverrideProperty)
		{
			return;
		}

		// The lighting properties exposed in the node always exist on the component, not the light class itself.
		FProperty* ComponentProperty = FindFProperty<FProperty>(InComponentClass, InComponentPropertyName);
		if (!ComponentProperty)
		{
			return;
		}

		constexpr UClass* LightClass = nullptr;
		FMovieGraphPropertyReference& NewProperty =
			PropertiesToAdd.Add_GetRef(FMovieGraphPropertyReference(LightClass, InComponentClass, InComponentPropertyName, ComponentProperty));

		// Queue up setting the value in the property bag *after* the property was added to the property bag.
		DeferredValueSetters.Add([this, &ModifierValues, NewProperty, InNodePropertyName]()
		{
			const FProperty* NodeProperty = FindFProperty<FProperty>(UMovieGraphLightModifierNode::StaticClass(), InNodePropertyName);
			if (ensureMsgf(NodeProperty, TEXT("Could not find property named [%s] on the Light Modifier node."), *InNodePropertyName.ToString()))
			{
				const FName ModifierPropertyName = Modifier->GetPropertyNameForSettingCustomValue(NewProperty);
				
				FString PropertyValue;
				NodeProperty->ExportTextItem_InContainer(PropertyValue, this, nullptr, nullptr, PPF_None);

				// Zero out the struct's memory (within the modifier's property bag). When calling ExportTextItem_InContainer() above, the exported/
				// serialized text provides a *sparse* representation of what was changed on the struct, not a representation of all properties. Thus
				// if we want the modifier's representation of the struct to represent the value set on the node, the struct first needs to be zero'd
				// out, THEN the sparse representation can be applied.
				if (const FStructProperty* StructTargetProperty = CastField<const FStructProperty>(NodeProperty))
				{
					const TObjectPtr<UScriptStruct> TargetScriptStruct = StructTargetProperty->Struct;

					FStructView StructView;
					if (ModifierValues->GetValueStruct(ModifierPropertyName, StructView, TargetScriptStruct.Get()))
					{
						void* StructData = StructView.GetMemory();
						
						TargetScriptStruct->DestroyStruct(StructData);
						FMemory::Memzero(StructData, TargetScriptStruct->GetStructureSize());
					}
				}

				ModifierValues->SetValueSerializedString(ModifierPropertyName, PropertyValue);
			}
		});
	};

	/** Adds a lighting property to the modifier. This propery applies to all light types. */
	auto AddGeneralPredefinedLightProperty = [this, &AddClassPredefinedLightProperty]
		(const bool& InOverrideProperty, const FName& InComponentPropertyName, const FName& InNodePropertyName) -> void
	{
		static const TArray<UClass*> ComponentClasses = {
			UDirectionalLightComponent::StaticClass(), USkyLightComponent::StaticClass(), UPointLightComponent::StaticClass(),
			USpotLightComponent::StaticClass(), URectLightComponent::StaticClass()
		};

		// Add this property for all light component classes.
		for (UClass* ComponentClass : ComponentClasses)
		{
			AddClassPredefinedLightProperty(InOverrideProperty, InComponentPropertyName, InNodePropertyName, ComponentClass);
		}
	};

	// First, add in the properties that are always defined on the node.
	AddGeneralPredefinedLightProperty(bOverride_LightColor, LightColorName, LightColorName);
	AddGeneralPredefinedLightProperty(bOverride_bAffectsWorld, AffectsWorldName, AffectsWorldName);
	AddGeneralPredefinedLightProperty(bOverride_CastShadows, CastShadowsName, CastShadowsName);
	AddGeneralPredefinedLightProperty(bOverride_IndirectLightingIntensity, IndirectLightingIntensityName, IndirectLightingIntensityName);
	AddGeneralPredefinedLightProperty(bOverride_VolumetricScatteringIntensity, VolumetricScatteringIntensityName, VolumetricScatteringIntensityName);
	AddGeneralPredefinedLightProperty(bOverride_LightingChannels, LightingChannelsName, LightingChannelsName);
	AddGeneralPredefinedLightProperty(bOverride_SamplesPerPixel, SamplesPerPixelName, SamplesPerPixelName);
	AddClassPredefinedLightProperty(bOverride_DirectionalLightIntensity, IntensityName, DirectionalLightIntensityName, UDirectionalLightComponent::StaticClass());
	AddClassPredefinedLightProperty(bOverride_SkyLightIntensityScale, IntensityName, SkyLightIntensityName, USkyLightComponent::StaticClass());

	if (IntensityMethod == EMovieGraphLightModifierIntensityMethod::PointRectSpot)
	{
		AddClassPredefinedLightProperty(bOverride_Intensity, IntensityName, IntensityName, UPointLightComponent::StaticClass());
		AddClassPredefinedLightProperty(bOverride_Intensity, IntensityName, IntensityName, URectLightComponent::StaticClass());
		AddClassPredefinedLightProperty(bOverride_Intensity, IntensityName, IntensityName, USpotLightComponent::StaticClass());

		AddClassPredefinedLightProperty(bOverride_IntensityUnits, IntensityUnitsName, IntensityUnitsName, UPointLightComponent::StaticClass());
		AddClassPredefinedLightProperty(bOverride_IntensityUnits, IntensityUnitsName, IntensityUnitsName, URectLightComponent::StaticClass());
		AddClassPredefinedLightProperty(bOverride_IntensityUnits, IntensityUnitsName, IntensityUnitsName, USpotLightComponent::StaticClass());
	}
	else
	{
		AddClassPredefinedLightProperty(bOverride_PointLightIntensity, IntensityName, PointLightIntensityName, UPointLightComponent::StaticClass());
		AddClassPredefinedLightProperty(bOverride_PointLightIntensityUnits, IntensityUnitsName, PointLightIntensityUnitsName, UPointLightComponent::StaticClass());

		AddClassPredefinedLightProperty(bOverride_RectLightIntensity, IntensityName, RectLightIntensityName, URectLightComponent::StaticClass());
		AddClassPredefinedLightProperty(bOverride_RectLightIntensityUnits, IntensityUnitsName, RectLightIntensityUnitsName, URectLightComponent::StaticClass());

		AddClassPredefinedLightProperty(bOverride_SpotLightIntensity, IntensityName, SpotLightIntensityName, USpotLightComponent::StaticClass());
		AddClassPredefinedLightProperty(bOverride_SpotLightIntensityUnits, IntensityUnitsName, SpotLightIntensityUnitsName, USpotLightComponent::StaticClass());
	}

	const UMovieGraphMutableValueView* CustomLightingValues = GetDynamicPropertiesView();

	// Second, add any properties that have been added in the "Custom" section.
	for (const FName& PropertyName : CustomLightingValues->GetPropertyNames())
	{
		const FString PropertyNameString = PropertyName.ToString();

		// Only examine bOverride_ properties. If the property isn't overridden, it won't be passed to the modifier.
		if (!PropertyNameString.StartsWith(TEXT("bOverride_")))
		{
			continue;
		}

		// Check that it's overridden.
		bool bIsOverridden = false;
		if (!CustomLightingValues->GetValueBool(PropertyName, bIsOverridden) || !bIsOverridden)
		{
			continue;
		}

		FString NonOverridePropertyName = PropertyNameString;
		NonOverridePropertyName.RemoveFromStart(TEXT("bOverride_"), ESearchCase::CaseSensitive);

		// Add the Custom property to the modifier if its supplemental property info was found.
		if (const FMovieGraphPropertyReference* CustomPropertyRef = CustomPropertyInfo.Find(FName(NonOverridePropertyName)))
		{
			PropertiesToAdd.Add(*CustomPropertyRef);

			// Queue up setting the value in the property bag *after* the property was added to the modifier property bag.
			FString PropertyValue = CustomLightingValues->GetValueSerializedString(FName(NonOverridePropertyName));
			DeferredValueSetters.Add([this, &ModifierValues, CustomPropertyRef, PropertyValue]()
			{
				ModifierValues->SetValueSerializedString(Modifier->GetPropertyNameForSettingCustomValue(*CustomPropertyRef), PropertyValue);
			});
		}
	}

	// Add all the new properties in bulk, then set their values.
	Modifier->AddProperties(PropertiesToAdd);
	for (const TFunction<void()>& ValueSetter : DeferredValueSetters)
	{
		ValueSetter();
	}

	return { Modifier };
}

bool UMovieGraphLightModifierNode::SupportsCollections() const
{
	return true;
}

TArray<FName> UMovieGraphLightModifierNode::GetAllCollections() const
{
	return Collections;
}

void UMovieGraphLightModifierNode::AddCollection(const FName& InCollectionName)
{
#if WITH_EDITOR
	Modify();
#endif

	if (InCollectionName == NAME_None)
	{
		return;
	}

	Collections.AddUnique(InCollectionName);
}

bool UMovieGraphLightModifierNode::RemoveCollection(const FName& InCollectionName)
{
#if WITH_EDITOR
	Modify();
#endif

	return Collections.Remove(InCollectionName) > 0;
}

FString UMovieGraphLightModifierNode::GetNodeInstanceName() const
{
	return ModifierName;
}

void UMovieGraphLightModifierNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesLightModifier = true;
}

bool UMovieGraphLightModifierNode::AddCustomLightProperty(UClass* InLightComponentClass, const FProperty* InLightProperty)
{
	if (!InLightComponentClass || !InLightProperty)
	{
		return false;
	}

	const FName PropertyName = GetLightingPropertyName(InLightComponentClass, InLightProperty);
	const UMovieGraphMutableValueView* LightingValues = GetDynamicPropertiesView();

	// Skip adding if the property already exists in the property bag.
	if (LightingValues->HasProperty(PropertyName))
	{
		return false;
	}

#if WITH_EDITOR
	Modify();
#endif

	// Add supplemental property information for this property, which will importantly track the origin component.
	// This has to be done via sidecar data, rather than via metadata on the property in the property bag, because runtime builds do not contain
	// property metadata. Note that the component class is tracked here instead of the actor class, since we want to potentially modify light
	// components outside of the typical light actor classes.
	constexpr UClass* LightClass = nullptr;
	FMovieGraphPropertyReference& NewCustomProperty = CustomPropertyInfo.Emplace(
		PropertyName,
		FMovieGraphPropertyReference(LightClass, InLightComponentClass, InLightProperty->GetFName(), const_cast<FProperty*>(InLightProperty)));
	NewCustomProperty.AuxPropertyName = PropertyName;

	// Update DynamicProperties with the properties referenced in CustomPropertyInfo. Although we could update DynamicProperties right here, there are
	// parts of the graph evaluation lifecycle that depend on GetDynamicPropertyDescriptions() reflecting the property layout of DynamicProperties.
	UpdateDynamicProperties();

	return true;
}

bool UMovieGraphLightModifierNode::AddCustomLightProperty(UClass* InLightComponentClass, const FName& InPropertyName)
{
	return AddCustomLightProperty(InLightComponentClass, FindFProperty<FProperty>(InLightComponentClass, InPropertyName));
}

bool UMovieGraphLightModifierNode::RemoveCustomLightProperty(const UClass* InLightComponentClass, const FProperty* InLightProperty)
{
	if (InLightComponentClass && InLightProperty)
	{
#if WITH_EDITOR
		Modify();
#endif

		const FName LightingPropertyName = GetLightingPropertyName(InLightComponentClass, InLightProperty);

		const bool bSuccess = CustomPropertyInfo.Remove(LightingPropertyName) > 0;

		// Trigger a refresh of DynamicProperties
		UpdateDynamicProperties();

		return bSuccess;
	}

	return false;
}

bool UMovieGraphLightModifierNode::RemoveCustomLightProperty(const UClass* InLightComponentClass, const FName& InPropertyName)
{
	return RemoveCustomLightProperty(InLightComponentClass, FindFProperty<FProperty>(InLightComponentClass, InPropertyName));
}

bool UMovieGraphLightModifierNode::HasCustomLightProperty(const UClass* InLightComponentClass, const FProperty* InLightProperty) const
{
	if (!InLightComponentClass || !InLightProperty)
	{
		return false;
	}

	const UMovieGraphMutableValueView* LightingValues = GetDynamicPropertiesView();
	
	return LightingValues->HasProperty(GetLightingPropertyName(InLightComponentClass, InLightProperty));
}

bool UMovieGraphLightModifierNode::HasCustomLightProperty(const UClass* InLightComponentClass, const FName& InPropertyName) const
{
	return HasCustomLightProperty(InLightComponentClass, FindFProperty<FProperty>(InLightComponentClass, InPropertyName));
}

bool UMovieGraphLightModifierNode::IsCustomLightPropertyOverridden(const UClass* InLightComponentClass, const FName& InPropertyName) const
{
	const UMovieGraphMutableValueView* LightingValues = GetDynamicPropertiesView();
	const FName MainPropertyName = GetLightingPropertyName(InLightComponentClass, FindFProperty<FProperty>(InLightComponentClass, InPropertyName));

	bool bIsOverridden = false;
	LightingValues->GetValueBool(GetLightingOverridePropertyName(MainPropertyName), bIsOverridden);

	return bIsOverridden;
}

bool UMovieGraphLightModifierNode::UpdateCustomLightPropertyOverrideState(const UClass* InLightComponentClass, const FName& InPropertyName, const bool bIsOverridden)
{
	UMovieGraphMutableValueView* LightingValues = GetDynamicPropertiesView();
	const FName MainPropertyName = GetLightingPropertyName(InLightComponentClass, FindFProperty<FProperty>(InLightComponentClass, InPropertyName));

	return LightingValues->SetValueBool(GetLightingOverridePropertyName(MainPropertyName), bIsOverridden);
}

FName UMovieGraphLightModifierNode::GetLightingPropertyName(const UClass* InLightComponentClass, const FProperty* InLightProperty)
{
	static const TCHAR* LightingPropertyNameFormat = TEXT("{0}_{1}");

	if (InLightComponentClass && InLightProperty)
	{
		return FName(FString::Format(LightingPropertyNameFormat, { InLightComponentClass->GetName(), InLightProperty->GetName() }));
	}

	return FName();
}

FName UMovieGraphLightModifierNode::GetLightingOverridePropertyName(const FName& InMainPropertyName)
{
	static const TCHAR* LightingPropertyOverrideNameFormat = TEXT("bOverride_{0}");

	return FName(FString::Format(LightingPropertyOverrideNameFormat, { InMainPropertyName.ToString() }));
}

int32 UMovieGraphLightModifierNode::GetNumCustomLightProperties() const
{
	const UMovieGraphMutableValueView* LightingValues = GetDynamicPropertiesView();

	// The divide by two is used to account for the bOverride_* properties; those should not be counted.
	return LightingValues ? (LightingValues->GetNumProperties() / 2) : 0;
}

#undef LOCTEXT_NAMESPACE // "MovieGraph"