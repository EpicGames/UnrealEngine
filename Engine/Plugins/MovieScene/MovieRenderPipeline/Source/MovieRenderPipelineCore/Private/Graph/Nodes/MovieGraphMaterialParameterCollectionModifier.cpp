// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphMaterialParameterCollectionModifier.h"

#include "Engine/World.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphMaterialParameterCollectionModifier)

void UMovieGraphMaterialParameterCollectionModifier::ApplyModifier(const UWorld* World)
{
	CachedWorld = World;
	
	constexpr bool bCacheCurrentValues = true;
	ApplyScalarAndVectorValues(ScalarParameterUpdates, VectorParameterUpdates, bCacheCurrentValues);
}

void UMovieGraphMaterialParameterCollectionModifier::UndoModifier()
{
	ApplyScalarAndVectorValues(PriorScalarValues, PriorVectorValues);
	
	PriorScalarValues.Reset();
	PriorVectorValues.Reset();
}

FText UMovieGraphMaterialParameterCollectionModifier::GetModifierName()
{
	static const FText ModifierName = NSLOCTEXT("MovieGraphNode", "MaterialParameterCollectionModifierName", "Material Parameter Collection");
	return ModifierName;
}

void UMovieGraphMaterialParameterCollectionModifier::ClearParameterValues()
{
	ScalarParameterUpdates.Reset();
	VectorParameterUpdates.Reset();
}

void UMovieGraphMaterialParameterCollectionModifier::ApplyScalarAndVectorValues(const TMap<FName, float>& InScalarValues, const TMap<FName, FLinearColor>& InVectorValues, const bool bCacheCurrentValues)
{
	if (!CachedWorld.IsValid())
	{
		return;
	}
	
	const UMaterialParameterCollection* MPC = MaterialParameterCollection.LoadSynchronous();
	if (!MPC)
	{
		return;
	}
	
	UMaterialParameterCollectionInstance* MpcInstance = CachedWorld->GetParameterCollectionInstance(MPC);
	if (!MpcInstance)
	{
		return;
	}

	// Cache + apply scalar parameters
	for (const TPair<FName, float>& ScalarPair : InScalarValues)
	{
		if (bCacheCurrentValues)
		{
			float ScalarParamValue = 0.f;
			if (MpcInstance->GetScalarParameterValue(ScalarPair.Key, ScalarParamValue))
			{
				PriorScalarValues.Add(ScalarPair.Key, ScalarParamValue);
			}
		}

		MpcInstance->SetScalarParameterValue(ScalarPair.Key, ScalarPair.Value);
	}

	// Cache + apply vector parameters
	for (const TPair<FName, FLinearColor>& VectorPair : InVectorValues)
	{
		if (bCacheCurrentValues)
		{
			FLinearColor VectorParamValue;
			if (MpcInstance->GetVectorParameterValue(VectorPair.Key, VectorParamValue))
			{
				PriorVectorValues.Add(VectorPair.Key, VectorParamValue);
			}
		}

		MpcInstance->SetVectorParameterValue(VectorPair.Key, VectorPair.Value);
	}
}
