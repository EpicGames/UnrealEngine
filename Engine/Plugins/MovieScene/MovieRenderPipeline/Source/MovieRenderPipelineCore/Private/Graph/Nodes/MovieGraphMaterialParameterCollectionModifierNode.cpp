// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphMaterialParameterCollectionModifierNode.h"

#include "Graph/Nodes/MovieGraphMaterialParameterCollectionModifier.h"
#include "Materials/MaterialParameterCollection.h"
#include "MoviePipelineTelemetry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphMaterialParameterCollectionModifierNode)

#define LOCTEXT_NAMESPACE "MovieGraph"

UMovieGraphMaterialParameterCollectionModifierNode::UMovieGraphMaterialParameterCollectionModifierNode()
{
	Modifier = CreateDefaultSubobject<UMovieGraphMaterialParameterCollectionModifier>(TEXT("MPC_Modifier"));
}

#if WITH_EDITOR
FText UMovieGraphMaterialParameterCollectionModifierNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText NodeName = LOCTEXT("NodeName_MPCModifier", "Material Parameter Collection Modifier");
	static const FText NodeDescription = LOCTEXT("NodeDescription_MPCModifier", "Material Parameter Collection Modifier\n{0}");

	const UMaterialParameterCollection* MPC = MaterialParameterCollection.LoadSynchronous(); 

	const FString AssetName = IsValid(MPC) ? MPC->GetName() : FString();
	const FString MpcDisplayName = !AssetName.IsEmpty() && bOverride_MaterialParameterCollection
		? AssetName
		: LOCTEXT("NodeNoNameWarning_MPCModifier", "No Material Parameter Collection Asset Chosen").ToString();
	
	if (bGetDescriptive)
	{
		return FText::Format(NodeDescription, FText::FromString(MpcDisplayName));
	}

	return NodeName;
}

FText UMovieGraphMaterialParameterCollectionModifierNode::GetMenuCategory() const
{
	return LOCTEXT("CollectionNode_Category", "Utility");
}

FLinearColor UMovieGraphMaterialParameterCollectionModifierNode::GetNodeTitleColor() const
{
	static const FLinearColor ModifierNodeColor = FLinearColor(0.6f, 0.113f, 0.113f);
	return ModifierNodeColor;
}

FSlateIcon UMovieGraphMaterialParameterCollectionModifierNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon ModifierIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit");

	OutColor = FLinearColor::White;
	return ModifierIcon;
}

void UMovieGraphMaterialParameterCollectionModifierNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Update the node and its dynamic properties if the MPC asset changes.
		if ((PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphMaterialParameterCollectionModifierNode, MaterialParameterCollection)) ||
			(PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphMaterialParameterCollectionModifierNode, bOverride_MaterialParameterCollection)))
		{
			UpdateDynamicProperties();
			OnNodeChangedDelegate.Broadcast(this);
		}
	}
}

void UMovieGraphMaterialParameterCollectionModifierNode::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Update the node's dynamic properties if any properties within the MPC asset changes (its scalar/vector parameters).
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UMovieGraphMaterialParameterCollectionModifierNode::OnObjectPropertyChanged);
		UpdateDynamicProperties();
	}
}
#endif // WITH_EDITOR

TArray<FPropertyBagPropertyDesc> UMovieGraphMaterialParameterCollectionModifierNode::GetDynamicPropertyDescriptions() const
{
	TArray<FPropertyBagPropertyDesc> ModifierDescs;
	
	UMaterialParameterCollection* MPC = MaterialParameterCollection.LoadSynchronous();
	if (!IsValid(MPC))
	{
		return ModifierDescs;
	}

	// Creates a new property bag property description for a specific MPC parameter.
	auto CreatePropertyDescFromMpcParameter = [&ModifierDescs](
		UMaterialParameterCollection* InMPC, const FCollectionParameterBase& InParameter,
		const EPropertyBagPropertyType ParamType, const UObject* ParamTypeObject = nullptr)
	{
		// The parameter name may contain spaces or other invalid characters for property names, so we have to sanitize it. Unfortunately this leaves
		// open a potential edge case where the MPC could contain two parameters (eg, "Param With Spaces" and "Param_With_Spaces") that would resolve
		// to the same sanitized name.
		const FName ParamName = FInstancedPropertyBag::SanitizePropertyName(InParameter.ParameterName, TEXT('_'));
		const FString OverrideName = FString::Printf(TEXT("bOverride_%s"), *ParamName.ToString());

		FPropertyBagPropertyDesc NewDesc(ParamName, ParamType, ParamTypeObject);
		NewDesc.ID = InParameter.Id;
#if WITH_EDITOR
		NewDesc.MetaData.Add(FPropertyBagPropertyDescMetaData(CategoryMetadataKey, InMPC->GetName()));
		NewDesc.MetaData.Add(FPropertyBagPropertyDescMetaData(EditConditionMetadataKey, OverrideName));
		NewDesc.MetaData.Add(FPropertyBagPropertyDescMetaData(DisplayNameMetadataKey, InParameter.ParameterName.ToString()));
#endif	// WITH_EDITOR
		ModifierDescs.Add(MoveTemp(NewDesc));

		// Need a bOverride_* property in order for this to be picked up properly by graph evaluation. Note that the ID of the the override property
		// is based off of the ID of the parameter. The overrides don't exist in the MPC asset, so we have to come up with new IDs for the overrides.
		// The easiest way to do this is to just use the string of the parameter's ID to create a new deterministic FGuid for the override.
		FPropertyBagPropertyDesc OverrideDesc(FName(OverrideName), EPropertyBagPropertyType::Bool);
		OverrideDesc.ID = FGuid::NewDeterministicGuid(NewDesc.ID.ToString());
#if WITH_EDITOR
		OverrideDesc.MetaData.Add(FPropertyBagPropertyDescMetaData(InlineEditConditionToggleMetadataKey, FString(TEXT("true"))));
#endif	// WITH_EDITOR
		ModifierDescs.Add(MoveTemp(OverrideDesc));
	};

	// Create property descs for each MPC in the MPC "chain"
	UMaterialParameterCollection* CurrentMPC = MPC;
	while (CurrentMPC)
	{
		for (const FCollectionScalarParameter& MPCScalarParameter : CurrentMPC->ScalarParameters)
		{
			CreatePropertyDescFromMpcParameter(CurrentMPC, MPCScalarParameter, EPropertyBagPropertyType::Float);
		}

		for (const FCollectionVectorParameter& MPCVectorParameter : CurrentMPC->VectorParameters)
		{
			CreatePropertyDescFromMpcParameter(CurrentMPC, MPCVectorParameter, EPropertyBagPropertyType::Struct, TBaseStructure<FLinearColor>::Get());
		}

		CurrentMPC = CurrentMPC->GetBaseParameterCollection();
	}

	return ModifierDescs;
}

bool UMovieGraphMaterialParameterCollectionModifierNode::GetDynamicPropertyValue(const FName PropertyName, FString& OutValue)
{
	TValueOrError<FString, EPropertyBagResult> Result = DynamicProperties.GetValueSerializedString(PropertyName);
	if (Result.HasValue())
	{
		OutValue = Result.GetValue();
		return true;
	}

	return false;
}

TArray<FMovieGraphPropertyInfo> UMovieGraphMaterialParameterCollectionModifierNode::GetOverrideablePropertyInfo() const
{
	TArray<FMovieGraphPropertyInfo> PropertyInfo = Super::GetOverrideablePropertyInfo();
	
	// Don't allow the Material Parameter Collection asset property to be overridden. The individual MPC parameters can, but not the asset itself.
	PropertyInfo.RemoveAll([](const FMovieGraphPropertyInfo& InPropertyInfo)
	{
		return InPropertyInfo.Name == GET_MEMBER_NAME_CHECKED(UMovieGraphMaterialParameterCollectionModifierNode, MaterialParameterCollection);
	});

	return PropertyInfo;
}

TArray<UMovieGraphModifierBase*> UMovieGraphMaterialParameterCollectionModifierNode::GetAllModifiers() const
{
	// Update the modifier with the scalar/vector parameters set on the node.

	Modifier->ClearParameterValues();
	Modifier->MaterialParameterCollection = MaterialParameterCollection;

	if (!MaterialParameterCollection.LoadSynchronous())
	{
		return {};
	}

	// First, figure out which parameters have actually been overridden on the node. Only these parameter values should be set on the modifier.
	TSet<FName> OverriddenParameterNames;
	for (const FPropertyBagPropertyDesc& Desc : GetDynamicPropertyDescriptions())
	{
		// Determine if this is a bOverride_* property. The only properties in the bag that are bools are override properties.
		const bool bIsOverrideProperty = (Desc.ValueType == EPropertyBagPropertyType::Bool);

		// Regular property, not a bOverride_*
		if (!bIsOverrideProperty)
		{
			continue;
		}

		const TValueOrError<bool, EPropertyBagResult> OverrideResult = DynamicProperties.GetValueBool(Desc.Name);
		if (OverrideResult.HasValue() && (OverrideResult.GetValue() == true))
		{
			// Generate the parameter name from the override's property name. Unfortunately we cannot rely on metadata here to link the parameter's
			// property and the override's property (since metadata is not available in non-editor builds). The next-best way to link them is via
			// name.
			const FName ParameterName = FName(Desc.Name.ToString().Replace(TEXT("bOverride_"), TEXT(""), ESearchCase::CaseSensitive));
			
			OverriddenParameterNames.Add(ParameterName);
		}
	}

	// Second, for all overridden parameters, transfer over their values to the modifier. 
	for (const FPropertyBagPropertyDesc& Desc : GetDynamicPropertyDescriptions())
	{
		const EPropertyBagPropertyType ParamType = Desc.ValueType;
		const FName ParamName = Desc.Name;

		if (!OverriddenParameterNames.Contains(ParamName))
		{
			continue;
		}

		// Note: When adding a new entry to ScalarParameterUpdates/VectorParameterUpdates, note that GetParameterName() is used *rather than* the
		// Desc's name. The parameter name may contain spaces or other property-name-incompatible characters, so we need to use the desc's ID (which
		// is the param's ID) to fetch its param name properly. Unfortunately MPC parameters can only be set by name, not ID.
		const FName ActualParamName = MaterialParameterCollection->GetParameterName(Desc.ID);
		if (ActualParamName.IsNone())
		{
			continue;
		}
		
		if (ParamType == EPropertyBagPropertyType::Float)
		{
			const TValueOrError<float, EPropertyBagResult> FloatResult = DynamicProperties.GetValueFloat(ParamName);
			if (FloatResult.HasValue())
			{
				Modifier->ScalarParameterUpdates.Add(ActualParamName, FloatResult.GetValue());
			}
		}
		else if (ParamType == EPropertyBagPropertyType::Struct)
		{
			const TValueOrError<FLinearColor*, EPropertyBagResult> StructResult = DynamicProperties.GetValueStruct<FLinearColor>(ParamName);
			if (StructResult.HasValue())
			{
				Modifier->VectorParameterUpdates.Add(ActualParamName, *StructResult.GetValue());
			}
		}
		else
		{
			// Invalid parameter type	
			check(false);
		}
	}

	return { Modifier.Get() };
}

bool UMovieGraphMaterialParameterCollectionModifierNode::SupportsCollections() const
{
	return false;
}

FString UMovieGraphMaterialParameterCollectionModifierNode::GetNodeInstanceName() const
{
	if (!MaterialParameterCollection.IsNull())
	{
		return MaterialParameterCollection.GetAssetName();
	}

	return FString();
}

void UMovieGraphMaterialParameterCollectionModifierNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesMpcModifier = true;
}

void UMovieGraphMaterialParameterCollectionModifierNode::PrepareForFlattening(const UMovieGraphSettingNode* InSourceNode)
{
	Super::PrepareForFlattening(InSourceNode);

	// This node's dynamic properties rely on the MPC being valid and correct in order to be properly initialized.
	if (const UMovieGraphMaterialParameterCollectionModifierNode* SourcePresetNode = Cast<UMovieGraphMaterialParameterCollectionModifierNode>(InSourceNode))
	{
		MaterialParameterCollection = SourcePresetNode->MaterialParameterCollection;
	}
}

void UMovieGraphMaterialParameterCollectionModifierNode::OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	// Since this method can be called at high frequency, early-out if possible
	if ((PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
		|| !ObjectBeingModified
		|| (ObjectBeingModified->GetClass() != UMaterialParameterCollection::StaticClass()))
	{
		return;
	}
	
	// Gather all of the MPCs in the MPC "chain"
	UMaterialParameterCollection* CurrentMPC = MaterialParameterCollection.LoadSynchronous();
	TArray<UObject*> RelevantMPCs;
	while (CurrentMPC)
	{
		RelevantMPCs.Add(CurrentMPC);
		CurrentMPC = CurrentMPC->GetBaseParameterCollection();
	}
	
	// If a scalar/vector parameter in the MPC changes, we need to update our property bag to reflect the changes
	if (RelevantMPCs.Contains(ObjectBeingModified))
	{
		UpdateDynamicProperties();
	}
}

#undef LOCTEXT_NAMESPACE // "MovieGraph"
