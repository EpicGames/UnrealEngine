// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphModifierNode.h"

#include "Graph/MovieGraphConfig.h"
#include "MoviePipelineTelemetry.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphModifierNode)

#define LOCTEXT_NAMESPACE "MovieGraph"

void UMovieGraphMergeableModifierContainer::Merge(const IMovieGraphTraversableObject* InSourceObject)
{
	const UMovieGraphMergeableModifierContainer* InSourceModifiers = Cast<UMovieGraphMergeableModifierContainer>(InSourceObject);
	checkf(InSourceModifiers, TEXT("UMovieGraphMergeableModifierContainer cannot merge with null or an object of another type."));

	for (TObjectPtr<UMovieGraphCollectionModifier>& ModifierInstance : Modifiers)
	{
		// Find the equivalent modifier in the source modifier container
		for (const TObjectPtr<UMovieGraphCollectionModifier>& SourceModifier : InSourceModifiers->Modifiers)
		{
			if (ModifierInstance && SourceModifier && (ModifierInstance->GetClass() == SourceModifier->GetClass()))
			{
				MergeProperties(ModifierInstance, SourceModifier);
				break;
			}
		}
	}
}

TArray<TPair<FString, FString>> UMovieGraphMergeableModifierContainer::GetMergedProperties() const
{
	TArray<TPair<FString, FString>> MergedProperties;

	// Include properties from modifiers if they have a bOverride_* counterpart
	for (const TObjectPtr<UMovieGraphCollectionModifier>& ModifierInstance : Modifiers)
	{
		for (TFieldIterator<FProperty> PropertyIterator(ModifierInstance->GetClass()); PropertyIterator; ++PropertyIterator)
		{
			const FProperty* ModifierProperty = *PropertyIterator;

			// Exclude the bOverride_* properties
			if (!UMovieGraphConfig::FindOverridePropertyForRealProperty(ModifierInstance->GetClass(), ModifierProperty))
			{
				continue;
			}
			
			// Prefix the property name with the modifier name to distinguish it from properties on other modifiers
#if WITH_EDITOR
			const FString ModifierClassName = ModifierInstance->GetClass()->GetDisplayNameText().ToString();
#else
			const FString ModifierClassName = ModifierInstance->GetClass()->GetName();
#endif
			const FString PropertyName = FString::Printf(TEXT("%s / %s"), *ModifierClassName, *ModifierProperty->GetName());
			
			TPair<FString, FString>& NewMergedProperty = MergedProperties.Add_GetRef(TPair<FString, FString>(PropertyName, FString()));

			// Use a string value of the property
			ModifierProperty->ExportTextItem_InContainer(NewMergedProperty.Value, ModifierInstance, nullptr, nullptr, PPF_None);
		}
	}
	
	return MergedProperties;
}

void UMovieGraphMergeableModifierContainer::MergeProperties(const TObjectPtr<UMovieGraphCollectionModifier>& InDestModifier, const TObjectPtr<UMovieGraphCollectionModifier>& InSourceModifier)
{
	for (TFieldIterator<FProperty> PropertyIterator(InDestModifier->GetClass()); PropertyIterator; ++PropertyIterator)
	{
		const FProperty* ModifierProperty = *PropertyIterator;
		const FBoolProperty* OverrideProp = UMovieGraphConfig::FindOverridePropertyForRealProperty(InSourceModifier->GetClass(), ModifierProperty);

		// Only consider properties which have a bOverride_* counterpart
		if (!OverrideProp)
		{
			continue;
		}

		// The graph is evaluated in reverse (Outputs to Inputs) so if this property has already been overridden on the dest, ignore the value.
		if (OverrideProp->GetPropertyValue_InContainer(InDestModifier))
		{
			continue;
		}

		// Set the value if the source has an override
		if (OverrideProp->GetPropertyValue_InContainer(InSourceModifier))
		{
			ModifierProperty->CopyCompleteValue_InContainer(InDestModifier, InSourceModifier);

			// Set the override indicator on the destination
			OverrideProp->SetPropertyValue_InContainer(InDestModifier, true);
		}
	}
}

UMovieGraphModifierNode::UMovieGraphModifierNode()
{
	ModifiersContainer = CreateDefaultSubobject<UMovieGraphMergeableModifierContainer>(TEXT("Modifiers"));
	
	// Add some default modifiers
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AddModifier(UMovieGraphRenderPropertyModifier::StaticClass());
		AddModifier(UMovieGraphMaterialModifier::StaticClass());
	}
}

#if WITH_EDITOR
FText UMovieGraphModifierNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText ModifierNodeName = LOCTEXT("NodeName_Modifier", "Modifier");
	static const FText ModifierNodeDescription = LOCTEXT("NodeDescription_Modifier", "Modifier\n{0}");
	
	const FString ModifierNameDisp = ModifierName.IsEmpty() ? LOCTEXT("NodeNoNameWarning_Modifier", "NO NAME").ToString() : ModifierName;

	if (bGetDescriptive)
	{
		return FText::Format(ModifierNodeDescription, FText::FromString(ModifierNameDisp));
	}

	return ModifierNodeName;
}

FText UMovieGraphModifierNode::GetMenuCategory() const
{
	return LOCTEXT("CollectionNode_Category", "Utility");
}

FLinearColor UMovieGraphModifierNode::GetNodeTitleColor() const
{
	static const FLinearColor ModifierNodeColor = FLinearColor(0.6f, 0.113f, 0.113f);
	return ModifierNodeColor;
}

FSlateIcon UMovieGraphModifierNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon ModifierIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit");

	OutColor = FLinearColor::White;
	return ModifierIcon;
}

void UMovieGraphModifierNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Broadcast a node-changed delegate so that the node title's UI gets updated.
	if ((PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphModifierNode, ModifierName)) ||
		(PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphModifierNode, bOverride_ModifierName)))
	{
		OnNodeChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR

FString UMovieGraphModifierNode::GetNodeInstanceName() const
{
	return ModifierName;
}

void UMovieGraphModifierNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesModifier = true;
}

UMovieGraphCollectionModifier* UMovieGraphModifierNode::GetModifier(TSubclassOf<UMovieGraphCollectionModifier> ModifierType) const
{
	const TObjectPtr<UMovieGraphCollectionModifier>* FoundModifier =
		ModifiersContainer->Modifiers.FindByPredicate([&ModifierType](const TObjectPtr<UMovieGraphCollectionModifier>& Modifier)
        {
           	return Modifier && (Modifier->GetClass() == ModifierType);
        });

	return (FoundModifier == nullptr) ? nullptr : *FoundModifier;
}

const TArray<UMovieGraphCollectionModifier*>& UMovieGraphModifierNode::GetModifiers() const
{
	return ModifiersContainer->Modifiers;
}

TArray<UMovieGraphModifierBase*> UMovieGraphModifierNode::GetAllModifiers() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<UMovieGraphCollectionModifier*>& Modifiers = GetModifiers();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	TArray<UMovieGraphModifierBase*> BaseModifiers;
	for (UMovieGraphCollectionModifier* CollectionModifier : Modifiers)
	{
		BaseModifiers.Add(Cast<UMovieGraphModifierBase>(CollectionModifier));
	}
	
	return BaseModifiers;
}

UMovieGraphCollectionModifier* UMovieGraphModifierNode::AddModifier(TSubclassOf<UMovieGraphCollectionModifier> ModifierType)
{
	const TObjectPtr<UMovieGraphCollectionModifier>* FoundModifier =
		ModifiersContainer->Modifiers.FindByPredicate([&ModifierType](const TObjectPtr<UMovieGraphCollectionModifier>& Modifier)
		{
			return Modifier && (Modifier->GetClass() == ModifierType);
		});

	// Don't add duplicate modifiers
	if (FoundModifier != nullptr)
	{
		return nullptr;
	}

#if WITH_EDITOR
	Modify();
#endif

	if (const UClass* ModifierClass = ModifierType.Get())
	{
		UMovieGraphCollectionModifier* NewModifier = NewObject<UMovieGraphCollectionModifier>(this, ModifierType, ModifierClass->GetFName(), RF_Transactional);
		ModifiersContainer->Modifiers.Add(NewModifier);

		return NewModifier;
	}

	return nullptr;
}

bool UMovieGraphModifierNode::RemoveModifier(TSubclassOf<UMovieGraphCollectionModifier> ModifierType)
{
#if WITH_EDITOR
	Modify();
#endif
	
	const int32 NumRemoved = ModifiersContainer->Modifiers.RemoveAll([&ModifierType](const TObjectPtr<UMovieGraphCollectionModifier>& Modifier)
	{
		return Modifier && (Modifier->GetClass() == ModifierType);
	});

	return NumRemoved > 0;
}

void UMovieGraphModifierNode::AddCollection(const FName& InCollectionName)
{
#if WITH_EDITOR
	Modify();
#endif
	
	if (InCollectionName == NAME_None)
	{
		return;
	}

	Collections.AddUnique(InCollectionName);
}

bool UMovieGraphModifierNode::RemoveCollection(const FName& InCollectionName)
{
#if WITH_EDITOR
	Modify();
#endif

	// Remove from disabled collections (in case this collection was disabled when it was deleted)
	DisabledCollections.Remove(InCollectionName);
	
	return Collections.Remove(InCollectionName) > 0;
}

const TArray<FName>& UMovieGraphModifierNode::GetCollections() const
{
	return Collections;
}

TArray<FName> UMovieGraphModifierNode::GetAllCollections() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetCollections();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UMovieGraphModifierNode::SupportsCollections() const
{
	return true;
}

void UMovieGraphModifierNode::SetCollectionEnabled(const FName& InCollectionName, const bool bIsCollectionEnabled)
{
#if WITH_EDITOR
	Modify();
#endif
	
	if (bIsCollectionEnabled)
	{
		DisabledCollections.Remove(InCollectionName);
	}
	else
	{
		DisabledCollections.Add(InCollectionName);
	}
}

bool UMovieGraphModifierNode::IsCollectionEnabled(const FName& InCollectionName) const
{
	return !DisabledCollections.Contains(InCollectionName);
}

#undef LOCTEXT_NAMESPACE // "MovieGraph"
