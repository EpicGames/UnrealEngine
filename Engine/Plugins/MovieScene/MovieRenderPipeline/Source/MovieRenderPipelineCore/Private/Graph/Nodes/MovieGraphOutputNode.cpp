// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphOutputNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphOutputNode)

UMovieGraphOutputNode::UMovieGraphOutputNode()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		RegisterDelegates();
	}
}

TArray<FMovieGraphPinProperties> UMovieGraphOutputNode::GetInputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;

	if (const UMovieGraphConfig* ParentGraph = GetGraph())
	{
		for (const UMovieGraphOutput* Output : ParentGraph->GetOutputs())
		{
			FMovieGraphPinProperties PinProperties(FName(Output->GetMemberName()), Output->GetValueType(), Output->GetValueTypeObject(), false);
			PinProperties.bIsBranch = Output->bIsBranch;
			Properties.Add(MoveTemp(PinProperties));
		}
	}
	
	return Properties;
}

bool UMovieGraphOutputNode::CanBeDisabled() const
{
	return false;
}

#if WITH_EDITOR
FText UMovieGraphOutputNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("MovieGraphNodes", "OutputNode_Description", "Output");
}

FText UMovieGraphOutputNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "OutputNode_Category", "Input/Output");
}

FLinearColor UMovieGraphOutputNode::GetNodeTitleColor() const
{
	return FLinearColor::Black;
}

FSlateIcon UMovieGraphOutputNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon OutputIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeBufferMode");

	OutColor = FLinearColor::White;
	return OutputIcon;
}
#endif // WITH_EDITOR

void UMovieGraphOutputNode::RegisterDelegates()
{
	Super::RegisterDelegates();
	
	if (UMovieGraphConfig* Graph = GetGraph())
	{
#if WITH_EDITOR
		// Register delegates for new outputs when they're added to the graph
		Graph->OnGraphOutputAddedDelegate.RemoveAll(this);
		Graph->OnGraphOutputAddedDelegate.AddUObject(this, &UMovieGraphOutputNode::RegisterOutputDelegates);
#endif

		// Register delegates for existing outputs
		for (UMovieGraphOutput* OutputMember : Graph->GetOutputs())
		{
			RegisterOutputDelegates(OutputMember);
		}
	}
}

void UMovieGraphOutputNode::RegisterOutputDelegates(UMovieGraphOutput* Output)
{
#if WITH_EDITOR
	if (Output)
	{
		Output->OnMovieGraphOutputChangedDelegate.RemoveAll(this);
		Output->OnMovieGraphOutputChangedDelegate.AddUObject(this, &UMovieGraphOutputNode::UpdateExistingPins);
	}
#endif
}

void UMovieGraphOutputNode::UpdateExistingPins(UMovieGraphMember* ChangedOutput) const
{
	if (const UMovieGraphConfig* Graph = GetGraph())
	{
		const TArray<UMovieGraphOutput*> OutputMembers = Graph->GetOutputs();
		if (OutputMembers.Num() == InputPins.Num())
		{
			for (int32 Index = 0; Index < OutputMembers.Num(); ++Index)
			{
				InputPins[Index]->Properties.Label = FName(OutputMembers[Index]->GetMemberName());
				InputPins[Index]->Properties.Type = OutputMembers[Index]->GetValueType();
				InputPins[Index]->Properties.TypeObject = OutputMembers[Index]->GetValueTypeObject();
				InputPins[Index]->Properties.bIsBranch = OutputMembers[Index]->bIsBranch;
			}
		}

		OnNodeChangedDelegate.Broadcast(this);
	}
}
