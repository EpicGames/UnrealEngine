// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphRemoveRenderSettingNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphRemoveRenderSettingNode)

TArray<FMovieGraphPinProperties> UMovieGraphRemoveRenderSettingNode::GetInputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;
	Properties.Add(FMovieGraphPinProperties::MakeBranchProperties());
	return Properties;
}

TArray<FMovieGraphPinProperties> UMovieGraphRemoveRenderSettingNode::GetOutputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;
	Properties.Add(FMovieGraphPinProperties::MakeBranchProperties());
	return Properties;
}

#if WITH_EDITOR
FText UMovieGraphRemoveRenderSettingNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText RemoveNode = NSLOCTEXT("MovieGraphNodes", "NodeName_RemoveNode", "Remove Render Setting");
	static const FText RemoveNodeDescription = NSLOCTEXT("MovieGraphNodes", "NodeDescription_RemoveNode", "Remove Render Setting\nType: {0}");

	if (bGetDescriptive && (NodeType.Get() != nullptr))
	{
		const UMovieGraphSettingNode* SettingNodeCDO = Cast<UMovieGraphSettingNode>(NodeType->GetDefaultObject());
		return FText::Format(RemoveNodeDescription, SettingNodeCDO->GetNodeTitle());
	}

	return RemoveNode;
}

FText UMovieGraphRemoveRenderSettingNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "RemoveNode_Category", "Utility");
}

FLinearColor UMovieGraphRemoveRenderSettingNode::GetNodeTitleColor() const
{
	static const FLinearColor RemoveNodeColor = FLinearColor(0.f, 0.f, 0.f);
	return RemoveNodeColor;
}

FSlateIcon UMovieGraphRemoveRenderSettingNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon ApplyCVarPresetIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete");

	OutColor = FLinearColor::White;
	return ApplyCVarPresetIcon;
}

void UMovieGraphRemoveRenderSettingNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphRemoveRenderSettingNode, NodeType))
	{
		OnNodeChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR
