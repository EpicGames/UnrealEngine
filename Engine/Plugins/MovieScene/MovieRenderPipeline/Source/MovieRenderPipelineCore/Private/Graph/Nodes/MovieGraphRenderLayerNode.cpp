// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphRenderLayerNode.h"

#include "Graph/MovieGraphConfig.h"
#include "MoviePipelineTelemetry.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphRenderLayerNode)

UMovieGraphRenderLayerNode::UMovieGraphRenderLayerNode()
	: LayerWarmUpFrames(0)
{
	LayerName = TEXT("beauty");
}

#if WITH_EDITOR
FText UMovieGraphRenderLayerNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText RenderLayerNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_RenderLayer", "Render Layer");
	static const FText RenderLayerNodeDescription = NSLOCTEXT("MovieGraphNodes", "NodeDescription_RenderLayer", "Render Layer\n{0}");

	if (bGetDescriptive && !LayerName.IsEmpty())
	{
		return FText::Format(RenderLayerNodeDescription, FText::FromString(LayerName));
	}
	
	return RenderLayerNodeName;
}

FText UMovieGraphRenderLayerNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "RenderLayerGraphNode_Category", "Rendering");
}

FLinearColor UMovieGraphRenderLayerNode::GetNodeTitleColor() const
{
	static const FLinearColor RenderLayerNodeColor = FLinearColor(0.192f, 0.258f, 0.615f);
	return RenderLayerNodeColor;
}

FSlateIcon UMovieGraphRenderLayerNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon RenderLayerIcon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.RenderLayer");

	OutColor = FLinearColor::White;
	return RenderLayerIcon;
}

void UMovieGraphRenderLayerNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphRenderLayerNode, LayerName))
	{
		OnNodeChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR

void UMovieGraphRenderLayerNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	++InTelemetry->RenderLayerCount;

	if (bOverride_LayerWarmUpFrames)
	{
		InTelemetry->LayerWarmUpFrames = FMath::Max(InTelemetry->LayerWarmUpFrames, FMath::Max(LayerWarmUpFrames, 0));
	}
}
