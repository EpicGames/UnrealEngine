// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphRerouteNode.h"

#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphRerouteNode)

UMovieGraphRerouteNode::UMovieGraphRerouteNode()
{
	InputOutputProperties = FMovieGraphPinProperties::MakeWildcardProperties();
}

TArray<FMovieGraphPinProperties> UMovieGraphRerouteNode::GetInputPinProperties() const
{
	return {InputOutputProperties};
}

TArray<FMovieGraphPinProperties> UMovieGraphRerouteNode::GetOutputPinProperties() const
{
	return {InputOutputProperties};
}

bool UMovieGraphRerouteNode::CanBeDisabled() const
{
	return false;
}

#if WITH_EDITOR
FText UMovieGraphRerouteNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText RerouteNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_Reroute", "Reroute");

	return RerouteNodeName;
}

FText UMovieGraphRerouteNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "RerouteGraphNode_Category", "Utility");
}
#endif // WITH_EDITOR

UMovieGraphPin* UMovieGraphRerouteNode::GetPassThroughPin(const UMovieGraphPin* InFromPin) const
{
	if (InFromPin)
	{
		if (InputPins.Contains(InFromPin))
		{
			return OutputPins[0];
		}

		if (OutputPins.Contains(InFromPin))
		{
			return InputPins[0];
		}
	}

	return nullptr;
}

FMovieGraphPinProperties UMovieGraphRerouteNode::GetPinProperties() const
{
	return InputOutputProperties;
}

void UMovieGraphRerouteNode::SetPinProperties(const FMovieGraphPinProperties& InPinProperties)
{
	InputOutputProperties = InPinProperties;
}
