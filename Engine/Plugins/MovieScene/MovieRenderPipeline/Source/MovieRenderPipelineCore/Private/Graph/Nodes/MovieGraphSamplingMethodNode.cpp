// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphSamplingMethodNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphLinearTimeStep.h"
#include "MoviePipelineTelemetry.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphSamplingMethodNode)

UMovieGraphSamplingMethodNode::UMovieGraphSamplingMethodNode()
	: SamplingMethodClass(UMovieGraphLinearTimeStep::StaticClass())
	, TemporalSampleCount(1)
{
}

EMovieGraphBranchRestriction UMovieGraphSamplingMethodNode::GetBranchRestriction() const
{
	return EMovieGraphBranchRestriction::Globals;
}

void UMovieGraphSamplingMethodNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	OutMergedFormatArgs.FilenameArguments.Add(TEXT("ts_count"), FString::FromInt(TemporalSampleCount));
    OutMergedFormatArgs.FileMetadata.Add(TEXT("unreal/sampling/temporalSampleCount"), FString::FromInt(TemporalSampleCount));

	if (const UClass* SamplingClass = SamplingMethodClass.ResolveClass())
	{
#if WITH_EDITOR
		const FString SamplingModeDisplayName = SamplingClass->GetDisplayNameText().ToString();
#else
		const FString SamplingModeDisplayName = SamplingClass->GetName();
#endif
		
		OutMergedFormatArgs.FilenameArguments.Add(TEXT("sampling_mode"), SamplingModeDisplayName);
		OutMergedFormatArgs.FileMetadata.Add(TEXT("unreal/sampling/mode"), SamplingModeDisplayName);
	}
	else
	{
		OutMergedFormatArgs.FilenameArguments.Add(TEXT("sampling_mode"), TEXT("Unknown"));
		OutMergedFormatArgs.FileMetadata.Add(TEXT("unreal/sampling/mode"), TEXT("Unknown"));
	}
}

#if WITH_EDITOR
FText UMovieGraphSamplingMethodNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText SamplingMethodNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_SamplingMethod", "Sampling Method");
	return SamplingMethodNodeName;
}

FText UMovieGraphSamplingMethodNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "SamplingMethodGraphNode_Category", "Rendering");
}

FLinearColor UMovieGraphSamplingMethodNode::GetNodeTitleColor() const
{
	static const FLinearColor SamplingMethodNodeColor = FLinearColor(0.572f, 0.274f, 1.f);
	return SamplingMethodNodeColor;
}

FSlateIcon UMovieGraphSamplingMethodNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon SamplingMethodPresetIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.Cache.Statistics");

	OutColor = FLinearColor::White;
	return SamplingMethodPresetIcon;
}
#endif // WITH_EDITOR

void UMovieGraphSamplingMethodNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->TemporalSampleCount = TemporalSampleCount;
}
