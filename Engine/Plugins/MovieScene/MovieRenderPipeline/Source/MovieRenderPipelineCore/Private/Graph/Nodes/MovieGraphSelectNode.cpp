// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphSelectNode.h"

#include "MovieRenderPipelineCoreModule.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphUtils.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphSelectNode)

namespace UE::MovieGraph::SelectNode
{
	static const FName SelectedOption("Selected Option");
	static const FName DefaultBranch("Default");
}

UMovieGraphSelectNode::UMovieGraphSelectNode()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		SelectOptions = CreateDefaultSubobject<UMovieGraphValueContainer>(TEXT("SelectOptions"));
		SelectOptions->SetValueContainerType(EMovieGraphContainerType::Array);
		SelectOptions->SetValueType(EMovieGraphValueType::String);
		SelectOptions->SetPropertyName(FName(TEXT("Options")));

		SelectedOption = CreateDefaultSubobject<UMovieGraphValueContainer>(TEXT("SelectedOption"));
		SelectedOption->SetValueType(EMovieGraphValueType::String);
		SelectedOption->SetPropertyName(FName(TEXT("Selection")));
	}
}

TArray<FMovieGraphPinProperties> UMovieGraphSelectNode::GetInputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;

	// Ensure the select options array is valid
	TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> SelectOptionsArrayResult = SelectOptions->GetArrayRef();
	if (!SelectOptionsArrayResult.HasValue() || SelectOptionsArrayResult.HasError())
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "When generating pins on the Select node, found an invalid options array. Select node will not work as intended.");
		return Properties;
	}
	
	const FPropertyBagArrayRef SelectOptionsArray = SelectOptionsArrayResult.GetValue();
	
	// Generate branch pins for each option
	for (int32 Index = 0; Index < SelectOptionsArray.Num(); ++Index)
	{
		switch (const EMovieGraphValueType ValueType = SelectOptions->GetValueType())
		{
		case EMovieGraphValueType::Bool:
			{
				TValueOrError<bool, EPropertyBagResult> Result = SelectOptionsArray.GetValueBool(Index);
				if (Result.IsValid())
				{
					FMovieGraphPinProperties BranchProperties = FMovieGraphPinProperties::MakeBranchProperties(FName(LexToString(Result.GetValue())));
					BranchProperties.bIsBuiltIn = false;
					Properties.Add(MoveTemp(BranchProperties));
				}
			}
			break;

		case EMovieGraphValueType::Byte:
			{
				TValueOrError<uint8, EPropertyBagResult> Result = SelectOptionsArray.GetValueByte(Index);
				if (Result.IsValid())
				{
					FMovieGraphPinProperties BranchProperties = FMovieGraphPinProperties::MakeBranchProperties(FName(LexToString(Result.GetValue())));
					BranchProperties.bIsBuiltIn = false;
					Properties.Add(MoveTemp(BranchProperties));
				}
			}
			break;
			
		case EMovieGraphValueType::String:
			{
				TValueOrError<FString, EPropertyBagResult> Result = SelectOptionsArray.GetValueString(Index);
				if (Result.IsValid())
				{
					FMovieGraphPinProperties BranchProperties = FMovieGraphPinProperties::MakeBranchProperties(FName(Result.GetValue()));
					BranchProperties.bIsBuiltIn = false;
					Properties.Add(MoveTemp(BranchProperties));
				}
			}
			break;

		case EMovieGraphValueType::Int32:
			{
				TValueOrError<int32, EPropertyBagResult> Result = SelectOptionsArray.GetValueInt32(Index);
				if (Result.IsValid())
				{
					FMovieGraphPinProperties BranchProperties = FMovieGraphPinProperties::MakeBranchProperties(FName(LexToString(Result.GetValue())));
					BranchProperties.bIsBuiltIn = false;
					Properties.Add(MoveTemp(BranchProperties));
				}
			}
			break;

		case EMovieGraphValueType::Int64:
			{
				TValueOrError<int64, EPropertyBagResult> Result = SelectOptionsArray.GetValueInt64(Index);
				if (Result.IsValid())
				{
					FMovieGraphPinProperties BranchProperties = FMovieGraphPinProperties::MakeBranchProperties(FName(LexToString(Result.GetValue())));
					BranchProperties.bIsBuiltIn = false;
					Properties.Add(MoveTemp(BranchProperties));
				}
			}
			break;

		case EMovieGraphValueType::Name:
			{
				TValueOrError<FName, EPropertyBagResult> Result = SelectOptionsArray.GetValueName(Index);
				if (Result.IsValid())
				{
					FMovieGraphPinProperties BranchProperties = FMovieGraphPinProperties::MakeBranchProperties(Result.GetValue());
					BranchProperties.bIsBuiltIn = false;
					Properties.Add(MoveTemp(BranchProperties));
				}
			}
			break;

		case EMovieGraphValueType::Text:
			{
				TValueOrError<FText, EPropertyBagResult> Result = SelectOptionsArray.GetValueText(Index);
				if (Result.IsValid())
				{
					FMovieGraphPinProperties BranchProperties = FMovieGraphPinProperties::MakeBranchProperties(FName(Result.GetValue().ToString()));
					BranchProperties.bIsBuiltIn = false;
					Properties.Add(MoveTemp(BranchProperties));
				}
			}
			break;

		case EMovieGraphValueType::Enum:
			{
				const UEnum* Enum = Cast<UEnum>(SelectOptions->GetValueTypeObject());
				TValueOrError<uint8, EPropertyBagResult> Result = SelectOptionsArray.GetValueEnum(Index, Enum);
				if (Result.IsValid())
				{
					const FText DisplayName = Enum->GetDisplayNameTextByValue(Result.GetValue());
					FMovieGraphPinProperties BranchProperties = FMovieGraphPinProperties::MakeBranchProperties(FName(DisplayName.ToString()));
					BranchProperties.bIsBuiltIn = false;
					Properties.Add(MoveTemp(BranchProperties));
				}
			}
			break;
			
		default:
			const UEnum* EnumType = StaticEnum<EMovieGraphValueType>();
			UE_LOGF(LogMovieRenderPipeline, Warning, "Encountered an unexpected type in the Select node when generating pins: %ls", *EnumType->GetNameStringByValue(static_cast<uint8>(ValueType)));
			break;
		}
	}

	Properties.Add(FMovieGraphPinProperties::MakeBranchProperties(UE::MovieGraph::SelectNode::DefaultBranch));
	Properties.Add(FMovieGraphPinProperties(UE::MovieGraph::SelectNode::SelectedOption, SelectOptions->GetValueType(), SelectOptions->GetValueTypeObject(), false));
	
	return Properties;
}

TArray<FMovieGraphPinProperties> UMovieGraphSelectNode::GetOutputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;
	Properties.Add(FMovieGraphPinProperties::MakeBranchProperties());
	return Properties;
}

TArray<UMovieGraphPin*> UMovieGraphSelectNode::EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const
{
	TArray<UMovieGraphPin*> PinsToFollow;

	// If the node is disabled, follow the pin for the first option available.
	if (IsDisabled())
	{
		if (UMovieGraphPin* GraphPin = GetFirstConnectedInputPin())
		{
			PinsToFollow.Add(GraphPin);
		}
		
		return PinsToFollow;
	}

	// The resolved value of the "Selected Option" property. May come from a connection or a value specified on the node.
	TObjectPtr<UMovieGraphValueContainer> ResolvedSelectedOption;

	// Try getting the value from a connection first
	bool bGotValueFromConnection = false;
	if (UMovieGraphPin* SelectPin = GetInputPin(UE::MovieGraph::SelectNode::SelectedOption, EMovieGraphPinQueryRequirement::BuiltIn))
	{
		constexpr bool bFollowRerouteConnections = true;
		if (UMovieGraphPin* OtherPin = SelectPin->GetFirstConnectedPin(bFollowRerouteConnections))
		{
			FString ResolvedValue;
			bGotValueFromConnection = UE::MovieGraph::ResolveConnectedPinValue(SelectPin, {OtherPin}, InContext, ResolvedValue);

			if (bGotValueFromConnection)
			{
				ResolvedSelectedOption = NewObject<UMovieGraphValueContainer>();
				ResolvedSelectedOption->SetValueType(OtherPin->Properties.Type);
				ResolvedSelectedOption->SetValueTypeObject(OtherPin->Properties.TypeObject);
				ResolvedSelectedOption->SetValueSerializedString(ResolvedValue);
			}
		}
	}

	if (!bGotValueFromConnection)
	{
		ResolvedSelectedOption = SelectedOption;
	}

	// Ensure the select options array is valid
	TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> SelectOptionsArrayResult = SelectOptions->GetArrayRef();
	if (!SelectOptionsArrayResult.HasValue() || SelectOptionsArrayResult.HasError())
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "When evaluating pins on the Select node, found an invalid options array. Select node will not work as intended.");
		return PinsToFollow;
	}
	
	const FPropertyBagArrayRef SelectOptionsArray = SelectOptionsArrayResult.GetValue();

	// Iterate over all select options, and follow the pin that matches the selected option
	for (int32 Index = 0; Index < SelectOptionsArray.Num(); ++Index)
	{
		if (!ensureMsgf(InputPins.IsValidIndex(Index), TEXT("Found more select options than pins on the node")))
		{
			continue;
		}
		
		const TObjectPtr<UMovieGraphPin>& InputPin = InputPins[Index];

		// If the pin isn't connected, there's no point in determining if it's a match
		if (!InputPin->IsConnected())
		{
			continue;
		}

		switch(SelectOptions->GetValueType())
		{
		case EMovieGraphValueType::Bool:
			{
				TValueOrError<bool, EPropertyBagResult> PinValue = SelectOptionsArray.GetValueBool(Index);
				bool bSelectedOption;
				if (PinValue.HasValue() && !PinValue.HasError() && ResolvedSelectedOption->GetValueBool(bSelectedOption) && (PinValue.GetValue() == bSelectedOption))
				{
					PinsToFollow.Add(InputPin);
				}
			}
			break;

		case EMovieGraphValueType::Byte:
			{
				TValueOrError<uint8, EPropertyBagResult> PinValue = SelectOptionsArray.GetValueByte(Index);
				uint8 SelectedValue;
				if (PinValue.HasValue() && !PinValue.HasError() && ResolvedSelectedOption->GetValueByte(SelectedValue) && (PinValue.GetValue() == SelectedValue))
				{
					PinsToFollow.Add(InputPin);
				}
			}
			break;

		case EMovieGraphValueType::String:
			{
				TValueOrError<FString, EPropertyBagResult> PinValue = SelectOptionsArray.GetValueString(Index);
				FString SelectedValue;
				if (PinValue.HasValue() && !PinValue.HasError() && ResolvedSelectedOption->GetValueString(SelectedValue) && (PinValue.GetValue() == SelectedValue))
				{
					PinsToFollow.Add(InputPin);
				}
			}
			break;
		
		case EMovieGraphValueType::Int32:
			{
				TValueOrError<int32, EPropertyBagResult> PinValue = SelectOptionsArray.GetValueInt32(Index);
				int32 SelectedValue;
				if (PinValue.HasValue() && !PinValue.HasError() && ResolvedSelectedOption->GetValueInt32(SelectedValue) && (PinValue.GetValue() == SelectedValue))
				{
					PinsToFollow.Add(InputPin);
				}
			}
			break;

		case EMovieGraphValueType::Int64:
			{
				TValueOrError<int64, EPropertyBagResult> PinValue = SelectOptionsArray.GetValueInt64(Index);
				int64 SelectedValue;
				if (PinValue.HasValue() && !PinValue.HasError() && ResolvedSelectedOption->GetValueInt64(SelectedValue) && (PinValue.GetValue() == SelectedValue))
				{
					PinsToFollow.Add(InputPin);
				}
			}
			break;

		case EMovieGraphValueType::Name:
			{
				TValueOrError<FName, EPropertyBagResult> PinValue = SelectOptionsArray.GetValueName(Index);
				FName SelectedValue;
				if (PinValue.HasValue() && !PinValue.HasError() && ResolvedSelectedOption->GetValueName(SelectedValue) && (PinValue.GetValue() == SelectedValue))
				{
					PinsToFollow.Add(InputPin);
				}
			}
			break;

		case EMovieGraphValueType::Text:
			{
				TValueOrError<FText, EPropertyBagResult> PinValue = SelectOptionsArray.GetValueText(Index);
				FText SelectedValue;
				if (PinValue.HasValue() && !PinValue.HasError() && ResolvedSelectedOption->GetValueText(SelectedValue) && PinValue.GetValue().EqualTo(SelectedValue))
				{
					PinsToFollow.Add(InputPin);
				}
			}
			break;
			
		case EMovieGraphValueType::Enum:
			{
				const UEnum* OptionsEnum = Cast<UEnum>(SelectOptions->GetValueTypeObject());
				const UEnum* SelectedEnum = Cast<UEnum>(ResolvedSelectedOption->GetValueTypeObject());
				
				TValueOrError<uint8, EPropertyBagResult> PinValue = SelectOptionsArray.GetValueEnum(Index, OptionsEnum);
				uint8 SelectedValue;
				
				if (OptionsEnum && SelectedEnum && PinValue.HasValue() && !PinValue.HasError() && ResolvedSelectedOption->GetValueEnum(SelectedValue, SelectedEnum) && (PinValue.GetValue() == SelectedValue))
				{
					PinsToFollow.Add(InputPin);
				}
			}
			break;

		default:
			const UEnum* EnumType = StaticEnum<EMovieGraphValueType>();
			UE_LOGF(LogMovieRenderPipeline, Warning, "Encountered an unexpected type in the Select node when following pins: %ls", *EnumType->GetNameStringByValue(static_cast<uint8>(SelectOptions->GetValueType())));
			break;
		}
	}

	// Follow the Default branch if no pins match
	if (PinsToFollow.IsEmpty())
	{
		PinsToFollow.Add(GetInputPin(UE::MovieGraph::SelectNode::DefaultBranch, EMovieGraphPinQueryRequirement::BuiltIn));
	}	

	return PinsToFollow;
}

void UMovieGraphSelectNode::SetDataType(const EMovieGraphValueType ValueType, UObject* InValueTypeObject)
{
	SelectOptions->SetValueType(ValueType, InValueTypeObject);
	SelectedOption->SetValueType(ValueType, InValueTypeObject);

	UpdatePins();
}

EMovieGraphValueType UMovieGraphSelectNode::GetValueType() const
{
	return SelectOptions->GetValueType();
}

const UObject* UMovieGraphSelectNode::GetValueTypeObject() const
{
	return SelectOptions->GetValueTypeObject();
}

#if WITH_EDITOR
FText UMovieGraphSelectNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText SelectNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_Select", "Select");
	static const FText SelectNodeNameDescription = NSLOCTEXT("MovieGraphNodes", "NodeDescription_Select", "Select\n{0}");

	if (bGetDescriptive)
	{
		if (const UEnum* Enum = Cast<UEnum>(SelectOptions->GetValueTypeObject()))
		{
			return FText::Format(SelectNodeNameDescription, FText::FromString(Enum->GetName()));
		}
	}

	return SelectNodeName;
}

FText UMovieGraphSelectNode::GetMenuCategory() const
{
	static const FText NodeCategory_Conditionals = NSLOCTEXT("MovieGraphNodes", "NodeCategory_Conditionals", "Conditionals");
	return NodeCategory_Conditionals;
}

FText UMovieGraphSelectNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "SelectNode_Keywords", "select if logic conditional");
	return Keywords;
}

FLinearColor UMovieGraphSelectNode::GetNodeTitleColor() const
{
	static const FLinearColor SelectNodeColor = FLinearColor(0.266f, 0.266f, 0.266f);
	return SelectNodeColor;
}

FSlateIcon UMovieGraphSelectNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon SelectIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Merge");

	OutColor = FLinearColor::White;
	return SelectIcon;
}

void UMovieGraphSelectNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphSelectNode, SelectOptions))
	{
		UpdatePins();
	}
}
#endif // WITH_EDITOR
