// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphSetCVarValueNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "MoviePipelineTelemetry.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphSetCVarValueNode)

FString UMovieGraphSetCVarValueNode::GetNodeInstanceName() const
{
	return Name;
}

EMovieGraphBranchRestriction UMovieGraphSetCVarValueNode::GetBranchRestriction() const
{
	return EMovieGraphBranchRestriction::Globals;
}

void UMovieGraphSetCVarValueNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bSetsConsoleVariables = !Name.TrimStartAndEnd().IsEmpty();
}

TArray<FMovieGraphPropertyInfo> UMovieGraphSetCVarValueNode::GetOverrideablePropertyInfo() const
{
	TArray<FMovieGraphPropertyInfo> PropertyInfo = Super::GetOverrideablePropertyInfo();

	for (FMovieGraphPropertyInfo& Info : PropertyInfo)
	{
		if (Info.Name == GET_MEMBER_NAME_CHECKED(UMovieGraphSetCVarValueNode, Value))
		{
			const FString CVarName = Name.IsEmpty() ? TEXT("NoCVarName") : Name;
			
			// This custom display name will show up in the property promotion context menu
			Info.ContextMenuName = FText::Format(NSLOCTEXT("MovieGraphNodes", "SetCvarValueNode_ValuePromotionContextMenuName", "Value ({0})"), FText::FromString(CVarName));

			// If the property is promoted, the created variable should have the name of the cvar
			Info.PromotionName = FName(CVarName);
		}
	}

	// Also remove the "Name" property -- it should not show up as a property that can be promoted
	PropertyInfo.RemoveAll([](const FMovieGraphPropertyInfo& InPropertyInfo)
	{
		return InPropertyInfo.Name == GET_MEMBER_NAME_CHECKED(UMovieGraphSetCVarValueNode, Name);
	});

	return PropertyInfo;
}

#if WITH_EDITOR
FText UMovieGraphSetCVarValueNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText SetCVarNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_SetCVar", "Set Console Variable");
	static const FText SetCVarNodeDescription = NSLOCTEXT("MovieGraphNodes", "NodeDescription_SetCVar", "Set Console Variable\n{0} ({1})");

	if (bGetDescriptive && !Name.IsEmpty())
	{
		// Format the float value so it looks nice for display purposes
		const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMinimumFractionalDigits(0)
			.SetMaximumFractionalDigits(4);
		const FString CVarValue = FastDecimalFormat::NumberToString(Value, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
		
		return FText::Format(SetCVarNodeDescription, FText::FromString(Name), FText::FromString(CVarValue));
	}
	
	return SetCVarNodeName;
}

FText UMovieGraphSetCVarValueNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "SetCVarGraphNode_Category", "Utility");
}

FText UMovieGraphSetCVarValueNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "SetCVarGraphNode_Keywords", "cvar console variable");
	return Keywords;
}

FLinearColor UMovieGraphSetCVarValueNode::GetNodeTitleColor() const
{
	static const FLinearColor SetCVarNodeColor = FLinearColor(0.04f, 0.22f, 0.36f);
	return SetCVarNodeColor;
}

FSlateIcon UMovieGraphSetCVarValueNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon SetCVarValueIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.BrowseCVars");

	OutColor = FLinearColor::White;
	return SetCVarValueIcon;
}

void UMovieGraphSetCVarValueNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	// Skip rapid updates from properties; only refresh on commit
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}
	
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphSetCVarValueNode, Name) ||
		PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphSetCVarValueNode, Value))
	{
		OnNodeChangedDelegate.Broadcast(this);
	}
}

#endif // WITH_EDITOR
