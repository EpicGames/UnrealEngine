// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphSetMetadataAttributesNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphSetMetadataAttributesNode)

void UMovieGraphMetadataAttributeCollection::Merge(const IMovieGraphTraversableObject* InSourceObject)
{
	const UMovieGraphMetadataAttributeCollection* MovieGraphMetadataAttributeCollection = Cast<UMovieGraphMetadataAttributeCollection>(InSourceObject);
	checkf(MovieGraphMetadataAttributeCollection, TEXT("UMovieGraphMetadataAttributeCollection cannot merge with null or an object of another type."));

	// The Movie Graph is evaluated right to left, i.e., from the output to the input, thus we need to keep track of the metadata attributes that
	// have already been merged so they are not added again upon merging a new source object.
	TSet<FString> ExistingMetadataAttributes;
	for (const FMovieGraphMetadataAttribute& MetadataAttribute : MetadataAttributes)
	{
		if (MetadataAttribute.bIsEnabled && !MetadataAttribute.Name.IsEmpty())
		{
			ExistingMetadataAttributes.Add(MetadataAttribute.Name);
		}
	}

	for (const FMovieGraphMetadataAttribute& SourceMetadataAttribute : MovieGraphMetadataAttributeCollection->MetadataAttributes)
	{
		if (SourceMetadataAttribute.bIsEnabled && !SourceMetadataAttribute.Name.IsEmpty() && !ExistingMetadataAttributes.Contains(SourceMetadataAttribute.Name))
		{
			MetadataAttributes.Add(FMovieGraphMetadataAttribute(SourceMetadataAttribute.Name, SourceMetadataAttribute.Value, SourceMetadataAttribute.bIsEnabled));
		}
	}
}

TArray<TPair<FString, FString>> UMovieGraphMetadataAttributeCollection::GetMergedProperties() const
{
	TArray<TPair<FString, FString>> MergedProperties;

	for (const FMovieGraphMetadataAttribute& Metadata : MetadataAttributes)
	{
		if (Metadata.bIsEnabled && !Metadata.Name.IsEmpty())
		{
			MergedProperties.Add({Metadata.Name, Metadata.Value});
		}
	}
	
	return MergedProperties;
}

UMovieGraphSetMetadataAttributesNode::UMovieGraphSetMetadataAttributesNode()
{
	MetadataAttributeCollection = CreateDefaultSubobject<UMovieGraphMetadataAttributeCollection>(TEXT("Metadata Attributes"));
}

void UMovieGraphSetMetadataAttributesNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	for (const FMovieGraphMetadataAttribute& MetadataAttribute: MetadataAttributeCollection->MetadataAttributes)
	{
		if (MetadataAttribute.bIsEnabled && !MetadataAttribute.Name.IsEmpty())
		{
			OutMergedFormatArgs.FileMetadata.Add(MetadataAttribute.Name, MetadataAttribute.Value);
		}
	}
}

#if WITH_EDITOR
FText UMovieGraphSetMetadataAttributesNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText SetMetadataAttributesNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_SetMetadataAttributes", "Set Metadata Attributes");
	static const FText SetMetadataAttributeNodesDescription = NSLOCTEXT("MovieGraphNodes", "NodeDescription_SetMetadataAttributes", "Set Metadata Attributes\n{0}");

	if (bGetDescriptive && IsValid(MetadataAttributeCollection))
	{
		TArray<FString> MetadataAttributeNames;
		for (const FMovieGraphMetadataAttribute& MetadataAttribute: MetadataAttributeCollection->MetadataAttributes)
		{
			if (MetadataAttribute.bIsEnabled && !MetadataAttribute.Name.IsEmpty())
			{
				MetadataAttributeNames.Add(MetadataAttribute.Name);
			}
		}

		if (!MetadataAttributeNames.IsEmpty())
		{
			// Shorten the list of attribute names if needed so it doesn't make the node too wide
			FString AttributeNames = FString::Join(MetadataAttributeNames, TEXT(", "));
			if (AttributeNames.Len() > 50)
			{
				AttributeNames = AttributeNames.Left(50) + TEXT("...");
			}
			
			return FText::Format(SetMetadataAttributeNodesDescription, FText::FromString(AttributeNames));
		}
	}
	
	return SetMetadataAttributesNodeName;
}

FText UMovieGraphSetMetadataAttributesNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "SetMetadataAttributesGraphNode_Category", "Utility");
}

FText UMovieGraphSetMetadataAttributesNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "SetMetadataAttributesGraphNode_Keywords", "metadata attributes exr");
	return Keywords;
}

FLinearColor UMovieGraphSetMetadataAttributesNode::GetNodeTitleColor() const
{
	static const FLinearColor SetMetadataAttributesNodeColor = FLinearColor(0.22f, 0.04f,  0.36f);
	return SetMetadataAttributesNodeColor;
}

FSlateIcon UMovieGraphSetMetadataAttributesNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon SetMetadataAttributesIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.ResourceUsage");

	OutColor = FLinearColor::White;
	return SetMetadataAttributesIcon;
}

void UMovieGraphSetMetadataAttributesNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	// Skip rapid updates from properties; only refresh on commit
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}
	
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphSetMetadataAttributesNode, MetadataAttributeCollection))
	{
		OnNodeChangedDelegate.Broadcast(this);
	}
}

#endif // WITH_EDITOR
