// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphSetStartEndConsoleCommandsNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphSetStartEndConsoleCommandsNode)

void UMovieGraphStartEndConsoleCommands::Merge(const IMovieGraphTraversableObject* InSourceObject)
{
	const UMovieGraphStartEndConsoleCommands* InConsoleCommands = Cast<UMovieGraphStartEndConsoleCommands>(InSourceObject);
	checkf(InConsoleCommands, TEXT("UMovieGraphStartEndConsoleCommands cannot merge with null or an object of another type."));

	// The Movie Graph is evaluated right to left, i.e., from the output to the input.

	// First, merge in the "remove" console commands. This needs to be done before merging the start/end commands to ensure that commands that need to
	// be removed are never added to the start/end commands.
	for (const FString& RemoveStartCommand : InConsoleCommands->RemoveStartCommands)
	{
		if (!RemoveStartCommand.IsEmpty())
		{
			RemoveStartCommands.AddUnique(RemoveStartCommand);
		}
	}

	for (const FString& RemoveEndCommand : InConsoleCommands->RemoveEndCommands)
	{
		if (!RemoveEndCommand.IsEmpty())
		{
			RemoveEndCommands.AddUnique(RemoveEndCommand);
		}
	}
	
	// Next, add in the start/end console commands, but only if they are NOT in the "remove" console commands.
	for (const FString& StartCommand : InConsoleCommands->AddStartCommands)
	{
		if (!StartCommand.IsEmpty() && !RemoveStartCommands.Contains(StartCommand))
		{
			AddStartCommands.AddUnique(StartCommand);
		}
	}

	for (const FString& EndCommand : InConsoleCommands->AddEndCommands)
	{
		if (!EndCommand.IsEmpty() && !RemoveEndCommands.Contains(EndCommand))
		{
			AddEndCommands.AddUnique(EndCommand);
		}
	}
}

TArray<TPair<FString, FString>> UMovieGraphStartEndConsoleCommands::GetMergedProperties() const
{
	TArray<TPair<FString, FString>> MergedProperties;

	// Don't show the Remove* properties in the evaluated view. They have no meaning after evaluation is finished.
	MergedProperties.Add(TPair<FString, FString>("Start Console Commands", FString::Join(AddStartCommands, TEXT(", "))));
	MergedProperties.Add(TPair<FString, FString>("End Console Commands", FString::Join(AddEndCommands, TEXT(", "))));

	return MergedProperties;
}

UMovieGraphSetStartEndConsoleCommandsNode::UMovieGraphSetStartEndConsoleCommandsNode()
{
	ConsoleCommands = CreateDefaultSubobject<UMovieGraphStartEndConsoleCommands>(TEXT("Console Commands"));
}

EMovieGraphBranchRestriction UMovieGraphSetStartEndConsoleCommandsNode::GetBranchRestriction() const
{
	return EMovieGraphBranchRestriction::Globals;
}

#if WITH_EDITOR
FText UMovieGraphSetStartEndConsoleCommandsNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText StartEndConsoleCommandsNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_StartEndConsoleCommands", "Set Start/End Console Commands");
	static const FText StartEndConsoleCommandsNodeAddDescription = NSLOCTEXT("MovieGraphNodes", "NodeDescription_AddStartEndConsoleCommands", "Add {0} Start, {1} End Command(s)");
	static const FText StartEndConsoleCommandsNodeRemoveDescription = NSLOCTEXT("MovieGraphNodes", "NodeDescription_RemoveStartEndConsoleCommands", "Remove {0} Start, {1} End Command(s)");

	if (bGetDescriptive)
	{
		TArray<FText> DescriptionLines = {StartEndConsoleCommandsNodeName};

		// Add in the "Add" commands, if there are any
		const int32 NumAddedStartCommands = ConsoleCommands->AddStartCommands.Num();
		const int32 NumAddedEndCommands = ConsoleCommands->AddEndCommands.Num();
		if ((NumAddedStartCommands > 0) || (NumAddedEndCommands > 0))
		{
			DescriptionLines.Add(FText::Format(StartEndConsoleCommandsNodeAddDescription, NumAddedStartCommands, NumAddedEndCommands));
		}

		// Add in the "Remove" commands, if there are any
		const int32 NumRemovedStartCommands = ConsoleCommands->RemoveStartCommands.Num();
		const int32 NumRemovedEndCommands = ConsoleCommands->RemoveEndCommands.Num();
		if ((NumRemovedStartCommands > 0) || (NumRemovedEndCommands > 0))
		{
			DescriptionLines.Add(FText::Format(StartEndConsoleCommandsNodeRemoveDescription, NumRemovedStartCommands, NumRemovedEndCommands));
		}

		return FText::Join(INVTEXT("\n"), DescriptionLines);
	}
	
	return StartEndConsoleCommandsNodeName;
}

FText UMovieGraphSetStartEndConsoleCommandsNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "StartEndConsoleCommandsGraphNode_Category", "Utility");
}

FText UMovieGraphSetStartEndConsoleCommandsNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "StartEndConsoleCommandsGraphNode_Keywords", "start end console commands");
	return Keywords;
}

FLinearColor UMovieGraphSetStartEndConsoleCommandsNode::GetNodeTitleColor() const
{
	static const FLinearColor StartEndConsoleCommandsNodeColor = FLinearColor(0.04f, 0.22f, 0.36f);
	return StartEndConsoleCommandsNodeColor;
}

FSlateIcon UMovieGraphSetStartEndConsoleCommandsNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon StartEndConsoleCommandsIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.BrowseCVars");

	OutColor = FLinearColor::White;
	return StartEndConsoleCommandsIcon;
}

void UMovieGraphSetStartEndConsoleCommandsNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Update the node's title once the user has committed their changes to the commands
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphSetStartEndConsoleCommandsNode, ConsoleCommands))
		{
			OnNodeChangedDelegate.Broadcast(this);
		}
	}
}
#endif // WITH_EDITOR
