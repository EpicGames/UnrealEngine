// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphSubgraphNode.h"

#include "MovieRenderPipelineCoreModule.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphSubgraphNode)

#define LOCTEXT_NAMESPACE "MovieGraphNode"

namespace UE::MovieGraph::Private
{
	/** Converts graph input/output members to pin properties. */
	TArray<FMovieGraphPinProperties> MembersToPinProperties(const TArray<UMovieGraphInterfaceBase*>& InMembers)
	{
		TArray<FMovieGraphPinProperties> PinProperties;

		Algo::TransformIf(InMembers, PinProperties,
			[](const UMovieGraphInterfaceBase* Member) -> bool
			{
				return Member != nullptr;
			},
			[](const UMovieGraphInterfaceBase* Member) -> FMovieGraphPinProperties
			{
				constexpr bool bAllowMultipleConnections = false;
				FMovieGraphPinProperties Properties(FName(Member->GetMemberName()), Member->GetValueType(), Member->GetValueTypeObject(), bAllowMultipleConnections);
				Properties.bIsBranch = Member->bIsBranch;
				return Properties;
			});

		return PinProperties;
	}
}

UMovieGraphSubgraphNode::UMovieGraphSubgraphNode()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		RefreshOnSubgraphAssetSaved();
	}
}

TArray<FMovieGraphPinProperties> UMovieGraphSubgraphNode::GetInputPinProperties() const
{
	if (const UMovieGraphConfig* Config = GetSubgraphAsset())
	{
		TArray<UMovieGraphInterfaceBase*> InputMembers;
		Algo::Transform(Config->GetInputs(), InputMembers, [](UMovieGraphInput* Input) { return Input; });
		
		return UE::MovieGraph::Private::MembersToPinProperties(InputMembers);
	}

	return {};
}

TArray<FMovieGraphPinProperties> UMovieGraphSubgraphNode::GetOutputPinProperties() const
{
	if (const UMovieGraphConfig* Config = GetSubgraphAsset())
	{
		TArray<UMovieGraphInterfaceBase*> OutputMembers;
		Algo::Transform(Config->GetOutputs(), OutputMembers, [](UMovieGraphOutput* Output) { return Output; });
		
		return UE::MovieGraph::Private::MembersToPinProperties(OutputMembers);
	}

	return {};
}

TArray<UMovieGraphPin*> UMovieGraphSubgraphNode::EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const
{
	TArray<UMovieGraphPin*> PinsToFollow;

	if (!ensure(InContext.PinBeingFollowed))
	{
		return PinsToFollow;
	}

	// If the node is disabled, follow the pin for the first option available.
	// This is only important for branch connections. For data pins, just continue following the connection.
	if (IsDisabled() && InContext.PinBeingFollowed->Properties.bIsBranch)
	{
		if (UMovieGraphPin* GraphPin = GetFirstConnectedInputPin())
		{
			PinsToFollow.Add(GraphPin);
		}
		
		return PinsToFollow;
	}
	
	const UMovieGraphConfig* SubgraphPtr = SubgraphAsset.LoadSynchronous();
	if (!SubgraphPtr)
	{
		return PinsToFollow;
	}

	// If this subgraph has already been visited, throw a circular graph reference error
	if (InContext.SubgraphStack.Contains(this))
	{
		InContext.bCircularGraphReferenceFound = true;

		// Normally there shouldn't be two identical graphs in the stack (cycle!), but for the purposes of error reporting, include this graph
		// as the last in the stack so the cycle is clear
		InContext.SubgraphStack.Add(this);
		
		return PinsToFollow;
	}

	// If an input on the node is being followed, just follow the connection on the pin, nothing fancy required.
	if (InContext.PinBeingFollowed->IsInputPin())
	{
		constexpr bool bFollowRerouteConnections = true;
		if (UMovieGraphPin* ConnectedInputPin = InContext.PinBeingFollowed->GetFirstConnectedPin(bFollowRerouteConnections))
		{
			PinsToFollow.Add(ConnectedInputPin);
		}
		
		return PinsToFollow;
	}

	// Find the input pin on the subgraph asset's outputs node that corresponds to the pin being followed
	//
	// Subgraph
	// -----------------+
	//     Outputs node |
	//     +----------+ |
	// --->| Out1     | |Out1 <--- Pin being followed ---
	//     | Out2     | |Out2
	//     | Out3     | |Out3
	//     +----------+ |
	// -----------------+
	
	const UMovieGraphNode* OutputNode = SubgraphPtr->GetOutputNode();
	if (!ensure(OutputNode))
	{
		return PinsToFollow;
	}

	if (UMovieGraphPin* OutputNodePin = OutputNode->GetInputPin(InContext.PinBeingFollowed->Properties.Label))
	{
		PinsToFollow.Add(OutputNodePin);
	}

	// Update the subgraph traversal stack so traversal can resume in parent graphs when traversal reaches the input
	// node of this subgraph
	InContext.SubgraphStack.Add(this);
	
	return PinsToFollow;
}

FString UMovieGraphSubgraphNode::GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext) const
{
	if (const UMovieGraphConfig* Subgraph = SubgraphAsset.LoadSynchronous())
	{
		for (UMovieGraphOutput* Output : Subgraph->GetOutputs())
		{
			if (Output && (FName(Output->GetMemberName()) == InPinName))
			{
				return Output->GetValueSerializedString();
			}
		}
	}

	return FString();
}

#if WITH_EDITOR
void UMovieGraphSubgraphNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Prevent circular subgraph references. This won't catch everything, but it will at least catch the common case interactively. Other cases will
	// be caught when the graph is fully evaluated.
	if (GetTypedOuter<UMovieGraphConfig>() == SubgraphAsset)
	{
		SubgraphAsset = nullptr;

#if WITH_EDITOR
		FNotificationInfo Info(LOCTEXT("CircularGraphAssignmentWarning", "Assigning the subgraph to the provided asset would cause a circular reference."));
		Info.ExpireDuration = 3.0f;
		Info.bUseLargeFont = false;
		Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
		const TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_None);
		}
#endif

		UE_LOGF(LogMovieRenderPipeline, Warning, "Found a circular reference when assigning the asset to the subgraph. Reverting.");
	}

	// Update pins when the subgraph asset is changed
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphSubgraphNode, SubgraphAsset))
	{
		UpdatePins();
	}
}
#endif

#if WITH_EDITOR
FText UMovieGraphSubgraphNode::GetNodeTitle(const bool bGetDescriptive) const
{
	// Note: Only the descriptive title indicates invalidity because we don't want the node creation menu to say the
	// subgraph node is invalid
	static const FText SubgraphTitle = LOCTEXT("SubgraphTitle", "Subgraph");
	static const FText DescriptiveSubgraphTitle = LOCTEXT("DescriptiveSubgraphTitle", "Invalid Subgraph\nNo subgraph asset specified");

	if (const UMovieGraphConfig* Subgraph = GetSubgraphAsset())
	{
		return FText::Format(LOCTEXT("SubgraphTitle_IncludesAsset", "Subgraph: {0}"), FText::FromString(Subgraph->GetName()));
	}

	return bGetDescriptive ? DescriptiveSubgraphTitle : SubgraphTitle;
}

FText UMovieGraphSubgraphNode::GetMenuCategory() const
{
	static const FText NodeCategory = LOCTEXT("NodeCategory_Utility", "Utility");

	return NodeCategory;
}
#endif

void UMovieGraphSubgraphNode::SetSubGraphAsset(const TSoftObjectPtr<UMovieGraphConfig>& InSubgraphAsset)
{
	SubgraphAsset = InSubgraphAsset;
	UpdatePins();
}

UMovieGraphConfig* UMovieGraphSubgraphNode::GetSubgraphAsset() const
{
	return SubgraphAsset.LoadSynchronous();
}

void UMovieGraphSubgraphNode::RefreshOnSubgraphAssetSaved()
{
	UPackage::PackageSavedWithContextEvent.AddWeakLambda(this, [this](const FString& PackageFilename, const UPackage* Package, FObjectPostSaveContext Context)
	{
		// Ignore procedural saves (ie, saves not triggered by a user)
		if (Context.IsProceduralSave())
		{
			return;
		}

		// Refresh the node if the subgraph asset was saved
		if (const UMovieGraphConfig* Subgraph = GetSubgraphAsset())
		{
			if (Subgraph->GetPackage() == Package)
			{
				UpdatePins();
			}
		}
	});
}

#undef LOCTEXT_NAMESPACE
