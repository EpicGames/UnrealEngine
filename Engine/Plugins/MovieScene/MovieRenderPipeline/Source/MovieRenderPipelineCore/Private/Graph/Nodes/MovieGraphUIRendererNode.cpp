// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphUIRendererNode.h"

#include "Engine/GameViewportClient.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphPipeline.h"
#include "MoviePipelineTelemetry.h"
#include "Slate/SGameLayerManager.h"
#include "TextureResource.h"
#include "UnrealClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphUIRendererNode)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FString UMovieGraphUIRendererNode::RendererName = FString("UIRenderer");
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UMovieGraphUIRendererNode::UMovieGraphUIRendererNode()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Super::RendererName = RendererName;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	RendererSubName = TEXT("ui");
}

#if WITH_EDITOR
FText UMovieGraphUIRendererNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("MovieGraphNodes", "UIRendererGraphNode_Description", "UI Renderer");
}

FLinearColor UMovieGraphUIRendererNode::GetNodeTitleColor() const
{
	return FLinearColor(0.572f, 0.274f, 1.f);
}

FSlateIcon UMovieGraphUIRendererNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon UIRendererIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "SequenceRecorder.TabIcon");

	OutColor = FLinearColor::White;
	return UIRendererIcon;
}
#endif	// WITH_EDITOR

void UMovieGraphUIRendererNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesUIRenderer = true;
}

void UMovieGraphUIRendererNode::FMovieGraphUIPass::Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, const FMovieGraphRenderPassLayerData& InLayer)
{
	FMovieGraphWidgetPass::Setup(InRenderer, InLayer);

	RenderDataIdentifier.SubResourceName = InLayer.RenderPassNode->GetRendererSubName();
}

TSharedPtr<SWidget> UMovieGraphUIRendererNode::FMovieGraphUIPass::GetWidget(UMovieGraphWidgetRendererBaseNode* InNodeThisFrame)
{
	const UMovieGraphPipeline* Pipeline = Renderer->GetOwningGraph();
	const ULocalPlayer* LocalPlayer = Pipeline->GetWorld()->GetFirstLocalPlayerFromController();
	
	// Cast the interface to a widget is a little yucky but the implementation is unlikely to change.
	const TSharedPtr<SGameLayerManager> GameLayerManager = StaticCastSharedPtr<SGameLayerManager>(LocalPlayer->ViewportClient->GetGameLayerManager());
	
	return GameLayerManager;
}

int32 UMovieGraphUIRendererNode::FMovieGraphUIPass::GetCompositingSortOrder() const
{
	// We want UI Widgets to be underneath the burn-in.
	return 80;
}

TUniquePtr<UMovieGraphWidgetRendererBaseNode::FMovieGraphWidgetPass> UMovieGraphUIRendererNode::GeneratePass()
{
	return MakeUnique<FMovieGraphUIPass>();
}
