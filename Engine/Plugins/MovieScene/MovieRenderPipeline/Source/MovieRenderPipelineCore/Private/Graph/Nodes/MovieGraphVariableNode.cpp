// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphVariableNode.h"

#include "Graph/MovieGraphConfig.h"
#include "MoviePipelineQueue.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphVariableNode)

void UMovieGraphVariableNode::PostEditImport()
{
	Super::PostEditImport();

	// Allow pasted/duplicated nodes to register delegates
	RegisterDelegates();
}

TArray<FMovieGraphPinProperties> UMovieGraphVariableNode::GetOutputPinProperties() const
{
	TArray<FMovieGraphPinProperties> Properties;
	
	if (GraphVariable)
	{
		Properties.Add(FMovieGraphPinProperties(FName(GraphVariable->GetMemberName()), GraphVariable->GetValueType(), GraphVariable->GetValueTypeObject(), false));
	}
	else
	{
		Properties.Add(FMovieGraphPinProperties(TEXT("Unknown"), EMovieGraphValueType::None, nullptr, false));
	}
	
	return Properties;
}

FString UMovieGraphVariableNode::GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext) const
{
	if (GraphVariable && (GraphVariable->GetMemberName() == InPinName))
	{
		TObjectPtr<UMovieJobVariableAssignmentContainer> VariableAssignment;
		if (ContextHasEnabledAssignmentForVariable(GraphVariable, InContext, VariableAssignment))
		{
			return VariableAssignment->GetValueSerializedString(GraphVariable);
		}

		// No valid variable assignment: just get the value from the variable
		return GraphVariable->GetValueSerializedString();
	}
	
	return FString();
}

bool UMovieGraphVariableNode::GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext, TObjectPtr<UMovieGraphValueContainer>& OutValueContainer) const
{
	if (GraphVariable && (GraphVariable->GetMemberName() == InPinName))
	{
		TObjectPtr<UMovieJobVariableAssignmentContainer> VariableAssignment;
		if (ContextHasEnabledAssignmentForVariable(GraphVariable, InContext, VariableAssignment))
		{
			return VariableAssignment->GetValueContainer(GraphVariable, OutValueContainer);
		}

		// No valid variable assignment: just get the value from the variable
		OutValueContainer = GraphVariable;
		return true;
	}
	
	return false;
}

UMovieGraphVariable* UMovieGraphVariableNode::GetVariable() const
{
	return GraphVariable;
}

void UMovieGraphVariableNode::SetVariable(UMovieGraphVariable* InVariable)
{
	if (InVariable)
	{
		GraphVariable = InVariable;

		// Update the output pin to reflect the new variable, and update the pin whenever the variable changes
		// (eg, when the variable is renamed)
		UpdateOutputPin(GraphVariable);

		RegisterDelegates();
	}
}

bool UMovieGraphVariableNode::IsGlobalVariable() const
{
	return GraphVariable && GraphVariable->IsGlobal();
}

bool UMovieGraphVariableNode::GetResolvedVariableValue(const TObjectPtr<UMovieGraphVariable> InGraphVariable, const FMovieGraphTraversalContext* InContext, TObjectPtr<UMovieGraphValueContainer>& OutValueContainer)
{
	TObjectPtr<UMovieJobVariableAssignmentContainer> VariableAssignment;
	if (ContextHasEnabledAssignmentForVariable(InGraphVariable.Get(), InContext, VariableAssignment))
	{
		return VariableAssignment->GetValueContainer(InGraphVariable.Get(), OutValueContainer);
	}
	
	// No valid variable assignment: just get the value from the variable
	OutValueContainer = InGraphVariable;
	return true;
}

#if WITH_EDITOR
FText UMovieGraphVariableNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return GraphVariable ? FText::FromString(GraphVariable->GetMemberName()) : FText();
}
	
FText UMovieGraphVariableNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNode", "VariableNode_Category", "Variables");
}

FLinearColor UMovieGraphVariableNode::GetNodeTitleColor() const
{
	static const FLinearColor VariableNodeColor = FLinearColor(0.188f, 0.309f, 0.286f);
	return VariableNodeColor;
}

FSlateIcon UMovieGraphVariableNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon VariableIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.VariableIcon");

	OutColor = FLinearColor::White;
	return VariableIcon;
}
#endif // WITH_EDITOR

void UMovieGraphVariableNode::RegisterDelegates()
{
	Super::RegisterDelegates();

#if WITH_EDITOR
	if (GraphVariable)
	{
		GraphVariable->OnMovieGraphVariableChangedDelegate.RemoveAll(this);
		GraphVariable->OnMovieGraphVariableChangedDelegate.AddUObject(this, &UMovieGraphVariableNode::UpdateOutputPin);
	}
#endif
}

void UMovieGraphVariableNode::UpdateOutputPin(UMovieGraphMember* ChangedVariable) const
{
	if (!OutputPins.IsEmpty() && ChangedVariable)
	{
		// Update the output pin to reflect the variable data model
		OutputPins[0]->Properties.Label = FName(ChangedVariable->GetMemberName());
		OutputPins[0]->Properties.Type = ChangedVariable->GetValueType();
		OutputPins[0]->Properties.TypeObject = ChangedVariable->GetValueTypeObject();
	}

	OnNodeChangedDelegate.Broadcast(this);
}

bool UMovieGraphVariableNode::ContextHasEnabledAssignmentForVariable(const TObjectPtr<UMovieGraphVariable> InGraphVariable, const FMovieGraphTraversalContext* InContext, TObjectPtr<UMovieJobVariableAssignmentContainer>& OutVariableAssignment)
{
	if (!InContext)
	{
		return false;
	}

	TObjectPtr<UMovieJobVariableAssignmentContainer> ShotVariableAssignments = nullptr;
	TObjectPtr<UMovieJobVariableAssignmentContainer> ShotVariableAssignments_PrimaryOverrides = nullptr;
	TObjectPtr<UMovieJobVariableAssignmentContainer> JobVariableAssignments = nullptr;

	const TObjectPtr<UMoviePipelineExecutorJob> PrimaryJob = InContext->Job;
	const TObjectPtr<UMoviePipelineExecutorShot> Shot = InContext->Shot;
	
	if (Shot)
	{
		ShotVariableAssignments = Shot->GetOrCreateJobVariableAssignmentsForGraph(InGraphVariable->GetOwningGraph());
		
		// The shot can also override variables on the primary job's graph (in addition to the shot-level ones fetched/created above)
		if (PrimaryJob)
		{
			constexpr bool bIsForPrimaryOverrides = true;
			ShotVariableAssignments_PrimaryOverrides = Shot->GetOrCreateJobVariableAssignmentsForGraph(InGraphVariable->GetOwningGraph(), bIsForPrimaryOverrides);
		}
	}

	if (PrimaryJob)
	{
		JobVariableAssignments = PrimaryJob->GetOrCreateJobVariableAssignmentsForGraph(InGraphVariable->GetOwningGraph());
	}
	
	// Check the shot job first for an enabled job variable assignment for this variable. Shot jobs take precedence over primary jobs.
	bool bIsEnabled = false;
	if (ShotVariableAssignments && ShotVariableAssignments->GetVariableAssignmentEnableState(InGraphVariable.Get(), bIsEnabled))
	{
		if (bIsEnabled)
		{
			OutVariableAssignment = ShotVariableAssignments;
			return true;
		}
	}

	// Next check for shot-level overrides to the primary graph variables.
	if (ShotVariableAssignments_PrimaryOverrides && ShotVariableAssignments_PrimaryOverrides->GetVariableAssignmentEnableState(InGraphVariable.Get(), bIsEnabled))
	{
		if (bIsEnabled)
		{
			OutVariableAssignment = ShotVariableAssignments_PrimaryOverrides;
			return true;
		}
	}

	// Check the primary job last.
	if (JobVariableAssignments && JobVariableAssignments->GetVariableAssignmentEnableState(InGraphVariable.Get(), bIsEnabled))
	{
		if (bIsEnabled)
		{
			OutVariableAssignment = JobVariableAssignments;
			return true;
		}
	}

	return false;
}
