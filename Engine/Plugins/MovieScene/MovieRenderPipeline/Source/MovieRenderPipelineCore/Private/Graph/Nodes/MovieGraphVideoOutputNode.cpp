// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphVideoOutputNode.h"

#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "ImageWriteTask.h"
#include "Misc/Paths.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineDataTypes.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"
#include "SampleBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphVideoOutputNode)

namespace MovieRenderGraph::Audio
{
	void WriteShotAudioSamples(
		const MoviePipeline::FAudioState& InAudioData,
		const TMap<int32, IVideoCodecWriter::FLightweightSourceData>& InLightweightSourceData,
		const FFrameRate& InOutputFrameRate,
		TFunctionRef<void(TArrayView<int16>)> InWriteSampleCallback)
	{
		// TMap iteration is unordered; sort keys so shots are processed in order.
		TArray<int32> SortedShotKeys;
		InLightweightSourceData.GetKeys(SortedShotKeys);
		SortedShotKeys.Sort();

		struct FShotAudioWork
		{
			int32 NumChannels = 0;
			int32 SampleRate = 0;
			int32 ExpectedSampleCount = 0;  // interleaved sample count
			::Audio::AlignedFloatBuffer Data;
		};
		TArray<FShotAudioWork> Shots;
		Shots.Reserve(SortedShotKeys.Num());

		for (const int32 ShotKey : SortedShotKeys)
		{
			if (!InAudioData.FinishedSegments.IsValidIndex(ShotKey))
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "Invalid shot index was requested for audio data, skipping audio writes.");
				continue;
			}

			const IVideoCodecWriter::FLightweightSourceData& SourceData = InLightweightSourceData[ShotKey];
			const MoviePipeline::FAudioState::FAudioSegment& AudioSegment = InAudioData.FinishedSegments[ShotKey];

			// ToDo: truncates fractional samples when sample rate isn't evenly divisible by frame rate (e.g. 48 kHz / 29.97 fps).
			const int32 SamplesPerFrame = AudioSegment.SampleRate * InOutputFrameRate.AsInterval();

			FShotAudioWork& Shot = Shots.AddDefaulted_GetRef();
			Shot.NumChannels = AudioSegment.NumChannels;
			Shot.SampleRate = AudioSegment.SampleRate;
			Shot.ExpectedSampleCount = SourceData.SubmittedFrameCount * SamplesPerFrame * AudioSegment.NumChannels;
			Shot.Data = AudioSegment.SegmentData;
		}

		// Snapshot each shot's *original* recorded length so the borrow decision uses each shot's real overflow,
		// not the residual after earlier iterations have already moved samples around (which is what caused the
		// shortfall to cascade across all shots in remote renders).
		TArray<int32> OriginalRecordedSampleCount;
		OriginalRecordedSampleCount.Reserve(Shots.Num());
		for (const FShotAudioWork& Shot : Shots)
		{
			OriginalRecordedSampleCount.Add(Shot.Data.Num());
		}

		// Submix recording latency typically makes shot N short by ~24 ms while shot N+1's first ~24 ms is
		// actually shot N's spilled-over tail. The remote/new-process render path has effectively zero submix
		// tail latency: shot N+1's head is its own legitimate audio, not spillover from N. Borrowing in that
		// regime shifts audio relative to picture and creates an audible cut at every shot boundary.
		// Gate the borrow on shot N+1's *original* overflow above its expected length: only that overflow can
		// plausibly be spillover from shot N. If shot N+1 has no overflow, shot N stays short here and is
		// padded with leading silence in the next loop.
		for (int32 i = 0; i + 1 < Shots.Num(); ++i)
		{
			const int32 Shortfall = Shots[i].ExpectedSampleCount - Shots[i].Data.Num();
			if (Shortfall <= 0 || Shots[i].NumChannels <= 0)
			{
				continue;
			}

			const int32 NextShotOriginalOverflow = OriginalRecordedSampleCount[i + 1] - Shots[i + 1].ExpectedSampleCount;
			if (NextShotOriginalOverflow <= 0)
			{
				continue;
			}

			// Snap to a whole multi-channel frame; splitting one across the boundary would
			// channel-misalign the rest of shot[i+1] and decode as noise.
			const int32 NextShotAvailable = Shots[i + 1].Data.Num();
			int32 BorrowCount = FMath::Min3(Shortfall, NextShotOriginalOverflow, NextShotAvailable);
			BorrowCount -= BorrowCount % Shots[i].NumChannels;
			if (BorrowCount > 0)
			{
				Shots[i].Data.Append(Shots[i + 1].Data.GetData(), BorrowCount);
				Shots[i + 1].Data.RemoveAt(0, BorrowCount, EAllowShrinking::No);
			}
		}

		for (const FShotAudioWork& Shot : Shots)
		{
			const int32 WriteCount = FMath::Min(Shot.ExpectedSampleCount, Shot.Data.Num());
			::Audio::TSampleBuffer<int16> SampleBuffer(Shot.Data.GetData(), WriteCount, Shot.NumChannels, Shot.SampleRate);
			InWriteSampleCallback(SampleBuffer.GetArrayView());
		}
	}
}

UMovieGraphVideoOutputNode::UMovieGraphVideoOutputNode()
	: bHasError(false)
{
	FileNameFormat = TEXT("{sequence_name}.{layer_name}");
}

void UMovieGraphVideoOutputNode::OnAllShotFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, const UMoviePipelineExecutorShot* InShot, TObjectPtr<UMovieGraphEvaluatedConfig>& InShotEvaluatedGraph, const bool bFlushToDisk)
{
	constexpr bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSettingsNode =
		InShotEvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);
	
	if (bFlushToDisk)
	{
		// If they don't have {shot_name} or {camera_name} in their output path, this probably doesn't do what they expect
		// as it will finalize and then overwrite itself.
		const FString FullPath = OutputSettingsNode->OutputDirectory.Path / FileNameFormat;
		if (!(FullPath.Contains(TEXT("{shot_name}")) || FullPath.Contains(TEXT("{camera_name}"))))
		{
			UE_LOGF(LogMovieRenderPipelineIO, Warning, "Asked Movie Render Graph to flush file writes to disk after each shot (via Global Output Settings), but filename format doesn't seem to separate video files per shot. This will cause the file to overwrite itself, is this intended?");
		}

		UE_LOGF(LogMovieRenderPipelineIO, Log, "MoviePipelineVideoOutputBase flushing %d tasks to disk...", AllWriters.Num());
		const double FlushBeginTime = FPlatformTime::Seconds();
		
		// Finalize will clear the AllWriters array, so any subsequent requests to write to that shot will try to generate a new file.
		// Note: The evaluated graph provided here is technically incorrect (it's the shot graph instead of the job graph), but these methods
		// don't use the evaluated graph anyway.
		OnAllFramesSubmitted(InPipeline, InShotEvaluatedGraph);
		OnAllFramesFinalized(InPipeline, InShotEvaluatedGraph);

		const float ElapsedS = static_cast<float>(FPlatformTime::Seconds() - FlushBeginTime);
		UE_LOGF(LogMovieRenderPipelineIO, Log, "Finished flushing tasks to disk after %2.2fs!", ElapsedS);
	}
}

void UMovieGraphVideoOutputNode::OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask)
{
	// ------------------------
	// This method is called on the CDO!
	// ------------------------

	// Don't continue processing if there was an error
	if (bHasError)
	{
		return;
	}

	UMovieGraphEvaluatedConfig* EvaluatedConfig = InRawFrameData->EvaluatedConfig.Get();

	// Gather the composited passes (eg, Burn Ins and Widget Renderer) 
	TArray<FMovieGraphPassData> CompositedPasses = GetCompositedPasses(InRawFrameData);

	// Do a first pass to determine which layers/payloads should actually be written by this output node. This needs to be done upfront so we can
	// provide a list of payloads output to this node to the filename disambiguation logic.
	TMap<const UMovieGraphFileOutputNode*, TArray<UE::MovieGraph::FMovieGraphSampleState*>> PayloadsToWrite;
	for (FMovieGraphPassData& RenderPassData : InRawFrameData->ImageOutputData)
	{
		FImagePixelData* RawRenderPassData = RenderPassData.Value.Get();
		UE::MovieGraph::FMovieGraphSampleState* Payload = RawRenderPassData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();

		// This payload may have opted out of being written by this output type.
		if (!Payload->OutputTypeRestrictions.IsEmpty() && !Payload->OutputTypeRestrictions.Contains(FSoftClassPath(GetClass())))
		{
			continue;
		}
		
		// Don't write out a composited pass in this loop, as it will be composited by the encoder and not written separately. 
		if (CompositedPasses.ContainsByPredicate([&RenderPassData](const FMovieGraphPassData& CompositedPass)
			{
				return RenderPassData.Key == CompositedPass.Key;
			}))
		{
			continue;
		}

		// Skip this layer if it's not part of the mask provided (ie, this node isn't part of the branch the layer is on).
		if (!InMask.Contains(RenderPassData.Key))
		{
			continue;
		}

		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = true;
		const UMovieGraphVideoOutputNode* ParentNode = Cast<UMovieGraphVideoOutputNode>(
			InRawFrameData->EvaluatedConfig->GetSettingForBranch(GetClass(), RenderPassData.Key.RootBranchName, bIncludeCDOs, bExactMatch));
		checkf(ParentNode, TEXT("Video output should not exist without a parent node in the graph."));

		PayloadsToWrite.FindOrAdd(ParentNode).Add(Payload);
	}
	
	// NodeInstanceToPayloads needs to be populated before the loop below; it's used in filename disambiguation
	InRawFrameData->NodeInstanceToPayloads.Append(PayloadsToWrite);

	for (FMovieGraphPassData& RenderPassData : InRawFrameData->ImageOutputData)
	{
		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = true;
		const UMovieGraphVideoOutputNode* ParentNode = Cast<UMovieGraphVideoOutputNode>(
			InRawFrameData->EvaluatedConfig->GetSettingForBranch(GetClass(), RenderPassData.Key.RootBranchName, bIncludeCDOs, bExactMatch));
		if (!IsValid(ParentNode))
		{
			continue;
		}

		TArray<UE::MovieGraph::FMovieGraphSampleState*>* EligiblePayloads = PayloadsToWrite.Find(ParentNode);
		if (!EligiblePayloads)
		{
			continue;
		}

		// Only write payloads that have been vetted by the prior loop and placed in NodeInstanceToPayloads.
		FImagePixelData* RawRenderPassData = RenderPassData.Value.Get();
		UE::MovieGraph::FMovieGraphSampleState* Payload = RawRenderPassData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
		if (!EligiblePayloads->Contains(Payload))
		{
			continue;
		}

		// GetOrCreateOutputWriter() can set bHasError, so even if a writer was created, don't continue if there was an error initializing it
		const FMovieGraphCodecWriterWithPromise* OutputWriter = GetOrCreateOutputWriter(InPipeline, InRawFrameData, RenderPassData, CompositedPasses);
		if (!OutputWriter || bHasError)
		{
			// This payload is no longer being written, so remove it from NodeInstanceToPayloads
			InRawFrameData->NodeInstanceToPayloads.Find(ParentNode)->Remove(Payload);
			continue;
		}
		
		// We can have multiple shots in one file, so we find/add the shot in case this is a new shot, and the counter will start over at zero.
		MovieRenderGraph::IVideoCodecWriter::FLightweightSourceData& LightweightData = OutputWriter->CodecWriter->LightweightSourceData.FindOrAdd(Payload->TraversalContext.ShotIndex);
		LightweightData.SubmittedFrameCount++;

		// Pixel data pointer for the cropped result to ensure it stays in memory while the image write is processed
		TUniquePtr<FImagePixelData> CroppedPixelData;
		const bool bIsCropRectValid = !Payload->CropRectangle.IsEmpty();
		const bool bCanCropResolution = RenderPassData.Value->GetSize() == Payload->OverscannedResolution;
		if (ShouldCropOverscanImpl() && bIsCropRectValid && bCanCropResolution)
		{
			switch (RenderPassData.Value->GetType())
			{
			case EImagePixelType::Color:
				{
					TAsyncCropImage<FColor> CropImagePreprocessor(Payload->CropRectangle);
					CropImagePreprocessor(RawRenderPassData);
					CroppedPixelData = MoveTemp(CropImagePreprocessor.OutCroppedImage);
				}
				break;

			case EImagePixelType::Float16:
				{
					TAsyncCropImage<FFloat16Color> CropImagePreprocessor(Payload->CropRectangle);
					CropImagePreprocessor(RawRenderPassData);
					CroppedPixelData = MoveTemp(CropImagePreprocessor.OutCroppedImage);
				}
				break;

			case EImagePixelType::Float32:
				{
					TAsyncCropImage<FLinearColor> CropImagePreprocessor(Payload->CropRectangle);
					CropImagePreprocessor(RawRenderPassData);
					CroppedPixelData = MoveTemp(CropImagePreprocessor.OutCroppedImage);
				}
				break;
			}
			
			RawRenderPassData = CroppedPixelData.Get();
		}
		
		// Write this frame's beauty pass, and pass along other passes so the encoder can composite them
		if (Payload->bIsBeautyPass)
		{
			TArray<FMovieGraphPassData> CompositesForThisCamera;
			for (FMovieGraphPassData& CompositePass : CompositedPasses)
			{
				// This pass may not allow other passes to be composited on it 
				if (!Payload->bAllowsCompositing)
				{
					continue;
				}

				UE::MovieGraph::FMovieGraphSampleState* CompositePayload = CompositePass.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();

				// This composite may have opted out of being written by this output type.
				if (!CompositePayload->OutputTypeRestrictions.IsEmpty() && !CompositePayload->OutputTypeRestrictions.Contains(FSoftClassPath(GetClass())))
				{
					continue;
				}

				// There might be composites across multiple branches; pick out the composites specific to this branch.
				if (RenderPassData.Key.RootBranchName != CompositePass.Key.RootBranchName)
				{
					continue;
				}
				
				// Match them up by camera name so multiple passes intended for different camera names work.
				if (RenderPassData.Key.CameraName == CompositePass.Key.CameraName)
				{
					// Create a new composite pass, otherwise when the main
					// loop tries to check if we should skip writing out this image pass in the future, it fails
					// because we've MoveTemp'd CompositePass and thus the name checks no longer pass.
					FMovieGraphPassData NewPassInfo;
					NewPassInfo.Key = CompositePass.Key;

					// Copy the pixel data if more than one node is using this composited pass
					TUniquePtr<FImagePixelData> PixelData;
					if (GetNumFileOutputNodes(*InRawFrameData->EvaluatedConfig, CompositePass.Key.RootBranchName) > 1)
					{
						NewPassInfo.Value = CompositePass.Value->CopyImageData();
					}
					else
					{
						NewPassInfo.Value = CompositePass.Value->MoveImageDataToNew();
					}
					
					CompositesForThisCamera.Add(MoveTemp(NewPassInfo));
				}
			}

			// Notify the encoder to write this frame
			WriteFrame_EncodeThread(OutputWriter->CodecWriter.Get(), RawRenderPassData, MoveTemp(CompositesForThisCamera), EvaluatedConfig, RenderPassData.Key.RootBranchName.ToString());
		}
		else
		{
			TArray<FMovieGraphPassData> Dummy;
			WriteFrame_EncodeThread(OutputWriter->CodecWriter.Get(), RawRenderPassData, MoveTemp(Dummy), EvaluatedConfig, RenderPassData.Key.RootBranchName.ToString());
		}
	}
}

bool UMovieGraphVideoOutputNode::IsFinishedWritingToDiskImpl() const
{
	// Most encoders should be writing to disk every frame, so by the time we get to this point, the video should be on disk. OnFramesFinalized()
	// will still be called to allow encoders to finish up.
	return true;
}

void UMovieGraphVideoOutputNode::OnAllFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph)
{
	for (const FMovieGraphCodecWriterWithPromise& Writer : AllWriters)
	{
		if (Writer.NodeType == GetClass())
		{
			MovieRenderGraph::IVideoCodecWriter* RawWriter = Writer.CodecWriter.Get();
			BeginFinalize_EncodeThread(RawWriter);
		}
	}
}

void UMovieGraphVideoOutputNode::OnAllFramesFinalizedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph)
{
	for (FMovieGraphCodecWriterWithPromise& Writer : AllWriters)
	{
		if (Writer.NodeType != GetClass())
		{
			continue;
		}
		
		MovieRenderGraph::IVideoCodecWriter* RawWriter = Writer.CodecWriter.Get();

		Finalize_EncodeThread(RawWriter);

		// Note: If there was an error, the promise should already be set to false, so we only set it to true here if there's no error. This isn't
		// ideal and it would be nice to have another way of tracking this, but double-setting the promise will trigger a check().
		if (!bHasError)
		{
			Writer.Promise.SetValue(!bHasError);
		}
	}

	// AllWriters is static and shared across *all* video output node types. Only delete writers of the current node's type.
	AllWriters.RemoveAll([this](const FMovieGraphCodecWriterWithPromise& InWriter)
	{
		return InWriter.NodeType == GetClass();
	});
	
	bHasError = false;
}

UMovieGraphVideoOutputNode::FMovieGraphCodecWriterWithPromise::FMovieGraphCodecWriterWithPromise(TUniquePtr<MovieRenderGraph::IVideoCodecWriter>&& InWriter, TPromise<bool>&& InPromise, UClass* InNodeType)
	: CodecWriter(MoveTemp(InWriter))
	, Promise(MoveTemp(InPromise))
	, NodeType(InNodeType)
{
	
}

void UMovieGraphVideoOutputNode::GetOutputFilePaths(const UMovieGraphPipeline* InPipeline, const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, FMovieGraphPassData& InRenderPassData, const TArray<FMovieGraphPassData>& InCompositedPasses, FString& OutFinalFilePath, FString& OutStableFilePath)
{
	const TObjectPtr<UMoviePipelineExecutorShot> Shot = InRawFrameData->TraversalContext.Shot;
	UMovieGraphEvaluatedConfig* EvaluatedConfig = InRawFrameData->EvaluatedConfig.Get();

	constexpr bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	UMovieGraphGlobalOutputSettingNode* OutputSettingsNode =
		EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);

	const UMovieGraphVideoOutputNode* EvaluatedNode = Cast<UMovieGraphVideoOutputNode>(
		EvaluatedConfig->GetSettingForBranch(GetClass(), InRenderPassData.Key.RootBranchName, bIncludeCDOs, bExactMatch));

	const UE::MovieGraph::FMovieGraphSampleState* Payload = InRenderPassData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();

	// The file name format usually comes from the output node directly, but the payload has a chance to override it.
	FString FileNameFormatString = !Payload->FilenameFormatOverride.IsEmpty() ? Payload->FilenameFormatOverride : EvaluatedNode->FileNameFormat;

	const TMap<FString, FString> FileNameFormatOverrides =  {
		{TEXT("camera_name"), InRenderPassData.Key.CameraName},
		{TEXT("render_pass"), InRenderPassData.Key.RendererName},
		{TEXT("ext"), GetFilenameExtension()},
	};
	FMovieGraphFilenameResolveParams ResolveParams =
		FMovieGraphFilenameResolveParams::MakeResolveParams(InRenderPassData.Key, InPipeline, EvaluatedConfig, InRawFrameData->TraversalContext, FileNameFormatOverrides, Payload);
	ResolveParams.FileNameOverride = FileNameFormatString;
	ResolveParams.Version = Shot ? Shot->ShotInfo.VersionNumber : UMovieGraphBlueprintLibrary::ResolveVersionNumber(ResolveParams);
	
	constexpr bool bIncludeRenderPass = false;
	constexpr bool bTestFrameNumber = false;
	constexpr bool bIncludeCameraName = false;
	UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber, bIncludeCameraName);
	
	FMovieGraphResolveArgs FinalFormatArgs;
	
	// Strip any frame number tags so we don't get one video file per frame.
	UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);

	const FString OutputDirectory = OutputSettingsNode->OutputDirectory.Path;
	FString FullFilepathFormatString = OutputDirectory / FileNameFormatString;

	// Insert tokens like {layer_name} as appropriate to make sure outputs don't clash with each other.
	DisambiguateFilename(FullFilepathFormatString, InRawFrameData, EvaluatedNode, InRenderPassData);

	// This is a bit crummy but we don't have a good way to implement bOverwriteFiles = False for video files. When you don't have the
	// override file flag set, we need to increment the number by 1, and write to that. This works fine for image sequences as image
	// sequences are also differentiated by their {frame_number}, but for video files (which strip it all out), instead we end up
	// with each incoming image sample trying to generate the same filename, finding a file that already exists (from the previous
	// frame) and incrementing again, giving us 1 video file per frame. To work around this, we're going to force bOverwriteExisting
	// off so we can generate a "stable" filename for the incoming image sample, then compare it against existing writers, and
	// if we don't find a writer, then generate a new one (respecting Overwrite Existing) to add the sample to.
	{
		const bool bPreviousOverwriteExisting = OutputSettingsNode->bOverwriteExistingOutput;
		OutputSettingsNode->bOverwriteExistingOutput = true;

		OutStableFilePath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FullFilepathFormatString, ResolveParams, FinalFormatArgs);
		
		// Restore user setting
		OutputSettingsNode->bOverwriteExistingOutput = bPreviousOverwriteExisting;

		if (FPaths::IsRelative(OutStableFilePath))
		{
			OutStableFilePath = FPaths::ConvertRelativePathToFull(OutStableFilePath);
		}
	}

	// Then we add the OutputDirectory, and resolve the filename format arguments again so the arguments in the directory get resolved.
	OutFinalFilePath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FullFilepathFormatString, ResolveParams, FinalFormatArgs);

	if (FPaths::IsRelative(OutFinalFilePath))
	{
		OutFinalFilePath = FPaths::ConvertRelativePathToFull(OutFinalFilePath);
	}

	// Ensure the directory is created
	{
		const FString FolderPath = FPaths::GetPath(OutFinalFilePath);
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		PlatformFile.CreateDirectoryTree(*FolderPath);
	}
}

UMovieGraphVideoOutputNode::FMovieGraphCodecWriterWithPromise* UMovieGraphVideoOutputNode::GetOrCreateOutputWriter(UMovieGraphPipeline* InPipeline, const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, FMovieGraphPassData& InRenderPassData, const TArray<FMovieGraphPassData>& InCompositedPasses)
{
	FMovieGraphCodecWriterWithPromise* OutputWriter = nullptr;

	const UE::MovieGraph::FMovieGraphSampleState* Payload = InRenderPassData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
	UMovieGraphEvaluatedConfig* EvaluatedConfig = InRawFrameData->EvaluatedConfig.Get();
	
	FString FinalFilePath;
	FString StableFilePath;
	GetOutputFilePaths(InPipeline, InRawFrameData, InRenderPassData, InCompositedPasses, FinalFilePath, StableFilePath);
	
	for (int32 Index = 0; Index < AllWriters.Num(); Index++)
	{
		if (AllWriters[Index].CodecWriter->StableFileName == StableFilePath)
		{
			OutputWriter = &AllWriters[Index];
			break;
		}
	}
	
	if (!OutputWriter)
	{
		FIntPoint OutputResolution = InRenderPassData.Value->GetSize();
		
		const bool bIsCropRectValid = !Payload->CropRectangle.IsEmpty();
		const bool bCanCropResolution = InRenderPassData.Value->GetSize() == Payload->OverscannedResolution;
		if (ShouldCropOverscanImpl() && bIsCropRectValid && bCanCropResolution)
		{
			OutputResolution = Payload->CropRectangle.Size();
		}
		
		FMovieGraphVideoNodeInitializationContext InitializationContext;
		InitializationContext.Pipeline = InPipeline;
		InitializationContext.EvaluatedConfig = EvaluatedConfig;
		InitializationContext.TraversalContext = &InRawFrameData->TraversalContext;
		InitializationContext.PassData = &InRenderPassData;
		InitializationContext.Resolution = OutputResolution;
		InitializationContext.FileName = FinalFilePath;
		InitializationContext.bAllowOCIO = Payload->bAllowOCIO;

		// Create a new writer for this file name (and output settings)
		if (TUniquePtr<MovieRenderGraph::IVideoCodecWriter> NewWriter = Initialize_GameThread(InitializationContext))
		{
			// Store the stable filename this was generated with so we can match them up later.
			NewWriter->StableFileName = StableFilePath;
			
			UE::MovieGraph::FMovieGraphOutputFutureData OutputData;
			OutputData.Shot = InPipeline->GetActiveShotList()[Payload->TraversalContext.ShotIndex];
			OutputData.DataIdentifier = InRenderPassData.Key;
			OutputData.FilePath = FinalFilePath;
			OutputData.OriginNodeClass = GetClass();
			OutputData.RenderLayerIndex = Payload->RenderLayerIndex;

			TPromise<bool> Completed;
			InPipeline->AddOutputFuture(Completed.GetFuture(), OutputData);

			AllWriters.Add(FMovieGraphCodecWriterWithPromise(MoveTemp(NewWriter), MoveTemp(Completed), GetClass()));
			OutputWriter = &AllWriters.Last();

			// If it fails to initialize, immediately mark the promise as failed so the render queue stops.
			bHasError = !Initialize_EncodeThread(OutputWriter->CodecWriter.Get());
			if (bHasError)
			{
				OutputWriter->Promise.SetValue(false);
				UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to initialize encoder for FileName: %ls", *FinalFilePath);
			}
		}
	}

	if (!OutputWriter)
	{
		UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to generate writer for FileName: %ls", *FinalFilePath);
	}

	return OutputWriter;
}
