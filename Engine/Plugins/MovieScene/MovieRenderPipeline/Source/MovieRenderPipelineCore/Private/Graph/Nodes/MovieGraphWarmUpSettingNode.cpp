// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphWarmUpSettingNode.h"

#include "MoviePipelineTelemetry.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphWarmUpSettingNode)

UMovieGraphWarmUpSettingNode::UMovieGraphWarmUpSettingNode()
	: NumWarmUpFrames(0)
	, LayerWarmUpFrames(0)
	, bEmulateMotionBlur(false)
{}

EMovieGraphBranchRestriction UMovieGraphWarmUpSettingNode::GetBranchRestriction() const 
{ 
	return EMovieGraphBranchRestriction::Globals; 
}

void UMovieGraphWarmUpSettingNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	if (bOverride_NumWarmUpFrames)
	{
		InTelemetry->WarmUpFrames = FMath::Max(InTelemetry->WarmUpFrames, FMath::Max(NumWarmUpFrames, 0));
	}

	if (bOverride_LayerWarmUpFrames)
	{
		InTelemetry->LayerWarmUpFrames = FMath::Max(InTelemetry->LayerWarmUpFrames, FMath::Max(LayerWarmUpFrames, 0));
	}
}

#if WITH_EDITOR
FText UMovieGraphWarmUpSettingNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText WarmUpSettingsNodeName = NSLOCTEXT("MoviePipelineGraph", "NodeName_WarmUpSettings", "Warm Up Settings");
	return WarmUpSettingsNodeName;
}

FText UMovieGraphWarmUpSettingNode::GetMenuCategory() const 
{
	return NSLOCTEXT("MoviePipelineGraph", "Settings_Category", "Settings");
}

FLinearColor UMovieGraphWarmUpSettingNode::GetNodeTitleColor() const 
{
	static const FLinearColor WarmUpSettingsColor = FLinearColor(0.854f, 0.509f, 0.039f);
	return WarmUpSettingsColor;
}

FSlateIcon UMovieGraphWarmUpSettingNode::GetIconAndTint(FLinearColor& OutColor) const 
{
	static const FSlateIcon SettingsIcon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.WarmUpSettings");

	OutColor = FLinearColor::White;
	return SettingsIcon;
}
#endif // WITH_EDITOR
