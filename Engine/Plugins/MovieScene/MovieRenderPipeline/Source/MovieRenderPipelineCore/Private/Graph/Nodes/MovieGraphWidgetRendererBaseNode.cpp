// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphWidgetRendererBaseNode.h"

#include "Engine/TextureRenderTarget.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphBurnInNode.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Slate/WidgetRenderer.h"
#include "Styling/AppStyle.h"
#include "TextureResource.h"
#include "Widgets/SVirtualWindow.h"
#include "RenderingThread.h"
#include "MoviePipelineQueue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphWidgetRendererBaseNode)

FString UMovieGraphWidgetRendererBaseNode::GetFileNameFormatOverride() const
{
	// By default, there's no override
	return FString();
}

FString UMovieGraphWidgetRendererBaseNode::GetLayerNameOverride() const
{
	// By default, there's no override
	return FString();
}

void UMovieGraphWidgetRendererBaseNode::SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData)
{
	static const FString MultiLayerExrNodeOutputName = TEXT("MULTI_EXR");
	bool bIncludedMultiExrBurnIn = false;

	for (const FMovieGraphRenderPassLayerData& LayerData : InSetupData.Layers)
	{
		// Temp hack: For multi-layer EXRs that include a burn-in pass, ensure that only one burn-in is included if multiple layers are being
		// rendered. The current multi-layer EXR node does not have a way to include {tokens} (like {layer_name}) in the Layer Name Format property,
		// so all burn-in layers will be named the same (and duplicate layer names are not allowed in EXRs). So, for now, restrict the number of
		// burn-ins to one per multi-layer EXR, which is likely what most users want anyway.
		if (const UMovieGraphOutputBurnInNode* OutputBurnInNode = Cast<UMovieGraphOutputBurnInNode>(LayerData.RenderPassNode.Get()))
		{
			if (OutputBurnInNode->OutputName == MultiLayerExrNodeOutputName)
			{
				if (bIncludedMultiExrBurnIn)
				{
					continue;
				}

				bIncludedMultiExrBurnIn = true;
			}
		}

		TUniquePtr<FMovieGraphWidgetPass> RendererInstance = GeneratePass();
		RendererInstance->Setup(InSetupData.Renderer, LayerData);

		CurrentInstances.Add(MoveTemp(RendererInstance));
	}

	constexpr bool bApplyGammaCorrection = false;
	WidgetRenderer = MakeShared<FWidgetRenderer>(bApplyGammaCorrection);
}

void UMovieGraphWidgetRendererBaseNode::TeardownImpl()
{
	for (const TUniquePtr<FMovieGraphWidgetPass>& Instance : CurrentInstances)
	{
		Instance->Teardown();
	}
	
	if (FSlateApplication::IsInitialized())
	{
		for (const TTuple<const FIntPoint, TSharedPtr<SVirtualWindow>>& Entry : SharedVirtualWindows)
		{
			const TSharedPtr<SVirtualWindow>& VirtualWindow = Entry.Value;
			if (VirtualWindow.IsValid())
			{
				FSlateApplication::Get().UnregisterVirtualWindow(VirtualWindow.ToSharedRef());
			}
		}
	}
    
	SharedVirtualWindows.Empty();
	CurrentInstances.Reset();
	WidgetRenderer = nullptr;
}

void UMovieGraphWidgetRendererBaseNode::RenderImpl(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData)
{
	for (const TUniquePtr<FMovieGraphWidgetPass>& Instance : CurrentInstances)
	{
		Instance->Render(InFrameTraversalContext, InTimeData);
	}
}

void UMovieGraphWidgetRendererBaseNode::GatherOutputPassesImpl(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const
{
	for (const TUniquePtr<FMovieGraphWidgetPass>& Instance : CurrentInstances)
    {
    	Instance->GatherOutputPasses(OutExpectedPasses);
    }
}

void UMovieGraphWidgetRendererBaseNode::FMovieGraphWidgetPass::Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, const FMovieGraphRenderPassLayerData& InLayer)
{
	LayerData = InLayer;
	Renderer = InRenderer;

	RenderDataIdentifier.RootBranchName = LayerData.BranchName;
	RenderDataIdentifier.LayerName = LayerData.LayerName;
	RenderDataIdentifier.RendererName = InLayer.RenderPassNode->GetRendererName();

	// NOTE: Subclasses should specify the SubResourceName

	RenderDataIdentifier.CameraName = InLayer.CameraName;
}

void UMovieGraphWidgetRendererBaseNode::FMovieGraphWidgetPass::Teardown()
{
	// Nothing to do
}

UMovieGraphWidgetRendererBaseNode* UMovieGraphWidgetRendererBaseNode::FMovieGraphWidgetPass::GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const
{
	const bool bIncludeCDOs = true;
	UMovieGraphWidgetRendererBaseNode* ParentNode = InConfig->GetSettingForBranch<UMovieGraphWidgetRendererBaseNode>(RenderDataIdentifier.RootBranchName, bIncludeCDOs);
	if (!ensureMsgf(ParentNode, TEXT("FMovieGraphWidgetPass should not exist without parent node in graph.")))
	{
		return nullptr;
	}

	return ParentNode;
}


void UMovieGraphWidgetRendererBaseNode::FMovieGraphWidgetPass::Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData)
{
	const UMovieGraphPipeline* Pipeline = Renderer->GetOwningGraph();
	UMovieGraphWidgetRendererBaseNode* ParentNodeThisFrame = GetParentNode(InTimeData.EvaluatedConfig);

	if(Pipeline->GetActiveShotList()[Pipeline->GetCurrentShotIndex()]->ShotInfo.State != EMovieRenderShotState::Rendering)
	{
		return;
	}

	UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = Renderer->GetCameraInfo(LayerData.CameraIndex);
	// 0.f will preserve original behavior (where aspect ratio is assumed to be the output aspect ratio)
	const float CameraAspectRatio = CameraInfo.bAllowCameraAspectRatio && CameraInfo.ViewInfo.bConstrainAspectRatio ? CameraInfo.ViewInfo.AspectRatio : 0.f;
	const float CameraOverscan = Renderer->GetCameraOverscan(LayerData.CameraIndex);
	const FIntPoint OverscannedResolution = UMovieGraphBlueprintLibrary::GetOverscannedResolution(InTimeData.EvaluatedConfig, CameraOverscan, CameraAspectRatio);
	const FIntRect CropRect = UMovieGraphBlueprintLibrary::GetOverscanCropRectangle(InTimeData.EvaluatedConfig, CameraOverscan, CameraAspectRatio);
	
	// Composited elements should be sized to the original frustum size, as the final image is either cropped to that size, or the composite will be offset
	// to match the original frustum
	const FIntPoint OutputResolution = CropRect.Size();
	const int32 MaxResolution = GetMax2DTextureDimension();
	if ((OutputResolution.X > MaxResolution) || (OutputResolution.Y > MaxResolution))
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Resolution %dx%d exceeds maximum allowed by GPU. Widget renders (burn-ins, etc) do not support high-resolution tiling and thus can't exceed %dx%d.", OutputResolution.X, OutputResolution.Y, MaxResolution, MaxResolution);
		return;
	}
	if (OutputResolution.X == 0 || OutputResolution.Y == 0)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Resolution %dx%d must be greater than zero in both dimensions.", OutputResolution.X, OutputResolution.Y);
		return;
	}

	// Create the render target the widget will be rendered into
	UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams RenderTargetInitParams;
	RenderTargetInitParams.Size = OutputResolution;
	RenderTargetInitParams.TargetGamma = UTextureRenderTarget::GetDefaultDisplayGamma();
	RenderTargetInitParams.PixelFormat = PF_B8G8R8A8;
	RenderTargetInitParams.bIncludeInPreviewWindow = false;
	UTextureRenderTarget2D* RenderTarget = Renderer->GetOrCreateViewRenderTarget(RenderTargetInitParams, RenderDataIdentifier);

	if (InFrameTraversalContext.Time.bIsFirstTemporalSampleForFrame)
	{
		FRenderTarget* BackbufferRenderTarget = RenderTarget->GameThread_GetRenderTargetResource();
		TSharedPtr<UE::MovieGraph::IMovieGraphOutputMerger> OutputMerger = Pipeline->GetOutputMerger();
		
		// The CDO contains the resources which are shared with all pass instances
		UMovieGraphWidgetRendererBaseNode* NodeCDO = ParentNodeThisFrame->GetClass()->GetDefaultObject<UMovieGraphWidgetRendererBaseNode>();
		const TSharedPtr<SVirtualWindow> VirtualWindow = NodeCDO->GetOrCreateVirtualWindow(OutputResolution);

		const TSharedPtr<SWidget> WidgetToRender = GetWidget(ParentNodeThisFrame);
		if (!WidgetToRender)
		{
			// The subclass implementation of GetWidget() is responsible for emitting error messages if getting the widget failed
			return;
		}
		
		// Put the widget in our window
		VirtualWindow->SetContent(WidgetToRender.ToSharedRef());

		// Draw the widget to the render target.  This leaves the texture in SRV state so no transition is needed.
		// Note: If this draw call is updated, be sure to also update the one below.
		NodeCDO->WidgetRenderer->DrawWindow(RenderTarget, VirtualWindow->GetHittestGrid(), VirtualWindow.ToSharedRef(),
			1.f, OutputResolution, InTimeData.FrameDeltaTime);

		// If this is the first frame being rendered, then re-render. This will ensure that all layout computations are correct (for example, large
		// text blocks may not have their layouts computed by the time the first render above occurs, especially if the text is driven by the
		// UMG blueprint).
		if (InFrameTraversalContext.Time.ShotOutputFrameNumber == 0)
		{
			NodeCDO->WidgetRenderer->DrawWindow(RenderTarget, VirtualWindow->GetHittestGrid(), VirtualWindow.ToSharedRef(),
				1.f, OutputResolution, InTimeData.FrameDeltaTime);
		}

		ENQUEUE_RENDER_COMMAND(WidgetRenderTargetResolveCommand)(
			[
				InTimeData,
				InFrameTraversalContext,
				RenderTargetInitParams,
				RenderDataIdentifier = this->RenderDataIdentifier,
				bComposite = ParentNodeThisFrame->bCompositeOntoFinalImage,
				CompositingSortOrder = GetCompositingSortOrder(),
				BackbufferRenderTarget,
				OutputMerger,
				OverscannedResolution,
				CropRect,
				OutputResolution,
				RenderLayerIndex = this->LayerData.LayerIndex,
				OutputTypeRestrictions = ParentNodeThisFrame->GetOutputTypeRestrictions(),
				FileNameFormatOverride = ParentNodeThisFrame->GetFileNameFormatOverride(),
				LayerNameFormatOverride = ParentNodeThisFrame->GetLayerNameOverride()
			](FRHICommandListImmediate& RHICmdList)
			{
				const FIntRect SourceRect = FIntRect(0, 0, BackbufferRenderTarget->GetSizeXY().X, BackbufferRenderTarget->GetSizeXY().Y);

				// Read the data back to the CPU
				TArray<FColor> RawPixels;
				RawPixels.SetNumUninitialized(SourceRect.Width() * SourceRect.Height());

				FReadSurfaceDataFlags ReadDataFlags(ERangeCompressionMode::RCM_MinMax);
				ReadDataFlags.SetLinearToGamma(false);

				{
					// TODO: The readback is taking ~37ms on a 4k image. This is definitely an area that should be a target for optimization in the future.
					TRACE_CPUPROFILER_EVENT_SCOPE(MRQ::FMovieGraphWidgetPass::ReadSurfaceData);
					RHICmdList.ReadSurfaceData(BackbufferRenderTarget->GetRenderTargetTexture(), SourceRect, RawPixels, ReadDataFlags);
				}

				// Take our per-frame Traversal Context and update it with context specific to this sample.
				FMovieGraphTraversalContext UpdatedTraversalContext = InFrameTraversalContext;
				UpdatedTraversalContext.Time = InTimeData;
				UpdatedTraversalContext.RenderDataIdentifier = RenderDataIdentifier;

				TSharedRef<UE::MovieGraph::FMovieGraphSampleState> SampleStatePayload =
					MakeShared<UE::MovieGraph::FMovieGraphSampleState>();
				SampleStatePayload->TraversalContext = MoveTemp(UpdatedTraversalContext);
				SampleStatePayload->OverscannedResolution = OverscannedResolution;
				SampleStatePayload->BackbufferResolution = RenderTargetInitParams.Size;
				SampleStatePayload->CropRectangle = CropRect;
				SampleStatePayload->bRequiresAccumulator = false;
				SampleStatePayload->bFetchFromAccumulator = false;
				SampleStatePayload->bCompositeOnOtherRenders = bComposite;
				SampleStatePayload->CompositingSortOrder = CompositingSortOrder;
				SampleStatePayload->RenderLayerIndex = RenderLayerIndex;
				SampleStatePayload->OutputTypeRestrictions = OutputTypeRestrictions;
				SampleStatePayload->FilenameFormatOverride = FileNameFormatOverride;
				SampleStatePayload->LayerNameOverride = LayerNameFormatOverride;

				TUniquePtr<FImagePixelData> PixelData =
					MakeUnique<TImagePixelData<FColor>>(OutputResolution, TArray64<FColor>(MoveTemp(RawPixels)), SampleStatePayload);

				OutputMerger->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(PixelData));
			});
	}
}

void UMovieGraphWidgetRendererBaseNode::FMovieGraphWidgetPass::GatherOutputPasses(TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const
{
	OutExpectedPasses.Add(RenderDataIdentifier);
}

TSharedPtr<SVirtualWindow> UMovieGraphWidgetRendererBaseNode::GetOrCreateVirtualWindow(const FIntPoint& InResolution)
{
	if (const TSharedPtr<SVirtualWindow>* ExistingVirtualWindow = SharedVirtualWindows.Find(InResolution))
	{
		return *ExistingVirtualWindow;
	}

	const TSharedPtr<SVirtualWindow> NewVirtualWindow = SNew(SVirtualWindow).Size(FVector2D(InResolution.X, InResolution.Y));
	SharedVirtualWindows.Emplace(InResolution, NewVirtualWindow);

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().RegisterVirtualWindow(NewVirtualWindow.ToSharedRef());
	}

	return NewVirtualWindow;
}
