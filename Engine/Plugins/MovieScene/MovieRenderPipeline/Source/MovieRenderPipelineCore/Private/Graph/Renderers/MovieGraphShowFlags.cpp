// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Renderers/MovieGraphShowFlags.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphShowFlags)

UMovieGraphShowFlags::UMovieGraphShowFlags()
	: ShowFlags(ESFIM_Game)
{
	// Using ESFIM_Game as a base, update the show flags based on the serialized state
	CopyOverridesToShowFlags(ShowFlags);
}

FEngineShowFlags UMovieGraphShowFlags::GetShowFlags() const
{
	// A copy is necessary because we keep ShowFlags around as the default (including values set by the renderer as
	// default), and sparse overrides are stored on top of it. The sparse overrides need to be applied to a copy of the
	// default.

	FEngineShowFlags Copy(ShowFlags);
	CopyOverridesToShowFlags(Copy);

	return Copy;
}

bool UMovieGraphShowFlags::IsShowFlagEnabled(const uint32 ShowFlagIndex) const
{
	if (IsShowFlagOverridden(ShowFlagIndex))
	{
		if (const bool* EnableState = ShowFlagEnableState.Find(ShowFlagIndex))
		{
			return *EnableState;
		}
	}

	return ShowFlags.GetSingleFlag(ShowFlagIndex);
}

void UMovieGraphShowFlags::SetShowFlagEnabled(const uint32 ShowFlagIndex, const bool bIsShowFlagEnabled)
{
	if (!IsShowFlagOverridden(ShowFlagIndex))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "SetShowFlagEnabled() call for '%ls' ignored because the flag has not been marked as overridden.", *ShowFlags.FindNameByIndex(ShowFlagIndex));
		return;
	}

#if WITH_EDITOR
	Modify();
#endif

	bool& EnableState = ShowFlagEnableState.FindOrAdd(ShowFlagIndex);
	EnableState = bIsShowFlagEnabled;
}

bool UMovieGraphShowFlags::IsShowFlagOverridden(const uint32 ShowFlagIndex) const
{
	return OverriddenShowFlags.Contains(ShowFlagIndex);
}

void UMovieGraphShowFlags::SetShowFlagOverridden(const uint32 ShowFlagIndex, const bool bIsOverridden)
{
#if WITH_EDITOR
	Modify();
#endif
	
	if (bIsOverridden)
	{
		OverriddenShowFlags.Add(ShowFlagIndex);

		// If the flag has not been marked as overridden in the past, then set the enable state to the default from
		// ShowFlags. Otherwise, marking the flag as overridden will pick up the previously-set enable state.
		if (!ShowFlagEnableState.Contains(ShowFlagIndex))
		{
			ShowFlagEnableState.Add(ShowFlagIndex, ShowFlags.GetSingleFlag(ShowFlagIndex));
		}
	}
	else
	{
		OverriddenShowFlags.Remove(ShowFlagIndex);

		// State from ShowFlagEnableState isn't removed; this is retained to restore the previously-set state if the
		// user re-enables the override
	}
}

const TSet<uint32>& UMovieGraphShowFlags::GetOverriddenShowFlags() const
{
	return OverriddenShowFlags;
}

bool UMovieGraphShowFlags::IsShowFlagSetToDefaultValue(const uint32 ShowFlagIndex) const
{
	const bool* ShowFlagEnableValue = ShowFlagEnableState.Find(ShowFlagIndex);
	
	return !ShowFlagEnableValue ||
		(ShowFlagEnableValue && (ShowFlags.GetSingleFlag(ShowFlagIndex) == *ShowFlagEnableValue)); 
}

void UMovieGraphShowFlags::RevertShowFlagToDefaultValue(const uint32 ShowFlagIndex)
{
#if WITH_EDITOR
	Modify();
#endif
	
	ShowFlagEnableState.Remove(ShowFlagIndex);
}

void UMovieGraphShowFlags::ApplyDefaultShowFlagValue(const uint32 ShowFlagIndex, const bool bShowFlagState)
{
	// If there's no current override on this flag, then the default can be set
	if (!OverriddenShowFlags.Contains(ShowFlagIndex))
	{
		ShowFlags.SetSingleFlag(ShowFlagIndex, bShowFlagState);
	}
}

void UMovieGraphShowFlags::Merge(const IMovieGraphTraversableObject* InSourceObject)
{
	const UMovieGraphShowFlags* InSourceFlags = Cast<UMovieGraphShowFlags>(InSourceObject);
	checkf(InSourceFlags, TEXT("UMovieGraphShowFlags cannot merge with null or an object of another type."));

	for (const uint32& ShowFlagIndex : InSourceFlags->GetOverriddenShowFlags())
	{
		// The graph is iterated in reverse, from Outputs to Inputs. If this property is already overridden (on a
		// object upstream, aka this object), then don't merge in the downstream (InSourceObject) override.
		if (!OverriddenShowFlags.Contains(ShowFlagIndex))
		{
			SetShowFlagOverridden(ShowFlagIndex, true);
			SetShowFlagEnabled(ShowFlagIndex, InSourceFlags->IsShowFlagEnabled(ShowFlagIndex));
		}
	}
}

TArray<TPair<FString, FString>> UMovieGraphShowFlags::GetMergedProperties() const
{
	TArray<TPair<FString, FString>> MergedProperties;

	for (const uint32& ShowFlagIndex : OverriddenShowFlags)
	{
		MergedProperties.Add({ FEngineShowFlags::FindNameByIndex(ShowFlagIndex), LexToString(IsShowFlagEnabled(ShowFlagIndex)) });
	}

	// Sort by show flag name (helps w/ keeping order consistent in the UI)
	MergedProperties.Sort([](const TPair<FString, FString>& Pair1, const TPair<FString, FString>& Pair2)
		{
			return Pair1.Key < Pair2.Key;
		});

	return MergedProperties;
}

void UMovieGraphShowFlags::CopyOverridesToShowFlags(FEngineShowFlags& InEngineShowFlags) const
{
	for (const uint32 ShowFlagIndex : OverriddenShowFlags)
	{
		// ShowFlagEnableState should always have an entry if there's an override, but check just in case
		if (const bool* EnableState = ShowFlagEnableState.Find(ShowFlagIndex))
		{
			InEngineShowFlags.SetSingleFlag(ShowFlagIndex, *EnableState);
		}
	}
}
