// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieJobVariableAssignmentContainer.h"

#include "Graph/MovieGraphCommon.h"
#include "Graph/MovieGraphConfig.h"

// The prefix for property descriptions that are in charge of variable assignment EditCondition enable/disable state

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieJobVariableAssignmentContainer)
static const TCHAR* EditConditionPrefix = TEXT("bOverride_");

void UMovieJobVariableAssignmentContainer::SetGraphConfig(const TSoftObjectPtr<UMovieGraphConfig>& InGraphConfig)
{
	GraphPreset = InGraphConfig;
}

TSoftObjectPtr<UMovieGraphConfig> UMovieJobVariableAssignmentContainer::GetGraphConfig() const
{
	return GraphPreset;
}

uint32 UMovieJobVariableAssignmentContainer::GetNumAssignments() const
{
	uint32 NumAssignments = 0;
	
	if (const UPropertyBag* PropertyBag = Value.GetPropertyBagStruct())
	{
		// Only count the descs that aren't in charge of the enable/disable state of a variable assignment
		for (const FPropertyBagPropertyDesc& Desc : PropertyBag->GetPropertyDescs())
		{
			if (!Desc.Name.ToString().StartsWith(EditConditionPrefix))
			{
				NumAssignments++;
			}
		}
	}

	return NumAssignments;
}

bool UMovieJobVariableAssignmentContainer::GetValueBool(const UMovieGraphVariable* InGraphVariable, bool& bOutValue) const
{
	TValueOrError<bool, EPropertyBagResult> Result = Value.GetValueBool(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<bool>(Result, bOutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueByte(const UMovieGraphVariable* InGraphVariable, uint8& OutValue) const
{
	TValueOrError<uint8, EPropertyBagResult> Result = Value.GetValueByte(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<uint8>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueInt32(const UMovieGraphVariable* InGraphVariable, int32& OutValue) const
{
	TValueOrError<int32, EPropertyBagResult> Result = Value.GetValueInt32(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<int32>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueInt64(const UMovieGraphVariable* InGraphVariable, int64& OutValue) const
{
	TValueOrError<int64, EPropertyBagResult> Result = Value.GetValueInt64(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<int64>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueFloat(const UMovieGraphVariable* InGraphVariable, float& OutValue) const
{
	TValueOrError<float, EPropertyBagResult> Result = Value.GetValueFloat(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<float>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueDouble(const UMovieGraphVariable* InGraphVariable, double& OutValue) const
{
	TValueOrError<double, EPropertyBagResult> Result = Value.GetValueDouble(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<double>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueName(const UMovieGraphVariable* InGraphVariable, FName& OutValue) const
{
	TValueOrError<FName, EPropertyBagResult> Result = Value.GetValueName(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<FName>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueString(const UMovieGraphVariable* InGraphVariable, FString& OutValue) const
{
	TValueOrError<FString, EPropertyBagResult> Result = Value.GetValueString(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<FString>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueText(const UMovieGraphVariable* InGraphVariable, FText& OutValue) const
{
	TValueOrError<FText, EPropertyBagResult> Result = Value.GetValueText(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<FText>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueEnum(const UMovieGraphVariable* InGraphVariable, uint8& OutValue, const UEnum* RequestedEnum) const
{
	TValueOrError<uint8, EPropertyBagResult> Result = Value.GetValueEnum(ConvertVariableToInternalName(InGraphVariable), RequestedEnum);
	return UE::MovieGraph::Private::GetOptionalValue<uint8>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueStruct(const UMovieGraphVariable* InGraphVariable, FStructView& OutValue, const UScriptStruct* RequestedStruct) const
{
	TValueOrError<FStructView, EPropertyBagResult> Result = Value.GetValueStruct(ConvertVariableToInternalName(InGraphVariable), RequestedStruct);
	return UE::MovieGraph::Private::GetOptionalValue<FStructView>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueObject(const UMovieGraphVariable* InGraphVariable, UObject*& OutValue, const UClass* RequestedClass) const
{
	TValueOrError<UObject*, EPropertyBagResult> Result = Value.GetValueObject(ConvertVariableToInternalName(InGraphVariable), RequestedClass);
	return UE::MovieGraph::Private::GetOptionalValue<UObject*>(Result, OutValue);
}

bool UMovieJobVariableAssignmentContainer::GetValueClass(const UMovieGraphVariable* InGraphVariable, UClass*& OutValue) const
{
	TValueOrError<UClass*, EPropertyBagResult> Result = Value.GetValueClass(ConvertVariableToInternalName(InGraphVariable));
	return UE::MovieGraph::Private::GetOptionalValue<UClass*>(Result, OutValue);
}

FString UMovieJobVariableAssignmentContainer::GetValueSerializedString(const UMovieGraphVariable* InGraphVariable)
{
	TValueOrError<FString, EPropertyBagResult> Result = Value.GetValueSerializedString(ConvertVariableToInternalName(InGraphVariable));
	FString ResultString;
	UE::MovieGraph::Private::GetOptionalValue<FString>(Result, ResultString);
	return ResultString;
}

bool UMovieJobVariableAssignmentContainer::SetValueBool(const UMovieGraphVariable* InGraphVariable, const bool bInValue)
{
	return Value.SetValueBool(ConvertVariableToInternalName(InGraphVariable), bInValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueByte(const UMovieGraphVariable* InGraphVariable, const uint8 InValue)
{
	return Value.SetValueByte(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueInt32(const UMovieGraphVariable* InGraphVariable, const int32 InValue)
{
	return Value.SetValueInt32(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueInt64(const UMovieGraphVariable* InGraphVariable, const int64 InValue)
{
	return Value.SetValueInt64(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueFloat(const UMovieGraphVariable* InGraphVariable, const float InValue)
{
	return Value.SetValueFloat(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueDouble(const UMovieGraphVariable* InGraphVariable, const double InValue)
{
	return Value.SetValueDouble(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueName(const UMovieGraphVariable* InGraphVariable, const FName InValue)
{
	return Value.SetValueName(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueString(const UMovieGraphVariable* InGraphVariable, const FString& InValue)
{
	return Value.SetValueString(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueText(const UMovieGraphVariable* InGraphVariable, const FText& InValue)
{
	return Value.SetValueText(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueEnum(const UMovieGraphVariable* InGraphVariable, const uint8 InValue, const UEnum* Enum)
{
	return Value.SetValueEnum(ConvertVariableToInternalName(InGraphVariable), InValue, Enum) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueStruct(const UMovieGraphVariable* InGraphVariable, FConstStructView InValue)
{
	return Value.SetValueStruct(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueObject(const UMovieGraphVariable* InGraphVariable, UObject* InValue)
{
	return Value.SetValueObject(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueClass(const UMovieGraphVariable* InGraphVariable, UClass* InValue)
{
	return Value.SetValueClass(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
}

bool UMovieJobVariableAssignmentContainer::SetValueSerializedString(const UMovieGraphVariable* InGraphVariable, const FString& NewValue)
{
	return Value.SetValueSerializedString(ConvertVariableToInternalName(InGraphVariable), NewValue) == EPropertyBagResult::Success;
}

EMovieGraphValueType UMovieJobVariableAssignmentContainer::GetValueType(const UMovieGraphVariable* InGraphVariable) const
{
	if (const FPropertyBagPropertyDesc* Desc = Value.FindPropertyDescByName(ConvertVariableToInternalName(InGraphVariable)))
	{
		return static_cast<EMovieGraphValueType>(Desc->ValueType);
	}

	return EMovieGraphValueType::None;
}

const UObject* UMovieJobVariableAssignmentContainer::GetValueTypeObject(const UMovieGraphVariable* InGraphVariable) const
{
	if (const FPropertyBagPropertyDesc* Desc = Value.FindPropertyDescByName(ConvertVariableToInternalName(InGraphVariable)))
	{
		return Desc->ValueTypeObject;
	}
	
	return nullptr;
}

EMovieGraphContainerType UMovieJobVariableAssignmentContainer::GetValueContainerType(const UMovieGraphVariable* InGraphVariable) const
{
	if (const FPropertyBagPropertyDesc* Desc = Value.FindPropertyDescByName(ConvertVariableToInternalName(InGraphVariable)))
	{
		return static_cast<EMovieGraphContainerType>(Desc->ContainerTypes.GetFirstContainerType());
	}

	return EMovieGraphContainerType::None;
}

bool UMovieJobVariableAssignmentContainer::GetValueContainer(const UMovieGraphVariable* InGraphVariable, TObjectPtr<UMovieGraphValueContainer>& OutValueContainer)
{
	const FName PropertyName = ConvertVariableToInternalName(InGraphVariable);
	if (const FPropertyBagPropertyDesc* Desc = Value.FindPropertyDescByName(PropertyName))
	{
		// TODO: This object allocation is unfortunate -- it would be nice to find a way to avoid it
		OutValueContainer = NewObject<UMovieGraphValueContainer>();
		OutValueContainer->SetFromDesc(Desc, GetValueSerializedString(InGraphVariable));
		return true;
	}

	return false;
}

bool UMovieJobVariableAssignmentContainer::FindOrGenerateVariableOverride(
	const UMovieGraphVariable* InGraphVariable, FPropertyBagPropertyDesc* OutPropDesc, FPropertyBagPropertyDesc* OutEditConditionPropDesc, bool bGenerateIfNotExists)
{
	if (InGraphVariable)
	{
		if (const FPropertyBagPropertyDesc* Desc = Value.FindPropertyDescByName(FName(InGraphVariable->GetMemberName())))
		{
			if (OutPropDesc)
			{
				*OutPropDesc = *Desc;
			}

			if (OutEditConditionPropDesc)
			{
				const FString EditCondPropName = FString::Format(TEXT("{0}{1}"), {EditConditionPrefix, Desc->ID.ToString()});
				if (const FPropertyBagPropertyDesc* EditCondPropDesc = Value.FindPropertyDescByName(FName(EditCondPropName)))
				{
					*OutEditConditionPropDesc = *EditCondPropDesc;
				}
			}
			
			return true;
		}

		// Didn't find an existing one; add a new assignment
		if (bGenerateIfNotExists)
		{
			return GenerateVariableOverride(InGraphVariable, OutPropDesc, OutEditConditionPropDesc);
		}
	}

	return false;
}

bool UMovieJobVariableAssignmentContainer::GenerateVariableOverride(const UMovieGraphVariable* InGraphVariable,
	FPropertyBagPropertyDesc* OutPropDesc, FPropertyBagPropertyDesc* OutEditConditionPropDesc)
{
	// Create the description for the variable override
	FPropertyBagPropertyDesc NewProperty = FPropertyBagPropertyDesc();
	NewProperty.ValueType = static_cast<EPropertyBagPropertyType>(InGraphVariable->GetValueType());
	NewProperty.ValueTypeObject = InGraphVariable->GetValueTypeObject();
	NewProperty.ContainerTypes = { static_cast<EPropertyBagContainerType>(InGraphVariable->GetValueContainerType()) };
	NewProperty.Name = FName(InGraphVariable->GetMemberName());
	NewProperty.ID = FGuid::NewGuid();
#if WITH_EDITOR
	NewProperty.MetaData.Add(FPropertyBagPropertyDescMetaData(VariableGuidMetaDataKey, InGraphVariable->GetGuid().ToString()));
	NewProperty.MetaData.Add(FPropertyBagPropertyDescMetaData(ToolTipMetaDataKey, InGraphVariable->Description));
	NewProperty.MetaData.Add(FPropertyBagPropertyDescMetaData(CategoryMetaDataKey, InGraphVariable->GetCategory()));
	NewProperty.MetaData.Add(FPropertyBagPropertyDescMetaData(EnableCategoriesMetaDataKey, FString()));
	NewProperty.MetaData.Add(FPropertyBagPropertyDescMetaData(DisplayNameMetaDataKey, InGraphVariable->GetMemberName()));	// Display the raw variable name, not the "friendly" name
#endif

	// Track a separate EditCondition property that can enable/disable the above property. Since the variable can be
	// renamed, base this property's name on the guid rather than the desc name.
	const FString EditCondPropName = FString::Format(TEXT("{0}{1}"), {EditConditionPrefix, NewProperty.ID.ToString()});
	FPropertyBagPropertyDesc NewPropertyEditCondition = FPropertyBagPropertyDesc(FName(EditCondPropName), EPropertyBagPropertyType::Bool);
	NewPropertyEditCondition.ID = FGuid::NewGuid();
#if WITH_EDITOR
	NewPropertyEditCondition.MetaData.Add(FPropertyBagPropertyDescMetaData(VariableGuidMetaDataKey, InGraphVariable->GetGuid().ToString()));
#endif

#if WITH_EDITOR
	NewPropertyEditCondition.MetaData.Add(FPropertyBagPropertyDescMetaData("InlineEditConditionToggle", "true"));
	NewProperty.MetaData.Add(FPropertyBagPropertyDescMetaData("EditCondition", EditCondPropName));
#endif

	*OutPropDesc = NewProperty;
	*OutEditConditionPropDesc = NewPropertyEditCondition;
	
	return true;
}

#if WITH_EDITOR
void UMovieJobVariableAssignmentContainer::UpdateGraphVariableOverrides()
{
	auto GetVariableGuidFromDesc = [](const FPropertyBagPropertyDesc& Desc) -> FGuid
	{
		const FPropertyBagPropertyDescMetaData* Meta = Desc.MetaData.FindByPredicate([](const FPropertyBagPropertyDescMetaData& MetaData)
		{
			return MetaData.Key == VariableGuidMetaDataKey;
		});
		
		if (Meta)
		{
			FGuid VariableGUID;
			if (FGuid::Parse(Meta->Value, VariableGUID))
			{
				return VariableGUID;
			}
		}
		
		return FGuid();
	};
	
	UMovieGraphConfig* LoadedPreset = GraphPreset.LoadSynchronous();
	if (!LoadedPreset)
	{
		return;
	}

	// Get the existing property descs. Modify (update, add, remove) the descs as needed to reflect the variables in the
	// graph preset. If the descs were changed, flag bNeedsToRegenerate to trigger rebuilding the property bag.
	const UPropertyBag* PropertyBag = Value.GetPropertyBagStruct();
	bool bNeedsToRegenerate = false;
	TArray<FPropertyBagPropertyDesc> ModifiedDescs;
	if (PropertyBag)
	{
		ModifiedDescs = PropertyBag->GetPropertyDescs();
	}

	// First, do a sweep to remove any variable overrides which no longer correspond to graph variables. Bag migration will
	// not remove properties from the bag, so this needs to be done as another step.
	if (PropertyBag)
	{
		const int32 NumDescsRemoved = ModifiedDescs.RemoveAll([&GetVariableGuidFromDesc, LoadedPreset](const FPropertyBagPropertyDesc& Desc)
		{
			const FGuid VariableGUID = GetVariableGuidFromDesc(Desc);
		
			const bool bVariableExists = LoadedPreset->GetVariables().ContainsByPredicate([&VariableGUID](const UMovieGraphVariable* Variable)
				{ return Variable && (Variable->GetGuid() == VariableGUID); });

			return !bVariableExists;
		});

		if (NumDescsRemoved > 0)
		{
			bNeedsToRegenerate = true;
		}
	}
	
	TArray<UMovieGraphVariable*> GraphVariables = LoadedPreset->GetVariables();

	// Second, ensure all variables in the bag match all variables in the graph; if something doesn't match,
	// update the desc and flag that the bag needs to be regenerated.
	if (PropertyBag)
	{		
		for (FPropertyBagPropertyDesc& Desc : ModifiedDescs)
		{
			// Skip descs that are responsible for the override checkmark; nothing about them will need to be updated
			if (Desc.Name.ToString().StartsWith(EditConditionPrefix))
			{
				continue;
			}
		
			const FGuid VariableGUID = GetVariableGuidFromDesc(Desc);
		
			UMovieGraphVariable** FoundVariable = GraphVariables.FindByPredicate([&VariableGUID](const UMovieGraphVariable* Variable)
				{ return Variable && (Variable->GetGuid() == VariableGUID); });

			// This shouldn't happen since descs that don't correspond to a real variable were removed above
			if (!ensure(FoundVariable != nullptr))
			{
				continue;
			}

			const UMovieGraphVariable* Variable = *FoundVariable;

			if (Desc.Name != FName(Variable->GetMemberName()))
			{
				bNeedsToRegenerate = true;
				Desc.Name = FName(Variable->GetMemberName());
			}

			if ((uint8)Desc.ValueType != (uint8)(Variable->GetValueType()))
			{
				bNeedsToRegenerate = true;
				Desc.ValueType = static_cast<EPropertyBagPropertyType>(Variable->GetValueType());
			}

			if (Desc.ValueTypeObject != Variable->GetValueTypeObject())
			{
				bNeedsToRegenerate = true;
				Desc.ValueTypeObject = Variable->GetValueTypeObject();
			}

			if ((uint8)Desc.ContainerTypes.GetFirstContainerType() != (uint8)(Variable->GetValueContainerType()))
			{
				bNeedsToRegenerate = true;
				Desc.ContainerTypes = { static_cast<EPropertyBagContainerType>(Variable->GetValueContainerType()) };
			}

			// Check the ToolTip metadata
			FPropertyBagPropertyDescMetaData* ToolTipMetaDataEntry =
				Desc.MetaData.FindByPredicate([](const FPropertyBagPropertyDescMetaData& InMetaDataEntry)
				{
					return InMetaDataEntry.Key == ToolTipMetaDataKey;
				});
			if (ToolTipMetaDataEntry && (ToolTipMetaDataEntry->Value != Variable->Description))
			{
				bNeedsToRegenerate = true;
				ToolTipMetaDataEntry->Value = Variable->Description;
			}

			// Check the Category metadata
			FPropertyBagPropertyDescMetaData* CategoryMetaDataEntry =
				Desc.MetaData.FindByPredicate([](const FPropertyBagPropertyDescMetaData& InMetaDataEntry)
				{
					return InMetaDataEntry.Key == CategoryMetaDataKey;
				});
			if (CategoryMetaDataEntry && (CategoryMetaDataEntry->Value != FName(Variable->GetCategory())))
			{
				bNeedsToRegenerate = true;
				CategoryMetaDataEntry->Value = Variable->GetCategory();
			}
		}
	}

	// Third, ensure that each variable has a corresponding property in the bag. If not, generate one and flag that the
	// bag needs to be regenerated. Any new job variables will get a default value assigned to them that matches the default
	// for the associated graph variable.
	TMap<UMovieGraphVariable*, FName> VariablesThatNeedDefaultAssigned;
	for (UMovieGraphVariable* Variable : GraphVariables)
	{
		bool bFoundMatchingDesc = false;
		
		for (const FPropertyBagPropertyDesc& Desc : ModifiedDescs)
		{
			if (Variable->GetGuid() == GetVariableGuidFromDesc(Desc))
			{
				bFoundMatchingDesc = true;
				break;
			}
		}

		if (!bFoundMatchingDesc)
		{
			// Generate the desc for both the property and the associated EditCondition
			FPropertyBagPropertyDesc NewPropertyDesc;
			FPropertyBagPropertyDesc EditConditionPropertyDesc;
			if (GenerateVariableOverride(Variable, &NewPropertyDesc, &EditConditionPropertyDesc))
			{
				ModifiedDescs.Add(NewPropertyDesc);
				ModifiedDescs.Add(EditConditionPropertyDesc);
				bNeedsToRegenerate = true;

				// Mark this variable as needing a default assigned (property descs do not contain the property value)
				VariablesThatNeedDefaultAssigned.Add({Variable, NewPropertyDesc.Name});
			}
		}
	}

	// Fourth, make sure that the variables are in the correct order. They should reflect the exact order that they're defined within the graph.
	TArray<FGuid> OriginalOrder;
	Algo::Transform(ModifiedDescs, OriginalOrder, [&GetVariableGuidFromDesc](const FPropertyBagPropertyDesc& Desc) { return GetVariableGuidFromDesc(Desc); });
	Algo::SortBy(ModifiedDescs, [&GraphVariables, &GetVariableGuidFromDesc](const FPropertyBagPropertyDesc& Desc)
	{
		// Sort the descs by the order of variable GUIDs within the graph
		return GraphVariables.IndexOfByPredicate([&Desc, &GetVariableGuidFromDesc](const UMovieGraphVariable* Variable)
		{
			return Variable->GetGuid() == GetVariableGuidFromDesc(Desc);
		});
	});
	TArray<FGuid> ChangedOrder;
	Algo::Transform(ModifiedDescs, ChangedOrder, [&GetVariableGuidFromDesc](const FPropertyBagPropertyDesc& Desc) { return GetVariableGuidFromDesc(Desc); });

	if (OriginalOrder != ChangedOrder)
	{
		bNeedsToRegenerate = true;
	}

	if (bNeedsToRegenerate)
	{
		const UPropertyBag* NewBagStruct = UPropertyBag::GetOrCreateFromDescs(ModifiedDescs);
		Value.MigrateToNewBagStruct(NewBagStruct);

		// Also update the default for newly-added job variables to match the associated graph variables
		for (const TPair<UMovieGraphVariable*, FName>& VariableNeedingDefault : VariablesThatNeedDefaultAssigned)
		{
			UMovieGraphVariable* GraphVariable = VariableNeedingDefault.Key;
			const FName VariableName = VariableNeedingDefault.Value;
			
			// Not ideal that this is being set by string, but it's the most universal method of setting all variable types and we don't need
			// to special-case for each variable type
			Value.SetValueSerializedString(VariableName, GraphVariable->GetValueSerializedString());
		}
	}
}
#endif

bool UMovieJobVariableAssignmentContainer::AddVariableAssignment(const UMovieGraphVariable* InGraphVariable)
{
	return InGraphVariable && FindOrGenerateVariableOverride(InGraphVariable);
}

bool UMovieJobVariableAssignmentContainer::SetVariableAssignmentEnableState(const UMovieGraphVariable* InGraphVariable, bool bIsEnabled)
{
	FPropertyBagPropertyDesc PropDesc;
	FPropertyBagPropertyDesc EditConditionPropDesc;
	const bool bAddIfNotExists = false;
	if (FindOrGenerateVariableOverride(InGraphVariable, &PropDesc, &EditConditionPropDesc, bAddIfNotExists))
	{
		const EPropertyBagResult Result = Value.SetValueBool(EditConditionPropDesc.Name, bIsEnabled);
		
		return Result == EPropertyBagResult::Success;
	}

	return false;
}

bool UMovieJobVariableAssignmentContainer::GetVariableAssignmentEnableState(const UMovieGraphVariable* InGraphVariable, bool& bOutIsEnabled)
{
	FPropertyBagPropertyDesc* PropDesc = nullptr;
	FPropertyBagPropertyDesc EditConditionPropDesc;
	const bool bAddIfNotExists = false;
	if (FindOrGenerateVariableOverride(InGraphVariable, PropDesc, &EditConditionPropDesc, bAddIfNotExists))
	{
		TValueOrError<bool, EPropertyBagResult> Result = Value.GetValueBool(EditConditionPropDesc.Name);
		if (Result.HasValue())
		{
			bOutIsEnabled = Result.GetValue();
		}
		
		return Result.HasValue();
	}

	return false;
}

FName UMovieJobVariableAssignmentContainer::ConvertVariableToInternalName(const UMovieGraphVariable* InGraphVariable) const
{
	FPropertyBagPropertyDesc PropDesc;
	FPropertyBagPropertyDesc EditConditionPropDesc;
	const bool bAddIfNotExists = false;

	// FindOrGenerateVariableOverride is non-const (because of the bAddIfNotExists option), but since we never add,
	// we're going to const-cast it so that we can call it from here.
	UMovieJobVariableAssignmentContainer* NonConstThis = const_cast<UMovieJobVariableAssignmentContainer*>(this);
	if (NonConstThis->FindOrGenerateVariableOverride(InGraphVariable, &PropDesc, &EditConditionPropDesc, bAddIfNotExists))
	{
		return PropDesc.Name;
	}

	return NAME_None;
}
