// Copyright Epic Games, Inc. All Rights Reserved.
#include "MoviePipeline.h"
#include "MovieScene.h"
#include "MovieRenderPipelineCoreModule.h"
#include "LevelSequence.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "EngineUtils.h"
#include "Engine/GameViewportClient.h"
#include "MovieSceneSection.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "CanvasTypes.h"
#include "AudioDeviceManager.h"
#include "MovieSceneTimeHelpers.h"
#include "Engine/World.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Modules/ModuleManager.h"
#include "GameFramework/PlayerController.h"
#include "MovieRenderDebugWidget.h"
#include "MoviePipelineShotConfig.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineRenderPass.h"
#include "MoviePipelineOutputBase.h"
#include "ShaderCompiler.h"
#include "ImageWriteStream.h"
#include "MoviePipelineAntiAliasingSetting.h"
#include "MoviePipelineOutputBuilder.h"
#include "DistanceFieldAtlas.h"
#include "UObject/SoftObjectPath.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "ImageWriteQueue.h"
#include "MoviePipelineHighResSetting.h"
#include "MoviePipelineCameraSetting.h"
#include "MoviePipelineDebugSettings.h"
#include "MoviePipelineQueue.h"
#include "HAL/FileManager.h"
#include "Misc/CoreDelegates.h"
#include "MoviePipelineCommandLineEncoder.h"
#include "Kismet/KismetSystemLibrary.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "MoviePipelineTelemetry.h"
#include "MovieSceneCommonHelpers.h"
#include "Compilation/MovieSceneCompiledDataManager.h"

#if WITH_EDITOR
#include "MovieSceneExportMetadata.h"
#endif
#include "Interfaces/Interface_PostProcessVolume.h"

#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipeline)

#define LOCTEXT_NAMESPACE "MoviePipeline"

static TAutoConsoleVariable<int32> CVarMovieRenderPipelineFrameStepper(
	TEXT("MovieRenderPipeline.FrameStepDebug"),
	-1,
	TEXT("How many frames should the Movie Render Pipeline produce before pausing. Set to zero on launch to stall at the first frame. Debug tool.\n")
	TEXT("-1: Don't pause after each frame (default)\n")
	TEXT("0: Process engine ticks but don't progress in the movie rendering pipeline.\n")
	TEXT("1+: Run this many loops of the movie rendering pipeline before pausing again.\n"),
	ECVF_Default);

FString UMoviePipeline::DefaultDebugWidgetAsset = TEXT("/MovieRenderPipeline/Blueprints/UI_MovieRenderPipelineScreenOverlay.UI_MovieRenderPipelineScreenOverlay_C");

UMoviePipeline::UMoviePipeline()
	: CustomTimeStep(nullptr)
	, CachedPrevCustomTimeStep(nullptr)
	, TargetSequence(nullptr)
	, LevelSequenceActor(nullptr)
	, DebugWidget(nullptr)
	, PipelineState(EMovieRenderPipelineState::Uninitialized)
	, CurrentShotIndex(-1)
	, bPrevGScreenMessagesEnabled(true)
	, bHasRunBeginFrameOnce(false)
	, bPauseAtEndOfFrame(false)
	, bShutdownRequested(false)
	, bIsTransitioningState(false)
	, AccumulatedTickSubFrameDeltas(0.f)
	, bHasRenderedFirstViewThisFrame(false)
	, CurrentJob(nullptr)
{
	CustomTimeStep = CreateDefaultSubobject<UMoviePipelineCustomTimeStep>("MoviePipelineCustomTimeStep");
	CustomSequenceTimeController = MakeShared<FMoviePipelineTimeController>();
	OutputBuilder = MakeShared<FMoviePipelineOutputMerger, ESPMode::ThreadSafe>(this);

	ImageWriteQueue = &FModuleManager::Get().LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue").GetWriteQueue();
}


void UMoviePipeline::ValidateSequenceAndSettings() const
{
	// ToDo: 
	// Warn for Blueprint Streaming Levels
}

void UMoviePipeline::Initialize(UMoviePipelineExecutorJob* InJob)
{
	// This function is called after the PIE world has finished initializing, but before
	// the PIE world is ticked for the first time. We'll end up waiting for the next tick
	// for FCoreDelegateS::OnBeginFrame to get called to actually start processing.
	UE_LOGF(LogMovieRenderPipeline, Log, "[%lld] Initializing overall Movie Pipeline", GFrameCounter);

	bPrevGScreenMessagesEnabled = GAreScreenMessagesEnabled;
	GAreScreenMessagesEnabled = false;

	if (!ensureAlwaysMsgf(InJob, TEXT("MoviePipeline cannot be initialized with null job. Aborting.")))
	{
		Shutdown(true);
		return;
	}

	if (!ensureAlwaysMsgf(InJob->GetConfiguration(), TEXT("MoviePipeline cannot be initialized with null configuration. Aborting.")))
	{
		Shutdown(true);
		return;
	}

	{
		// If they have a preset origin set, we  will attempt to load from it and copy it into our configuration.
		// A preset origin is only set if they have not modified the preset using the UI, if they have it will have
		// been copied into the local configuration when it was modified and the preset origin cleared. This resolves 
		// an issue where if a preset asset is updated after this job is made, the job uses the wrong settings because
		//  the UI is the one who updates the configuration from the preset.
		if (InJob->GetPresetOrigin())
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "Job has a primary preset specified, updating local primary configuration from preset.");
			InJob->GetConfiguration()->CopyFrom(InJob->GetPresetOrigin());
		}

		// Now we need to update each shot as well.
		for (UMoviePipelineExecutorShot* Shot : InJob->ShotInfo)
		{
			if (Shot->GetShotOverridePresetOrigin())
			{
				UE_LOGF(LogMovieRenderPipeline, Log, "Shot has a preset specified, updating local override configuraton from preset.");
				Shot->GetShotOverrideConfiguration()->CopyFrom(Shot->GetShotOverridePresetOrigin());
			}
		}
	}
	
	if (!ensureAlwaysMsgf(PipelineState == EMovieRenderPipelineState::Uninitialized, TEXT("Pipeline cannot be reused. Create a new pipeline to execute a job.")))
	{
		Shutdown(true);
		return;
	}

	// Ensure this object has the World as part of its Outer (so that it has context to spawn things)
	if (!ensureAlwaysMsgf(GetWorld(), TEXT("Pipeline does not contain the world as an outer.")))
	{
		Shutdown(true);
		return;
	}

	CurrentJob = InJob;
	
	ULevelSequence* OriginalSequence = Cast<ULevelSequence>(InJob->Sequence.TryLoad());
	if (!ensureAlwaysMsgf(OriginalSequence, TEXT("Failed to load Sequence Asset from specified path, aborting movie render! Attempted to load Path: %s"), *InJob->Sequence.ToString()))
	{
		Shutdown(true);
		return;
	}

	UMoviePipelineDebugSettings* DebugSetting = FindOrAddSettingForShot<UMoviePipelineDebugSettings>(nullptr);
	if (DebugSetting)
	{
		if (DebugSetting->bCaptureUnrealInsightsTrace)
		{
			StartUnrealInsightsCapture();
		}
	}

	TargetSequence = Cast<ULevelSequence>(GetCurrentJob()->Sequence.TryLoad());

	CachedSequenceHierarchyRoot = MakeShared<MoviePipeline::FCameraCutSubSectionHierarchyNode>();
	MoviePipeline::CacheCompleteSequenceHierarchy(TargetSequence, CachedSequenceHierarchyRoot);
	CustomTimeStep->CacheWorldSettings();

	// Override the frame range on the target sequence if needed first before anyone has a chance to modify it.
	{
		UMoviePipelineOutputSetting* OutputSetting = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
		if (OutputSetting->bUseCustomPlaybackRange)
		{
			FFrameNumber StartFrameTickResolution = FFrameRate::TransformTime(FFrameTime(FFrameNumber(OutputSetting->CustomStartFrame)), TargetSequence->GetMovieScene()->GetDisplayRate(), TargetSequence->GetMovieScene()->GetTickResolution()).FloorToFrame();
			FFrameNumber EndFrameTickResolution = FFrameRate::TransformTime(FFrameTime(FFrameNumber(OutputSetting->CustomEndFrame)), TargetSequence->GetMovieScene()->GetDisplayRate(), TargetSequence->GetMovieScene()->GetTickResolution()).CeilToFrame();

			TRange<FFrameNumber> CustomPlaybackRange = TRange<FFrameNumber>(StartFrameTickResolution, EndFrameTickResolution);
#if WITH_EDITOR
			TargetSequence->GetMovieScene()->SetPlaybackRangeLocked(false);
			TargetSequence->GetMovieScene()->SetReadOnly(false);
#endif
			TargetSequence->GetMovieScene()->SetPlaybackRange(CustomPlaybackRange);
		}
	}
	
	// Initialize all of our primary config settings. Shot specific ones will be called for their appropriate shot.
	TArray<UMoviePipelineSetting*> Settings = GetPipelinePrimaryConfig()->GetAllSettings();
	Algo::SortBy(Settings, [](const UMoviePipelineSetting* Setting) { return Setting->GetPriority(); });
	for (UMoviePipelineSetting* Setting : Settings)
	{
		Setting->OnMoviePipelineInitialized(this);
	}

	// Now that we've fixed up the sequence, we're going to build a list of shots that we need
	// to produce in a simplified data structure. The simplified structure makes the flow/debugging easier.
	BuildShotListFromSequence();

	// Now that we've built up the shot list, we're going to run a validation pass on it. This will produce warnings
	// for anything we can't fix that might be an issue - extending sections, etc. This should be const as this
	// validation should re-use what was used in the UI.
	ValidateSequenceAndSettings();

#if WITH_EDITOR
	// Next, initialize the output metadata with the shot list data we just built
	OutputMetadata.Shots.Empty(ActiveShotList.Num());
	for (UMoviePipelineExecutorShot* Shot : ActiveShotList)
	{
		UMoviePipelineOutputSetting* OutputSettings = FindOrAddSettingForShot<UMoviePipelineOutputSetting>(Shot);

		FMovieSceneExportMetadataShot& ShotMetadata = OutputMetadata.Shots.AddDefaulted_GetRef();

		// The XML exporter only supports a root sequence that contains shots, but we support deep hierarchies.
		// To resolve this, we only take the highest sub-sequence node.
		TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> CurNode = Shot->ShotInfo.SubSectionHierarchy;
		while (CurNode)
		{
			// Shorthand for checking if this node has subsequence data for the root level
			if (CurNode->GetParent() && CurNode->GetParent()->GetParent() == nullptr)
			{
				ShotMetadata.MovieSceneShotSection = Cast<UMovieSceneCinematicShotSection>(CurNode->Section);
				break;
			}

			CurNode = CurNode->GetParent();
		}

		ShotMetadata.HandleFrames = OutputSettings->HandleFrameCount;
	}
#endif

	// Finally, we're going to create a Level Sequence Actor in the world that has its settings configured by us.
	// Because this callback is at the end of startup (and before tick) we should be able to spawn the actor
	// and give it a chance to tick once (where it should do nothing) before we start manually manipulating it.
	InitializeLevelSequenceActor();

	// Register any additional engine callbacks needed.
	{
		// Called before the Custom Timestep is updated. This gives us a chance to calculate
		// what we want the frame to look like and then cache that information so that the
		// Custom Timestep doesn't have to perform its own logic.
		FCoreDelegates::OnBeginFrame.AddUObject(this, &UMoviePipeline::OnEngineTickBeginFrame);
		// Called at the end of the frame after everything has been ticked and rendered for the frame.
		FCoreDelegates::OnEndFrame.AddUObject(this, &UMoviePipeline::OnEngineTickEndFrame);
	}

	// Construct a debug UI and bind it to this instance.
	LoadDebugWidget();
	
	if (UGameViewportClient* Viewport = GetWorld()->GetGameViewport())
	{
		Viewport->bDisableWorldRendering = !ViewportInitArgs.bRenderViewport;
	}

	for (ULevelStreaming* Level : GetWorld()->GetStreamingLevels())
	{
		UClass* StreamingClass = Level->GetClass();

		if (StreamingClass == ULevelStreamingDynamic::StaticClass())
		{
			const FString NonPrefixedLevelName = UWorld::StripPIEPrefixFromPackageName(Level->GetWorldAssetPackageName(), GetWorld()->StreamingLevelsPrefix);
			UE_LOGF(LogMovieRenderPipeline, Warning, "Sub-level '%ls' is set to blueprint streaming and will not be visible during a render unless a Sequencer Visibility Track controls its visibility or you have written other code to handle loading it.",
				*NonPrefixedLevelName);
		}
	}

	SetupAudioRendering();
	CurrentShotIndex = 0;
	CachedOutputState.ShotCount = ActiveShotList.Num();

	// Initialization is complete. This engine frame is a wash (because the tick started with a 
	// delta time not generated by us) so we'll wait until the next engine frame to start rendering.
	InitializationTime = FDateTime::UtcNow();
	InitializationTimeOffset = FDateTime::Now() - InitializationTime;

	// We need to start our CachedOutputState at the time the Game World is currently. This way
	// as MRQ counts up time, the render world stays in sync allowing comparisons between last rendered time
	// and game world time.
	CachedOutputState.TimeData.WorldSeconds = GetWorld()->GetTimeSeconds();

	// Resolve the version number for each shot. If the version number comes before the shot name in
	// the format string, all shots will land on the same 'global' version number (as this is done
	// before any file writing, so all shots within this job will calculate the same value). If the
	// version number comes after theo shot name, then each shot may have its own 'local' version number
	FMoviePipelineFilenameResolveParams Params;
	Params.InitializationTime = InitializationTime;
	Params.InitializationTimeOffset = InitializationTimeOffset;
	Params.Job = GetCurrentJob();

	for (UMoviePipelineExecutorShot* Shot : ActiveShotList)
	{
		Params.ShotOverride = Shot;
		Shot->ShotInfo.VersionNumber = UMoviePipelineBlueprintLibrary::ResolveVersionNumber(Params);
	}

	// Warn about zero length playback ranges, often happens because they set the Start/End frame to the same frame.
	const bool bIsEmptyPlaybackRange = TargetSequence->GetMovieScene()->GetPlaybackRange().IsEmpty();
	if (bIsEmptyPlaybackRange)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Playback Range was zero. End Frames are exclusive, did you mean [n, n+1]?");
	}

	// Warn if the shot mask entirely disabled everything.
	const bool bAllShotsDisabled = ActiveShotList.IsEmpty();
	if (bAllShotsDisabled)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "[%lld] No shots detected to render. Either all outside playback range, or disabled via shot mask, bailing.", GFrameCounter);
	}

	// If there's no work to do, shut down.
	if (bIsEmptyPlaybackRange || bAllShotsDisabled)
	{
		// We have to transition twice as Uninitialized -> n state is a no-op, so the second tick will take us to Finished which shuts down.
		TransitionToState(EMovieRenderPipelineState::Export);
		TransitionToState(EMovieRenderPipelineState::Finished);
	}
	else
	{
		TransitionToState(EMovieRenderPipelineState::ProducingFrames);
	}
}

void UMoviePipeline::RestoreTargetSequenceToOriginalState()
{
	if (PipelineState == EMovieRenderPipelineState::Uninitialized)
	{
		return;
	}

	if (!TargetSequence)
	{
		return;
	}

	MoviePipeline::RestoreCompleteSequenceHierarchy(TargetSequence, CachedSequenceHierarchyRoot);
}


void UMoviePipeline::RequestShutdownImpl(bool bIsError)
{
	// It's possible for a previous call to RequestionShutdown to have set an error before this call that may not
	// We don't want to unset a previously set error state
	if (bIsError)
	{
		bFatalError = true;
	}

	// The user has requested a shutdown, it will be read the next available chance and possibly acted on.
	bShutdownRequested = true;
	switch (PipelineState)
	{
		// It is valid to call Shutdown at any point during these two states.
	case EMovieRenderPipelineState::Uninitialized:
	case EMovieRenderPipelineState::ProducingFrames:
		break;
		// You can call Shutdown during these two, but they won't do anything as we're already shutting down at that point.
	case EMovieRenderPipelineState::Finalize:
	case EMovieRenderPipelineState::Export:
		UE_LOGF(LogMovieRenderPipeline, Log, "[GFrameCounter: %lld] Async Shutdown Requested, ignoring due to already being on the way to shutdown.", GFrameCounter);
		break;
	}
}

void UMoviePipeline::ShutdownImpl(bool bIsError)
{
	check(IsInGameThread());

	// We flag this so you can check if the shutdown was requested even when we do a stall-stop.
	bShutdownRequested = true;

	// It's possible for a previous call to RequestionShutdown to have set an error before this call that may not
	// We don't want to unset a previously set error state
	if (bIsError)
	{
		bFatalError = true;
	}

	// This is a blocking operation which abandons any outstanding work to be submitted but finishes
	// the existing work already processed.
	if (PipelineState == EMovieRenderPipelineState::Uninitialized)
	{
		// If initialize is not called, no need to do anything.
		return;
	}
	
	if (PipelineState == EMovieRenderPipelineState::Finished)
	{
		// If already shut down, no need to do anything.
		return;
	}

	if (PipelineState == EMovieRenderPipelineState::ProducingFrames)
	{

		// Teardown the currently active shot (if there is one). This will flush any outstanding rendering
		// work that has already submitted - it cannot be canceled, so we may as well execute it and save the results.
		TransitionToState(EMovieRenderPipelineState::Finalize);

		// Abandon the current frame. When using temporal sampling we may had canceled mid-frame, so the rendering
		// commands were never submitted, thus the output builder will still be expecting a frame to come in.
		if (CachedOutputState.TemporalSampleCount > 1)
		{
			OutputBuilder->AbandonOutstandingWork();
		}
	}

	if (PipelineState == EMovieRenderPipelineState::Finalize)
	{
		// We were either in the middle of writing frames to disk, or we have moved to Finalize as a result of the above block.
		// Tick output containers until they report they have finished writing to disk. This is a blocking operation. 
		// Finalize automatically switches our state to Export so no need to manually transition afterwards.
		TickFinalizeOutputContainers(true);
	}

	if (PipelineState == EMovieRenderPipelineState::Export)
	{
		// All frames have been written to disk but we're doing a post-export step (such as encoding). Flush this operation as well.
		// Export automatically switches our state to Finished so no need to manually transition afterwards.
		TickPostFinalizeExport(true);
	}
}

void UMoviePipeline::TransitionToState(const EMovieRenderPipelineState InNewState)
{
	// No re-entrancy. This isn't an error as tearing down a shot may try to move to
	// Finalize on its own, but we don't want that.
	if (bIsTransitioningState)
	{
		return;
	}

	TGuardValue<bool> StateTransitionGuard(bIsTransitioningState, true);

	bool bInvalidTransition = true;
	switch (PipelineState)
	{
	case EMovieRenderPipelineState::Uninitialized:
		PipelineState = InNewState;
		bInvalidTransition = false;
		break;
	case EMovieRenderPipelineState::ProducingFrames:
		if (InNewState == EMovieRenderPipelineState::Finalize)
		{
			bInvalidTransition = false;

			// If we had naturally finished the last shot before doing this transition it will have
			// already been torn down, so this only catches mid-shot transitions to ensure teardown.
			// We also check that the shot was actually initialized (State != Uninitialized) to avoid
			// restoring hierarchy state that was never saved. This can happen when PIE ends on the same
			// frame as MRQ initialization, before the first TickProducingFrames/InitializeShot call.
			if (CurrentShotIndex < ActiveShotList.Num()
				&& ActiveShotList[CurrentShotIndex]->ShotInfo.IsInitialized())
			{
				// Ensures all in-flight work for that shot is handled.
				TeardownShot(ActiveShotList[CurrentShotIndex]);
			}

			// Unregister our OnEngineTickEndFrame delegate. We don't unregister BeginFrame as we need
			// to continue to call it to allow ticking the Finalization stage.
			FCoreDelegates::OnEndFrame.RemoveAll(this);

			// Reset the Custom Timestep because we don't care how long the engine takes now
			GEngine->SetCustomTimeStep(CachedPrevCustomTimeStep);

			// Ensure all frames have been processed by the GPU and sent to the Output Merger
			FlushRenderingCommands();

			// And then make sure all frames are sent to the Output Containers before we finalize.
			ProcessOutstandingFinishedFrames();

			PreviewTexture = nullptr;

			// This is called once notifying output containers that all frames that will be submitted have been submitted.
			PipelineState = EMovieRenderPipelineState::Finalize;
			BeginFinalize();
		}
		break;
	case EMovieRenderPipelineState::Finalize:
		if (InNewState == EMovieRenderPipelineState::Export)
		{
			bInvalidTransition = false;

			// This is called once notifying our export step that they can begin the export.
			PipelineState = EMovieRenderPipelineState::Export;

			// Restore the sequence so that the export processes can operate on the original sequence. 
			// This is also done in the finished state because it's not guaranteed that the Export state 
			// will be set when the render is canceled early
			LevelSequenceActor->GetSequencePlayer()->Stop();
			RestoreTargetSequenceToOriginalState();

			// Ensure all of our Futures have been converted to the GeneratedOutputData. This has to happen
			// after finalize finishes, because the futures won't be available until actually written to disk.
			ProcessOutstandingFutures();
	
			BeginExport();

			// Clear out MRQ tick information for external consumers as we are done
			FMovieRenderPipelineCoreModule::SetTickInfo(FMoviePipelineLightweightTickInfo());
		}
		break;
	case EMovieRenderPipelineState::Export:
		if (InNewState == EMovieRenderPipelineState::Finished)
		{
			bInvalidTransition = false;
			PipelineState = InNewState;

			// Uninitialize our primary config settings. Reverse sorted so settings that cached values restore correctly.
			TArray<UMoviePipelineSetting*> Settings = GetPipelinePrimaryConfig()->GetAllSettings();
			Algo::SortBy(Settings, [](const UMoviePipelineSetting* Setting) { return Setting->GetPriority(); },  TLess<int32>());
			for (UMoviePipelineSetting* Setting : Settings)
			{
				Setting->OnMoviePipelineShutdown(this);
			}

			// Restore any custom Time Step that may have been set before.
			GEngine->SetCustomTimeStep(CachedPrevCustomTimeStep);

			// Ensure our delegates don't get called anymore as we're going to become null soon.
			FCoreDelegates::OnBeginFrame.RemoveAll(this);
			FCoreDelegates::OnEndFrame.RemoveAll(this);

			if (DebugWidget)
			{
				DebugWidget->RemoveFromParent();
				DebugWidget = nullptr;
			}

			TArray<UMoviePipelineOutputBase*> ContainerSettings = GetPipelinePrimaryConfig()->GetOutputContainers();
			Algo::SortBy(ContainerSettings, [](const UMoviePipelineOutputBase* Setting) { return Setting->GetPriority(); });
			for (UMoviePipelineOutputBase* Setting : ContainerSettings)
			{
				Setting->OnPipelineFinished();
			}

			TeardownAudioRendering();
			LevelSequenceActor->GetSequencePlayer()->Stop();
			RestoreTargetSequenceToOriginalState();

			if (UGameViewportClient* Viewport = GetWorld()->GetGameViewport())
			{
				Viewport->bDisableWorldRendering = false;
			}

			// Because the render target pool is shared, if you had a high-resolution render in editor the entire gbuffer
			// has been resized up to match the new maximum extent. This console command will reset the size of the pool
			// and cause it to re-allocate at the currrent size on the next render request, which is likely to be the size
			// of the PIE window (720p) or the Viewport itself.
			UKismetSystemLibrary::ExecuteConsoleCommand(GetWorld(), TEXT("r.ResetRenderTargetsExtent"), nullptr);
			CustomTimeStep->RestoreCachedWorldSettings();

			GAreScreenMessagesEnabled = bPrevGScreenMessagesEnabled;

			UE_LOGF(LogMovieRenderPipeline, Log, "Movie Pipeline completed. Duration: %ls", *(FDateTime::UtcNow() - InitializationTime).ToString());

			UMoviePipelineDebugSettings* DebugSetting = FindOrAddSettingForShot<UMoviePipelineDebugSettings>(nullptr);
			if (DebugSetting)
			{
				if (DebugSetting->bCaptureUnrealInsightsTrace)
				{
					StopUnrealInsightsCapture();
				}
			}

			OnMoviePipelineFinishedImpl();
		}
		break;
	}

	if (!ensureAlwaysMsgf(!bInvalidTransition, TEXT("[GFrameCounter: %lld] An invalid transition was requested (from: %d to: %d), ignoring transition request."),
		GFrameCounter, EnumToUnderlyingType(PipelineState), EnumToUnderlyingType(InNewState)))
	{
		return;
	}


}


void UMoviePipeline::OnEngineTickBeginFrame()
{
	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "OnEngineTickBeginFrame (Start) Engine Frame: [%lld]", GFrameCounter);
	
	// We should have a custom timestep set up by now.
	check(CustomTimeStep);

	switch (PipelineState)
	{
	case EMovieRenderPipelineState::Uninitialized:
		// We shouldn't register this delegate until we're initialized.
		check(false);
		break;
	case EMovieRenderPipelineState::ProducingFrames:
		TickProducingFrames();
		break;
	case EMovieRenderPipelineState::Finalize:
		// Don't flush the finalize to keep the UI responsive.
		TickFinalizeOutputContainers(false);
		break;
	case EMovieRenderPipelineState::Export:
		// Don't flush the export to keep the UI responsive.
		TickPostFinalizeExport(false);
		break;
	}

	bHasRunBeginFrameOnce = true;
	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "OnEngineTickBeginFrame (End) Engine Frame: [%lld]", GFrameCounter);
}

void UMoviePipeline::OnSequenceEvaluated(const UMovieSceneSequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime)
{
	// This callback exists for logging purposes. DO NOT HINGE LOGIC ON THIS CALLBACK
	// because this may get called multiple times per frame and may be the result of
	// a seek operation which is reverted before a frame is even rendered.
	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "[GFrameCounter: %lld] Sequence Evaluated. CurrentTime: %ls PreviousTime: %ls", GFrameCounter, *LexToString(CurrentTime), *LexToString(PreviousTime));
}

void UMoviePipeline::OnEngineTickEndFrame()
{
	LLM_SCOPE_BYNAME(TEXT("MoviePipeline"));

	// Unfortunately, since we can't control when our Initialization function is called
	// we can end up in a situation where this callback is registered but the matching
	// OnEngineTickBeginFrame() hasn't been called for that given engine tick. Instead of
	// changing this registration to hang off of the end of the first OnEngineTickBeginFrame()
	// we instead just early out here if that hasn't actually been called once. This decision
	// is designed to minimize places where callbacks are registered and where flow changes.
	if (!bHasRunBeginFrameOnce)
	{
		return;
	}

	// Early out if we're idling as we don't want to process a frame. This prevents us from
	// overwriting render state when the engine is processing ticks but we're not trying to
	// change the evaluation. 
	if (IsDebugFrameStepIdling())
	{
		return;
	}

	// It is important that there is no early out that skips hitting this
	// (Otherwise we don't pause on the frame we transition from step -> idle
	// and the world plays even though the state is frozen).
	if (bPauseAtEndOfFrame)
	{
		GetWorld()->GetFirstPlayerController()->SetPause(true);
		bPauseAtEndOfFrame = false;
	}

	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "OnEngineTickEndFrame (Start) Engine Frame: [%lld]", GFrameCounter);

	ProcessAudioTick();
	RenderFrame();

	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "OnEngineTickEndFrame (End) Engine Frame: [%lld]", GFrameCounter);
}

void UMoviePipeline::ProcessEndOfCameraCut(UMoviePipelineExecutorShot* InCameraCut)
{
	UE_LOGF(LogMovieRenderPipeline, Log, "[%lld] Finished processing Camera Cut [%d/%d].", GFrameCounter, CurrentShotIndex + 1, ActiveShotList.Num());
	InCameraCut->ShotInfo.State = EMovieRenderShotState::Finished;

	// We pause at the end too, just so that frames during finalize don't continue to trigger Sequence Eval messages.
	LevelSequenceActor->GetSequencePlayer()->Pause();

	TeardownShot(InCameraCut);
}

void UMoviePipeline::BeginFinalize()
{
	// Notify all of our output containers that we have finished producing and
	// submitting all frames to them and that they should start any async flushes.
	TArray<UMoviePipelineOutputBase*> Settings = GetPipelinePrimaryConfig()->GetOutputContainers();
	Algo::SortBy(Settings, [](const UMoviePipelineOutputBase* Setting) { return Setting->GetPriority(); });

	for (UMoviePipelineOutputBase* Container : Settings)
	{
		Container->BeginFinalize();
	}
}

void UMoviePipeline::BeginExport()
{
	TArray<UMoviePipelineSetting*> Settings = GetPipelinePrimaryConfig()->GetAllSettings();
	Algo::SortBy(Settings, [](const UMoviePipelineSetting* Setting) { return Setting->GetPriority(); });
	for (UMoviePipelineSetting* Setting : Settings)
	{
		Setting->BeginExport();
	}
}

void UMoviePipeline::TickFinalizeOutputContainers(const bool bInForceFinish)
{
	// Tick all containers until they all report that they have finalized.
	bool bAllContainsFinishedProcessing;

	do
	{
		bAllContainsFinishedProcessing = true;

		// Ask the containers if they're all done processing.
		for (UMoviePipelineOutputBase* Container : GetPipelinePrimaryConfig()->GetOutputContainers())
		{
			bAllContainsFinishedProcessing &= Container->HasFinishedProcessing();
		}

		// If we aren't forcing a finish, early out after one loop to keep
		// the editor/ui responsive.
		if (!bInForceFinish || bAllContainsFinishedProcessing)
		{
			break;
		}

		// If they've reached here, they're forcing them to finish so we'll sleep for a touch to give
		// everyone a chance to actually do work before asking them if they're done.
		FPlatformProcess::Sleep(1.f);

	} while (true);

	// If an output container is still working, we'll early out to keep the UI responsive.
	// If they've forced a finish this will have to be true before we can reach this block.
	if (!bAllContainsFinishedProcessing)
	{
		return;
	}

	TArray<UMoviePipelineOutputBase*> Settings = GetPipelinePrimaryConfig()->GetOutputContainers();
	Algo::SortBy(Settings, [](const UMoviePipelineOutputBase* Setting) { return Setting->GetPriority(); });
	for (UMoviePipelineOutputBase* Container : Settings)
	{
		// All containers have finished processing, final shutdown.
		Container->Finalize();
	}

	TransitionToState(EMovieRenderPipelineState::Export);
}

void UMoviePipeline::TickPostFinalizeExport(const bool bInForceFinish)
{
	// This step assumes you have produced data and filled the data structures.
	check(PipelineState == EMovieRenderPipelineState::Export);
	UE_LOGF(LogMovieRenderPipeline, Verbose, "[%lld] PostFinalize Export (Start).", GFrameCounter);

	// ToDo: Loop through any extensions (such as XML export) and let them export using all of the
	// data that was generated during this run such as containers, output names and lengths.
	// Tick all containers until they all report that they have finalized.
	bool bAllContainsFinishedProcessing;

	do
	{
		bAllContainsFinishedProcessing = true;

		// Ask the containers if they're all done processing.
		for (UMoviePipelineSetting* Setting : GetPipelinePrimaryConfig()->GetAllSettings())
		{
			bAllContainsFinishedProcessing &= Setting->HasFinishedExporting();
		}
		
		// If we aren't forcing a finish, early out after one loop to keep
		// the editor/ui responsive.
		if (!bInForceFinish || bAllContainsFinishedProcessing)
		{
			break;
		}

		// If they've reached here, they're forcing them to finish so we'll sleep for a touch to give
		// everyone a chance to actually do work before asking them if they're done.
		FPlatformProcess::Sleep(1.f);

	} while (true);

	UE_LOGF(LogMovieRenderPipeline, Verbose, "[%lld] PostFinalize Export (End).", GFrameCounter);

	// If an output container is still working, we'll early out to keep the UI responsive.
	// If they've forced a finish this will have to be true before we can reach this block.
	if (!bAllContainsFinishedProcessing)
	{
		return;
	}

	TransitionToState(EMovieRenderPipelineState::Finished);
}

bool UMoviePipelineCustomTimeStep::UpdateTimeStep(UEngine* /*InEngine*/)
{
	if (ensureMsgf(!FMath::IsNearlyZero(TimeCache.UndilatedDeltaTime), TEXT("An incorrect or uninitialized time step was used! Delta Time of 0 isn't allowed.")))
	{
		FApp::UpdateLastTime();
		FApp::SetDeltaTime(TimeCache.UndilatedDeltaTime);
		FApp::SetCurrentTime(FApp::GetCurrentTime() + FApp::GetDeltaTime());

		// The UWorldSettings can clamp the delta time inside the Level Tick function, this creates a mismatch between component
		// velocity and render thread velocity and becomes an issue at high temporal sample counts. 
		if (AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings())
		{
			WorldSettings->MinUndilatedFrameTime = TimeCache.UndilatedDeltaTime;
			WorldSettings->MaxUndilatedFrameTime = TimeCache.UndilatedDeltaTime;
		}
	}

	// Clear our cached time to ensure we're always explicitly telling this what to do and never relying on the last set value.
	// (This will cause the ensure above to check on the next tick if someone didn't reset our value.)
	TimeCache = MoviePipeline::FFrameTimeStepCache();

	// Return false so the engine doesn't run its own logic to overwrite FApp timings.
	return false;
}

void UMoviePipelineCustomTimeStep::CacheWorldSettings()
{
	if (AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings())
	{
		PrevMinUndilatedFrameTime = WorldSettings->MinUndilatedFrameTime;
		PrevMaxUndilatedFrameTime = WorldSettings->MaxUndilatedFrameTime;
	}
}

void UMoviePipelineCustomTimeStep::RestoreCachedWorldSettings()
{
	if (AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings())
	{
		WorldSettings->MinUndilatedFrameTime = PrevMinUndilatedFrameTime;
		WorldSettings->MaxUndilatedFrameTime = PrevMaxUndilatedFrameTime;
	}
}

void UMoviePipelineCustomTimeStep::SetCachedFrameTiming(const MoviePipeline::FFrameTimeStepCache& InTimeCache)
{ 
	if (ensureMsgf(!FMath::IsNearlyZero(InTimeCache.UndilatedDeltaTime), TEXT("An incorrect or uninitialized time step was used! Delta Time of 0 isn't allowed.")))
	{
		TimeCache = InTimeCache;
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "SetCachedFrameTiming called with zero delta time, falling back to 1/24");
		TimeCache = MoviePipeline::FFrameTimeStepCache(1 / 24.0);
	}
}

void UMoviePipeline::ModifySequenceViaExtensions(ULevelSequence* InSequence)
{
}


void UMoviePipeline::InitializeLevelSequenceActor()
{
	// There is a reasonable chance that there exists a Level Sequence Actor in the world already set up to play this sequence.
	ALevelSequenceActor* ExistingActor = nullptr;

	for (auto It = TActorIterator<ALevelSequenceActor>(GetWorld()); It; ++It)
	{
		// Iterate through all of them in the event someone has multiple copies in the world on accident.
		if (It->GetSequence() == TargetSequence)
		{
			// Found it!
			ExistingActor = *It;

			// Stop it from playing if it's already playing.
			if (ExistingActor->GetSequencePlayer())
			{
				ExistingActor->GetSequencePlayer()->Stop();
			}
		}
	}

	LevelSequenceActor = ExistingActor;
	if (!LevelSequenceActor)
	{
		// Spawn a new level sequence
		LevelSequenceActor = GetWorld()->SpawnActor<ALevelSequenceActor>();
		check(LevelSequenceActor);
	}
	
	// Enforce settings.
	LevelSequenceActor->PlaybackSettings.LoopCount.Value = 0;
	LevelSequenceActor->PlaybackSettings.bAutoPlay = false;
	LevelSequenceActor->PlaybackSettings.bPauseAtEnd = true;
	LevelSequenceActor->PlaybackSettings.FinishCompletionStateOverride = EMovieSceneCompletionModeOverride::ForceRestoreState;

#if !WITH_EDITOR
	// In shipping builds the Level Sequence data is assumed to be non-volatile. This prevents MRQ from automatically
	// expanding sections (for temporal sub-samples, etc.). To solve this, we want to convert the user provided level
	// sequence to be volatile, but unfortunately there's a couple of gotchas in the Sequencer API for doing that at runtime.
	ULevelSequence* DummySequence = NewObject<ULevelSequence>(this, NAME_None, RF_Transient);
	DummySequence->Initialize();
	
	// We need to create a dummy level sequence and specifically tell the LevelSequenceActor to switch to it. This is required
	// because once we reset the precompiled data for the level sequence, if the LevelSequenceActor was already pointed to this
	// sequence, then calling SetSequence again doesn't recompile the necessary data.
	LevelSequenceActor->SetSequence(DummySequence);
	
	// Ensure the Volatile flag is set on the user provided level sequence.
	TargetSequence->SetSequenceFlags(TargetSequence->GetFlags() | EMovieSceneSequenceFlags::Volatile);

	// Reset the data in the Precompiled Data Manager so that it fetches the new Volatile value from the 
	// sequence flags (otherwise it will used the cached data in the compiled hierarchy, which doesn't
	// have the flag set).	
	UMovieSceneCompiledDataManager::GetPrecompiledData()->Reset(TargetSequence);
#endif

	LevelSequenceActor->SetSequence(TargetSequence);

	LevelSequenceActor->GetSequencePlayer()->SetTimeController(CustomSequenceTimeController);
	LevelSequenceActor->GetSequencePlayer()->Stop();

	LevelSequenceActor->GetSequencePlayer()->OnSequenceUpdated().AddUObject(this, &UMoviePipeline::OnSequenceEvaluated);
}

void UMoviePipeline::BuildShotListFromSequence()
{
	// Synchronize our shot list with our target sequence. New shots will be added and outdated shots removed.
	// Shots that are already in the list will be updated but their enable flag will be respected.
	bool bShotsChanged = false;
	UMoviePipelineBlueprintLibrary::UpdateJobShotListFromSequence(TargetSequence, GetCurrentJob(), bShotsChanged);

	for (UMoviePipelineExecutorShot* Shot : GetCurrentJob()->ShotInfo)
	{
		// We need to run a pre-pass on shot expansion. This doesn't actually expand the data  in pre-pass mode, but it
		// runs the calculations like it would and updates our metrics so that frame-count estimates are correct later.
		// We do the actual expansion of each shot right before rendering.
		UMoviePipelineOutputSetting* OutputSettings = FindOrAddSettingForShot<UMoviePipelineOutputSetting>(Shot);
		UMoviePipelineAntiAliasingSetting* AntiAliasingSettings = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(Shot);
		UMoviePipelineHighResSetting* HighResSettings = FindOrAddSettingForShot<UMoviePipelineHighResSetting>(Shot);

		// This info is read in ExpandShot so needs to be set first
		Shot->ShotInfo.NumTemporalSamples = AntiAliasingSettings->TemporalSampleCount;
		Shot->ShotInfo.NumSpatialSamples = AntiAliasingSettings->SpatialSampleCount;
		Shot->ShotInfo.CachedFrameRate = GetPipelinePrimaryConfig()->GetEffectiveFrameRate(TargetSequence);
		Shot->ShotInfo.CachedTickResolution = TargetSequence->GetMovieScene()->GetTickResolution();
		Shot->ShotInfo.CachedShotTickResolution = Shot->ShotInfo.CachedTickResolution;
		if (Shot->ShotInfo.SubSectionHierarchy.IsValid() && Shot->ShotInfo.SubSectionHierarchy->MovieScene.IsValid())
		{
			Shot->ShotInfo.CachedShotTickResolution = Shot->ShotInfo.SubSectionHierarchy->MovieScene->GetTickResolution();
		}
		Shot->ShotInfo.NumTiles = FIntPoint(HighResSettings->TileCount, HighResSettings->TileCount);

		// Expand the shot (but don't actually modify the sections)
		const bool bPrePass = true;
		ExpandShot(Shot, OutputSettings->HandleFrameCount, bPrePass);

		bool bUseCameraCutForWarmUp = AntiAliasingSettings->bUseCameraCutForWarmUp;
		if (Shot->ShotInfo.NumEngineWarmUpFramesRemaining == 0 && bUseCameraCutForWarmUp)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Shot was asked to use excess Camera Cut section data for warm-up but no warmup range was detected. Extend the Camera Cut section to the left for this shot or disable bUseCameraCutForWarmUp to resolve this issue.");
			// If they don't have enough data for warmup (no camera cut extended track) fall back to emulated warmup.
			bUseCameraCutForWarmUp = false;
		}

		// Warm Up Frames. If there are any render samples we require at least one engine warm up frame.
		int32 NumWarmupFrames = bUseCameraCutForWarmUp ? Shot->ShotInfo.NumEngineWarmUpFramesRemaining : AntiAliasingSettings->EngineWarmUpCount;
		Shot->ShotInfo.NumEngineWarmUpFramesRemaining = FMath::Max(NumWarmupFrames, AntiAliasingSettings->RenderWarmUpCount > 0 ? 1 : 0);

		// When using real warmup we don't emulate a first frame motion blur as we actually have real data.
		Shot->ShotInfo.bEmulateFirstFrameMotionBlur = !bUseCameraCutForWarmUp;
		Shot->ShotInfo.CalculateWorkMetrics();

		// When we expanded the shot above, it pushed the first/last camera cuts ranges to account for Handle Frames.
		// We want to start rendering from the first handle frame. Shutter Timing is a fixed offset from this number.
		Shot->ShotInfo.CurrentTickInRoot = Shot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue();
	}

	// The active shot-list is a subset of the whole shot-list; The ShotInfo contains information about every range it detected to render
	// but if the user has turned the shot off in the UI then we don't want to render it.
	ActiveShotList.Empty();
	for (UMoviePipelineExecutorShot* Shot : GetCurrentJob()->ShotInfo)
	{
		if (Shot->ShouldRender())
		{
			ActiveShotList.Add(Shot);;
		}
	}
}



void UMoviePipeline::InitializeShot(UMoviePipelineExecutorShot* InShot)
{
	CurrentShotRenderedFrameCount = 0;

	// Set the new shot as the active shot. This enables the specified shot section and disables all other shot sections.
	SetSoloShot(InShot);

	// Loop through just our primary settings and let them know which shot we're about to start.
	TArray<UMoviePipelineSetting*> Settings = GetPipelinePrimaryConfig()->GetAllSettings();
	Algo::SortBy(Settings, [](const UMoviePipelineSetting* Setting) { return Setting->GetPriority(); });
	for (UMoviePipelineSetting* Setting : Settings)
	{
		Setting->OnSetupForShot(InShot);
	}

	if (InShot->GetShotOverrideConfiguration() != nullptr)
	{
		// Any shot-specific overrides haven't had first time initialization. So we'll do that now.
		TArray<UMoviePipelineSetting*> ShotSettings = InShot->GetShotOverrideConfiguration()->GetUserSettings();
		Algo::SortBy(ShotSettings, [](const UMoviePipelineSetting* Setting) { return Setting->GetPriority(); });
		for (UMoviePipelineSetting* Setting : ShotSettings)
		{
			Setting->OnMoviePipelineInitialized(this);
		}
	}

	FMoviePipelineBeginShotRenderTelemetryContext BeginShotTelemetry;
	BeginShotTelemetry.Shot = InShot;
	FMoviePipelineTelemetry::SendBeginShotRenderTelemetry(BeginShotTelemetry);
}

void UMoviePipeline::TeardownShot(UMoviePipelineExecutorShot* InShot)
{
	// Teardown happens at the start of the first frame the shot is finished so we'll stop recording
	// audio, which will prevent it from capturing any samples for this frame. We don't do a similar
	// Start in InitializeShot() because we don't want to record samples during warm up/motion blur.
	StopAudioRecording();

	// Teardown any rendering architecture for this shot. This needs to happen first because it'll flush outstanding rendering commands
	TeardownRenderingPipelineForShot(InShot);

	if (IsFlushDiskWritesPerShot())
	{
		// Moves them from the Output Builder to the output containers
		ProcessOutstandingFinishedFrames();
	}

	// Notify our containers that the current shot has ended.
	TArray<UMoviePipelineOutputBase*> OutputSettings = GetPipelinePrimaryConfig()->GetOutputContainers();
	Algo::SortBy(OutputSettings, [](const UMoviePipelineOutputBase* Setting) { return Setting->GetPriority(); },  TLess<int32>());
	for (UMoviePipelineOutputBase* Container : OutputSettings)
	{
		Container->OnShotFinished(InShot, IsFlushDiskWritesPerShot());
	}

	if (InShot->GetShotOverrideConfiguration() != nullptr)
	{
		// Any shot-specific overrides should get shutdown now.
		TArray<UMoviePipelineOutputBase*> ShotSettings = GetPipelinePrimaryConfig()->GetOutputContainers();
		Algo::SortBy(ShotSettings, [](const UMoviePipelineOutputBase* Setting) { return Setting->GetPriority(); },  TLess<int32>());
		for (UMoviePipelineSetting* Setting : ShotSettings)
		{
			Setting->OnMoviePipelineShutdown(this);
		}
	}

	// Loop through just our primary settings and let them know which shot we're about to end.
	TArray<UMoviePipelineSetting*> AllSettings = GetPipelinePrimaryConfig()->GetAllSettings();
	Algo::SortBy(AllSettings, [](const UMoviePipelineSetting* Setting) { return Setting->GetPriority(); },  TLess<int32>());
	for (UMoviePipelineSetting* Setting : AllSettings)
	{
		Setting->OnTeardownForShot(InShot);
	}

	// Restore the sequence to the original state. We made changes to this when we solo'd it, so we want to unsolo now.
	for (UMoviePipelineExecutorShot* Shot : GetCurrentJob()->ShotInfo)
	{
		TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> Node = Shot->ShotInfo.SubSectionHierarchy;

		const bool bSaveSettings = false;
		MoviePipeline::SaveOrRestoreSubSectionHierarchy(Node, bSaveSettings);
	}

	UE::MoviePipeline::RestoreSkeletalMeshClothSubSteps(ClothSimCache);
	ClothSimCache.Reset();

	FMoviePipelineEndShotRenderTelemetryContext EndShotTelemetry;
	EndShotTelemetry.bIsGraph = false;
	EndShotTelemetry.bWasSuccessful = !bFatalError;
	EndShotTelemetry.bWasCanceled = bShutdownRequested;
	EndShotTelemetry.TotalRenderedFrameCount = CurrentShotRenderedFrameCount;
	FMoviePipelineTelemetry::SendEndShotRenderTelemetry(EndShotTelemetry);
	
	if (IsFlushDiskWritesPerShot())
	{
		ProcessOutstandingFutures();

		TArray<FMoviePipelineShotOutputData> LatestShotData;
		if (GeneratedShotOutputData.Num() > 0)
		{
			LatestShotData.Add(GeneratedShotOutputData.Last());

			// Temporarily remove it from the global array, as the encode may modify it.
			GeneratedShotOutputData.RemoveAt(GeneratedShotOutputData.Num() - 1);
		}

		// We call the command line encoder as a special case here because it may want to modify the file list
		// ie: if it deletes the file after use we probably don't want scripting looking for those files.
		const bool bIncludeDisabledSettings = false;
		UMoviePipelineCommandLineEncoder* Encoder = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineCommandLineEncoder>(bIncludeDisabledSettings);
		if (Encoder)
		{
			const bool bInIsShotEncode = true;
			Encoder->StartEncodingProcess(LatestShotData, bInIsShotEncode);
		}

		FMoviePipelineOutputData Params;
		Params.Pipeline = this;
		Params.Job = GetCurrentJob();
		Params.bSuccess = !bFatalError;

		// The per-shot callback only includes data from the latest shot, but packed into an
		// array to re-use the same datastructures.
		Params.ShotData = LatestShotData;

		// Re-add this to our global list (as it has potentially been modified by the CLI encoder)
		GeneratedShotOutputData.Append(LatestShotData);

		UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Files written to disk for current shot:");
		PrintVerboseLogForFiles(LatestShotData);
		UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Completed outputting files written to disk.");

		OnMoviePipelineShotWorkFinishedDelegateNative.Broadcast(Params);
		OnMoviePipelineShotWorkFinishedDelegate.Broadcast(Params);
	}

	CurrentShotIndex++;

	// Check to see if this was the last shot in the Pipeline, otherwise on the next
	// tick the new shot will be initialized and processed.
	if (CurrentShotIndex >= ActiveShotList.Num())
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "[%lld] Finished rendering last shot. Moving to Finalize to finish writing items to disk.", GFrameCounter);
		TransitionToState(EMovieRenderPipelineState::Finalize);
	}
}



void UMoviePipeline::SetSoloShot(UMoviePipelineExecutorShot* InShot)
{
	// We need to 'solo' shots whichs means disabling any other sections that may overlap with the one currently being
	// rendered. This is because temporal samples, handle frames, warmup frames, etc. all need to evaluate outside of
	// their original bounds and we don't want to end up evaluating something that should have been clipped by another shot.
	for (UMoviePipelineExecutorShot* Shot : GetCurrentJob()->ShotInfo)
	{
		TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> Node = Shot->ShotInfo.SubSectionHierarchy;
		const bool bSaveSettings = true;
		MoviePipeline::SaveOrRestoreSubSectionHierarchy(Node, bSaveSettings);

		MoviePipeline::SetSubSectionHierarchyActive(Node, false);
	}

	// Historically shot expansion was done all at once up front, however this creates a lot of complications when a movie scene isn't filled with unique data
	// such as re-using shots or using different parts of shots. To resolve this, we expand the entire tree needed for a given range, render it, and then restore the original
	// values before moving onto the next shot so that each shot has no effect on the others.
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Expanding Shot %d/%d (Shot: %ls Camera: %ls)", CurrentShotIndex + 1, ActiveShotList.Num(), *InShot->OuterName, *InShot->InnerName);
		
		// Enable the one hierarchy we do want for rendering. We will re-disable it later when we restore the current Sequence state.
		MoviePipeline::SetSubSectionHierarchyActive(InShot->ShotInfo.SubSectionHierarchy, true);

		
		UMoviePipelineOutputSetting* OutputSettings = FindOrAddSettingForShot<UMoviePipelineOutputSetting>(InShot);

		// Expand the shot to encompass handle frames. This will modify the sections required for expansion, etc.
		const bool bIsPrePass = false;
		ExpandShot(InShot, OutputSettings->HandleFrameCount, bIsPrePass);
	}
}



void UMoviePipeline::ExpandShot(UMoviePipelineExecutorShot* InShot, const int32 InNumHandleFrames, const bool bIsPrePass)
{
	const MoviePipeline::FFrameConstantMetrics FrameMetrics = CalculateShotFrameMetrics(InShot);
	int32 LeftDeltaFrames = 0;
	int32 RightDeltaFrames = 0;

	// Calculate the number of ticks added for warmup frames. These are added to both sides. The rendering
	// code is unaware of handle frames, we just pretend the shot is bigger than it actually is.
	LeftDeltaFrames += InNumHandleFrames;
	RightDeltaFrames += InNumHandleFrames;

	// We expand both the first and last frames so Temporal Sub-Sampling can correctly evaluate either side of a frame.
	UMoviePipelineAntiAliasingSetting* AntiAliasingSettings = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(InShot);
	const bool bHasMultipleTemporalSamples = AntiAliasingSettings->TemporalSampleCount > 1;
	if (bHasMultipleTemporalSamples)
	{
		LeftDeltaFrames +=1;
		RightDeltaFrames += 1;
	}

	// Check to see if the detected range was not aligned to a whole frame on the root.
	const FFrameRate RootDisplayRate = GetPipelinePrimaryConfig()->GetEffectiveFrameRate(TargetSequence);
	FFrameTime StartTimeInRoot = FFrameRate::TransformTime(InShot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue(), InShot->ShotInfo.CachedTickResolution, RootDisplayRate);
	if(bIsPrePass && StartTimeInRoot.GetSubFrame() != 0.f)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Shot: %ls Camera: %ls starts on a sub-frame. Rendered range has been rounded to the previous frame to match Sequencer.", *InShot->OuterName, *InShot->InnerName);
		FFrameNumber NewStartFrame = FFrameRate::TransformTime(FFrameTime(StartTimeInRoot.GetFrame()), RootDisplayRate, InShot->ShotInfo.CachedTickResolution).FloorToFrame();
		InShot->ShotInfo.TotalOutputRangeRoot.SetLowerBoundValue(NewStartFrame);
	}

	FFrameNumber LeftDeltaTicks = FFrameRate::TransformTime(FFrameTime(LeftDeltaFrames), FrameMetrics.FrameRate, FrameMetrics.TickResolution).CeilToFrame().Value;
	FFrameNumber RightDeltaTicks = FFrameRate::TransformTime(FFrameTime(RightDeltaFrames), FrameMetrics.FrameRate, FrameMetrics.TickResolution).CeilToFrame().Value;

	// We auto-expand into the warm-up ranges, but users are less concerned about 'early' data there. So we cache how many frames
	// the user expects to check beforehand, so we can use this for a warning later.
	const int32 LeftDeltaFramesUserPoV = LeftDeltaFrames;
	FFrameNumber LeftDeltaTicksUserPoV = FFrameRate::TransformTime(FFrameTime(LeftDeltaFramesUserPoV), FrameMetrics.FrameRate, FrameMetrics.TickResolution).CeilToFrame().Value;

	// We generate this from excess camera cut length. If they don't want real warmup then this is overriden later.
	// Needs to happen after we expand the TotalOutputRangeRoot for handle frames - the MoviePipelineTiming re-calculates
	// the required offset when jumping based on NumEngineWarmUpFramesRemaining.
	if (!InShot->ShotInfo.WarmupRangeRoot.IsEmpty())
	{
		if (bIsPrePass)
		{
			FFrameNumber TicksForWarmUp = InShot->ShotInfo.WarmupRangeRoot.Size<FFrameNumber>();
			InShot->ShotInfo.NumEngineWarmUpFramesRemaining = FFrameRate::TransformTime(FFrameTime(TicksForWarmUp), FrameMetrics.TickResolution, FrameMetrics.FrameRate).CeilToFrame().Value;

			// Handle frames weren't accounted for when we calculated the warm up range, so just reduce the amount of warmup by that.
			// When we actually evaluate we will start our math from the first handle frame so we're still starting from the same
			// absolute position regardless of the handle frame count.
			InShot->ShotInfo.NumEngineWarmUpFramesRemaining = FMath::Max(InShot->ShotInfo.NumEngineWarmUpFramesRemaining - InNumHandleFrames, 0);
		}

		LeftDeltaFrames += InShot->ShotInfo.NumEngineWarmUpFramesRemaining;
	}

	TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> Node = InShot->ShotInfo.SubSectionHierarchy;
	while (Node)
	{
		LeftDeltaTicks = FFrameRate::TransformTime(FFrameTime(LeftDeltaFrames), FrameMetrics.FrameRate, FrameMetrics.TickResolution).CeilToFrame().Value;
		RightDeltaTicks = FFrameRate::TransformTime(FFrameTime(RightDeltaFrames), FrameMetrics.FrameRate, FrameMetrics.TickResolution).CeilToFrame().Value;
		// We need to expand the inner playback bounds to cover three features:
		// 1) Temporal Sampling (+1 frame each end)
		// 2) Handle frames (+n frames left/right)
		// 3) Using the camera-cut as real warm-up frames (+n frames left side only)
		// To keep the inner movie scene and outer sequencer section in sync we can calculate the tick delta
		// to each side and simply expand both sections like that - ignoring all start frame offsets, etc.
		if (!bIsPrePass)
		{
			if (Node->CameraCutSection.IsValid())
			{
				// Expand the camera cut section because there's no harm in doing it.
				Node->CameraCutSection->SetRange(UE::MovieScene::DilateRange(Node->CameraCutSection->GetRange(), -LeftDeltaTicks, RightDeltaTicks));
				Node->CameraCutSection->MarkAsChanged();
			}

			if (Node->Section.IsValid())
			{
				// Expand the MovieSceneSubSequenceSection
				Node->Section->SetRange(UE::MovieScene::DilateRange(Node->Section->GetRange(), -LeftDeltaTicks, RightDeltaTicks));
				Node->Section->MarkAsChanged();
			}

			if (Node->MovieScene.IsValid())
			{
				// Expand the Playback Range of the movie scene as well. Expanding this at the same time as expanding the 
				// SubSequenceSection will result in no apparent change to the evaluated time. ToDo: This doesn't work if
				// sub-sequences have different tick resolutions?
				Node->MovieScene->SetPlaybackRange(UE::MovieScene::DilateRange(Node->MovieScene->GetPlaybackRange(), -LeftDeltaTicks, RightDeltaTicks));
				Node->MovieScene->MarkAsChanged();
			}

			FFrameNumber LowerCheckBound = InShot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue() - LeftDeltaTicksUserPoV;
			FFrameNumber UpperCheckBound = InShot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue();

			TRange<FFrameNumber> CheckRange = TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Exclusive(LowerCheckBound), TRangeBound<FFrameNumber>::Inclusive(UpperCheckBound));

			for (const TTuple<UMovieSceneSection*, TRange<FFrameNumber>>& Pair : Node->AdditionalSectionsToExpand)
			{
				// Expand the section. Because it's an infinite range, we know the contents won't get shifted.
				TRange<FFrameNumber> NewRange = TRange<FFrameNumber>::Hull(Pair.Key->GetRange(), CheckRange);
				Pair.Key->SetRange(NewRange);
				Pair.Key->MarkAsChanged();
			}
		}
		else
		{
			if (Node->MovieScene.IsValid())
			{
				for (UMovieSceneSection* Section : Node->MovieScene->GetAllSections())
				{
					if (!Section)
					{
						continue;
					}

					// Their data is already cached for restore elsewhere.
					if (Section == Node->Section || Section == Node->CameraCutSection)
					{
						continue;
					}

					if (Section->GetSupportsInfiniteRange())
					{
						Node->AdditionalSectionsToExpand.Add(MakeTuple(Section, Section->GetRange()));
					}
				}
			}

			// We only do our warnings during the pre-pass
			// Check for sections that start in the expanded evaluation range and warn user. Only check the frames user expects to (handle + temporal, no need for warm up frames to get checked as well)
			MoviePipeline::CheckPartialSectionEvaluationAndWarn(LeftDeltaTicksUserPoV, Node, InShot, RootDisplayRate);
		}

		Node = Node->GetParent();
	}

	// Expand the Total Output Range Root by Handle Frames. The expansion of TotalOutputRangeRoot has to come after we do partial evaluation checks,
	// otherwise the expanded range makes it check the wrong area for partial evaluations.
	if (bIsPrePass)
	{
		// We expand on the pre-pass so that we have the correct number of frames set up in our datastructures before we reach each shot so that metrics
		// work as expected.
		FFrameNumber LeftHandleTicks = FFrameRate::TransformTime(FFrameTime(InNumHandleFrames), FrameMetrics.FrameRate, FrameMetrics.TickResolution).CeilToFrame().Value;
		FFrameNumber RightHandleTicks = FFrameRate::TransformTime(FFrameTime(InNumHandleFrames), FrameMetrics.FrameRate, FrameMetrics.TickResolution).CeilToFrame().Value;

		InShot->ShotInfo.TotalOutputRangeRoot = UE::MovieScene::DilateRange(InShot->ShotInfo.TotalOutputRangeRoot, -LeftHandleTicks, RightHandleTicks);
	}
}

bool UMoviePipeline::IsDebugFrameStepIdling() const
{
	// We're only idling when we're at zero, otherwise there's more frames to process.
	// Caveat is that this will be zero on the last frame we want to render, so we
	// take into account whether or not we've queued up a pause at the end of the frame
	// which is indicator that we want to process the current frame.
	int32 DebugFrameStepValue = CVarMovieRenderPipelineFrameStepper.GetValueOnGameThread();
	return DebugFrameStepValue == 0 && !bPauseAtEndOfFrame;
}

bool UMoviePipeline::DebugFrameStepPreTick()
{
	int32 DebugFrameStepValue = CVarMovieRenderPipelineFrameStepper.GetValueOnGameThread();
	if (DebugFrameStepValue == 0)
	{
		// A value of 0 means that they are using the frame stepper and that we have stepped
		// the specified number of frames. We will create a DeltaTime for the engine
		// and not process anything below, which prevents us from trying to produce an
		// output frame later.
		CustomTimeStep->SetCachedFrameTiming(MoviePipeline::FFrameTimeStepCache(1 / 24.0));
		return true;
	}
	else if (DebugFrameStepValue > 0)
	{
		// They want to process at least one frame, deincrement, then we
		// process the frame. We pause the game here to preserve render state.
		CVarMovieRenderPipelineFrameStepper->Set(DebugFrameStepValue - 1, ECVF_SetByConsole);

		// We want to run this one frame and then pause again at the end.
		bPauseAtEndOfFrame = true;
	}

	return false;
}

void UMoviePipeline::LoadDebugWidget()
{
	TSubclassOf<UMovieRenderDebugWidget> DebugWidgetClassToUse = ViewportInitArgs.DebugWidgetClass;
	if (DebugWidgetClassToUse.Get() == nullptr)
	{
		DebugWidgetClassToUse = LoadClass<UMovieRenderDebugWidget>(nullptr, *DefaultDebugWidgetAsset, nullptr, LOAD_None, nullptr);
	}

	if (DebugWidgetClassToUse.Get() != nullptr)
	{
		DebugWidget = CreateWidget<UMovieRenderDebugWidget>(GetWorld(), DebugWidgetClassToUse.Get());
		if (DebugWidget)
		{
			DebugWidget->OnInitializedForPipeline(this);
			DebugWidget->AddToViewport();
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to create Debug Screen UMG Widget. No debug overlay available.");
		}
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to find Debug Screen UMG Widget class. No debug overlay available.");
	}
}

FFrameTime FMoviePipelineTimeController::OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate)
{
	FFrameTime RequestTime = FFrameRate::TransformTime(TimeCache.Time, TimeCache.Rate, InCurrentTime.Rate);
	UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "[%lld] OnRequestCurrentTime: %d %f", GFrameCounter, RequestTime.FloorToFrame().Value, RequestTime.GetSubFrame());

	return RequestTime;
}

MoviePipeline::FFrameConstantMetrics UMoviePipeline::CalculateShotFrameMetrics(const UMoviePipelineExecutorShot* InShot) const
{
	MoviePipeline::FFrameConstantMetrics Output;
	Output.TickResolution = TargetSequence->GetMovieScene()->GetTickResolution();
	Output.ShotTickResolution = InShot->ShotInfo.CachedShotTickResolution;
	Output.FrameRate = GetPipelinePrimaryConfig()->GetEffectiveFrameRate(TargetSequence);
	Output.TicksPerOutputFrame = FFrameRate::TransformTime(FFrameTime(FFrameNumber(1)), Output.FrameRate, Output.TickResolution);

	UMoviePipelineCameraSetting* CameraSettings = FindOrAddSettingForShot<UMoviePipelineCameraSetting>(InShot);
	UMoviePipelineAntiAliasingSetting* AntiAliasingSettings = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(InShot);

	// We are overriding blur settings to account for how we sample multiple frames, so
	// we need to process any camera and post process volume settings for motion blur manually

	// Start with engine default for motion blur in the event no one overrides it.
	Output.ShutterAnglePercentage = 0.5;

	APlayerCameraManager* PlayerCameraManager = GetWorld()->GetFirstPlayerController()->PlayerCameraManager;
	if (PlayerCameraManager)
	{
		// Apply any motion blur settings from post process volumes in the world
		FVector ViewLocation = PlayerCameraManager->GetCameraLocation();
		for (IInterface_PostProcessVolume& PPVolume : GetWorld()->GetPostProcessVolumeIterator())
		{
			const FPostProcessVolumeProperties VolumeProperties = PPVolume.GetProperties();

			// Skip any volumes which are either disabled or don't modify blur amount
			if (!VolumeProperties.bIsEnabled || !VolumeProperties.Settings->bOverride_MotionBlurAmount)
			{
				continue;
			}

			float LocalWeight = FMath::Clamp(VolumeProperties.BlendWeight, 0.0f, 1.0f);

			if (!VolumeProperties.bIsUnbound)
			{
				float DistanceToPoint = 0.0f;
				PPVolume.EncompassesPoint(ViewLocation, 0.0f, &DistanceToPoint);

				if (DistanceToPoint >= 0 && DistanceToPoint < VolumeProperties.BlendRadius)
				{
					LocalWeight *= FMath::Clamp(1.0f - DistanceToPoint / VolumeProperties.BlendRadius, 0.0f, 1.0f);
				}
				else
				{
					LocalWeight = 0.0f;
				}
			}

			if (LocalWeight > 0.0f)
			{
				Output.ShutterAnglePercentage = FMath::Lerp(Output.ShutterAnglePercentage, (double)VolumeProperties.Settings->MotionBlurAmount, LocalWeight);
			}
		}

		// Now try from the camera, which takes priority over post processing volumes.
		ACameraActor* CameraActor = Cast<ACameraActor>(PlayerCameraManager->GetViewTarget());
		if (CameraActor)
		{
			UCameraComponent* CameraComponent = CameraActor->GetCameraComponent();
			if (CameraComponent && CameraComponent->PostProcessSettings.bOverride_MotionBlurAmount)
			{
				Output.ShutterAnglePercentage = FMath::Lerp(Output.ShutterAnglePercentage, (double)CameraComponent->PostProcessSettings.MotionBlurAmount, (double)CameraComponent->PostProcessBlendWeight);
			}
		}
		
		// Apply any motion blur settings from post processing blends attached to the camera manager
		TArray<FPostProcessSettings> const* CameraAnimPPSettings;
		TArray<float> const* CameraAnimPPBlendWeights;
		PlayerCameraManager->GetCachedPostProcessBlends(CameraAnimPPSettings, CameraAnimPPBlendWeights);
		for (int32 PPIdx = 0; PPIdx < CameraAnimPPBlendWeights->Num(); ++PPIdx)
		{
			if ((*CameraAnimPPSettings)[PPIdx].bOverride_MotionBlurAmount)
			{
				Output.ShutterAnglePercentage = FMath::Lerp(Output.ShutterAnglePercentage, (double)(*CameraAnimPPSettings)[PPIdx].MotionBlurAmount, (*CameraAnimPPBlendWeights)[PPIdx]);
			}
		}
	}

	{
		/*
		* Calculate out how many ticks a normal sub-frame occupies.
		* (TickRes/FrameRate) gives you ticks-per-second, and then divide that by the percentage of time the
		* shutter is open. Finally, divide the percentage of time the shutter is open by the number of frames
		* we're accumulating.
		*
		* It is common that there is potential to have some leftover here. ie:
		* 24000 Ticks / 24fps = 1000 ticks per second. At 180 degree shutter angle that gives you 500 ticks
		* spread out amongst n=3 sub-frames. 500/3 = 166.66 ticks. We'll floor that when we use it, and ensure
		* we accumulate the sub-tick and choose when to apply it.
		*/

		// If the shutter angle is effectively zero, lie about how long a frame is to prevent divide by zero
		if (Output.ShutterAnglePercentage < 1.0 / 360.0)
		{
			// The minimum shutter fraction must guarantee at least 1 whole tick per temporal
			// sample, otherwise the per-sample delta floors to 0 ticks and causes zero-delta-time
			// errors. TemporalSampleCount / TicksPerFrame is the smallest fraction that achieves this.
			//
			// Example: 200 ticks/frame, 8 temporal samples
			//   clamp(8/200, 1/360, 1) = 0.04 -> 200 * 0.04 = 8 ticks / 8 samples = 1 tick/sample
			//   Total timing: 192 closed + (8 * 1) open = 200 ticks/frame
			const double MinShutterFraction = FMath::Clamp(
				(double)AntiAliasingSettings->TemporalSampleCount / Output.TicksPerOutputFrame.AsDecimal(),
				1.0 / 360.0, 1.0);
			Output.TicksWhileShutterOpen = Output.TicksPerOutputFrame * MinShutterFraction;
		}
		else
		{
			// Otherwise, calculate the amount of time the shutter is open.
			Output.TicksWhileShutterOpen = Output.TicksPerOutputFrame * Output.ShutterAnglePercentage;
		}

		// Divide that amongst all of our accumulation sample frames.
		Output.TicksPerSample = Output.TicksWhileShutterOpen / AntiAliasingSettings->TemporalSampleCount;

	}

	Output.ShutterClosedFraction = 1.0 - Output.ShutterAnglePercentage;
	Output.TicksWhileShutterClosed = Output.TicksPerOutputFrame - Output.TicksWhileShutterOpen;

	// Shutter Offset
	switch (CameraSettings->ShutterTiming)
	{
		// Subtract the entire time the shutter is open.
	case EMoviePipelineShutterTiming::FrameClose:
		Output.ShutterOffsetTicks = -Output.TicksWhileShutterOpen;
		break;
		// Only subtract half the time the shutter is open.
	case EMoviePipelineShutterTiming::FrameCenter:
		Output.ShutterOffsetTicks = -Output.TicksWhileShutterOpen / 2.0;
		break;
		// No offset needed
	case EMoviePipelineShutterTiming::FrameOpen:
		break;
	}

	// Then, calculate our motion blur offset. Motion Blur in the engine is always
	// centered around the object so we offset our time sampling by half of the
	// motion blur distance so that the distance blurred represents that time.
	Output.MotionBlurCenteringOffsetTicks = Output.TicksPerSample / 2.0;

	return Output;
}


UMoviePipelinePrimaryConfig* UMoviePipeline::GetPipelinePrimaryConfig() const
{ 
	return CurrentJob->GetConfiguration(); 
}

TArray<UMoviePipelineSetting*> UMoviePipeline::FindSettingsForShot(TSubclassOf<UMoviePipelineSetting> InSetting, const UMoviePipelineExecutorShot* InShot) const
{
	TArray<UMoviePipelineSetting*> FoundSettings;

	// Find all enabled settings of given subclass in the shot override first
	if (UMoviePipelineShotConfig* ShotOverride = InShot->GetShotOverrideConfiguration())
	{
		for (UMoviePipelineSetting* Setting : ShotOverride->FindSettingsByClass(InSetting))
		{
			if (Setting && Setting->IsEnabled())
			{
				FoundSettings.Add(Setting);
			}
		}
	}

	// Add all enabled settings of given subclass not overridden by shot override
	for (UMoviePipelineSetting* Setting : GetPipelinePrimaryConfig()->FindSettingsByClass(InSetting))
	{
		if (Setting && Setting->IsEnabled())
		{
			TSubclassOf<UMoviePipelineSetting> SettingClass = Setting->GetClass();
			if (!FoundSettings.ContainsByPredicate([SettingClass](UMoviePipelineSetting* ExistingSetting) { return ExistingSetting && ExistingSetting->GetClass() == SettingClass; } ))
			{
				FoundSettings.Add(Setting);
			}
		}
	}

	return FoundSettings;
}

void UMoviePipeline::ResolveFilenameFormatArguments(const FString& InFormatString, const TMap<FString, FString>& InFormatOverrides, FString& OutFinalPath, FMoviePipelineFormatArgs& OutFinalFormatArgs, const FMoviePipelineFrameOutputState* InOutputState, const int32 InFrameNumberOffset) const
{
	FMoviePipelineFilenameResolveParams Params = FMoviePipelineFilenameResolveParams();
	if (InOutputState)
	{
		Params.FrameNumber = InOutputState->SourceFrameNumber;
		Params.FrameNumberShot = InOutputState->CurrentShotSourceFrameNumber;
		Params.FrameNumberRel = InOutputState->OutputFrameNumber;
		Params.FrameNumberShotRel = InOutputState->ShotOutputFrameNumber;
		Params.FileMetadata = InOutputState->FileMetadata;
		Params.ShotOverride = ActiveShotList[InOutputState->ShotIndex];
		Params.InitializationVersion = ActiveShotList[InOutputState->ShotIndex]->ShotInfo.VersionNumber;
		Params.CameraIndex = InOutputState->CameraIndex;
		Params.CameraNameOverride = InOutputState->CameraNameOverride;
	}

	UMoviePipelineOutputSetting* OutputSetting = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	check(OutputSetting);

	Params.ZeroPadFrameNumberCount = OutputSetting->ZeroPadFrameNumbers;

	// Ensure they used relative frame numbers in the output so they get the right number of output frames.
	bool bForceRelativeFrameNumbers = false;
	if (InFormatString.Contains(TEXT("{frame")) && InOutputState && InOutputState->TimeData.IsTimeDilated() && !InFormatString.Contains(TEXT("_rel}")))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Time Dilation was used but output format does not use relative time, forcing relative numbers. Change {frame_number} to {frame_number_rel} (or shot version) to remove this message.");
		bForceRelativeFrameNumbers = true;
	}

	Params.bForceRelativeFrameNumbers = bForceRelativeFrameNumbers;
	Params.FileNameFormatOverrides = InFormatOverrides;
	Params.InitializationTime = InitializationTime;
	Params.InitializationTimeOffset = InitializationTimeOffset;
	Params.Job = GetCurrentJob();
	Params.AdditionalFrameNumberOffset = InFrameNumberOffset;

	UMoviePipelineBlueprintLibrary::ResolveFilenameFormatArguments(InFormatString, Params, OutFinalPath, OutFinalFormatArgs);

	// This needs to come after ResolveFilenameFormatArguments as that resets the OutFinalFormatArgs.
	if (InOutputState)
	{
		const FRenderTimeStatistics* TimeStats = RenderTimeFrameStatistics.Find(InOutputState->OutputFrameNumber);
		if (TimeStats)
		{
			FString StartTimeStr = TimeStats->StartTime.ToString();
			FString EndTimeStr = TimeStats->EndTime.ToString();
			FString DurationTimeStr = (TimeStats->EndTime - TimeStats->StartTime).ToString();
			OutFinalFormatArgs.FileMetadata.Add(TEXT("unreal/frameRenderStartTimeUTC"), StartTimeStr);
			OutFinalFormatArgs.FileMetadata.Add(TEXT("unreal/frameRenderEndTimeUTC"), EndTimeStr);
			OutFinalFormatArgs.FileMetadata.Add(TEXT("unreal/frameRenderDuration"), DurationTimeStr);
		}
	}
}

void UMoviePipeline::SetProgressWidgetVisible(bool bVisible)
{
	if (DebugWidget)
	{
		DebugWidget->SetVisibility(bVisible ? ESlateVisibility::Visible : ESlateVisibility::Collapsed);
	}
}

FMoviePipelineOutputData UMoviePipeline::GetOutputDataParams()
{
	FMoviePipelineOutputData Params;

	Params.Pipeline = this;
	Params.Job = GetCurrentJob();
	Params.bSuccess = !bFatalError;
	Params.ShotData = GeneratedShotOutputData;

	return Params;
}

void UMoviePipeline::OnMoviePipelineFinishedImpl()
{
	// Generate a params struct containing the data generated by this job.
	FMoviePipelineOutputData Params;
	Params.Pipeline = this;
	Params.Job = GetCurrentJob();
	Params.bSuccess = !bFatalError;
	Params.ShotData = GeneratedShotOutputData;

	UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Files written to disk for entire sequence:");
	PrintVerboseLogForFiles(GeneratedShotOutputData);
	UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Completed outputting files written to disk.");

	OnMoviePipelineWorkFinishedDelegateNative.Broadcast(Params);
	OnMoviePipelineWorkFinishedDelegate.Broadcast(Params);
}

void UMoviePipeline::PrintVerboseLogForFiles(const TArray<FMoviePipelineShotOutputData>& InOutputData) const
{
	for (const FMoviePipelineShotOutputData& OutputData : InOutputData)
	{
		const UMoviePipelineExecutorShot* Shot = OutputData.Shot.Get();
		if (Shot)
		{
			UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Shot: %ls [%ls]", *Shot->OuterName, *Shot->InnerName);
		}
		for (const TPair<FMoviePipelinePassIdentifier, FMoviePipelineRenderPassOutputData>& Pair : OutputData.RenderPassData)
		{
			UE_LOGF(LogMovieRenderPipelineIO, Verbose, "Render Pass: %ls", *Pair.Key.Name);
			for (const FString& FilePath : Pair.Value.FilePaths)
			{
				UE_LOGF(LogMovieRenderPipelineIO, Verbose, "\t\t%ls", *FilePath);
			}
		}
	}
}

void UMoviePipeline::StartUnrealInsightsCapture()
{
	// Generate a filepath to attempt to store the trace file in.
	UMoviePipelineOutputSetting* OutputSetting = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	FString FileName = OutputSetting->FileNameFormat + TEXT("_UnrealInsights");
	FString FileNameFormatString = OutputSetting->OutputDirectory.Path / FileName;

	// Generate a filename for this encoded file
	TMap<FString, FString> FormatOverrides;
	FormatOverrides.Add(TEXT("ext"), TEXT("utrace"));
	FMoviePipelineFormatArgs FinalFormatArgs;

	FString FinalFilePath;
	ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, FinalFilePath, FinalFormatArgs);

	if (FPaths::IsRelative(FinalFilePath))
	{
		FinalFilePath = FPaths::ConvertRelativePathToFull(FinalFilePath);
	}	

	const bool bTraceStarted = FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::File, *FinalFilePath);
	if (bTraceStarted)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Started capturing UnrealInsights trace file to %ls", *FinalFilePath);
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to start capturing UnrealInsights trace. Is there already a trace session in progress?");
	}
}

void UMoviePipeline::StopUnrealInsightsCapture()
{
	FTraceAuxiliary::Stop();
}

void UMoviePipeline::GetSidecarCameraData(UMoviePipelineExecutorShot* InShot, int32 InCameraIndex, FMinimalViewInfo& OutViewInfo, UCameraComponent** OutCameraComponent) const
{
	UMovieScene* MovieScene = GetTargetSequence()->GetMovieScene();
	if (!InShot || !MovieScene)
	{
		return;
	}

	UObject* PrimaryViewTarget = nullptr;
	// Sequences don't always have a camera cut track, at which point we fall back to the regular player controller.
	if (InCameraIndex >= InShot->SidecarCameras.Num())
	{
		if (APlayerController* PlayerController = GetWorld()->GetFirstPlayerController())
		{
			if (APlayerCameraManager* PlayerCameraManager = PlayerController->PlayerCameraManager)
			{
				PrimaryViewTarget = PlayerCameraManager->GetViewTarget();
			}
		}
	}
	else
	{
		const FGuid CameraBindingRef = InShot->SidecarCameras[InCameraIndex].BindingId;
		const FMovieSceneSequenceID CameraSequenceRef = InShot->SidecarCameras[InCameraIndex].SequenceId;
		{
			TArrayView<TWeakObjectPtr<UObject>> BoundCameras = LevelSequenceActor->GetSequencePlayer()->FindBoundObjects(CameraBindingRef, CameraSequenceRef);
			if (BoundCameras.Num() > 0)
			{
				PrimaryViewTarget = BoundCameras[0].Get();
			}
		}
	}

	UCameraComponent* BoundCamera = MovieSceneHelpers::CameraComponentFromRuntimeObject(PrimaryViewTarget);
	if (BoundCamera)
	{
		*OutCameraComponent = BoundCamera;
		BoundCamera->GetCameraView(GetWorld()->GetDeltaSeconds(), OutViewInfo);
				
		// We override the current/previous transform based on cached data though to ensure we accurately handle cur/next frame positions
		if (FrameInfo.CurrSidecarViewLocations.IsValidIndex(InCameraIndex) && FrameInfo.CurrSidecarViewRotations.IsValidIndex(InCameraIndex) &&
			FrameInfo.PrevSidecarViewLocations.IsValidIndex(InCameraIndex) && FrameInfo.PrevSidecarViewRotations.IsValidIndex(InCameraIndex))
		{
			OutViewInfo.Location = FrameInfo.CurrSidecarViewLocations[InCameraIndex];
			OutViewInfo.Rotation = FrameInfo.CurrSidecarViewRotations[InCameraIndex];
			OutViewInfo.PreviousViewTransform = FTransform(FrameInfo.PrevSidecarViewRotations[InCameraIndex], FrameInfo.PrevSidecarViewLocations[InCameraIndex]);
		}
	}
}

bool UMoviePipeline::GetSidecarCameraViewPoints(UMoviePipelineExecutorShot* InShot, TArray<FVector>& OutSidecarViewLocations, TArray<FRotator>& OutSidecarViewRotations) const
{
	OutSidecarViewLocations.Reset();
	OutSidecarViewRotations.Reset();

	if (InShot)
	{
		// We can't use GetSidecarCameraData because it relies on data generated by this function to ensure the previous/next locations are correct.
		for (int32 Index = 0; Index < InShot->SidecarCameras.Num(); Index++)
		{
			const FGuid CameraBindingRef = InShot->SidecarCameras[Index].BindingId;
			const FMovieSceneSequenceID CameraSequenceRef = InShot->SidecarCameras[Index].SequenceId;

			TArrayView<TWeakObjectPtr<UObject>> BoundCameras = LevelSequenceActor->GetSequencePlayer()->FindBoundObjects(CameraBindingRef, CameraSequenceRef);
			if (BoundCameras.Num() > 0)
			{
				UCameraComponent* BoundCamera = MovieSceneHelpers::CameraComponentFromRuntimeObject(BoundCameras[0].Get());
				if (BoundCamera)
				{
					FMinimalViewInfo ViewInfo;
					BoundCamera->GetCameraView(GetWorld()->GetDeltaSeconds(), ViewInfo);

					OutSidecarViewLocations.Add(ViewInfo.Location);
					OutSidecarViewRotations.Add(ViewInfo.Rotation);
				}
			}
		}
	}

	// Jam a default value in but return false so that the caller can know the data was no good.
	if (OutSidecarViewLocations.Num() == 0)
	{
		OutSidecarViewLocations.AddDefaulted();
		OutSidecarViewRotations.AddDefaulted();
		return false;
	}

	return true;
}

float UMoviePipeline::GetCachedCameraOverscan(int32 InCameraIndex) const
{
	if (CameraOverscanCache.Contains(InCameraIndex))
	{
		return CameraOverscanCache[InCameraIndex];
	}

	// If no camera under the specified index is found, check if a cached value exists for the "default" camera
	if (CameraOverscanCache.Contains(INDEX_NONE))
	{
		return CameraOverscanCache[INDEX_NONE];
	}

	return 0.0f;
}

bool UMoviePipeline::HasCachedCameraOverscan(int32 InCameraIndex) const
{
	return CameraOverscanCache.Contains(InCameraIndex);
}

void UMoviePipeline::CacheCameraOverscan(int32 InCameraIndex, float InCameraOverscan)
{
	if (!CameraOverscanCache.Contains(InCameraIndex))
	{
		CameraOverscanCache.Add(InCameraIndex, InCameraOverscan);
	}
	else
	{
		CameraOverscanCache[InCameraIndex] = InCameraOverscan;
	}
}

void UMoviePipeline::WarnAboutAnimatedOverscan(float InInitialOverscan)
{
	if (!bHasWarnedAboutAnimatedOverscan)
	{
		UMoviePipelineExecutorShot* CurrentShot = ActiveShotList[CurrentShotIndex];
		UE_LOGF(
			LogMovieRenderPipeline,
			Warning,
			"Detected animated Camera Overscan value on shot %ls for camera %ls. MRQ does not support changing resolution between frames, and a resolution computed from the initial overscan (%f) will be used instead. Overscan can be affected by both the Overscan camera property or distortion parameters on the Lens component",
			*CurrentShot->OuterName,
			*CurrentShot->InnerName,
			InInitialOverscan);

		bHasWarnedAboutAnimatedOverscan = true;
	}
}

#undef LOCTEXT_NAMESPACE // "MoviePipeline"

