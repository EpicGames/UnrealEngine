// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineAntiAliasingSetting.h"

#include "MoviePipelineTelemetry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineAntiAliasingSetting)

void UMoviePipelineAntiAliasingSetting::ValidateStateImpl()
{
	Super::ValidateStateImpl();

	int32 NumTAASamples = 8;
	IConsoleVariable* AntiAliasingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.TemporalAASamples"));
	if(AntiAliasingCVar)
	{
		NumTAASamples = AntiAliasingCVar->GetInt();
	}
	
	if (IsTemporalAccumulationBasedMethod(UE::MovieRenderPipeline::GetEffectiveAntiAliasingMethod(this)))
	{
		if ((TemporalSampleCount*SpatialSampleCount) > NumTAASamples)
		{
			const FText ValidationWarning = NSLOCTEXT("MovieRenderPipeline", "AntiAliasing_BetterOffWithoutTAA", "If the product of Temporal and Spatial counts ({0}x{1}={2}) is greater than the number of TAA samples (controlled by r.TemporalAASamples, currently {3}) then TAA is ineffective and you should consider overriding AA to None for better quality.");
			ValidationResults.Add(FText::Format(ValidationWarning, TemporalSampleCount, SpatialSampleCount, TemporalSampleCount * SpatialSampleCount, NumTAASamples));
			ValidationState = EMoviePipelineValidationState::Warnings;
		}

		if (SpatialSampleCount % 2 == 0)
		{
			ValidationResults.Add(NSLOCTEXT("MovieRenderPipeline", "AntiAliasing_InsufficientJitters", "Temporal Anti-Aliasing does not converge when using an even number of samples. Disable TAA or increase sample count."));
			ValidationState = EMoviePipelineValidationState::Warnings;
		}
	}

	if (UE::MovieRenderPipeline::GetEffectiveAntiAliasingMethod(this) == EAntiAliasingMethod::AAM_None)
	{
		if ((TemporalSampleCount * SpatialSampleCount) < NumTAASamples)
		{
			const FText ValidationWarning = NSLOCTEXT("MovieRenderPipeline", "AntiAliasing_InsufficientSamples", "Temporal Anti-Aliasing uses at least {0} samples. Increase Spatial/Temporal sub-sample count to maintain visual anti-aliasing quality.");
			ValidationResults.Add(FText::Format(ValidationWarning, NumTAASamples));
			ValidationState = EMoviePipelineValidationState::Warnings;
		}
	}

}

void UMoviePipelineAntiAliasingSetting::GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const
{
	Super::GetFormatArguments(InOutFormatArgs);

	InOutFormatArgs.FilenameArguments.Add(TEXT("ts_count"), FString::FromInt(TemporalSampleCount));
	InOutFormatArgs.FilenameArguments.Add(TEXT("ss_count"), FString::FromInt(SpatialSampleCount));

	InOutFormatArgs.FileMetadata.Add(TEXT("unreal/aa/temporalSampleCount"), FString::FromInt(TemporalSampleCount));
	InOutFormatArgs.FileMetadata.Add(TEXT("unreal/aa/spatialSampleCount"), FString::FromInt(SpatialSampleCount));
}

void UMoviePipelineAntiAliasingSetting::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->TemporalSampleCount = FMath::Max(InTelemetry->TemporalSampleCount, TemporalSampleCount);
	InTelemetry->SpatialSampleCount = FMath::Max(InTelemetry->SpatialSampleCount, SpatialSampleCount);
	InTelemetry->WarmUpFrames = FMath::Max(InTelemetry->WarmUpFrames, FMath::Max(EngineWarmUpCount, 0));
	InTelemetry->AntiAliasingType = UEnum::GetValueAsString(UE::MovieRenderPipeline::GetEffectiveAntiAliasingMethod(this));
}
