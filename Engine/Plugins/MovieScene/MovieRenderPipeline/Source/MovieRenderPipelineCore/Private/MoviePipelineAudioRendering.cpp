// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipeline.h"
#include "HAL/IConsoleManager.h"
#include "AudioDeviceManager.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineOutputSetting.h"
#include "Engine/Engine.h"
#include "AudioMixerBlueprintLibrary.h"
#include "LevelSequence.h"
#include "LevelSequencePlayer.h"
#include "LevelSequenceActor.h"
#include "AudioDevice.h"
#include "AudioMixerDevice.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MovieRenderPipelineDataTypes.h"
#include "AudioMixerSubmix.h"
#include "DSP/BufferVectorOperations.h"
#include "MoviePipelineAntiAliasingSetting.h"
#include "AudioMixerPlatformNonRealtime.h"
#include "MoviePipelineQueue.h"

#if WITH_EDITOR
#include "Settings/LevelEditorMiscSettings.h"
#endif

static FAudioDevice* GetAudioDeviceFromWorldContext(const UObject* WorldContextObject)
{
	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->GetNetMode() == NM_DedicatedServer)
	{
		return nullptr;
	}

	return ThisWorld->GetAudioDeviceRaw();
}

static Audio::FMixerDevice* GetAudioMixerDeviceFromWorldContext(const UObject* WorldContextObject)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFromWorldContext(WorldContextObject))
	{
		return static_cast<Audio::FMixerDevice*>(AudioDevice);
	}
	return nullptr;
}

void UMoviePipeline::SetupAudioRendering()
{
	// Ensure that we try to play audio at full volume, even if we're unfocused.
	AudioState.PrevUnfocusedAudioMultiplier = FApp::GetUnfocusedVolumeMultiplier();
	FApp::SetUnfocusedVolumeMultiplier(1.f);

#if WITH_EDITOR
	// Force background audio on so audio is captured even when the editor is in the background
	{
		ULevelEditorMiscSettings* MiscSettings = GetMutableDefault<ULevelEditorMiscSettings>();
		AudioState.bPrevAllowBackgroundAudio = MiscSettings->bAllowBackgroundAudio;
		MiscSettings->bAllowBackgroundAudio = true;
	}
#endif

	IConsoleVariable* AudioRenderEveryTickCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.nrt.RenderEveryTick"));
	// Will be null if the NRT module wasn't loaded
	if (AudioRenderEveryTickCvar)
	{
		AudioState.PrevRenderEveryTickValue = AudioRenderEveryTickCvar->GetInt();
		// Override it to prevent it from automatically ticking, we'll control this below.
		AudioRenderEveryTickCvar->SetWithCurrentPriority(0, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}
}

void UMoviePipeline::TeardownAudioRendering()
{
	if (LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{
		TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = LevelSequenceActor->GetSequencePlayer()->GetSharedPlaybackState();
		if (UE::MovieScene::FMovieSceneAudioRecordingCapability* AudioRecordingCapability = SharedPlaybackState->FindCapability<UE::MovieScene::FMovieSceneAudioRecordingCapability>())
		{
			AudioRecordingCapability->bIsRecordingAudio = false; // flag to notify MovieScene::FEvaluateAudio::Evaluate that MRQ is NOT active.
		}
	}

	// Restore previous unfocused audio multiplier, to no longer force audio when unfocused
	FApp::SetUnfocusedVolumeMultiplier(AudioState.PrevUnfocusedAudioMultiplier);

#if WITH_EDITOR
	// Restore the original bAllowBackgroundAudio setting
	GetMutableDefault<ULevelEditorMiscSettings>()->bAllowBackgroundAudio = AudioState.bPrevAllowBackgroundAudio;
#endif

	// Restore our cached CVar value.
	IConsoleVariable* AudioRenderEveryTickCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.nrt.RenderEveryTick"));

	// This will be null if the NRT wasn't used (module not loaded).
	if (AudioRenderEveryTickCvar)
	{
		AudioRenderEveryTickCvar->SetWithCurrentPriority(AudioState.PrevRenderEveryTickValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}
}

void UMoviePipeline::StartAudioRecording()
{
	AudioState.bIsRecordingAudio = true;
	const UMoviePipelineExecutorShot* CurrentShot = ActiveShotList[CurrentShotIndex];
	UMoviePipelineOutputSetting* OutputSettings = FindOrAddSettingForShot<UMoviePipelineOutputSetting>(CurrentShot);
	check(OutputSettings);

	Audio::FMixerDevice* MixerDevice = GetAudioMixerDeviceFromWorldContext(this);
	if (MixerDevice)
	{
		TWeakPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> MasterSubmix = MixerDevice->GetMasterSubmix();
		if (MasterSubmix.Pin())
		{
			AudioState.ActiveSubmixes.Add(MasterSubmix);
		}

		for (USoundSubmix* Mix : OutputSettings->AdditionalSubmixOutputs)
		{
			TWeakPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> Submix = MixerDevice->GetSubmixInstance(Mix);
			if (Submix.Pin() && Submix != MasterSubmix)
			{
				AudioState.ActiveSubmixes.AddUnique(Submix);
			}
		}

		for (TWeakPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> WeakSubmix : AudioState.ActiveSubmixes)
		{
			const float ExpectedDuration = 30.f;
			WeakSubmix.Pin()->OnStartRecordingOutput(ExpectedDuration);
		}
	}
}

void UMoviePipeline::StopAudioRecording()
{
	AudioState.bIsRecordingAudio = false;

	for (TWeakPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> WeakSubmix : AudioState.ActiveSubmixes)
	{
		TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> Submix = WeakSubmix.Pin();
		if (Submix)
		{
			MoviePipeline::FAudioState::FAudioSegment& NewSegment = AudioState.FinishedSegments.AddDefaulted_GetRef();
			NewSegment.OutputState = CachedOutputState;

			// Copy the data returned by the Submix.
			Audio::AlignedFloatBuffer& AudioRecording = Submix->OnStopRecordingOutput(NewSegment.NumChannels, NewSegment.SampleRate);
			NewSegment.SegmentData = AudioRecording; 
			NewSegment.SubmixName = Submix->GetName();
		}
	}

	AudioState.ActiveSubmixes.Reset();
}

void UMoviePipeline::ProcessAudioTick()
{
	// Only supported on the new audio mixer (with the non-realtime device, windows only).
	Audio::FMixerDevice* MixerDevice = GetAudioMixerDeviceFromWorldContext(this);
	if (!MixerDevice)
	{
		return;
	}

	Audio::IAudioMixerPlatformInterface* AudioMixerPlatform = MixerDevice->GetAudioMixerPlatform();
	Audio::FMixerPlatformNonRealtime* NRTPlatform = nullptr;
	if (AudioMixerPlatform && AudioMixerPlatform->IsNonRealtime())
	{
		// There is only one non-realtime audio platform at this time, so we can safely static_cast this due to the IsNonRealtime check.
		NRTPlatform = static_cast<Audio::FMixerPlatformNonRealtime*>(MixerDevice->GetAudioMixerPlatform());
	}

	if (!NRTPlatform)
	{
		return;
	}


	UMoviePipelineExecutorShot* CurrentShot = ActiveShotList[CurrentShotIndex];

	if (CurrentShot->ShotInfo.State == EMovieRenderShotState::WarmingUp)
	{
		// There can be time in the Unreal world that passes when MRQ is not yet active. When MRQ isn't active,
		// then the NRT audio platform doesn't process anything as all the AudioComponents that would have been playing are paused.
		// On capturing the next camera, The AudioComponents resume playback, ensuring sample-accurate continuity across camera cuts.
		if (LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
		{
			TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = LevelSequenceActor->GetSequencePlayer()->GetSharedPlaybackState();
			if (UE::MovieScene::FMovieSceneAudioRecordingCapability* AudioRecordingCapability = SharedPlaybackState->FindCapability<UE::MovieScene::FMovieSceneAudioRecordingCapability>())
			{
				AudioRecordingCapability->bIsRecordingAudio = true; // flag to notify MovieScene::FEvaluateAudio::Evaluate that MRQ is active.
			}
		}
	}
	// Start capturing any produced samples on the same frame we start submitting samples that will make it to disk.
	// This comes before we process samples for this frame (below).
	if (CurrentShot->ShotInfo.State == EMovieRenderShotState::Rendering && !AudioState.bIsRecordingAudio)
	{
		StartAudioRecording();
	}

	// This needs to tick every frame so we process and clear audio that happens between shots while the world is running.
	// We have special ticking logic while rendering due to temporal sampling, and otherwise just fall back to normal.
	bool bCanRenderAudio = true;
	double AudioDeltaTime = FApp::GetDeltaTime();
	
	if(CurrentShot->ShotInfo.State == EMovieRenderShotState::Rendering)
	{
		// The non-real time audio renderer desires even engine time steps. Unfortunately, when using temporal sampling
		// we don't have an even timestep. However, because it's non-real time, and we're accumulating the results into
		// a single frame anyways, we can bunch up the audio work and then process it when we've reached the end of a frame.
		bCanRenderAudio = CachedOutputState.IsLastTemporalSample();
		
		// Process work that has been submitted from the game thread to the audio thread over the temporal samples of this frame.
		AudioDeltaTime = GetPipelinePrimaryConfig()->GetEffectiveFrameRate(TargetSequence).AsInterval();
	}

	// Handle any game logic that changed Audio State.
	MixerDevice->Update(true);

	if (bCanRenderAudio)
	{
		// Process work that has been submitted from the game thread to the audio thread over the temporal samples of this frame.
		NRTPlatform->RenderAudio(AudioDeltaTime);
	}
}
