// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineBasicConfig.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/Nodes/MovieGraphFileOutputNode.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "Graph/Nodes/MovieGraphSamplingMethodNode.h"
#include "Graph/Nodes/MovieGraphWarmUpSettingNode.h"
#include "MovieRenderPipelineCoreModule.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "PackageHelperFunctions.h"
#include "Settings/EditorLoadingSavingSettings.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineBasicConfig)

namespace UE::MoviePipeline::BasicConfig
{
	// Reflected property names on nodes from the MovieRenderPipelineRenderPasses module (avoiding compile-time dependency)
	static const FName PropName_SpatialSampleCount					= TEXT("SpatialSampleCount");
	static const FName PropName_bOverride_SpatialSampleCount		= TEXT("bOverride_SpatialSampleCount");
	static const FName PropName_AntiAliasingMethod					= TEXT("AntiAliasingMethod");
	static const FName PropName_bOverride_AntiAliasingMethod		= TEXT("bOverride_AntiAliasingMethod");
	static const FName PropName_bEnableDenoiser						= TEXT("bEnableDenoiser");
	static const FName PropName_bOverride_bEnableDenoiser			= TEXT("bOverride_bEnableDenoiser");
	static const FName PropName_DenoiserType						= TEXT("DenoiserType");
	static const FName PropName_bOverride_DenoiserType				= TEXT("bOverride_DenoiserType");
	static const FName PropName_bEnableBurnIn						= TEXT("bEnableBurnIn");
	static const FName PropName_bOverride_bEnableBurnIn				= TEXT("bOverride_bEnableBurnIn");
	static const FName PropName_BurnInClass							= TEXT("BurnInClass");
	static const FName PropName_bOverride_BurnInClass				= TEXT("bOverride_BurnInClass");
	static const FName PropName_bCompositeOntoFinalImage			= TEXT("bCompositeOntoFinalImage");
	static const FName PropName_bOverride_bCompositeOntoFinalImage	= TEXT("bOverride_bCompositeOntoFinalImage");

	/** Set a property on a UObject by name using its serialized string representation. */
	static void SetReflectedProperty(UObject* InObject, const FName PropertyName, const FString& Value)
	{
		if (!InObject)
		{
			return;
		}

		const FProperty* Property = FindFProperty<FProperty>(InObject->GetClass(), PropertyName);
		if (!Property)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "UMoviePipelineBasicConfig: Could not find property '%ls' on '%ls'", *PropertyName.ToString(), *InObject->GetClass()->GetName());
			return;
		}

		void* ValuePtr = Property->ContainerPtrToValuePtr<void>(InObject);
		if (Property->ImportText_Direct(*Value, ValuePtr, InObject, PPF_None) == nullptr)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "UMoviePipelineBasicConfig: Failed to set property '%ls' on '%ls' from value '%ls'",
				*PropertyName.ToString(), *InObject->GetClass()->GetName(), *Value);
		}
	}

	/** Set a bOverride_ flag on a UObject to true. */
	static void SetOverrideFlag(UObject* InObject, const FName FlagPropertyName)
	{
		if (!InObject)
		{
			return;
		}

		if (const FBoolProperty* BoolProp = CastField<FBoolProperty>(FindFProperty<FProperty>(InObject->GetClass(), FlagPropertyName)))
		{
			void* ValuePtr = BoolProp->ContainerPtrToValuePtr<void>(InObject);
			BoolProp->SetPropertyValue(ValuePtr, true);
		}
	}

	/** Template graph asset path. */
	static const FSoftObjectPath TemplateGraphPath(TEXT("/MovieRenderPipeline/BasicConfigTemplateGraph.BasicConfigTemplateGraph"));
	
	/** The path of the user-saved default for Basic configurations. This asset may not exist if a default has not been saved. */
	static const FString DefaultBasicConfigPath = TEXT("/Temp/MovieRenderPipeline/BasicConfigDefaults");

#if WITH_EDITOR
	/** Lay out all nodes in each branch left-to-right with a fixed stride. */
	static void LayoutGraphBranches(const UMovieGraphConfig* Graph)
	{
		constexpr int32 NodeWidth   	= 200;
		constexpr int32 NodeSpacing 	= 100;
		constexpr int32 Stride      	= NodeWidth + NodeSpacing;
		constexpr int32 BranchYSpacing	= 300;

		// Globals branch first, then all others
		TArray<FName> BranchOrder = Graph->GetBranchNames();
		BranchOrder.Remove(UMovieGraphNode::GlobalsPinName);
		BranchOrder.Insert(UMovieGraphNode::GlobalsPinName, 0);

		const int32 InputNodeX = Graph->GetInputNode()->GetNodePosX();
		int32 MaxNodeX = InputNodeX;
		int32 BranchY  = 0;

		for (const FName& BranchName : BranchOrder)
		{
			// Walk from InputNode's branch output pin through intermediate nodes to the OutputNode
			TArray<UMovieGraphNode*> IntermediateNodes;
			if (const UMovieGraphPin* BranchOutPin = Graph->GetInputNode()->GetOutputPin(BranchName))
			{
				const UMovieGraphPin* Next = BranchOutPin->GetFirstConnectedPin();
				while (Next && Next->Node != Graph->GetOutputNode())
				{
					IntermediateNodes.Add(Next->Node);
					const UMovieGraphPin* NodeOut = Next->Node->GetOutputPin(FName());
					if (!NodeOut)
					{
						break;
					}
					
					Next = NodeOut->GetFirstConnectedPin();
				}
			}

			// Assign X positions left-to-right, starting one Stride past InputNode
			for (int32 i = 0; i < IntermediateNodes.Num(); ++i)
			{
				const int32 NodeX = InputNodeX + Stride * (i + 1);
				IntermediateNodes[i]->SetNodePosX(NodeX);
				IntermediateNodes[i]->SetNodePosY(BranchY);
				MaxNodeX = FMath::Max(MaxNodeX, NodeX);
			}

			BranchY += BranchYSpacing;
		}

		// Center InputNode and OutputNode vertically across all branches
		const int32 TotalBranches = BranchOrder.Num();
		const int32 MidY = (TotalBranches > 1) ? ((TotalBranches - 1) * BranchYSpacing) / 2 : 0;
		Graph->GetInputNode()->SetNodePosY(MidY);

		// Push OutputNode to the right of all intermediate nodes
		Graph->GetOutputNode()->SetNodePosX(MaxNodeX + Stride);
		Graph->GetOutputNode()->SetNodePosY(MidY);
	}
#endif // WITH_EDITOR
}

UMoviePipelineBasicConfig::UMoviePipelineBasicConfig()
	: bOverride_OutputDirectory(0)
	, bOverride_FileNameFormat(0)
	, bOverride_OutputResolution(0)
	, bOverride_EnabledOutputTypes(0)
	, bOverride_CustomStartFrame(0)
	, bOverride_CustomEndFrame(0)
	, bOverride_bUseDeferredRenderer(0)
	, bOverride_DeferredSpatialSampleCount(0)
	, bOverride_DeferredAntiAliasingMethod(0)
	, bOverride_bUsePathTracedRenderer(0)
	, bOverride_PathTracedSpatialSampleCount(0)
	, bOverride_PathTracedDenoiserType(0)
	, bOverride_NumWarmUpFrames(0)
	, bOverride_TemporalSampleCount(0)
	, bOverride_BurnInClass(0)
	, CustomStartFrame(0)
	, CustomEndFrame(0)
	, bUseDeferredRenderer(true)
	, DeferredSpatialSampleCount(1)
	, DeferredAntiAliasingMethod(EAntiAliasingMethod::AAM_TSR)
	, bUsePathTracedRenderer(false)
	, PathTracedSpatialSampleCount(1)
	, PathTracedDenoiserType(EMoviePipelineBasicDenoiserType::Spatial)
	, NumWarmUpFrames(64)
	, TemporalSampleCount(1)
{
	OutputDirectory.Path = TEXT("{project_dir}/Saved/MovieRenders/");
	FileNameFormat = TEXT("{sequence_name}.{frame_number}");
	OutputResolution = FMovieGraphNamedResolution(FMovieGraphNamedResolution::DefaultResolutionName, FMovieGraphNamedResolution::DefaultResolution, FString());

	// Default to PNG output
	EnabledOutputTypes.Add(TSoftClassPtr<UMovieGraphFileOutputNode>(
		FSoftClassPath(TEXT("/Script/MovieRenderPipelineRenderPasses.MovieGraphImageSequenceOutputNode_PNG"))));

	BurnInClass = FSoftClassPath(TEXT("/MovieRenderPipeline/Blueprints/Graph/DefaultGraphBurnIn.DefaultGraphBurnIn_C"));
}

bool UMoviePipelineBasicConfig::HasAnyOverrides() const
{
	for (TFieldIterator<FBoolProperty> It(GetClass()); It; ++It)
	{
		if (It->GetName().StartsWith(TEXT("bOverride_")) && It->GetPropertyValue_InContainer(this))
		{
			return true;
		}
	}
	return false;
}

void UMoviePipelineBasicConfig::EnforceInvariants()
{
	if (!bUseDeferredRenderer && !bUsePathTracedRenderer)
	{
		bUseDeferredRenderer = true;
	}

	if (EnabledOutputTypes.IsEmpty())
	{
		EnabledOutputTypes.Add(TSoftClassPtr<UMovieGraphFileOutputNode>(
			FSoftClassPath(TEXT("/Script/MovieRenderPipelineRenderPasses.MovieGraphImageSequenceOutputNode_PNG"))));
	}
}

#if WITH_EDITOR
void UMoviePipelineBasicConfig::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	EnforceInvariants();
}
#endif

void UMoviePipelineBasicConfig::MergeOverrides(const UMoviePipelineBasicConfig* InShotOverrides, UMoviePipelineBasicConfig& OutMerged) const
{
	// For each property, use the shot override value if its bOverride_ flag is set; otherwise use the job value.
	// When InShotOverrides is null, all flags are effectively false, so job values are always used.
#define MERGE_PROPERTY(PropName) \
	OutMerged.bOverride_##PropName = (InShotOverrides && InShotOverrides->bOverride_##PropName) ? InShotOverrides->bOverride_##PropName : bOverride_##PropName; \
	OutMerged.PropName = (InShotOverrides && InShotOverrides->bOverride_##PropName) ? InShotOverrides->PropName : PropName;

	MERGE_PROPERTY(OutputDirectory);
	MERGE_PROPERTY(FileNameFormat);
	MERGE_PROPERTY(OutputResolution);
	MERGE_PROPERTY(EnabledOutputTypes);
	MERGE_PROPERTY(bUseDeferredRenderer);
	MERGE_PROPERTY(DeferredSpatialSampleCount);
	MERGE_PROPERTY(DeferredAntiAliasingMethod);
	MERGE_PROPERTY(bUsePathTracedRenderer);
	MERGE_PROPERTY(PathTracedSpatialSampleCount);
	MERGE_PROPERTY(PathTracedDenoiserType);
	MERGE_PROPERTY(NumWarmUpFrames);
	MERGE_PROPERTY(TemporalSampleCount);
	MERGE_PROPERTY(BurnInClass);

	// Custom playback range is applied while building the global shot list, before shot graphs are evaluated.
	// The value for a shot override is thus ignored.
	OutMerged.bOverride_CustomStartFrame = bOverride_CustomStartFrame;
	OutMerged.CustomStartFrame = CustomStartFrame;
	OutMerged.bOverride_CustomEndFrame = bOverride_CustomEndFrame;
	OutMerged.CustomEndFrame = CustomEndFrame;

#undef MERGE_PROPERTY
}

UMovieGraphConfig* UMoviePipelineBasicConfig::GenerateGraph(UMoviePipelineBasicConfig* InConfig, UObject* InOuter)
{
	using namespace UE::MoviePipeline::BasicConfig;

	if (!InConfig)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "UMoviePipelineBasicConfig::GenerateGraph called with null config.");
		return nullptr;
	}

	// Enforce invariants defensively — config may have been mutated programmatically or via scripting
	// without going through PostEditChangeProperty.
	InConfig->EnforceInvariants();

	// Load and duplicate the default graph so we can modify it freely.
	const UMovieGraphConfig* TemplateGraph = Cast<UMovieGraphConfig>(TemplateGraphPath.TryLoad());
	if (!TemplateGraph)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "UMoviePipelineBasicConfig::GenerateGraph: Failed to load the template graph.");
		return nullptr;
	}

	UMovieGraphConfig* Graph = DuplicateObject<UMovieGraphConfig>(TemplateGraph, InOuter);
	if (!Graph)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "UMoviePipelineBasicConfig::GenerateGraph: Failed to duplicate the template graph.");
		return nullptr;
	}

	// Find an existing node of the given class in the graph. Returns the first match, or nullptr.
	auto FindExistingNode = [Graph](const UClass* NodeClass) -> UMovieGraphNode*
	{
		for (const TObjectPtr<UMovieGraphNode>& Node : Graph->GetNodes())
		{
			if (Node && Node->GetClass() == NodeClass)
			{
				return Node;
			}
		}
		return nullptr;
	};

	// Find an existing node of the given class in the duplicated graph, or insert a new one after the
	// InputNode on the specified pin. New nodes are inserted between the InputNode and whatever is
	// already downstream, so existing nodes (e.g. Render Layer) stay near the OutputNode.
	UMovieGraphNode* InputNode = Graph->GetInputNode();
	auto FindOrInsertNode = [&](const FName& InputPinName, const TSubclassOf<UMovieGraphNode> NodeClass) -> UMovieGraphNode*
	{
		if (!NodeClass) { return nullptr; }

		if (UMovieGraphNode* Existing = FindExistingNode(NodeClass))
		{
			return Existing;
		}

		return Graph->InsertAfter(InputNode, NodeClass, InputPinName);
	};

	// Gather pin names from InputNode's output pins. The Globals pin and all other pins (render layers).
	const FName GlobalsPin = UMovieGraphNode::GlobalsPinName;
	FName RenderLayerPin;
	for (const UMovieGraphPin* Pin : InputNode->GetOutputPins())
	{
		if (Pin && Pin->Properties.Label != GlobalsPin)
		{
			RenderLayerPin = Pin->Properties.Label;
			break;
		}
	}

	// ── Globals branch ────────────────────────────────────────────────────────

	// GlobalOutputSettingNode: output directory, resolution, optional custom frame range
	UMovieGraphGlobalOutputSettingNode* GlobalOutputNode = Cast<UMovieGraphGlobalOutputSettingNode>(
		FindOrInsertNode(GlobalsPin, UMovieGraphGlobalOutputSettingNode::StaticClass()));
	if (GlobalOutputNode)
	{
		GlobalOutputNode->bOverride_OutputDirectory = true;
		GlobalOutputNode->OutputDirectory = InConfig->OutputDirectory;
		GlobalOutputNode->bOverride_OutputResolution = true;
		GlobalOutputNode->OutputResolution = InConfig->OutputResolution;

		if (InConfig->bOverride_CustomStartFrame)
		{
			GlobalOutputNode->bOverride_CustomPlaybackRangeStart = true;
			GlobalOutputNode->CustomPlaybackRangeStart.Type = EMovieGraphSequenceRangeType::Custom;
			GlobalOutputNode->CustomPlaybackRangeStart.Value = InConfig->CustomStartFrame;
		}

		if (InConfig->bOverride_CustomEndFrame)
		{
			GlobalOutputNode->bOverride_CustomPlaybackRangeEnd = true;
			GlobalOutputNode->CustomPlaybackRangeEnd.Type = EMovieGraphSequenceRangeType::Custom;
			GlobalOutputNode->CustomPlaybackRangeEnd.Value = InConfig->CustomEndFrame;
		}
	}

	// SamplingMethodNode: temporal sample count
	if (UMovieGraphSamplingMethodNode* SamplingNode = Cast<UMovieGraphSamplingMethodNode>(
		FindOrInsertNode(GlobalsPin, UMovieGraphSamplingMethodNode::StaticClass())))
	{
		SamplingNode->bOverride_TemporalSampleCount = true;
		SamplingNode->TemporalSampleCount = InConfig->TemporalSampleCount;
	}

	// WarmUpSettingNode: warm-up frame count
	if (UMovieGraphWarmUpSettingNode* WarmUpNode = Cast<UMovieGraphWarmUpSettingNode>(
		FindOrInsertNode(GlobalsPin, UMovieGraphWarmUpSettingNode::StaticClass())))
	{
		WarmUpNode->bOverride_NumWarmUpFrames = true;
		WarmUpNode->NumWarmUpFrames = InConfig->NumWarmUpFrames;
	}

	// ── Render layer branch ───────────────────────────────────────────────────
	if (!RenderLayerPin.IsNone())
	{
		// Deferred renderer — only add/configure if enabled
		if (InConfig->bUseDeferredRenderer)
		{
			const TSubclassOf<UMovieGraphNode> DeferredRendererClass =
				FSoftClassPath(TEXT("/Script/MovieRenderPipelineRenderPasses.MovieGraphDeferredRenderPassNode")).TryLoadClass<UMovieGraphNode>();

			if (UMovieGraphNode* DeferredNode = FindOrInsertNode(RenderLayerPin, DeferredRendererClass))
			{
				SetOverrideFlag(DeferredNode, PropName_bOverride_SpatialSampleCount);
				SetReflectedProperty(DeferredNode, PropName_SpatialSampleCount,
					FString::FromInt(InConfig->DeferredSpatialSampleCount));

				if (InConfig->bOverride_DeferredAntiAliasingMethod)
				{
					SetOverrideFlag(DeferredNode, PropName_bOverride_AntiAliasingMethod);
					SetReflectedProperty(DeferredNode, PropName_AntiAliasingMethod,
						StaticEnum<EAntiAliasingMethod>()->GetNameStringByValue(static_cast<int64>(InConfig->DeferredAntiAliasingMethod.GetValue())));
				}
			}
		}

		// PathTracer renderer — only add/configure if enabled
		if (InConfig->bUsePathTracedRenderer)
		{
			const TSubclassOf<UMovieGraphNode> PathTracerRendererClass =
				FSoftClassPath(TEXT("/Script/MovieRenderPipelineRenderPasses.MovieGraphPathTracerRenderPassNode")).TryLoadClass<UMovieGraphNode>();

			if (UMovieGraphNode* PathTracerNode = FindOrInsertNode(RenderLayerPin, PathTracerRendererClass))
			{
				SetOverrideFlag(PathTracerNode, PropName_bOverride_SpatialSampleCount);
				SetReflectedProperty(PathTracerNode, PropName_SpatialSampleCount, FString::FromInt(InConfig->PathTracedSpatialSampleCount));
				SetOverrideFlag(PathTracerNode, PropName_bOverride_bEnableDenoiser);
				SetReflectedProperty(PathTracerNode, PropName_bEnableDenoiser, InConfig->bOverride_PathTracedDenoiserType ? TEXT("true") : TEXT("false"));

				if (InConfig->bOverride_PathTracedDenoiserType)
				{
					SetOverrideFlag(PathTracerNode, PropName_bOverride_DenoiserType);
					SetReflectedProperty(PathTracerNode, PropName_DenoiserType,
						StaticEnum<EMoviePipelineBasicDenoiserType>()->GetNameStringByValue(static_cast<int64>(InConfig->PathTracedDenoiserType)));
				}
			}
		}

		// File output nodes from EnabledOutputTypes — route to the correct branch based on restriction.
		// Also configures burn-in on each output node that supports it (when the burn-in override is active).
		for (const TSoftClassPtr<UMovieGraphFileOutputNode>& OutputClassPtr : InConfig->EnabledOutputTypes)
		{
			UClass* OutputClass = OutputClassPtr.LoadSynchronous();
			if (!OutputClass || OutputClass->HasAnyClassFlags(CLASS_Abstract))
			{
				UE_LOGF(LogMovieRenderPipeline, Warning,
					"UMoviePipelineBasicConfig::GenerateGraph: Skipping invalid/abstract output type '%ls'.",
					*OutputClassPtr.ToString());
				continue;
			}

			// Check branch restriction via CDO — Globals-restricted nodes go on the Globals branch
			const UMovieGraphFileOutputNode* CDO = GetDefault<UMovieGraphFileOutputNode>(OutputClass);
			const bool bGlobalsOnly = CDO && CDO->GetBranchRestriction() == EMovieGraphBranchRestriction::Globals;
			const FName& TargetPin = bGlobalsOnly ? GlobalsPin : RenderLayerPin;

			UMovieGraphFileOutputNode* OutputNode = Cast<UMovieGraphFileOutputNode>(
				FindOrInsertNode(TargetPin, OutputClass));
			if (OutputNode)
			{
				OutputNode->bOverride_FileNameFormat = true;
				OutputNode->FileNameFormat = InConfig->FileNameFormat;

				// Burn-in: if the override is active and the node supports it, enable burn-in
				if (InConfig->bOverride_BurnInClass && InConfig->BurnInClass.IsValid()
					&& FindFProperty<FProperty>(OutputNode->GetClass(), PropName_bEnableBurnIn))
				{
					SetOverrideFlag(OutputNode, PropName_bOverride_bEnableBurnIn);
					SetReflectedProperty(OutputNode, PropName_bEnableBurnIn, TEXT("true"));
					SetOverrideFlag(OutputNode, PropName_bOverride_BurnInClass);
					SetReflectedProperty(OutputNode, PropName_BurnInClass, InConfig->BurnInClass.ToString());

					// Enable compositing only if the node exposes bCompositeOntoFinalImage
					if (FindFProperty<FProperty>(OutputNode->GetClass(), PropName_bCompositeOntoFinalImage))
					{
						SetOverrideFlag(OutputNode, PropName_bOverride_bCompositeOntoFinalImage);
						SetReflectedProperty(OutputNode, PropName_bCompositeOntoFinalImage, TEXT("true"));
					}
				}
			}
		}
	}

	// ── Layout ────────────────────────────────────────────────────────────────
#if WITH_EDITOR
	LayoutGraphBranches(Graph);
#endif

	return Graph;
}

const UMoviePipelineBasicConfig* UMoviePipelineBasicConfig::GetSavedDefault()
{
	using namespace UE::MoviePipeline::BasicConfig;

	// Use FindPackage first to avoid triggering the async loading path if already in memory.
	const UPackage* Package = FindPackage(nullptr, *DefaultBasicConfigPath);
	if (!Package)
	{
		Package = LoadPackage(nullptr, *DefaultBasicConfigPath, LOAD_None);
	}

	if (Package)
	{
		return Cast<UMoviePipelineBasicConfig>(FindObjectWithOuter(Package, StaticClass()));
	}

	return nullptr;
}

#if WITH_EDITOR
void UMoviePipelineBasicConfig::SaveAsDefault(const UMoviePipelineBasicConfig* InConfig)
{
	using namespace UE::MoviePipeline::BasicConfig;

	if (!InConfig)
	{
		return;
	}

	UPackage* Package = CreatePackage(*DefaultBasicConfigPath);
	UMoviePipelineBasicConfig* DuplicatedConfig = CastChecked<UMoviePipelineBasicConfig>(
		StaticDuplicateObject(InConfig, Package, TEXT("BasicConfigDefaults")));
	DuplicatedConfig->SetFlags(RF_Public | RF_Transactional | RF_Standalone);

	// Zero out shot-only bOverride_ flags via reflection. Preserve flags that also have job-level meaning.
	for (TFieldIterator<FBoolProperty> It(StaticClass()); It; ++It)
	{
		if (It->GetName().StartsWith(TEXT("bOverride_"))
			&& It->GetFName() != GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, bOverride_CustomStartFrame)
			&& It->GetFName() != GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, bOverride_CustomEndFrame)
			&& It->GetFName() != GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, bOverride_DeferredAntiAliasingMethod)
			&& It->GetFName() != GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, bOverride_PathTracedDenoiserType))
		{
			It->SetPropertyValue_InContainer(DuplicatedConfig, false);
		}
	}

	// Save to disk. Temporarily disable SCC auto-add for generated local files.
	const FString PackageFileName = FPackageName::LongPackageNameToFilename(DefaultBasicConfigPath, FPackageName::GetAssetPackageExtension());
	UEditorLoadingSavingSettings* SaveSettings = GetMutableDefault<UEditorLoadingSavingSettings>();
	const uint32 bSCCAutoAddNewFiles = SaveSettings->bSCCAutoAddNewFiles;
	SaveSettings->bSCCAutoAddNewFiles = 0;

	const bool bSuccess = SavePackageHelper(Package, *PackageFileName);

	SaveSettings->bSCCAutoAddNewFiles = bSCCAutoAddNewFiles;

	if (!bSuccess)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning,
			"UMoviePipelineBasicConfig::SaveAsDefault: Failed to save default config to '%ls'.", *PackageFileName);
	}
}
#endif // WITH_EDITOR
