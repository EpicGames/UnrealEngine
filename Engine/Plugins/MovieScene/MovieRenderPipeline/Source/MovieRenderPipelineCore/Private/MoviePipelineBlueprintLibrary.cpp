// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineBlueprintLibrary.h"
#include "Misc/PackageName.h"
#include "MoviePipeline.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieScenePossessable.h"
#include "MovieSceneBinding.h"
#include "LevelSequence.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "MoviePipelineQueue.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Sections/MovieSceneSpawnSection.h"
#include "MovieSceneTimeHelpers.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineSetting.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineCameraSetting.h"
#include "HAL/FileManager.h"
#include "Internationalization/Regex.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Misc/StringBuilder.h"
#include "MovieSceneSequenceVisitor.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneTransformTypes.h"
#include "MoviePipelineUtils.h"
#include "UObject/UObjectHash.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Features/IModularFeatures.h"
#include "Misc/EngineVersion.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Decorations/MovieSceneMuteSoloDecoration.h"

// For camera settings
#include "GameFramework/PlayerController.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "MoviePipelineHighResSetting.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineBlueprintLibrary)

EMovieRenderPipelineState UMoviePipelineBlueprintLibrary::GetPipelineState(const UMoviePipeline* InPipeline)
{
	if (InPipeline)
	{
		return InPipeline->GetPipelineState();
	}

	return EMovieRenderPipelineState::Uninitialized;
}

EMovieRenderShotState UMoviePipelineBlueprintLibrary::GetCurrentSegmentState(UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		int32 ShotIndex = InMoviePipeline->GetCurrentShotIndex();
		if (ShotIndex < InMoviePipeline->GetActiveShotList().Num())
		{
			return InMoviePipeline->GetActiveShotList()[ShotIndex]->ShotInfo.State;
		}
	}

	return EMovieRenderShotState::Uninitialized;
}

FText UMoviePipelineBlueprintLibrary::GetJobName(UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		return FText::FromString(InMoviePipeline->GetCurrentJob()->JobName);
	}

	return FText();
}

FText UMoviePipelineBlueprintLibrary::GetJobAuthor(UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		return FText::FromString(UE::MoviePipeline::GetJobAuthor(InMoviePipeline->GetCurrentJob()));
	}

	return FText();
}

void UMoviePipelineBlueprintLibrary::GetOverallOutputFrames(const UMoviePipeline* InMoviePipeline, int32& OutCurrentIndex, int32& OutTotalCount)
{
	OutCurrentIndex = 0;
	OutTotalCount = 0;
	if (InMoviePipeline)
	{
		OutCurrentIndex = InMoviePipeline->GetOutputState().OutputFrameNumber;

		for (const UMoviePipelineExecutorShot* Shot : InMoviePipeline->GetActiveShotList())
		{
			OutTotalCount += Shot->ShotInfo.WorkMetrics.TotalOutputFrameCount;
		}
	}
}

void UMoviePipelineBlueprintLibrary::GetCurrentSegmentName(UMoviePipeline* InMoviePipeline, FText& OutOuterName, FText& OutInnerName)
{
	if (InMoviePipeline)
	{
		int32 ShotIndex = InMoviePipeline->GetCurrentShotIndex();
		if (ShotIndex < InMoviePipeline->GetActiveShotList().Num())
		{
			OutOuterName = FText::FromString(InMoviePipeline->GetActiveShotList()[ShotIndex]->OuterName);
			OutInnerName = FText::FromString(InMoviePipeline->GetActiveShotList()[ShotIndex]->InnerName);
		}
	}
}

FDateTime UMoviePipelineBlueprintLibrary::GetJobInitializationTime(const UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		return InMoviePipeline->GetInitializationTime();
	}

	return FDateTime();
}

FMoviePipelineSegmentWorkMetrics UMoviePipelineBlueprintLibrary::GetCurrentSegmentWorkMetrics(const UMoviePipeline* InMoviePipeline)
{
	if(InMoviePipeline)
	{
		int32 ShotIndex = InMoviePipeline->GetCurrentShotIndex();
		if (ShotIndex < InMoviePipeline->GetActiveShotList().Num())
		{
			return InMoviePipeline->GetActiveShotList()[ShotIndex]->ShotInfo.WorkMetrics;
		}
	}

	return FMoviePipelineSegmentWorkMetrics();
}

void UMoviePipelineBlueprintLibrary::GetOverallSegmentCounts(const UMoviePipeline* InMoviePipeline, int32& OutCurrentIndex, int32& OutTotalCount)
{
	OutCurrentIndex = InMoviePipeline->GetCurrentShotIndex();
	OutTotalCount = InMoviePipeline->GetActiveShotList().Num();
}

UMovieSceneSequence* UMoviePipelineBlueprintLibrary::DuplicateSequence(UObject* Outer, UMovieSceneSequence* InSequence)
{
	if (!InSequence)
	{
		return nullptr;
	}

	FObjectDuplicationParameters DuplicationParams(InSequence, Outer);
	DuplicationParams.DestName = MakeUniqueObjectName(Outer, UMovieSceneSequence::StaticClass());

	// Duplicate the given sequence.
	UMovieSceneSequence* DuplicatedSequence = CastChecked<UMovieSceneSequence>(StaticDuplicateObjectEx(DuplicationParams));

	// Now go through looking for Shot and SubSequence tracks. These currently point to the same (shared) sequence as the InSequence.
	TArray<UMovieSceneSection*> AllSubSequenceSections;

	UMovieSceneCinematicShotTrack* ShotTrack = DuplicatedSequence->GetMovieScene()->FindTrack<UMovieSceneCinematicShotTrack>();
	if (ShotTrack)
	{
		AllSubSequenceSections.Append(ShotTrack->GetAllSections());
	}

	UMovieSceneSubTrack* SubTrack = DuplicatedSequence->GetMovieScene()->FindTrack<UMovieSceneSubTrack>();
	if (SubTrack)
	{
		AllSubSequenceSections.Append(SubTrack->GetAllSections());
	}

	for (UMovieSceneSection* Section : AllSubSequenceSections)
	{
		UMovieSceneSubSection* SubSequenceSection = CastChecked<UMovieSceneSubSection>(Section);
		UMovieSceneSequence* OriginalSubSequence = SubSequenceSection->GetSequence();

		// Recursively duplicate this sequence, and then update our sequence to point to the duplicated one.
		UMovieSceneSequence* DuplicatedSubSequence = DuplicateSequence(Outer, OriginalSubSequence);
		SubSequenceSection->SetSequence(DuplicatedSubSequence);
	}

	return DuplicatedSequence;
}

bool UMoviePipelineBlueprintLibrary::GetEstimatedTimeRemaining(const UMoviePipeline* InPipeline, FTimespan& OutEstimate)
{
	if (!InPipeline)
	{
		OutEstimate = FTimespan();
		return false;
	}

	// If they haven't produced a single frame yet, we can't give an estimate.
	int32 OutputFrames;
	int32 TotalOutputFrames;
	GetOverallOutputFrames(InPipeline, OutputFrames, TotalOutputFrames);

	if (OutputFrames <= 0 || TotalOutputFrames <= 0)
	{
		OutEstimate = FTimespan();
		return false;
	}

	float CompletionPercentage = OutputFrames / float(TotalOutputFrames);;
	FTimespan CurrentDuration = FDateTime::UtcNow() - InPipeline->GetInitializationTime();

	// If it has taken us CurrentDuration to process CompletionPercentage samples, then we can get a total duration
	// estimate by taking (CurrentDuration/CompletionPercentage) and then take that total estimate minus elapsed
	// to get remaining.
	FTimespan EstimatedTotalDuration = CurrentDuration / CompletionPercentage;
	OutEstimate = EstimatedTotalDuration - CurrentDuration;

	return true;
}

float UMoviePipelineBlueprintLibrary::GetCompletionPercentage(const UMoviePipeline* InPipeline)
{
	if (!InPipeline)
	{
		return 0.f;
	}

	int32 OutputFrames;
	int32 TotalOutputFrames;
	GetOverallOutputFrames(InPipeline, OutputFrames, TotalOutputFrames);

	float CompletionPercentage = FMath::Clamp(OutputFrames / (float)TotalOutputFrames, 0.f, 1.f);
	return CompletionPercentage;
}

FTimecode UMoviePipelineBlueprintLibrary::GetRootTimecode(const UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		return InMoviePipeline->GetOutputState().SourceTimeCode;
	}

	return FTimecode();
}

FFrameNumber UMoviePipelineBlueprintLibrary::GetRootFrameNumber(const UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		if (InMoviePipeline->GetTargetSequence() && InMoviePipeline->GetTargetSequence()->GetMovieScene())
		{
			FFrameRate EffectiveFrameRate = InMoviePipeline->GetPipelinePrimaryConfig()->GetEffectiveFrameRate(InMoviePipeline->GetTargetSequence());
			return GetRootTimecode(InMoviePipeline).ToFrameNumber(EffectiveFrameRate);
		}
	}

	return FFrameNumber(-1);
}

FTimecode UMoviePipelineBlueprintLibrary::GetCurrentShotTimecode(const UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		return InMoviePipeline->GetOutputState().CurrentShotSourceTimeCode;
	}

	return FTimecode();
}

FFrameNumber UMoviePipelineBlueprintLibrary::GetCurrentShotFrameNumber(const UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		FFrameRate EffectiveFrameRate = InMoviePipeline->GetPipelinePrimaryConfig()->GetEffectiveFrameRate(InMoviePipeline->GetTargetSequence());
		return GetCurrentShotTimecode(InMoviePipeline).ToFrameNumber(EffectiveFrameRate);
	}

	return FFrameNumber(-1);
}

float UMoviePipelineBlueprintLibrary::GetCurrentFocusDistance(const UMoviePipeline* InMoviePipeline)
{
	if (const UCineCameraComponent* CineCameraComponent = Utility_GetCurrentCineCamera(InMoviePipeline->GetWorld()))
	{
		return CineCameraComponent->CurrentFocusDistance;
	}

	return -1.0f;
}

float UMoviePipelineBlueprintLibrary::GetCurrentFocalLength(const UMoviePipeline* InMoviePipeline)
{
	if (const UCineCameraComponent* CineCameraComponent = Utility_GetCurrentCineCamera(InMoviePipeline->GetWorld()))
	{
		return CineCameraComponent->CurrentFocalLength;
	}

	return -1.0f;
}

float UMoviePipelineBlueprintLibrary::GetCurrentAperture(const UMoviePipeline* InMoviePipeline)
{
	if (const UCineCameraComponent* CineCameraComponent = Utility_GetCurrentCineCamera(InMoviePipeline->GetWorld()))
	{
		return CineCameraComponent->CurrentAperture;
	}

	return 0.0f;
}

FString UMoviePipelineBlueprintLibrary::GetMapPackageName(UMoviePipelineExecutorJob* InJob)
{
	if (!InJob)
	{
		return FString();
	}

	return InJob->Map.GetLongPackageName();
}

UMoviePipelineQueue* UMoviePipelineBlueprintLibrary::LoadManifestFileFromString(const FString& InManifestFilePath)
{
	FString InFileName = TEXT("QueueManifest");
	FString InPackagePath = TEXT("/Engine/MovieRenderPipeline/Editor/Transient");

	FString NewPackageName = FPackageName::GetLongPackagePath(InPackagePath) + TEXT("/") + InFileName;

	// Relative paths are considered relative to the game's save directory to avoid having to hard code
	// game names into relative paths.
	FString ConfigAssetPath = InManifestFilePath;
	if (FPaths::IsRelative(InManifestFilePath))
	{
		ConfigAssetPath = FPaths::Combine(FPaths::ProjectSavedDir(), InManifestFilePath);
	}

	UPackage* OuterPackage = CreatePackage(*NewPackageName);
	UPackage* QueuePackage = LoadPackage(OuterPackage, *ConfigAssetPath, LOAD_None);
	UMoviePipelineQueue* OutQueue = nullptr;
	if (QueuePackage)
	{
		OutQueue = Cast<UMoviePipelineQueue>((UObject*)FindObjectWithOuter(QueuePackage, UMoviePipelineQueue::StaticClass()));
	}

	return OutQueue;
}

void UMoviePipelineBlueprintLibrary::UpdateJobShotListFromSequence(ULevelSequence* InSequence, UMoviePipelineExecutorJob* InJob, bool& bShotsChanged)
{
	using namespace UE::MovieScene;

	if (!ensureMsgf(InSequence && InJob, TEXT("Cannot generate shot list for null sequence/job.")))
	{
		return;
	}

	using FSequenceRangeInfo = TTuple<UMovieSceneSection*, FMovieSceneSequenceID, int16>;
	struct FSequenceRangeVisitor : ISequenceVisitor
	{
		// Add one root-space section range to the evaluation tree. All callers funnel through
		// this helper so normal inverse-transform results and fallback ranges preserve the
		// same section, sequence-id, and hierarchical-bias semantics.
		void AddRangeToTree(UMovieSceneSection* InSection, const TRange<FFrameNumber>& InRootRange, const FSubSequenceSpace& LocalSpace, TMovieSceneEvaluationTree<FSequenceRangeInfo>* InTree)
		{
			// Empty intersections are valid outcomes when a parent sequence clips a child section
			// completely out of the root evaluation range; those sections should not create shots.
			if (InRootRange.IsEmpty())
			{
				return;
			}

			if (const UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(InSection))
			{
				// Shot sections represent their child sequence, while camera cuts belong to the
				// currently visited sequence. Store the child sequence ID so the later hierarchy
				// walk can resolve the correct shot/subsequence instead of the parent container.
				InTree->Add(InRootRange, MakeTuple(InSection, ShotSection->GetSequenceID(), LocalSpace.HierarchicalBias));
			}
			else
			{
				// Non-shot sections, currently camera cuts for this visitor, are already authored
				// in the visited sequence's space, so LocalSpace.SequenceID is the owning sequence.
				InTree->Add(InRootRange, MakeTuple(InSection, LocalSpace.SequenceID, LocalSpace.HierarchicalBias));
			}
		}

		void GatherFromSection(UMovieSceneTrack* InTrack, const UE::MovieScene::FSubSequenceSpace& LocalSpace, TMovieSceneEvaluationTree<FSequenceRangeInfo>* InTree)
		{
			TArray<FSequenceRangeInfo> OutRanges;

			FMovieSceneInverseSequenceTransform LocalToRoot = LocalSpace.RootToSequenceTransform.Inverse();

			for (UMovieSceneSection* Section : InTrack->GetAllSections())
			{
				if(!Section)
				{
					continue;
				}
				
				if (!Section->IsActive())
				{
					continue;
				}

				// When a sequence is contained by a Shot Section, it can offset/clip the playback range. So we want to scan for any sections that
				// fall entirely out of the actually evaluated range, and discard them. To do this, we transform from our parent (RootToSequenceTransform)
				// to local space, and then compare the local camera range to that and discard it. When there is no parent, the RootToSequence transform
				// is identity, and thus it just clips it against the normal Playback Range.
				TRange<FFrameNumber> LocalCameraRange = Section->GetRange();

				bool bVisitedRange = false;
				auto Visit = [this, &LocalSpace, Section, InTree, &bVisitedRange](TRange<FFrameTime> InRange)
				{
					bVisitedRange = true;

					// Intersect it with the root range so that if the parent has trimmed down the sub-section we don't render outside that.
					TRange<FFrameNumber> RootCameraRange = TRange<FFrameNumber>::Intersection(LocalSpace.RootClampRange, ConvertToDiscreteRange(InRange));
					if (!RootCameraRange.IsEmpty())
					{
						AddRangeToTree(Section, RootCameraRange, LocalSpace, InTree);
					}
					return true;
				};

				LocalToRoot.TransformFiniteRangeWithinRange(ConvertToFrameTimeRange(LocalCameraRange), Visit, LocalSpace.StartBreadcrumbs, LocalSpace.EndBreadcrumbs);

				// Time-warp curves can make inverse range solving fail even though the section is still editorially active.
				// Fall back to the visited root clamp range so the shot remains visible in MRQ/MRG.
				if (!bVisitedRange)
				{
					AddRangeToTree(Section, LocalSpace.RootClampRange, LocalSpace, InTree);
				}
			}
		}

		virtual void VisitTrack(UMovieSceneTrack* InTrack, const FGuid&, const UE::MovieScene::FSubSequenceSpace& LocalSpace)
		{
			if (UMovieSceneCameraCutTrack* CameraCutTrack = Cast<UMovieSceneCameraCutTrack>(InTrack))
			{
				GatherFromSection(CameraCutTrack, LocalSpace, &CameraCutTree);
			}

			if (UMovieSceneCinematicShotTrack* ShotTrack = Cast<UMovieSceneCinematicShotTrack>(InTrack))
			{
				GatherFromSection(ShotTrack, LocalSpace, &ShotSectionTree);
			}
		}
		TMovieSceneEvaluationTree<FSequenceRangeInfo> CameraCutTree;
		TMovieSceneEvaluationTree<FSequenceRangeInfo> ShotSectionTree;
	};

	{
		UE::MovieScene::FSequenceVisitParams Params;
		Params.bVisitRootTracks = true;
		Params.bVisitSubSequences = true;
		FSequenceRangeVisitor RangeVisitor;

		// Visit all camera cuts
		VisitSequence(InSequence, Params, RangeVisitor);

		struct FLinearizedEntity
		{
			TRange<FFrameNumber> Range;
			UMovieSceneCameraCutSection* CameraCut;
			FMovieSceneSequenceID SequenceID;
			TArray<FMoviePipelineSidecarCamera> SidecarCameras;

			// These are only updated after the final linearized entities are chosen
			TTuple<FString, FString> Name;
			TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> LeafNode;
			TRange<FFrameNumber> CameraCutWarmUpRange;
			FMovieSceneSequenceTransform RootToInnerTransform;
		};

		TArray<FLinearizedEntity> Entities;

		// Give preference to the camera cut tree if we have any entries, otherwise we fall back to just shot sections
		TMovieSceneEvaluationTree<FSequenceRangeInfo>* RangeToWalk = RangeVisitor.CameraCutTree.IsEmpty() ? &RangeVisitor.ShotSectionTree : &RangeVisitor.CameraCutTree;
		
		// Walk the camera cut tree, linearizing the results
		FMovieSceneEvaluationTreeRangeIterator RangeIt(*RangeToWalk);
		for (; RangeIt; ++RangeIt)
		{
			// Look up the camera cuts for this range
			TMovieSceneEvaluationTreeDataIterator<FSequenceRangeInfo> DataIt = RangeToWalk->GetAllData(RangeIt.Node());
			if (!DataIt)
			{
				continue;
			}

			// Find the best Camera Cut based on HBias
			FSequenceRangeInfo BestInfo = *DataIt;
			++DataIt;
			for (; DataIt; ++DataIt)
			{
				int16 DataHBias = DataIt->Get<2>();
				int16 BestHBias = BestInfo.Get<2>();
				if (DataHBias > BestHBias)
				{
					BestInfo = *DataIt;
				}
			}

			TRange<FFrameNumber> IntersectedWithPlaybackBounds = TRange<FFrameNumber>::Intersection(RangeIt.Range(), InSequence->GetMovieScene()->GetPlaybackRange());

			// If it falls completely outside the playback bounds, no need to include it.
			if (IntersectedWithPlaybackBounds.IsEmpty())
			{
				continue;
			}

			// Use our intersected range to trim by the outmost Playback Range.
			FLinearizedEntity NewEntry{ IntersectedWithPlaybackBounds, Cast<UMovieSceneCameraCutSection>(BestInfo.Get<0>()), BestInfo.Get<1>() };
			
			bool bMerged = false;
			if (Entities.Num())
			{
				FLinearizedEntity& LastEntity = Entities.Last();
				// Check whether we should combine with the last one
				const bool bShotMatches = LastEntity.SequenceID == NewEntry.SequenceID;
				const bool bCameraCutMatches = (LastEntity.CameraCut == NewEntry.CameraCut) && LastEntity.CameraCut != nullptr;
				if (LastEntity.Range.Adjoins(NewEntry.Range) && bShotMatches && bCameraCutMatches)
				{
					LastEntity.Range.SetUpperBound(NewEntry.Range.GetUpperBound());
					bMerged = true;
				}
			}

			if(!bMerged)
			{
				Entities.Add(NewEntry);
			}
		}

		FMovieSceneSequenceHierarchy SequenceHierarchyCache = FMovieSceneSequenceHierarchy();
		UMovieSceneCompiledDataManager::CompileHierarchy(InSequence, &SequenceHierarchyCache, EMovieSceneServerClientMask::All);

		// We now have the ranges we want to render (for camera cuts), 
		for (FLinearizedEntity& Entity : Entities)
		{
			// Let's build a sub-section hierarchy for this node
			TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> LeafNode = MakeShared<MoviePipeline::FCameraCutSubSectionHierarchyNode>();
			
			// This can be null in the fallback case where no camera cuts were detected
			LeafNode->CameraCutSection = Entity.CameraCut;
			MoviePipeline::BuildSectionHierarchyRecursive(SequenceHierarchyCache, InSequence, Entity.SequenceID, MovieSceneSequenceID::Invalid, LeafNode);

			// If we did find a camera cut section, we're going to look for any object bindings in the same sequence that contain
			// camera components and track them as sidecar cameras. We do this even if sidecar data isn't being used.
			if (LeafNode->CameraCutSection.IsValid() && LeafNode->MovieScene.IsValid())
			{
				UMovieSceneSequence* Sequence = LeafNode->MovieScene->GetTypedOuter<UMovieSceneSequence>();
				TSharedRef<UE::MovieScene::FSharedPlaybackState> TransientPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(GWorld, Sequence);
				int32 MainCameraIndex = INDEX_NONE;
				FMovieSceneObjectBindingID MainBinding = LeafNode->CameraCutSection->GetCameraBindingID();

				for (int32 Index = 0; Index < LeafNode->MovieScene->GetPossessableCount(); Index++)
				{
					FMovieScenePossessable& Possessable = LeafNode->MovieScene->GetPossessable(Index);
					FGuid ValidBinding = FGuid();

#if WITH_EDITOR
					// This is only available in the editor, so in the editor we'll auto-detect the class
					// but in runtime builds they'll have to manually tag it with "Camera" :(
					const UClass* PossessedClass = Possessable.GetPossessedObjectClass();

					// A possessable might point to a unloaded class, but we can safely skip it
					// because we only need to look for UCameraComponents and those are core engine so
					// they will always be loaded.
					if (!PossessedClass)
					{
						continue;
					}
					if (PossessedClass->IsChildOf<UCameraComponent>())
					{
						ValidBinding = Possessable.GetGuid();
					}
#endif
					if (Possessable.Tags.Contains(TEXT("Camera")))
					{
						// Don't duplicate them up, but also run this name check in the editor
						ValidBinding = Possessable.GetGuid();
					}

					if (ValidBinding.IsValid())
					{
						FMoviePipelineSidecarCamera SidecarCamera;
						SidecarCamera.BindingId = ValidBinding;
						SidecarCamera.SequenceId = Entity.SequenceID; 
						bool bAddCamera = true;

						TArray<FGuid> CheckedBindings;
						
						// We break out of the while loop below when the parent guid is invalid, so we track
						// the last known valid GUID too, so we can compare the root binding against the camera.
						FGuid LastValidGuid = Possessable.GetGuid();
						FGuid ParentGuid = Possessable.GetParent();
						CheckedBindings.Add(LastValidGuid);

						while (ParentGuid.IsValid())
						{
							// Bypass if the binding is muted
							if (FMovieSceneBinding* Binding = LeafNode->MovieScene->FindBinding(ParentGuid))
							{
								if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Cast<UMovieSceneMuteSoloDecoration>(Binding->FindDecoration<UMovieSceneMuteSoloDecoration>()))
								{
									if (MuteSoloDecoration->IsMuted())
									{
										bAddCamera = false;
										UE_LOGF(LogMovieRenderPipeline, Log, "Skipping muted camera: %ls", *Possessable.GetName());
										break;
									}
								}
							}

							if (FMovieScenePossessable* ParentAsPossessable = LeafNode->MovieScene->FindPossessable(ParentGuid))
							{
								LastValidGuid = ParentGuid;
								ParentGuid = ParentAsPossessable->GetParent();
								SidecarCamera.Name = ParentAsPossessable->GetName();
								CheckedBindings.Add(LastValidGuid);

								if (MovieSceneHelpers::IsBoundToAnySpawnable(LeafNode->MovieScene->GetTypedOuter<UMovieSceneSequence>(), ParentAsPossessable->GetGuid(), TransientPlaybackState))
								{
									FMovieSceneBinding* Binding = LeafNode->MovieScene->FindBinding(ParentAsPossessable->GetGuid());

									if (Binding)
									{
										const TArray<UMovieSceneTrack*> AllBindingTracks = Binding->GetTracks();
										for (const UMovieSceneTrack* Track : AllBindingTracks)
										{
											// Bypass if the spawn track is disabled
											if (Track->IsEvalDisabled())
											{
												bAddCamera = false;
												UE_LOGF(LogMovieRenderPipeline, Log, "Skipping disabled camera: %ls", *ParentAsPossessable->GetName());
												continue;
											}

											if (const UMovieSceneSpawnTrack* SpawnTrack = Cast<UMovieSceneSpawnTrack>(Track))
											{
												for (const UMovieSceneSection* Section : SpawnTrack->GetAllSections())
												{
													if (const UMovieSceneSpawnSection* BoolSection = Cast<UMovieSceneSpawnSection>(Section))
													{
														bAddCamera = BoolSection->GetChannel().GetDefault().Get(true);
													}
												}
											}
										}
									}
								}
							}
							else
							{
								// If the content is set up wrong and doesn't have a valid parent, we need to break out of the loop
								UE_LOGF(LogMovieRenderPipeline, Error, "A camera binding was found while calculating multi-camera support, but the binding had an invalid parent (ParentGuid %ls), skipping...", *ParentGuid.ToString());
								break;
							}
						}

						if (bAddCamera)
						{
							// This comparison doesn't support cross-sequence references, technically we should create a 
							// FMovieSceneObjectBindingID etc, but because we only support finding sidecar cameras in the
							// same sequence as the main camera cut track, this is fine for now.
							if (CheckedBindings.Contains(MainBinding.GetGuid()))
							{
								// We want to know which sidecar camera this is, not the possessable index.
								MainCameraIndex = Entity.SidecarCameras.Num();
							}

							Entity.SidecarCameras.Add(SidecarCamera);
						}
					}
				}

				if (MainCameraIndex == INDEX_NONE)
				{
					// When the camera cut points to a camera in another Sequence it won't get detected by the above loop,
					// so we manually add it as the only sidecar camera. We need to check that the CameraCutSection points
					// to a real camera though, as they may have an unbound camera.
					if (LeafNode->CameraCutSection->GetCameraBindingID().GetGuid().IsValid())
					{
						FMoviePipelineSidecarCamera& SidecarCamera = Entity.SidecarCameras.AddDefaulted_GetRef();
						SidecarCamera.BindingId = LeafNode->CameraCutSection->GetCameraBindingID().GetGuid();
						SidecarCamera.SequenceId = LeafNode->CameraCutSection->GetCameraBindingID().GetRelativeSequenceID();

						// ToDo: We should fetch the actual camera name from the Object Binding in the other sequence.
						SidecarCamera.Name = TEXT("main_camera");
						MainCameraIndex = Entity.SidecarCameras.Num() - 1;
					}
				}

				// Ensure the "main" camera (specified by the Camera Cut Track) is always index 0, so that it's rendered
				// first, and the stable sort (on output files) still tries to keep it as the primary layer.
				if (MainCameraIndex >= 0)
				{
					Entity.SidecarCameras.Swap(MainCameraIndex, 0);
				}
			}

			// To make the camera cut range detection more consistent, we'll convert the start of this Entity into root sequence space, and then we'll convert the start of the
			// camera cut into root sequence space, and the difference between them is the range to use. This is a more reliable way than looking at the actual Playback Range.
			TRange<FFrameNumber> EntityRangeInRoot = Entity.Range;
			TRange<FFrameNumber> CameraCutRangeInRoot = EntityRangeInRoot;

			FMovieSceneSubSequenceData* SubSequenceData = SequenceHierarchyCache.FindSubData(Entity.SequenceID);
			if (SubSequenceData)
			{
				Entity.RootToInnerTransform = SubSequenceData->RootToSequenceTransform;
			}

			if (LeafNode->CameraCutSection.IsValid())
			{
				CameraCutRangeInRoot = LeafNode->CameraCutSection->GetRange();

				// Put the camera cut range in root space
				FMovieSceneInverseSequenceTransform InnerToRoot = Entity.RootToInnerTransform.Inverse();

				FMovieSceneTransformBreadcrumbs RangeStartBreadcrumbs, RangeEndBreadcrumbs;
				Entity.RootToInnerTransform.TransformTime(UE::MovieScene::DiscreteInclusiveLower(EntityRangeInRoot), UE::MovieScene::FTransformTimeParams().HarvestBreadcrumbs(RangeStartBreadcrumbs));
				Entity.RootToInnerTransform.TransformTime(UE::MovieScene::DiscreteExclusiveUpper(EntityRangeInRoot), UE::MovieScene::FTransformTimeParams().HarvestBreadcrumbs(RangeEndBreadcrumbs));

				bool bConvertedCameraCutRange = false;
				auto Visit = [&CameraCutRangeInRoot, &bConvertedCameraCutRange](TRange<FFrameTime> InRange)
				{
					CameraCutRangeInRoot = ConvertToDiscreteRange(InRange);
					bConvertedCameraCutRange = true;
					return false; // only call this once
				};
				InnerToRoot.TransformFiniteRangeWithinRange(ConvertToFrameTimeRange(CameraCutRangeInRoot), Visit, RangeStartBreadcrumbs, RangeEndBreadcrumbs);

				// If a time-warped camera cut cannot be mapped back to root space, use the
				// already-root-space entity range so warmup math does not mix coordinate spaces.
				if (!bConvertedCameraCutRange)
				{
					CameraCutRangeInRoot = EntityRangeInRoot;
				}
			}

			TRange<FFrameNumber> CameraCutWarmUpRange = TRange<FFrameNumber>::Empty();
			if (EntityRangeInRoot.GetLowerBound().IsClosed() && CameraCutRangeInRoot.GetLowerBound().IsClosed())
			{
				CameraCutWarmUpRange = TRange<FFrameNumber>(CameraCutRangeInRoot.GetLowerBound(), EntityRangeInRoot.GetLowerBound());
			}
			if (CameraCutRangeInRoot.IsDegenerate())
			{
				CameraCutRangeInRoot = TRange<FFrameNumber>::Empty();
			}

			Entity.CameraCutWarmUpRange = CameraCutWarmUpRange;
			Entity.LeafNode = LeafNode;
			Entity.Name = MoviePipeline::GetNameForShot(SequenceHierarchyCache, InSequence, LeafNode);
		}

		// Fallback case for no shot sections detected and no camera cut sections detected.
		if (Entities.Num() == 0)
		{
			FLinearizedEntity& Entity = Entities.AddDefaulted_GetRef();

			Entity.Range = InSequence->GetMovieScene()->GetPlaybackRange();
			Entity.CameraCut = nullptr;
			Entity.SequenceID = MovieSceneSequenceID::Root;

			TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> LeafNode = MakeShared<MoviePipeline::FCameraCutSubSectionHierarchyNode>();
			LeafNode->CameraCutSection = nullptr;
			MoviePipeline::BuildSectionHierarchyRecursive(SequenceHierarchyCache, InSequence, MovieSceneSequenceID::Root, MovieSceneSequenceID::Invalid, LeafNode);

			Entity.CameraCutWarmUpRange = TRange<FFrameNumber>::Empty();
			Entity.LeafNode = LeafNode;
			Entity.Name = MoviePipeline::GetNameForShot(SequenceHierarchyCache, InSequence, LeafNode);
			Entity.RootToInnerTransform = FMovieSceneSequenceTransform();
		}

		// We need to generate all of the linearized segments first so that we have all of the names available.
		// We need all of the names available because we need to ensure unique shot names (for XMLs), and they 
		// expect to be consistent (ie: if duplicates are found, the first one is retroactively changed to (1)).
		{
			TArray<FString> ShotNames;
			for (FLinearizedEntity& Entity : Entities)
			{
				// Insert the outer name portion (which is the shot)
				ShotNames.Add(Entity.Name.Get<1>());
			}

			// Now fix up the names to avoid duplicates
			UE::MoviePipeline::DeduplicateNameArray(ShotNames);

			// Now re-apply the fixed names
			for (int32 Index = 0; Index < Entities.Num(); Index++)
			{
				Entities[Index].Name.Get<1>() = ShotNames[Index];
			}
		}

		// Repeat for camera names within each shot.
		{
			for (FLinearizedEntity& Entity : Entities)
			{
				TArray<FString> CameraNames;
				for (FMoviePipelineSidecarCamera& SidecarCamera : Entity.SidecarCameras)
				{
					CameraNames.Add(SidecarCamera.Name);
				}
				
				UE::MoviePipeline::DeduplicateNameArray(CameraNames);

				for (int32 Index = 0; Index < Entity.SidecarCameras.Num(); Index++)
				{
					Entity.SidecarCameras[Index].Name = CameraNames[Index];
				}
			}
		}

		// Finally, we can take the entities and turn them into shots. We have to do all of the name
		// manipulation all upfront before looking for matching shots that already exist.
		TArray<UMoviePipelineExecutorShot*> NewShots;
		for (FLinearizedEntity& Entity : Entities)
		{
			// Try to find an existing shot to update - this way we respect the toggled UI state.
			UMoviePipelineExecutorShot* NewShot = nullptr;
			for (UMoviePipelineExecutorShot* Shot : InJob->ShotInfo)
			{
				if (!Shot)
				{
					UE_LOGF(LogMovieRenderPipeline, Warning, "Null ShotInfo in job, ignoring...");
					continue;
				}

				// For now we just compare the inner and outer names to decide if they match
				if (Shot->InnerName == Entity.Name.Get<0>() && Shot->OuterName == Entity.Name.Get<1>())
				{
					NewShot = Shot;
					break;
				}
			}

			// We can run into certain situations where the shots look the same ie: a root with a sub-scene and a camera being used multiple times
			// have the same inner and outer name. We don't want to re-use the same shot multiple times

			if (!NewShot)
			{
				NewShot = NewObject<UMoviePipelineExecutorShot>(InJob);
				NewShot->SetFlags(RF_Transactional);
				
				UE_LOGF(LogMovieRenderPipeline, Log, "Generated new ShotInfo for Inner: %ls Outer: %ls (No existing shot found in the job).", *Entity.Name.Get<0>(), *Entity.Name.Get<1>());
			}
			else
			{
				UE_LOGF(LogMovieRenderPipeline, Log, "Reusing existing ShotInfo for Inner: %ls Outer: %ls.", *Entity.Name.Get<0>(), *Entity.Name.Get<1>());

			}
			NewShots.Add(NewShot);

			// Reset the shot info (as it is all transient data) so it doesn't get re-used between runs.
			NewShot->ShotInfo = FMoviePipelineCameraCutInfo();
			NewShot->InnerName = Entity.Name.Get<0>();
			NewShot->OuterName = Entity.Name.Get<1>();
			NewShot->ShotInfo.SubSectionHierarchy = Entity.LeafNode;
			NewShot->ShotInfo.TotalOutputRangeRoot = Entity.Range;
			NewShot->ShotInfo.WarmupRangeRoot = Entity.CameraCutWarmUpRange;
			NewShot->ShotInfo.OuterToInnerTransform = Entity.RootToInnerTransform;
			NewShot->SidecarCameras = Entity.SidecarCameras;
			UE_LOGF(LogMovieRenderPipeline, Log, "Registering range: %ls (InnerName: %ls OuterName: %ls)", *LexToString(NewShot->ShotInfo.TotalOutputRangeRoot), *NewShot->InnerName, *NewShot->OuterName);
		}

		// Now that we've read the job's shot mask we will clear it and replace it with only things still valid.
		if (ToRawPtrTArrayUnsafe(InJob->ShotInfo) != NewShots)
		{
			InJob->ShotInfo.Reset();
			InJob->ShotInfo = NewShots;
			bShotsChanged = true;
		}
	}
}

int32 UMoviePipelineBlueprintLibrary::ResolveVersionNumber(FMoviePipelineFilenameResolveParams InParams, bool bGetNextVersion)
{
	if (!InParams.Job)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot resolve Version Number without a Job to pull settings from."), ELogVerbosity::Error);
		return -1;
	}

	UMoviePipelineOutputSetting* OutputSettings = InParams.Job->GetConfiguration()->FindSetting<UMoviePipelineOutputSetting>();
	if (!OutputSettings->bAutoVersion)
	{
		return OutputSettings->VersionNumber;
	}

	// Calculate a version number by looking at the output path and then scanning for a version token.
	FString FileNameFormatString = InParams.FileNameOverride.Len() > 0 ? InParams.FileNameOverride : OutputSettings->OutputDirectory.Path / OutputSettings->FileNameFormat;

	FString FinalPath;
	FMoviePipelineFormatArgs FinalFormatArgs;
	TMap<FString, FString> Overrides;
	// Force the Version string to stay as {version} so we can substring based on it later.
	InParams.FileNameFormatOverrides.Add(TEXT("version"), TEXT("{version}"));
	
	UMoviePipelineBlueprintLibrary::ResolveFilenameFormatArguments(FileNameFormatString, InParams, FinalPath, FinalFormatArgs);

	FinalPath = FPaths::ConvertRelativePathToFull(FinalPath);
	FPaths::NormalizeFilename(FinalPath);

	// If they're using the version token, try to resolve the directory that the files are in.
	if (FinalPath.Contains(TEXT("{version}")))
	{
		int32 HighestVersion = 0;

		// FinalPath can have {version} either in a folder name or in a file name. We need to find the 'parent' of either the file or folder that contains it. We can do this by substringing
		// for {version} and then finding the last "/" character, which will be the containing folder.
		int32 VersionStringIndex = FinalPath.Find(TEXT("{version}"), ESearchCase::Type::IgnoreCase, ESearchDir::Type::FromStart);
		if (VersionStringIndex >= 0)
		{
			int32 LastParentFolder = FinalPath.Find(TEXT("/"), ESearchCase::Type::IgnoreCase, ESearchDir::Type::FromEnd, VersionStringIndex);
			FinalPath.LeftInline(LastParentFolder + 1);

			// Now that we have the parent folder of either the folder with the version token, or the file with the version token, we will
			// look through all immediate children and scan for version tokens so we can find the highest one.
			const FRegexPattern VersionSearchPattern(TEXT("v([0-9]{3})"));
			TArray<FString> FilesAndFoldersInDirectory;
			IFileManager& FileManager = IFileManager::Get();
			FileManager.FindFiles(FilesAndFoldersInDirectory, *(FinalPath / TEXT("*.*")), true, true);

			for (const FString& Path : FilesAndFoldersInDirectory)
			{
				FRegexMatcher Regex(VersionSearchPattern, *Path);
				if (Regex.FindNext())
				{
					FString Result = Regex.GetCaptureGroup(0);
					if (Result.Len() > 0)
					{
						// Strip the "v" token off, expected pattern is vXXX
						Result.RightChopInline(1);
					}

					int32 VersionNumber = 0;
					LexFromString(VersionNumber, *Result);
					if (VersionNumber > HighestVersion)
					{
						HighestVersion = VersionNumber;
					}
				}
			}

		}

		return HighestVersion + (bGetNextVersion ? 1 : 0);
	}

	return 0;
}

int32 UMoviePipelineBlueprintLibrary::GetCurrentVersionNumber(const UMoviePipeline* InMoviePipeline)
{
	if (!InMoviePipeline)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get version number from null pipeline!"), ELogVerbosity::Error);
		return 0;
	}
	
	int32 CurrentShotIndex = InMoviePipeline->GetCurrentShotIndex();
	if(!InMoviePipeline->GetActiveShotList().IsValidIndex(CurrentShotIndex))
	{
		FFrame::KismetExecutionMessage(TEXT("No shot is currently active to get the version number from."), ELogVerbosity::Error);
		return 0;
	}

	return InMoviePipeline->GetActiveShotList()[CurrentShotIndex]->ShotInfo.VersionNumber;
}

FIntPoint UMoviePipelineBlueprintLibrary::GetEffectiveOutputResolution(UMoviePipelinePrimaryConfig* InPrimaryConfig, UMoviePipelineExecutorShot* InPipelineExecutorShot, float DefaultOverscan)
{
	return GetOverscannedResolution(InPrimaryConfig, InPipelineExecutorShot, DefaultOverscan);
}

FIntPoint UMoviePipelineBlueprintLibrary::GetDesiredOutputResolution(const UMoviePipelinePrimaryConfig* InPrimaryConfig)
{
	if (InPrimaryConfig)
	{
		const UMoviePipelineOutputSetting* OutputSetting = InPrimaryConfig->FindSetting<UMoviePipelineOutputSetting>();
		check(OutputSetting);
		return OutputSetting->OutputResolution;
	}

	return FIntPoint();
}

FIntPoint UMoviePipelineBlueprintLibrary::GetOverscannedResolution(const UMoviePipelinePrimaryConfig* InPrimaryConfig, const UMoviePipelineExecutorShot* InPipelineExecutorShot, float InDefaultOverscan)
{
	if (InPrimaryConfig && InPipelineExecutorShot)
	{
		const FIntPoint DesiredOutputResolution = GetDesiredOutputResolution(InPrimaryConfig);
		
		const UMoviePipelineCameraSetting* CameraSettings = Cast<const UMoviePipelineCameraSetting>(FindOrGetDefaultSettingForShot(UMoviePipelineCameraSetting::StaticClass(), InPrimaryConfig, InPipelineExecutorShot));
		check(CameraSettings);
		const float Overscan = CameraSettings->bOverrideCameraOverscan ? CameraSettings->OverscanPercentage : InDefaultOverscan;
		return UE::MoviePipeline::ScaleResolutionByOverscan(Overscan, DesiredOutputResolution);
	}

	return FIntPoint();
}

FIntPoint UMoviePipelineBlueprintLibrary::GetBackbufferResolution(const UMoviePipelinePrimaryConfig* InPrimaryConfig, const UMoviePipelineExecutorShot* InPipelineExecutorShot, float InDefaultOverscan)
{
	if (InPrimaryConfig && InPipelineExecutorShot)
	{
		const FIntPoint OverscannedResolution = GetOverscannedResolution(InPrimaryConfig, InPipelineExecutorShot, InDefaultOverscan);

		const UMoviePipelineHighResSetting* HighResSettings = Cast<const UMoviePipelineHighResSetting>(FindOrGetDefaultSettingForShot(UMoviePipelineHighResSetting::StaticClass(), InPrimaryConfig, InPipelineExecutorShot));
		check(HighResSettings);
		
		FIntPoint TileResolution = FIntPoint(
			FMath::CeilToInt((float)OverscannedResolution.X / (float)HighResSettings->TileCount),
			FMath::CeilToInt((float)OverscannedResolution.Y / (float)HighResSettings->TileCount));

		return HighResSettings->CalculatePaddedBackbufferSize(TileResolution);
	}

	return FIntPoint();
}

FIntRect UMoviePipelineBlueprintLibrary::GetOverscanCropRectangle(const UMoviePipelinePrimaryConfig* InPrimaryConfig, const UMoviePipelineExecutorShot* InPipelineExecutorShot, float DefaultOverscan)
{
	if (InPrimaryConfig && InPipelineExecutorShot)
	{
		const FIntPoint DesiredOutputResolution = GetDesiredOutputResolution(InPrimaryConfig);
		const FIntPoint OverscannedResolution = GetOverscannedResolution(InPrimaryConfig, InPipelineExecutorShot, DefaultOverscan);
		
		FIntRect CropRect;

		// Overscan is symmetrical, so this difference is always evenly divisible by 2.
		CropRect.Min = (OverscannedResolution - DesiredOutputResolution) / 2;
		CropRect.Max = CropRect.Min + DesiredOutputResolution;

		return CropRect;
	}

	return FIntRect();
}


UCineCameraComponent* UMoviePipelineBlueprintLibrary::Utility_GetCurrentCineCamera(const UWorld* InWorld)
{
	if (InWorld)
	{
		if (const TObjectPtr<APlayerCameraManager> PlayerCameraManager = InWorld->GetFirstPlayerController()->PlayerCameraManager)
		{
			// This only works if you use a Cine Camera (which is almost guaranteed with Sequencer) and it's easier (and less human error prone) than re-deriving the information
			if (const ACineCameraActor* CineCameraActor = Cast<ACineCameraActor>(PlayerCameraManager->GetViewTarget()))
			{
				return CineCameraActor->GetCineCameraComponent();
			}
		}
	}

	return nullptr;
}

UMoviePipelineSetting* UMoviePipelineBlueprintLibrary::FindOrGetDefaultSettingForShot(TSubclassOf<UMoviePipelineSetting> InSettingType, const UMoviePipelinePrimaryConfig* InPrimaryConfig, const UMoviePipelineExecutorShot* InShot)
{
	// Check to see if this setting is in the shot override, if it is we'll use the shot version of that.
	if (InShot && InShot->GetShotOverrideConfiguration())
	{
		UMoviePipelineSetting* Setting = InShot->GetShotOverrideConfiguration()->FindSettingByClass(InSettingType);
		if (Setting)
		{
			// If they specified the setting at all, respect the enabled setting. If it's not enabled, we return the
			// default instead which is the same as if they hadn't added the setting at all.
			return Setting->IsEnabled() ? Setting : InSettingType->GetDefaultObject<UMoviePipelineSetting>();
		}
	}

	// If they didn't have a shot override, or the setting wasn't enabled, we'll check the primary config.
	if (InPrimaryConfig)
	{
		UMoviePipelineSetting* Setting = InPrimaryConfig->FindSettingByClass(InSettingType);
		if (Setting && Setting->IsEnabled())
		{
			return Setting;
		}
	}

	// If no one overrode it, then we return the default.
	return InSettingType->GetDefaultObject<UMoviePipelineSetting>();
}


void UMoviePipelineBlueprintLibrary::ResolveFilenameFormatArguments(const FString& InFormatString, const FMoviePipelineFilenameResolveParams& InParams, FString& OutFinalPath, FMoviePipelineFormatArgs& OutMergedFormatArgs)
{
	if (!InParams.Job)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot resolve Filename Format Arguments without a Job to pull settings from."), ELogVerbosity::Error);
		return;
	}

	UMoviePipelineOutputSetting* OutputSetting = InParams.Job->GetConfiguration()->FindSetting<UMoviePipelineOutputSetting>();
	check(OutputSetting);

	// Gather all the variables
	OutMergedFormatArgs.InJob = InParams.Job;

	// Copy the file metadata from our InOutputState
	OutMergedFormatArgs.FileMetadata = InParams.FileMetadata;

	// Now get the settings from our config. This will expand the FileMetadata and assign the default values used in the UI.
	InParams.Job->GetConfiguration()->GetFormatArguments(OutMergedFormatArgs, true);

	// Big block of stuff that normally comes from the Output State
	{
		// Zero-pad our frame numbers when we format the strings. Some programs struggle when ingesting frames that 
		// go 1,2,3,...,10,11. To work around this issue we allow the user to specify how many zeros they want to
		// pad the numbers with, 0001, 0002, etc. We also allow offsetting the output frame numbers, this is useful
		// when your sequence starts at zero and you use handle frames (which would cause negative output frame 
		// numbers), so we allow the user to add a fixed amount to all output to ensure they are positive.
		int32 FrameNumberOffset = OutputSetting->FrameNumberOffset + InParams.AdditionalFrameNumberOffset;
		FString FrameNumber = UE::MoviePipeline::GetPaddingFormatString(OutputSetting->ZeroPadFrameNumbers, InParams.FrameNumber + FrameNumberOffset); // Sequence Frame #
		FString FrameNumberShot = UE::MoviePipeline::GetPaddingFormatString(OutputSetting->ZeroPadFrameNumbers, InParams.FrameNumberShot + FrameNumberOffset); // Shot Frame #
		FString FrameNumberRel = UE::MoviePipeline::GetPaddingFormatString(OutputSetting->ZeroPadFrameNumbers, InParams.FrameNumberRel + FrameNumberOffset); // Relative to 0
		FString FrameNumberShotRel = UE::MoviePipeline::GetPaddingFormatString(OutputSetting->ZeroPadFrameNumbers, InParams.FrameNumberShotRel + FrameNumberOffset); // Relative to 0 within the shot.

		// Ensure they used relative frame numbers in the output so they get the right number of output frames.
		if (InParams.bForceRelativeFrameNumbers)
		{
			FrameNumber = FrameNumberRel;
			FrameNumberShot = FrameNumberShotRel;
		}
		FString CameraName = InParams.ShotOverride ? InParams.ShotOverride->GetCameraName(InParams.CameraIndex) : FString();
		CameraName = CameraName.Len() > 0 ? CameraName : TEXT("NoCamera");
		if(InParams.CameraNameOverride.Len() > 0)
		{
			CameraName = InParams.CameraNameOverride;
		}

		FString ShotName = InParams.ShotOverride ? InParams.ShotOverride->OuterName : FString();
		ShotName = ShotName.Len() > 0 ? ShotName : TEXT("NoShot");
		if(InParams.ShotNameOverride.Len() > 0)
		{
			ShotName = InParams.ShotNameOverride;
		}

		MoviePipeline::GetOutputStateFormatArgs(OutMergedFormatArgs.FilenameArguments, OutMergedFormatArgs.FileMetadata, FrameNumber, FrameNumberShot, FrameNumberRel, FrameNumberShotRel, CameraName, ShotName);
	}


	// And from ourself
	{
		// Use the shared function as UMoviePipelinePrimaryConfig::GetFormatArguments to ensure all the time variables get overwritten.
		UE::MoviePipeline::GetSharedFormatArguments(OutMergedFormatArgs.FilenameArguments, OutMergedFormatArgs.FileMetadata, InParams.InitializationTime, InParams.InitializationVersion, InParams.Job, InParams.InitializationTimeOffset);
		
		// By default, we don't want to show frame duplication numbers. If we need to start writing them,
		// they need to come before the frame number (so that tools recognize them as a sequence).
		OutMergedFormatArgs.FilenameArguments.Add(TEXT("file_dup"), FString());
	}

	if (InParams.ShotOverride && InParams.ShotOverride->GetShotOverrideConfiguration())
	{
		UMoviePipelineShotConfig* ShotConfig = InParams.ShotOverride->GetShotOverrideConfiguration();
		for (UMoviePipelineSetting* Setting : ShotConfig->GetUserSettings())
		{
			if (!Setting)
			{
				UE_LOGF(LogMovieRenderPipeline, Error, "Null setting found in config: %ls - Did you disable a plugin that contained this setting?", *GetNameSafe(ShotConfig));
				continue;
			}

			if (Setting->IsEnabled())
			{
				Setting->GetFormatArguments(OutMergedFormatArgs);
			}
		}
	}

	// Overwrite the variables with overrides if needed. This allows different requesters to share the same variables (ie: filename extension, render pass name)
	for (const TPair<FString, FString>& KVP : InParams.FileNameFormatOverrides)
	{
		OutMergedFormatArgs.FilenameArguments.Add(KVP.Key, KVP.Value);
	}

	// No extension should be provided at this point, because we need to tack the extension onto the end after appending numbers (in the event of no overwrites)
	// We don't convert this to a full filename because sometimes this function is used to resolve just filenames without directories, and we can't tell if the
	// caller wants a full filepath or just a filename.
	FStringFormatNamedArguments NamedArgs;
	for (const TPair<FString, FString>& Argument : OutMergedFormatArgs.FilenameArguments)
	{
		NamedArgs.Add(Argument.Key, Argument.Value);
	}

	FString FileNameFormatString = InParams.FileNameOverride.Len() > 0 ? InParams.FileNameOverride : InFormatString;

	FString BaseFilename = FString::Format(*FileNameFormatString, NamedArgs);
	FPaths::NormalizeFilename(BaseFilename);


	// If we end with a "." character, remove it. The extension will put it back on. We can end up with this sometimes after resolving file format strings, ie:
	// {sequence_name}.{frame_number} becomes {sequence_name}. for videos (which can't use frame_numbers).
	BaseFilename.RemoveFromEnd(TEXT("."));

	FString Extension = FString::Format(TEXT(".{ext}"), NamedArgs);


	FString ThisTry = BaseFilename + Extension;

	if (UE::MoviePipeline::CanWriteToFile(*ThisTry, OutputSetting->bOverrideExistingOutput))
	{
		OutFinalPath = ThisTry;
		return;
	}

	int32 DuplicateIndex = 2;
	while (true)
	{
		NamedArgs.Add(TEXT("file_dup"), FString::Printf(TEXT("_(%d)"), DuplicateIndex));

		// Re-resolve the format string now that we've reassigned frame_dup to a number.
		ThisTry = FString::Format(*FileNameFormatString, NamedArgs) + Extension;

		// If the file doesn't exist, we can use that, else, increment the index and try again
		if (UE::MoviePipeline::CanWriteToFile(*ThisTry, OutputSetting->bOverrideExistingOutput))
		{
			OutFinalPath = ThisTry;
			return;
		}

		++DuplicateIndex;
	}
}

ULevelSequence* UMoviePipelineBlueprintLibrary::GetCurrentSequence(const UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		return InMoviePipeline->GetTargetSequence();
	}

	return nullptr;
}

UMoviePipelineExecutorShot* UMoviePipelineBlueprintLibrary::GetCurrentExecutorShot(const UMoviePipeline* InMoviePipeline)
{
	if (InMoviePipeline)
	{
		const TArray<UMoviePipelineExecutorShot*>& ActiveShotList = InMoviePipeline->GetActiveShotList();
		int32 CurrentShotIndex = InMoviePipeline->GetCurrentShotIndex();
		if (ActiveShotList.IsValidIndex(CurrentShotIndex))
		{
			return ActiveShotList[CurrentShotIndex];
		}
	}

	return nullptr;
}

FText UMoviePipelineBlueprintLibrary::GetMoviePipelineEngineChangelistLabel(const UMoviePipeline*)
{
	// This is a modular feature so that you can write a plugin that provides the string. If you don't write a plugin
	// then it falls back to the existing logic which pulls the engine build version.
	if(IModularFeatures::Get().IsModularFeatureAvailable(IMoviePipelineBurnInExtension::ModularFeatureName))
	{
		IMoviePipelineBurnInExtension& Extension = IModularFeatures::Get().GetModularFeature<IMoviePipelineBurnInExtension>(IMoviePipelineBurnInExtension::ModularFeatureName);
		const FText Label = Extension.GetEngineChangelistLabel();
		if (!Label.IsEmpty())
		{
			return Label;
		}
	}

	FString EngineVersion = FEngineVersion::Current().ToString();
	int32 ChangeListIndexStart = EngineVersion.Find(TEXT("-"));
	int32 ChangeListIndexEnd = EngineVersion.Find(TEXT("+"));

	if (ChangeListIndexStart > 0 && ChangeListIndexEnd > 0)
	{
		FString Substr = EngineVersion.Mid(ChangeListIndexStart + 1, (ChangeListIndexEnd - ChangeListIndexStart) - 1);
		return FText::FromString(Substr);
	}

	return FText();
}

