// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineCameraSetting.h"

#include "MoviePipelineTelemetry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineCameraSetting)

void UMoviePipelineCameraSetting::GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const
{
	Super::GetFormatArguments(InOutFormatArgs);

	InOutFormatArgs.FilenameArguments.Add(TEXT("shutter_timing"), StaticEnum<EMoviePipelineShutterTiming>()->GetNameStringByValue((int64)ShutterTiming));
	InOutFormatArgs.FilenameArguments.Add(TEXT("overscan_percentage"), FString::SanitizeFloat(OverscanPercentage));
	InOutFormatArgs.FileMetadata.Add(TEXT("unreal/camera/shutterTiming"), StaticEnum<EMoviePipelineShutterTiming>()->GetNameStringByValue((int64)ShutterTiming));
	InOutFormatArgs.FileMetadata.Add(TEXT("unreal/camera/overscanPercentage"), FString::SanitizeFloat(OverscanPercentage));
}

void UMoviePipelineCameraSetting::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesMultiCamera |= bRenderAllCameras;
}