// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineColorSetting.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineColorSetting)

UMoviePipelineColorSetting::UMoviePipelineColorSetting() : bDisableToneCurve(false)
{
}

#if WITH_EDITOR
bool UMoviePipelineColorSetting::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty->GetName() == GET_MEMBER_NAME_STRING_CHECKED(UMoviePipelineColorSetting, bDisableToneCurve))
	{
		return !OCIOConfiguration.bIsEnabled;
	}

	return Super::CanEditChange(InProperty);
}
#endif // #if WITH_EDITOR
