// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineCommandLineEncoder.h"
#include "MoviePipelineCommandLineEncoderSettings.h"
#include "MoviePipelineOutputSetting.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipeline.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelinePrimaryConfig.h"	
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "MoviePipelineUtils.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineDebugSettings.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelineTelemetry.h"
#include "Misc/FileHelper.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineCommandLineEncoder)

// Forward Declare
namespace UE
{
namespace MoviePipeline
{
	static TArray<FText> GetErrorTexts();
	static bool HasMultipleRenderPasses(const TArray<FMoviePipelineShotOutputData>& InData);
}
}

UMoviePipelineCommandLineEncoder::UMoviePipelineCommandLineEncoder()
{
	FileNameFormatOverride = TEXT("");
	Quality = EMoviePipelineEncodeQuality::Epic;
	bDeleteSourceFiles = false;
	bSkipEncodeOnRenderCanceled = true;
	bWriteEachFrameDuration = true;
}

bool UMoviePipelineCommandLineEncoder::HasFinishedExportingImpl()
{
	// Manually tick the output (which cleans up ActiveEncodeJobs). This is needed because
	// manually canceling a job stops ticking the engine and repeatedly calls HasFinishedExportingImpl
	OnTick();

	return ActiveEncodeJobs.Num() == 0;
}

void UMoviePipelineCommandLineEncoder::BeginExportImpl()
{
	// When we start exporting, we remove the OnEndFrame delegate because if they've hit escape to cancel a movie render
	// no frames will be ticked anymore. Instead we'll call OnTick from HasFinishedExportingImpl() by hand as that will
	// get called in a loop until it is finished.
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	// This is called at the end of the movie render. If we were starting encode jobs
	// per shot, then there's already an encode job going for all shots, so we early out.
	if (NeedsPerShotFlushing())
	{
		return;
	}

	// However, if they didn't want a per-shot flush (ie: rendering one video) then we start now.
	FMoviePipelineOutputData OutputData = GetPipeline()->GetOutputDataParams();
	const bool bIsShotEncode = false;
	StartEncodingProcess(OutputData.ShotData, bIsShotEncode);
}

void UMoviePipelineCommandLineEncoder::StartEncodingProcess(TArray<FMoviePipelineShotOutputData>& InOutData, const bool bInIsShotEncode)
{
	// If the format string isn't split per shot, we can't start encoding now.
	if(bInIsShotEncode && !NeedsPerShotFlushing())
	{
		return;
	}

	const UMoviePipelineCommandLineEncoderSettings* EncoderSettings = GetDefault<UMoviePipelineCommandLineEncoderSettings>();

	// Early out if there's any errors
	{
		TArray<FText> ErrorTexts = UE::MoviePipeline::GetErrorTexts();
		for (const FText& ErrorText : ErrorTexts)
		{
			UE_LOGF(LogMovieRenderPipelineIO, Error, "%ls", *ErrorText.ToString());
		}
		if (ErrorTexts.Num() > 0)
		{
			GetPipeline()->RequestShutdown(true);
			return;
		}
	}

	UMoviePipelineOutputSetting* OutputSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	FString OutputFilename = FileNameFormatOverride.Len() > 0 ? FileNameFormatOverride : OutputSetting->FileNameFormat;
	FString FileNameFormatString = OutputSetting->OutputDirectory.Path / OutputFilename;
	
	// If we're writing more than one render pass out, we need to ensure the file name has the format string in it so we don't
	// overwrite the same file multiple times. 
	const bool bIncludeRenderPass = UE::MoviePipeline::HasMultipleRenderPasses(InOutData);
	const bool bTestFrameNumber = false;

	UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber);
	UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);

	/** We produce one file per render pass we detect */
	TMap<FMoviePipelinePassIdentifier, FEncoderParams> RenderPasses;
	
	// The path shouldn't have quotes on it as it's already kept as a separate argument right up until creating the process, at which point
	// the platform puts quotes around the FString if needed.
	FString ExecutablePathNoQuotes = EncoderSettings->ExecutablePath.Replace(TEXT("\""), TEXT(""));
	FPaths::NormalizeFilename(ExecutablePathNoQuotes);
	
	FStringFormatNamedArguments SharedArguments;
	SharedArguments.Add(TEXT("Executable"), ExecutablePathNoQuotes);
	SharedArguments.Add(TEXT("AudioCodec"), EncoderSettings->AudioCodec);
	SharedArguments.Add(TEXT("VideoCodec"), EncoderSettings->VideoCodec);
	FFrameRate RenderFrameRate = GetPipeline()->GetPipelinePrimaryConfig()->GetEffectiveFrameRate(GetPipeline()->GetTargetSequence());
	SharedArguments.Add(TEXT("FrameRate"), RenderFrameRate.AsDecimal());
	SharedArguments.Add(TEXT("AdditionalLocalArgs"), AdditionalCommandLineArgs);
	SharedArguments.Add(TEXT("Quality"), GetQualitySettingString());

	for (FMoviePipelineShotOutputData& Data : InOutData)
	{
		for (const TPair<FMoviePipelinePassIdentifier, FMoviePipelineRenderPassOutputData>& RenderPass : Data.RenderPassData)
		{
			if (RenderPass.Key == FMoviePipelinePassIdentifier(TEXT("Audio")))
			{
				continue;
			}

			FEncoderParams& EncoderParams = RenderPasses.FindOrAdd(RenderPass.Key.Name);
			EncoderParams.Shot = Data.Shot;
			for (const FString& FilePath : RenderPass.Value.FilePaths)
			{
				FString Extension = FPaths::GetExtension(FilePath);
				EncoderParams.FilesByExtensionType.FindOrAdd(Extension).Add(FilePath);
			}

			// Search for audio to attach it to every render pass
			for (const TPair<FMoviePipelinePassIdentifier, FMoviePipelineRenderPassOutputData>& InnerRenderPass : Data.RenderPassData)
			{
				if (InnerRenderPass.Key == FMoviePipelinePassIdentifier(TEXT("Audio")))
				{
					for (const FString& FilePath : InnerRenderPass.Value.FilePaths)
					{
						FString Extension = FPaths::GetExtension(FilePath);
						EncoderParams.FilesByExtensionType.FindOrAdd(Extension).Add(FilePath);
					}
				}
			}
		}

		// If we're going to delete the source files, don't make them available to the scripting layer callback
		// because the files will be deleted out from underneath at a random point, so don't want the scripting
		// layer to think they can rely on them actually existing. If scripting layer really needs source files,
		// they need to not use the Command Line Encoder setting and instead roll their own.
		if (bDeleteSourceFiles)
		{
			Data.RenderPassData.Reset();
		}
	}

	for (TTuple<FMoviePipelinePassIdentifier, FEncoderParams>& RenderPass : RenderPasses)
	{
		// Copy the shared arguments into our render pass
		RenderPass.Value.NamedArguments = SharedArguments;

		// Generate a filename for this encoded file
		TMap<FString, FString> FormatOverrides;
		FormatOverrides.Add(TEXT("render_pass"), RenderPass.Key.Name);
		FormatOverrides.Add(TEXT("ext"), EncoderSettings->OutputFileExtension);
		UMoviePipelineExecutorShot* Shot = RenderPass.Value.Shot.Get();
		if (Shot)
		{
			FormatOverrides.Add(TEXT("shot_name"), Shot->OuterName);
			FormatOverrides.Add(TEXT("camera_name"), Shot->InnerName);
		}

		// We must manually resolve {version} tokens because they're intended to be per-shot/global (ie: individual file names
		// don't have versions when using image sequences).
		{
			FMoviePipelineFilenameResolveParams ResolveParams;
			ResolveParams.InitializationTime = GetPipeline()->GetInitializationTime();
			ResolveParams.InitializationTimeOffset = GetPipeline()->GetInitializationTimeOffset();
			ResolveParams.Job = GetPipeline()->GetCurrentJob();
			ResolveParams.ShotOverride = RenderPass.Value.Shot.Get();
			ResolveParams.FileNameOverride = FileNameFormatString;
			int32 VersionNumber = UMoviePipelineBlueprintLibrary::ResolveVersionNumber(ResolveParams);
			FileNameFormatString.ReplaceInline(TEXT("{version}"), *FString::Printf(TEXT("v%0*d"), 3, VersionNumber));
		}

		FMoviePipelineFormatArgs FinalFormatArgs;

		FString FinalFilePath;
		GetPipeline()->ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, FinalFilePath, FinalFormatArgs);

		if (FPaths::IsRelative(FinalFilePath))
		{
			FinalFilePath = FPaths::ConvertRelativePathToFull(FinalFilePath);
		}

		FPaths::NormalizeFilename(FinalFilePath);
		FPaths::CollapseRelativeDirectories(FinalFilePath);

		FString FinalFileDirectory = FPaths::GetPath(FinalFilePath);

		// Ensure the output directory is created
		IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
		if (!FileManager.CreateDirectoryTree(*FinalFileDirectory))
		{
			UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to create directory for output path '%ls'", *FinalFileDirectory);
		}

		// Manipulate the in/out data in case scripting tries to get access to the files. It's not a perfect solution
		// because the encoding won't be finished by the time the scripting layer is called. Need to manipulate the original
		// and not the copy that we're currently iterating through.
		for (FMoviePipelineShotOutputData& Data : InOutData)
		{
			if (Data.Shot != RenderPass.Value.Shot)
			{
				continue;
			}

			Data.RenderPassData.FindOrAdd(FMoviePipelinePassIdentifier("CommandLineEncoder")).FilePaths.Add(FinalFilePath);
		}
		
		RenderPass.Value.NamedArguments.Add(TEXT("OutputPath"), FinalFilePath);
		LaunchEncoder(RenderPass.Value);
	}
}

void UMoviePipelineCommandLineEncoder::LaunchEncoder(const FEncoderParams& InParams)
{
	UMoviePipelineOutputSetting* OutputSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();

	// Generate a text file for each input type which lists the files for that input type. We generate a FGuid in case there are
	// multiple encode jobs going at once.
	TStringBuilder<64> StringBuilder;
	TArray<FString> VideoInputs;
	TArray<FString> AudioInputs;

	double InFrameRate = InParams.NamedArguments[TEXT("FrameRate")].DoubleValue;
	double FrameRateAsDuration = 1.0 / InFrameRate;

	for (const TTuple<FString, TArray<FString>>& Pair : InParams.FilesByExtensionType)
	{
		FGuid FileGuid = FGuid::NewGuid();
		FString FilePath = OutputSetting->OutputDirectory.Path / FileGuid.ToString() + TEXT("_input");

		FMoviePipelineFormatArgs FinalFormatArgs;

		FString FinalFilePath;
		TMap<FString, FString> FormatOverrides;
		FormatOverrides.Add(TEXT("ext"), TEXT("txt"));

		GetPipeline()->ResolveFilenameFormatArguments(FilePath, FormatOverrides, FinalFilePath, FinalFormatArgs);
		

		UE_LOGF(LogMovieRenderPipelineIO, Log, "Generated Path '%ls' for input data.", *FinalFilePath);
		StringBuilder.Reset();
		for (const FString& Path : Pair.Value)
		{
			StringBuilder.Appendf(TEXT("file 'file:%s'%s"), *Path, LINE_TERMINATOR);

			// Some encoders require the duration of each file to be listed after the file.
			if (Pair.Key != TEXT("wav") && bWriteEachFrameDuration)
			{
				StringBuilder.Appendf(TEXT("duration %f%s"), FrameRateAsDuration, LINE_TERMINATOR);
			}
		}

		// Save this to disk.
		FFileHelper::SaveStringToFile(StringBuilder.ToString(), *FinalFilePath);

		// Not a great solution but best we've got right now
		if (Pair.Key == TEXT("wav"))
		{
			AudioInputs.Add(FinalFilePath);
		}
		else
		{
			VideoInputs.Add(FinalFilePath);
		}
	}

	// Build our final command line arguments
	const UMoviePipelineCommandLineEncoderSettings* EncoderSettings = GetDefault<UMoviePipelineCommandLineEncoderSettings>();
	FStringFormatNamedArguments FinalNamedArgs = InParams.NamedArguments;
	
	FString VideoInputArg;
	FString AudioInputArg;

	for (const FString& FilePath : VideoInputs)
	{
		FStringFormatNamedArguments NamedArgs;
		NamedArgs.Add(TEXT("InputFile"), *FilePath);
		NamedArgs.Add(TEXT("FrameRate"), InParams.NamedArguments[TEXT("FrameRate")]);
			
		VideoInputArg += TEXT(" ") + FString::Format(*EncoderSettings->VideoInputStringFormat, NamedArgs);
	}

	for (const FString& FilePath : AudioInputs)
	{
		FStringFormatNamedArguments NamedArgs;
		NamedArgs.Add(TEXT("InputFile"), *FilePath);

		AudioInputArg += TEXT(" ") + FString::Format(*EncoderSettings->AudioInputStringFormat, NamedArgs);
	}

	FinalNamedArgs.Add(TEXT("VideoInputs"), VideoInputArg);
	FinalNamedArgs.Add(TEXT("AudioInputs"), AudioInputArg);
	FString CommandLineArgs = FString::Format(*EncoderSettings->CommandLineFormat, FinalNamedArgs);
	UE_LOGF(LogMovieRenderPipelineIO, Log, "Final Command Line Arguments: %ls", *CommandLineArgs);

	const bool bLaunchDetached = false;
	const bool bLaunchHidden = true;
	const bool bLaunchReallyHidden = bLaunchHidden;

	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;
	int32 ReturnCode = -1;

	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	FString ExecutableArg = FString::Format(TEXT("{Executable}"), InParams.NamedArguments);
	FProcHandle ProcessHandle = FPlatformProcess::CreateProc(*ExecutableArg, *CommandLineArgs, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, nullptr, PipeWrite);
	if (ProcessHandle.IsValid())
	{
		FActiveJob& NewJob = ActiveEncodeJobs.AddDefaulted_GetRef();
		NewJob.ProcessHandle = ProcessHandle;
		NewJob.ReadPipe = PipeRead;
		NewJob.WritePipe = PipeWrite;

		// Automatically delete the input files we generated when the job is done
		bool bDeleteInputTexts = true;
		UMoviePipelineDebugSettings* DebugSettings = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineDebugSettings>();
		if (DebugSettings)
		{
			bDeleteInputTexts = !DebugSettings->bWriteAllSamples;
		}

		// We delete our auto-generated files (unless you've flagged them to keep with the debug setting)
		if (bDeleteInputTexts)
		{
			NewJob.FilesToDelete.Append(VideoInputs);
			NewJob.FilesToDelete.Append(AudioInputs);
		}

		// And the user's input files (if requested), though we ignore this if you have the Debug Setting asking you to write all samples.
		if (bDeleteSourceFiles && bDeleteInputTexts)
		{
			for (const TTuple<FString, TArray<FString>>& Pair : InParams.FilesByExtensionType)
			{
				NewJob.FilesToDelete.Append(Pair.Value);
			}
		}
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to launch encoder process, see output log for more details.");
		GetPipeline()->Shutdown(true);
	}
}

void UMoviePipelineCommandLineEncoder::OnTick()
{
	for (int32 Index = ActiveEncodeJobs.Num() - 1; Index >= 0; Index--)
	{
		FActiveJob& Job = ActiveEncodeJobs[Index];
		TArray<uint8> ProcessOutput;
		FString Results = FPlatformProcess::ReadPipe(Job.ReadPipe);
		if (Results.Len() > 0)
		{
			// The default global arguments suppress everything but errors. We unfortunately can't tell stdout from stderr
			// using a non-blocking platform process launch, so we'll just promote all the output as an error log.
			UE_LOGF(LogMovieRenderPipeline, Error, "Command Line Encoder: %ls", *Results);
		}

		// If they hit escape during  a render, (potentially) cancel the encode job
		bool bCancelEncode = false;
		if (bSkipEncodeOnRenderCanceled && GetPipeline()->IsShutdownRequested())
		{
			bCancelEncode = true;

			// We have to specifically terminate the process instead of just closing it
			const bool bKillTree = true;
			FPlatformProcess::TerminateProc(Job.ProcessHandle, bKillTree);
			FPlatformProcess::WaitForProc(Job.ProcessHandle);
		}

		// If the process has finished we'll clean up
		if (!FPlatformProcess::IsProcRunning(Job.ProcessHandle) || bCancelEncode)
		{
			FPlatformProcess::ClosePipe(Job.ReadPipe, Job.WritePipe);
			FPlatformProcess::CloseProc(Job.ProcessHandle);

			IFileManager& FileManager = IFileManager::Get();
			for (const FString& FilePath : Job.FilesToDelete)
			{
				const bool bRequireExist = false;
				const bool bEvenReadOnly = false;
				const bool bQuiet = false;
				FileManager.Delete(*FilePath, bRequireExist, bEvenReadOnly, bQuiet);
			}

			ActiveEncodeJobs.RemoveAt(Index);
		}
	}
}

bool UMoviePipelineCommandLineEncoder::NeedsPerShotFlushing() const
{
	UMoviePipelineOutputSetting* OutputSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	FString OutputFilename = FileNameFormatOverride.Len() > 0 ? FileNameFormatOverride : OutputSetting->FileNameFormat;
	FString FullPath = OutputSetting->OutputDirectory.Path / OutputFilename;

	if(FullPath.Contains(TEXT("{shot_name}")) || FullPath.Contains(TEXT("{camera_name}")))
	{
		return true;
	}
	
	return false;
}

void UMoviePipelineCommandLineEncoder::SetupForPipelineImpl(UMoviePipeline* InPipeline)
{
	if (InPipeline && NeedsPerShotFlushing())
	{
		InPipeline->SetFlushDiskWritesPerShot(true);
	}

	// Register a delegate so we can listen each frame for finished encode processes
	FCoreDelegates::OnEndFrame.AddUObject(this, &UMoviePipelineCommandLineEncoder::OnTick);
}

void UMoviePipelineCommandLineEncoder::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesCommandLineEncoder = true;
}

FString UMoviePipelineCommandLineEncoder::GetQualitySettingString() const
{
	const UMoviePipelineCommandLineEncoderSettings* EncoderSettings = GetDefault<UMoviePipelineCommandLineEncoderSettings>();
	switch (Quality)
	{
	case EMoviePipelineEncodeQuality::Low:
		return EncoderSettings->EncodeSettings_Low;
	case EMoviePipelineEncodeQuality::Medium:
		return EncoderSettings->EncodeSettings_Med;
	case EMoviePipelineEncodeQuality::High:
		return EncoderSettings->EncodeSettings_High;
	case EMoviePipelineEncodeQuality::Epic:
		return EncoderSettings->EncodeSettings_Epic;
	}

	return FString();
}

namespace UE
{
namespace MoviePipeline
{
	static TArray<FText> GetErrorTexts()
	{
		TArray<FText> OutErrors;
		const UMoviePipelineCommandLineEncoderSettings* EncoderSettings = GetDefault<UMoviePipelineCommandLineEncoderSettings>();
		if (EncoderSettings->ExecutablePath.Len() == 0)
		{
			OutErrors.Add(NSLOCTEXT("MovieRenderPipeline", "CommandLineEncode_MissingExecutable", "No encoder executable has been specified in the Project Settings. Please set an encoder executable in Project Settings > Movie Pipeline CLI Encoder"));
		}
		if (EncoderSettings->VideoCodec.Len() == 0)
		{
			OutErrors.Add(NSLOCTEXT("MovieRenderPipeline", "CommandLineEncode_MissingVideoCodec", "No video encoding codec has been specified in the Project Settings. Please set an video codec in Project Settings > Movie Pipeline CLI Encoder"));
		}
		if (EncoderSettings->AudioCodec.Len() == 0)
		{
			OutErrors.Add(NSLOCTEXT("MovieRenderPipeline", "CommandLineEncode_MissingAudioCodec", "No audio encoding codec has been specified in the Project Settings. Please set an audio codec in Project Settings > Movie Pipeline CLI Encoder"));
		}
		if (EncoderSettings->OutputFileExtension.Len() == 0)
		{
			OutErrors.Add(NSLOCTEXT("MovieRenderPipeline", "CommandLineEncode_MissingFileExtension", "No file extension has been specified in the Project Settings. Please set a file extension in Project Settings > Movie Pipeline CLI Encoder"));
		}

		return OutErrors;
	}

	static bool HasMultipleRenderPasses(const TArray<FMoviePipelineShotOutputData>& InData)
	{
		bool bHasMultipleRenderPasses = false;
		for (const FMoviePipelineShotOutputData& Data : InData)
		{
			int32 RenderPassCount = 0;
			for (const TTuple<FMoviePipelinePassIdentifier, FMoviePipelineRenderPassOutputData>& Pair : Data.RenderPassData)
			{
				if (Pair.Key.Name != TEXT("Audio"))
				{
					RenderPassCount++;
				}
			}

			bHasMultipleRenderPasses |= RenderPassCount > 1;
		}

		return bHasMultipleRenderPasses;
	}
}
}

void UMoviePipelineCommandLineEncoder::ValidateStateImpl()
{
	Super::ValidateStateImpl();

	TArray<FText> Errors = UE::MoviePipeline::GetErrorTexts();
	for (const FText& Error : Errors)
	{
		ValidationResults.Add(Error);
		ValidationState = EMoviePipelineValidationState::Warnings;
	}
}

