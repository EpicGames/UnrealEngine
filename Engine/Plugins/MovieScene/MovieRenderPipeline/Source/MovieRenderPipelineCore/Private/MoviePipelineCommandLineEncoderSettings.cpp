// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineCommandLineEncoderSettings.h"
#include "HAL/PlatformProcess.h"
#include "Misc/FileHelper.h"
#include "HAL/IConsoleManager.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineCommandLineEncoderSettings)


#if WITH_EDITOR
#include "Misc/MessageDialog.h"
#endif

#if WITH_EDITOR
static void PrintAvailableCodecs()
{
	const UMoviePipelineCommandLineEncoderSettings* Settings = GetDefault< UMoviePipelineCommandLineEncoderSettings>();
	if (Settings->ExecutablePath.Len() == 0)
	{
		FMessageDialog::Open(EAppMsgType::Type::Ok, NSLOCTEXT("MoviePipelineCLIEncoder", "NoExecutableSpecified", "No executable path specified, cannot determine codecs."));
		return;
	}

	// We'll attempt to launch the process now. This could still fail if the path they specified is invalid.
	FString CommandLineArgs = TEXT("-encoders");

	const bool bLaunchDetached = false;
	const bool bLaunchHidden = true;
	const bool bLaunchReallyHidden = bLaunchHidden;

	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;
	int32 ReturnCode = -1;

	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	FProcHandle ProcessHandle = FPlatformProcess::CreateProc(*Settings->ExecutablePath, *CommandLineArgs, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, nullptr, PipeWrite);
	if (ProcessHandle.IsValid())
	{
		FPlatformProcess::Sleep(0.01f);

		TArray<uint8> ProcessOutput;
		while (FPlatformProcess::IsProcRunning(ProcessHandle))
		{
			TArray<uint8> BinaryData;
			FPlatformProcess::ReadPipeToArray(PipeRead, BinaryData);
			if (BinaryData.Num() > 0)
			{
				ProcessOutput.Append(MoveTemp(BinaryData));
			}
		}
		
		const FString ProcessOutputStr(UTF8_TO_TCHAR(ProcessOutput.GetData()));
		TArray<FString> SplitStrings;
		ProcessOutputStr.ParseIntoArray(SplitStrings, LINE_TERMINATOR, true);

		int32 NumCodecsDumped = 0;
		TArray<FString> LineContents;
		for (const FString& Line : SplitStrings)
		{
			LineContents.Reset();
			Line.ParseIntoArray(LineContents, TEXT(" "), true);

			if (LineContents.Num() < 3)
			{
				continue;
			}

			const bool bCorrectStart = LineContents[0].StartsWith(TEXT("V")) || LineContents[0].StartsWith(TEXT("A"));
			if (bCorrectStart && LineContents[1] != TEXT("="))
			{
				TStringBuilder<64> Description;
				for (int32 Index = 2; Index < LineContents.Num(); Index++)
				{
					Description.Append(*LineContents[Index]);
					Description.Append(TEXT(" "));
				}

				FString CombinedDesc = Description.ToString();
				FString CodecType = LineContents[0].StartsWith(TEXT("V")) ? TEXT("Video") : TEXT("Audio");
				UE_LOGF(LogMovieRenderPipeline, Log, "%ls CodecName: %-20ls Codec Description: %ls", *CodecType, *LineContents[1], *CombinedDesc);
				NumCodecsDumped++;
			}
		}

		UE_LOGF(LogMovieRenderPipeline, Log, "Finished dumping audio and video codecs. Please select a video and audio codec from the list above for the settings.");
		FPlatformProcess::CloseProc(ProcessHandle);
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Type::Ok, NSLOCTEXT("MoviePipelineCLIEncoder", "ExecutableNotFound", "Failed to launch process, cannot determine codecs. Please verify that the executable exists at the path specified!"));
	}
	FPlatformProcess::ClosePipe(PipeRead, PipeWrite);
}

FAutoConsoleCommand GDumpCLIEncoderCodecs(TEXT("MovieRenderPipeline.DumpCLIEncoderCodecs"), TEXT("Dumps the available codecs for use with the Movie Pipeline Command Line Encoder settings dialog."), FConsoleCommandDelegate::CreateStatic(PrintAvailableCodecs));
#endif

