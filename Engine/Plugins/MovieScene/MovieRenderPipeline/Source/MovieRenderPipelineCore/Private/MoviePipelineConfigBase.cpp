// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineConfigBase.h"
#include "MoviePipelineSetting.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineConfigBase)

#define LOCTEXT_NAMESPACE "MoviePipelineConfigBase"

void UMoviePipelineConfigBase::RemoveSetting(UMoviePipelineSetting* InSetting)
{
	Settings.Remove(InSetting);
	OnSettingRemoved(InSetting);

	// Update our cached serial number so the UI rebuilds the tree
	++SettingsSerialNumber;
}

void UMoviePipelineConfigBase::CopyFrom(UMoviePipelineConfigBase* InConfig)
{
	if (!InConfig)
	{
		return;
	}
	
	// Don't allow copying ourself to ourself as we first wipe out our settings array
	// and then copy from the other config's settings array which will now be empty.
	if (InConfig == this)
	{
		return;
	}

	ConfigOrigin = InConfig->ConfigOrigin;

#if WITH_EDITOR
	Modify();
#endif

	// Copy the display name from the other config. When we copy from a preset, the preset will
	// have a display name matching the asset name so it will look like you're using that preset.
	DisplayName = InConfig->DisplayName;

	Settings.Empty();

	// Only access the direct Settings array
	for (UMoviePipelineSetting* Setting : InConfig->Settings)
	{
		if (!Setting)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Null setting found in config: %ls - Did you disable a plugin that contained this setting?", *GetNameSafe(InConfig));
			continue;
		}

		UMoviePipelineSetting* Duplicate = Cast<UMoviePipelineSetting>(StaticDuplicateObject(Setting, this, Setting->GetFName()));
		Duplicate->ValidateState();

		Settings.Add(Duplicate);
		OnSettingAdded(Duplicate);
	}

	// Manually bump this since we directly added to the Settings array
	SettingsSerialNumber++;
}

#undef LOCTEXT_NAMESPACE // "MoviePipelineConfigBase"
