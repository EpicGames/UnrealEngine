// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineDebugSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineDebugSettings)

#if WITH_EDITOR && !UE_BUILD_SHIPPING
#include "IRenderCaptureProvider.h"
#endif

UMoviePipelineDebugSettings::UMoviePipelineDebugSettings()
	: bWriteAllSamples(false)
	, bCaptureFramesWithRenderDoc(false)
	, CaptureFrame(0)
	, bIsRenderDebugCaptureAvailable(false)
{
#if WITH_EDITOR && !UE_BUILD_SHIPPING
	bIsRenderDebugCaptureAvailable = IRenderCaptureProvider::IsAvailable();
#endif
}

#if WITH_EDITOR

FText UMoviePipelineDebugSettings::GetFooterText(UMoviePipelineExecutorJob* InJob) const
{
	if (!bIsRenderDebugCaptureAvailable)
	{
		return NSLOCTEXT("MovieRenderPipeline", "DebugSettings_FooterText", "GPU Captures require RenderDoc to be installed and the plugin enabled for this project.");
	}

	return FText();
}


bool UMoviePipelineDebugSettings::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMoviePipelineDebugSettings, bCaptureFramesWithRenderDoc))
	{
		return bIsRenderDebugCaptureAvailable;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMoviePipelineDebugSettings, CaptureFrame))
	{
		return bIsRenderDebugCaptureAvailable && bCaptureFramesWithRenderDoc;
	}
	
	return Super::CanEditChange(InProperty);
}

#endif // WITH_EDITOR
