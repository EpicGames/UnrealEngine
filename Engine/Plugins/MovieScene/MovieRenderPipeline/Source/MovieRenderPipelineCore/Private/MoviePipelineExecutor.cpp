// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineExecutor.h"
#include "SocketSubsystem.h"
#include "Sockets.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineExecutor)

bool UMoviePipelineExecutorBase::ConnectSocket(const FString& InHostName, const int32 InPort)
{
	if (IsSocketConnected())
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "ConnectSocket called while existing socket connected, call DisconnectSocket first if you wish to change socket hosts. Ignoring request.");
		return false;
	}

	if (ExternalSocket)
	{
		ExternalSocket->Close();
		ExternalSocket = nullptr;
	}

	TSharedRef<FInternetAddr> Address = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
	bool bIsValid = false;
	Address->SetIp(*InHostName, bIsValid);
	Address->SetPort(InPort);

	if (!bIsValid)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Could not parse IP address for socket.");
		return false;
	}

	ExternalSocket = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateSocket(NAME_Stream, TEXT("MoviePipelineExternalSocket"), false);
	return ExternalSocket->Connect(*Address);
}

void UMoviePipelineExecutorBase::DisconnectSocket()
{
	if (ExternalSocket)
	{
		ExternalSocket->Close();
		ExternalSocket = nullptr;
	}
}

bool UMoviePipelineExecutorBase::SendSocketMessage(const FString& InMessage)
{
	if (!IsSocketConnected())
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Attempted to send a socket message but no socket connected, ignoring...");
		return false;
	}

	// Send the size header
	int32 MessageLength = FCStringAnsi::Strlen(TCHAR_TO_UTF8(*InMessage));
	if (!BlockingSendSocketMessageImpl((uint8*)&MessageLength, sizeof(uint32)))
	{
		UE_LOGF(LogMovieRenderPipeline, Verbose, "SendSocketMessage size write failed with code %d", (int32)ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLastErrorCode());
		return false;
	}

	// send the actual message
	if (!BlockingSendSocketMessageImpl((uint8*)TCHAR_TO_UTF8(*InMessage), MessageLength))
	{
		UE_LOGF(LogMovieRenderPipeline, Verbose, "SendSocketMessage write failed with code %d", (int32)ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLastErrorCode());
		return false;
	}

	return true;
}

bool UMoviePipelineExecutorBase::IsSocketConnected() const
{
	return ExternalSocket && ExternalSocket->GetConnectionState() == ESocketConnectionState::SCS_Connected;
}

bool UMoviePipelineExecutorBase::BlockingSendSocketMessageImpl(const uint8* Data, int32 BytesToSend)
{
	check(IsSocketConnected());

	int32 TotalBytes = BytesToSend;
	while (BytesToSend > 0)
	{
		while (!ExternalSocket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(1.0)))
		{
			if (ExternalSocket->GetConnectionState() == SCS_ConnectionError)
			{
				return false;
			}
		}

		int32 BytesSent = 0;
		if (!ExternalSocket->Send(Data, BytesToSend, BytesSent))
		{
			return false;
		}
		BytesToSend -= BytesSent;
		Data += BytesSent;
	}
	return true;
}

bool UMoviePipelineExecutorBase::ProcessIncomingSocketData()
{
	if (!IsSocketConnected())
	{
		return true;
	}

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	uint32 PendingDataSize = 0;

	auto GetReadableErrorCode = [SocketSubsystem]() -> FString
	{
		ESocketErrors LastError = SocketSubsystem->GetLastErrorCode();
		return FString::Printf(TEXT("%s (%d)"), SocketSubsystem->GetSocketError(LastError), (int32)LastError);
	};

	// Check if there is any pending data
	if (!ExternalSocket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromSeconds(0.0)))
	{
		return (ExternalSocket->GetConnectionState() != SCS_ConnectionError);
	}

	// Read until the whole message has come in.
	while(true)
	{
		int32 BytesRead = 0;
		// See if we're in the process of receiving a (large) message
		if (RecvMessageDataRemaining == 0)
		{
			// no partial message. Try to receive the size of a message
			if (!ExternalSocket->HasPendingData(PendingDataSize) || (PendingDataSize < sizeof(uint32)))
			{
				// no messages
				return true;
			}

			FArrayReader MessagesizeData = FArrayReader(true);
			MessagesizeData.SetNumUninitialized(sizeof(uint32));

			// read message size from the stream
			BytesRead = 0;
			if (!ExternalSocket->Recv(MessagesizeData.GetData(), sizeof(uint32), BytesRead))
			{
				UE_LOGF(LogMovieRenderPipeline, Verbose, "In progress read failed with code %ls", *GetReadableErrorCode());
				return false;
			}

			check(BytesRead == sizeof(uint32));

			// Setup variables to receive the message
			MessagesizeData << RecvMessageDataRemaining;

			if (RecvMessageDataRemaining <= 0)
			{
				UE_LOGF(LogMovieRenderPipeline, Error, "Read failed due to invalid Message Size: %d", RecvMessageDataRemaining);
				return false;
			}

			RecvMessageData = MakeShareable(new FArrayReader(true));
			RecvMessageData->SetNumUninitialized(RecvMessageDataRemaining);
		}

		BytesRead = 0;
		if (!ExternalSocket->Recv(RecvMessageData->GetData() + RecvMessageData->Num() - RecvMessageDataRemaining, RecvMessageDataRemaining, BytesRead))
		{
			UE_LOGF(LogMovieRenderPipeline, Verbose, "Read failed with code %ls", *GetReadableErrorCode());
			return false;
		}

		if (BytesRead > 0)
		{
			check(BytesRead <= RecvMessageDataRemaining);
			RecvMessageDataRemaining -= BytesRead;
			if (RecvMessageDataRemaining == 0)
			{
				RecvMessageData->Add(0); // Ensure Null Terminator
				
				// Convert the message to a string
				FString Message = FString(UTF8_TO_TCHAR((char*)RecvMessageData->GetData()));
				RecvMessageData.Reset();

				SocketMessageRecievedDelegate.Broadcast(Message);
			}
		}
		else
		{
			// no data
			return true;
		}
	}
}

void UMoviePipelineExecutorBase::OnBeginFrame_Implementation()
{
	ProcessIncomingSocketData();
}

void UMoviePipelineExecutorBase::SetPostRenderAction_Implementation(const EMoviePipelinePostRenderActionType PostRenderAction)
{
	if (GetSupportsPostRenderActions())
	{
		PostRenderActionType = PostRenderAction;
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Attempting to set a post-render action on an executor that does not support them.");
	}
}

int32 UMoviePipelineExecutorBase::SendHTTPRequest(const FString& InURL, const FString& InVerb, const FString& InMessage, const TMap<FString, FString>& InHeaders)
{
	FHttpRequestPtr HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetURL(*InURL);
	HttpRequest->SetVerb(*InVerb);

	for (TPair<FString, FString> KVP : InHeaders)
	{
		HttpRequest->SetHeader(*KVP.Key, *KVP.Value);
	}
	HttpRequest->SetContentAsString(*InMessage);
	HttpRequest->OnProcessRequestComplete().BindUObject(this, &UMoviePipelineExecutorBase::OnHttpRequestCompletedImpl);
	HttpRequest->ProcessRequest();

	static int32 RequestIndex = 0;
	FOutstandingRequest& NewRequest = OutstandingRequests.AddDefaulted_GetRef();
	NewRequest.RequestIndex = RequestIndex++;
	NewRequest.Request = HttpRequest;

	return NewRequest.RequestIndex;
}

void UMoviePipelineExecutorBase::OnHttpRequestCompletedImpl(FHttpRequestPtr InRequest, FHttpResponsePtr InResponse, bool bWasSuccessful)
{
	// Match it up with the request index we told the user this request was for originally.
	int32 RequestIndex = -1;
	for (const FOutstandingRequest& Request : OutstandingRequests)
	{
		if (Request.Request == InRequest)
		{
			RequestIndex = Request.RequestIndex;
			break;
		}
	}

	if (bWasSuccessful && InResponse)
	{
		int32 ResponseCode = InResponse->GetResponseCode();
		FString ResponseMessage = InResponse->GetContentAsString();
		HTTPResponseRecievedDelegate.Broadcast(RequestIndex, ResponseCode, ResponseMessage);
	}
	else
	{
		HTTPResponseRecievedDelegate.Broadcast(RequestIndex, -1, FString());
	}
}
