// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineFCPXMLExporterSetting.h"
#include "MoviePipeline.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformFileManager.h"

#if WITH_EDITOR
#include "FCPXML/FCPXMLMovieSceneTranslator.h"
#include "MovieSceneExportMetadata.h"
#include "MovieSceneToolHelpers.h"
#endif

#include "LevelSequence.h"
#include "MoviePipelineUtils.h"

// For logs
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineFCPXMLExporterSetting)

void UMoviePipelineFCPXMLExporter::BeginExportImpl()
{
	bHasFinishedExporting = true;

#if WITH_EDITOR
	UMoviePipelineOutputSetting* OutputSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();

	// Use our file name format on the end of the shared common directory.
	FString FileNameFormat = FileNameFormatOverride.Len() > 0 ? FileNameFormatOverride : OutputSetting->FileNameFormat;

	FString FileNameFormatString = OutputSetting->OutputDirectory.Path / FileNameFormat;

	const bool bIncludeRenderPass = false;
	const bool bTestFrameNumber = false;

	UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber);

	// Strip any frame number tags.
	UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);
	
	TMap<FString, FString> FormatOverrides;
	FormatOverrides.Add(TEXT("ext"), TEXT("xml"));

	// Create a full absolute path
	FMoviePipelineFormatArgs TempFormatArgs;
	GetPipeline()->ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, FilePath, TempFormatArgs);

	bool bSuccess = EnsureWritableFile();

	if (bSuccess)
	{
		ULevelSequence* Sequence = GetPipeline()->GetTargetSequence();
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		UMovieSceneCinematicShotTrack* ShotTrack = MovieScene->FindTrack<UMovieSceneCinematicShotTrack>();
		if (!ShotTrack)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "FCPXML Export only works with a Cinematic Shot track. No FCPXML file will be written.");
			return;
		}
		
		FString FilenameFormat = OutputSetting->FileNameFormat;
		int32 HandleFrames = OutputSetting->HandleFrameCount;
		FFrameRate FrameRate = GetPipeline()->GetPipelinePrimaryConfig()->GetEffectiveFrameRate(Sequence);
		uint32 ResX = OutputSetting->OutputResolution.X;
		uint32 ResY = OutputSetting->OutputResolution.Y;
		FString MovieExtension = ".mxf";

		FFCPXMLExporter* Exporter = new FFCPXMLExporter;

		TSharedRef<FMovieSceneTranslatorContext> ExportContext(new FMovieSceneTranslatorContext);
		ExportContext->Init();
		
		switch (DataSource)
		{
			case FCPXMLExportDataSource::OutputMetadata:
			{
				const FMovieSceneExportMetadata& OutputMetadata = GetPipeline()->GetOutputMetadata();
				bSuccess = Exporter->Export(MovieScene, FilenameFormat, FrameRate, ResX, ResY, HandleFrames, FilePath, ExportContext, MovieExtension, &OutputMetadata);
				break;
			}
			case FCPXMLExportDataSource::SequenceData:
			{
				bSuccess = Exporter->Export(MovieScene, FilenameFormat, FrameRate, ResX, ResY, HandleFrames, FilePath, ExportContext, MovieExtension);
				break;
			}
		}

		// Log any messages in context
		MovieSceneToolHelpers::MovieSceneTranslatorLogMessages(Exporter, ExportContext, false);

		delete Exporter;
	}

	if (!bSuccess)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to write Final Cut Pro XML");
	}
#else
	UE_LOGF(LogMovieRenderPipeline, Error, "Final Cut Pro XML writing only supported in editor.");
#endif
}

bool UMoviePipelineFCPXMLExporter::EnsureWritableFile()
{
	FString Directory = FPaths::GetPath(FilePath);

	if (!IFileManager::Get().DirectoryExists(*Directory))
	{
		IFileManager::Get().MakeDirectory(*Directory);
	}

	UMoviePipelineOutputSetting* OutputSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();

	// If the file doesn't exist, we're ok to continue
	if (IFileManager::Get().FileSize(*FilePath) == -1)
	{
		return true;
	}
	// If we're allowed to overwrite the file, and we deleted it ok, we can continue
	else if (OutputSetting->bOverrideExistingOutput && FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*FilePath))
	{
		return true;
	}
	// We can't write to the file
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to write Final Cut Pro XML to '%ls'. Should Overwrite: %d - If we should have overwritten the file, we failed to delete the file. If we shouldn't have overwritten the file the file already exists so we can't replace it.", *FilePath, OutputSetting->bOverrideExistingOutput);
		return false;
	}
}
