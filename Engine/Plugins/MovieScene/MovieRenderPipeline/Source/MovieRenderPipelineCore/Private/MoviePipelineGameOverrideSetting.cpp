// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineGameOverrideSetting.h"
#include "Scalability.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineGameOverrideSetting)

static TAutoConsoleVariable<int32> CVarMoviePipelineVTNaniteAutoLOD(
	TEXT("MoviePipeline.EnableVTInvalidateOnNaniteLOD"),
	1,
	TEXT("If true, the Movie Pipeline Game Overrides will automatically apply 'r.Nanite.VSMInvalidateOnLODDelta' during renders.\n"),
	ECVF_Default);

void UMoviePipelineGameOverrideSetting::SetupForPipelineImpl(UMoviePipeline* InPipeline)
{
	// Store the cvar values and apply the ones from this setting
	UE_LOGF(LogMovieRenderPipeline, Log, "Applying Game Override quality settings and cvars.");
	ApplyCVarSettings(true);
}

void UMoviePipelineGameOverrideSetting::TeardownForPipelineImpl(UMoviePipeline* InPipeline)
{
	// Restore the previous cvar values the user had
	UE_LOGF(LogMovieRenderPipeline, Log, "Restoring Game Override quality settings and cvars.");
	ApplyCVarSettings(false);
}

void UMoviePipelineGameOverrideSetting::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->ScalabilityQualityLevel = bCinematicQualitySettings ? TEXT("Cinematic") : TEXT("Default");
}

void UMoviePipelineGameOverrideSetting::PostLoad()
{
	Super::PostLoad();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (const UClass* GameModeOverrideClass = GameModeOverride.Get())
	{
		SoftGameModeOverride = GameModeOverrideClass;
		GameModeOverride = nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UMoviePipelineGameOverrideSetting::ApplyCVarSettings(const bool bOverrideValues)
{
	if (bCinematicQualitySettings)
	{
		if (bOverrideValues)
		{
			// Store their previous Scalability settings so we can revert back to them
			PreviousQualityLevels = Scalability::GetQualityLevels();

			// Create a copy and override to the maximum level for each Scalability category
			Scalability::FQualityLevels QualityLevels = PreviousQualityLevels;
			QualityLevels.SetFromSingleQualityLevelRelativeToMax(0);

			// Apply
			Scalability::SetQualityLevels(QualityLevels);
		}
		else
		{
			// We re-apply old scalability settings at the end of the function during teardown
			// so that any values that are also specified in Scalability don't get overwritten
			// with the wrong values from the ones below restoring.
		}
	}

	switch (TextureStreaming)
	{
	case EMoviePipelineTextureStreamingMethod::FullyLoad:
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousFramesForFullUpdate, TEXT("r.Streaming.FramesForFullUpdate"), 0, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousFullyLoadUsedTextures, TEXT("r.Streaming.FullyLoadUsedTextures"), 1, bOverrideValues);
		break;
	case EMoviePipelineTextureStreamingMethod::Disabled:
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousTextureStreaming, TEXT("r.TextureStreaming"), 0, bOverrideValues);
		break;
	default:
		// We don't change their texture streaming settings.
		break;
	}

	if (bUseLODZero)
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousForceLOD, TEXT("r.ForceLOD"), 0, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousSkeletalMeshBias, TEXT("r.SkeletalMeshLODBias"), -10, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousParticleLODBias, TEXT("r.ParticleLODBias"), -10, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousFoliageDitheredLOD, TEXT("foliage.DitheredLOD"), 0, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousFoliageForceLOD, TEXT("foliage.ForceLOD"), 0, bOverrideValues);
	}

	if (bDisableHLODs)
	{
		// It's a command and not an integer cvar (despite taking 1/0), so we can't cache it 
		if(GEngine)
		{
			GEngine->Exec(GetWorld(), TEXT("r.HLOD 0"));
		}
	}

	if (bUseHighQualityShadows)
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_FLOAT(PreviousShadowDistanceScale, TEXT("r.Shadow.DistanceScale"), ShadowDistanceScale, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousShadowQuality, TEXT("r.ShadowQuality"), 5, bOverrideValues);
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_FLOAT(PreviousShadowRadiusThreshold, TEXT("r.Shadow.RadiusThreshold"), ShadowRadiusThreshold, bOverrideValues);
	}

	if (bOverrideViewDistanceScale)
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_FLOAT(PreviousViewDistanceScale, TEXT("r.ViewDistanceScale"), ViewDistanceScale, bOverrideValues);
	}

	if (bOverrideGrassCullDistanceScale)
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_FLOAT(PreviousGrassCullDistanceScale, TEXT("grass.CullDistanceScale"), GrassCullDistanceScale, bOverrideValues);
	}

	if (bOverrideGrassDensityScale)
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_FLOAT(PreviousGrassDensityScale, TEXT("grass.densityScale"), GrassDensityScale, bOverrideValues);
	}

	if (bDisableGPUTimeout)
	{
		// This CVAR only exists if the D3D12RHI module is loaded
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT_IF_EXIST(PreviousGPUTimeout, TEXT("r.D3D12.GPUTimeout"), 0, bOverrideValues);
	}

	if (bFlushStreamingManagers)
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousStreamingManagerSyncState, TEXT("r.Streaming.SyncStatesWhenBlocking"), 1, bOverrideValues);
	}
	
#if WITH_EDITOR
	// To make sure the GeometryCache streamer doesn't skip frames and doesn't pop up notification during rendering
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT_IF_EXIST(PreviousGeoCacheStreamerBlockTillFinish, TEXT("GeometryCache.Streamer.BlockTillFinishStreaming"), 1, bOverrideValues);
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT_IF_EXIST(PreviousGeoCacheStreamerShowNotification, TEXT("GeometryCache.Streamer.ShowNotification"), 0, bOverrideValues);
#endif

	{
		// Disable systems that try to preserve performance in runtime games.
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousAnimationUROEnabled, TEXT("a.URO.Enable"), 0, bOverrideValues);
	}

	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousNeverMuteNonRealtimeAudio, TEXT("au.NeverMuteNonRealtimeAudioDevices"), 1, bOverrideValues);

	// Ensures that submixes will always render silence instead of disabling to maintain timing in the sequence
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_BOOL(PreviousNeverDisableSubmixes, TEXT("au.NeverDisableSubmixes"), 1, bOverrideValues);

	// To make sure that the skylight is always valid and consistent accross capture sessions, we enforce a full capture each frame, accepting a small GPU cost.
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousSkyLightRealTimeReflectionCaptureTimeSlice, TEXT("r.SkyLight.RealTimeReflectionCapture.TimeSlice"), 0, bOverrideValues);

	// Cloud are rendered using high quality volumetric render target mode 3: per pixel tracing and composition on screen, while supporting cloud on translucent.
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousVolumetricRenderTarget, TEXT("r.VolumetricRenderTarget"), 1, bOverrideValues);
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousVolumetricRenderTargetMode, TEXT("r.VolumetricRenderTarget.Mode"), 3, bOverrideValues);

	// To make sure that the world partition streaming doesn't end up in critical streaming performances and stops streaming low priority cells.
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousIgnoreStreamingPerformance, TEXT("wp.Runtime.BlockOnSlowStreaming"), 0, bOverrideValues);

	// Remove any minimum delta time requirements from Chaos Physics to ensure accuracy at high Temporal Sample counts
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_FLOAT(PreviousChaosImmPhysicsMinStepTime, TEXT("p.Chaos.ImmPhys.MinStepTime"), 0, bOverrideValues);

	// Cloth's time step smoothing messes up the change in number of simulation substeps that fixes the cloth simulation behavior when using Temporal Samples.
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousChaosClothUseTimeStepSmoothing, TEXT("p.ChaosCloth.UseTimeStepSmoothing"), 0, bOverrideValues);

	// Slate throttling can cause the world to not tick, leading to hitching/pausing in the rendered frames. Notifications/toast popping up can trigger
	// throttling, so force-disable it during a render.
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousAllowSlateThrottling, TEXT("Slate.bAllowThrottling"), 0, bOverrideValues);

	// Water skips water info texture when the world's game viewport rendering is disabled so we need to prevent this from happening.
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT_IF_EXIST(PreviousSkipWaterInfoTextureRenderWhenWorldRenderingDisabled, TEXT("r.Water.SkipWaterInfoTextureRenderWhenWorldRenderingDisabled"), 0, bOverrideValues);

	// This is only a temporary cvar while it's experimental so it's not exposed to the UI, but exposed as a cvar
	// so that users can turn it off in the event that it causes issues.
	if (CVarMoviePipelineVTNaniteAutoLOD.GetValueOnGameThread())
	{
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousNaniteVSMInvalidateOnLODDelta, TEXT("r.Nanite.VSMInvalidateOnLODDelta"), 1, bOverrideValues);
	}
	
	// Enable the alpha channel throughout the renderer
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_BOOL(PreviousAlphaOutput, TEXT("r.PostProcessing.PropagateAlpha"), UE::MoviePipeline::GetAlphaOutputOverride(), bOverrideValues);

	// Improve (combined) LUT precision to avoid quantization errors, especially with multi-sample accumulation
	MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PreviousLUTSize, TEXT("r.LUT.Size"), 64, bOverrideValues);

	// Must come after the above cvars so that if one of those cvars is also specified by the Scalability level, then we restore to the value in the original scalability level
	// not the value we cached in the Cinematic level (if applied).
	if (bCinematicQualitySettings)
	{
		if (!bOverrideValues)
		{
			Scalability::SetQualityLevels(PreviousQualityLevels);
		}
	}
}

void UMoviePipelineGameOverrideSetting::BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const
{
	if (!IsEnabled())
	{
		return;
	}

	// We don't provide the GameMode on the command line argument as we expect NewProcess to boot into an empty map and then it will
	// transition into the correct map which will then use the GameModeOverride setting.
	if (bCinematicQualitySettings)
	{
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.ViewDistanceQuality=4"));
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.AntiAliasingQuality=4"));
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.ShadowQuality=4"));
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.GlobalIlluminationQuality=4"));
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.ReflectionQuality=4"));
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.PostProcessQuality=4"));
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.TextureQuality=4"));
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.EffectsQuality=4"));
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.FoliageQuality=4"));
		InOutDeviceProfileCvars.AddUnique(TEXT("sg.ShadingQuality=4"));
	}

	switch (TextureStreaming)
	{
	case EMoviePipelineTextureStreamingMethod::FullyLoad:
		InOutDeviceProfileCvars.AddUnique(TEXT("r.Streaming.FramesForFullUpdate=0"));
		InOutDeviceProfileCvars.AddUnique(TEXT("r.Streaming.FullyLoadUsedTextures=1"));
		break;
	case EMoviePipelineTextureStreamingMethod::Disabled:
		InOutDeviceProfileCvars.AddUnique(TEXT("r.TextureStreaming=0"));
		break;
	default:
		// We don't change their texture streaming settings.
		break;
	}

	if (bUseLODZero)
	{
		InOutDeviceProfileCvars.AddUnique(TEXT("r.ForceLOD=0"));
		InOutDeviceProfileCvars.AddUnique(TEXT("r.SkeletalMeshLODBias=-10"));
		InOutDeviceProfileCvars.AddUnique(TEXT("r.ParticleLODBias=-10"));
		InOutDeviceProfileCvars.AddUnique(TEXT("foliage.DitheredLOD=0"));
		InOutDeviceProfileCvars.AddUnique(TEXT("foliage.ForceLOD=0"));
	}

	if (bDisableHLODs)
	{
		// It's a command and not an integer cvar (despite taking 1/0)
		InOutExecCmds.AddUnique(TEXT("r.HLOD 0"));
	}

	if (bUseHighQualityShadows)
	{
		InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.Shadow.DistanceScale=%d"), ShadowDistanceScale));
		InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.Shadow.RadiusThreshold=%f"), ShadowRadiusThreshold));
		InOutDeviceProfileCvars.AddUnique(TEXT("r.ShadowQuality=5"));
	}

	if (bOverrideViewDistanceScale)
	{
		InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.ViewDistanceScale=%d"), ViewDistanceScale));
	}

	if (bOverrideGrassCullDistanceScale)
	{
		InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("grass.CullDistanceScale=%f"), GrassCullDistanceScale));
	}

	if (bOverrideGrassDensityScale)
	{
		InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("grass.densityScale=%f"), GrassDensityScale));
	}

	if (bDisableGPUTimeout)
	{
		InOutDeviceProfileCvars.AddUnique(TEXT("r.D3D12.GPUTimeout=0"));
	}

	if (bFlushStreamingManagers)
	{
		InOutDeviceProfileCvars.AddUnique(TEXT("r.Streaming.SyncStatesWhenBlocking=1"));
	}
	
#if WITH_EDITOR
	{
		InOutDeviceProfileCvars.AddUnique(TEXT("GeometryCache.Streamer.BlockTillFinishStreaming=1"));
		InOutDeviceProfileCvars.AddUnique(TEXT("GeometryCache.Streamer.ShowNotification=0"));
	}
#endif

	{
		InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("a.URO.Enable=%d"), 0));
	}

	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("au.NeverMuteNonRealtimeAudioDevices=%d"), 1));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("au.NeverDisableSubmixes=%d"), 1));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.SkyLight.RealTimeReflectionCapture.TimeSlice=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.VolumetricRenderTarget=%d"), 1));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.VolumetricRenderTarget.Mode=%d"), 3));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("wp.Runtime.BlockOnSlowStreaming=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("p.Chaos.ImmPhys.MinStepTime=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("p.ChaosCloth.UseTimeStepSmoothing=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.Water.SkipWaterInfoTextureRenderWhenWorldRenderingDisabled=%d"), 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("Slate.bAllowThrottling=%d"), 0));

	if (CVarMoviePipelineVTNaniteAutoLOD.GetValueOnGameThread())
	{
		InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.Nanite.VSMInvalidateOnLODDelta=%d"), 1));
	}

	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.PostProcessing.PropagateAlpha=%d"), UE::MoviePipeline::GetAlphaOutputOverride() ? 1 : 0));
	InOutDeviceProfileCvars.AddUnique(FString::Printf(TEXT("r.LUT.Size=%d"), 64));
}
