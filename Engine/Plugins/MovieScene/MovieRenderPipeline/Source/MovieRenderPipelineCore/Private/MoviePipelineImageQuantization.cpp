// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineImageQuantization.h"
#include "ImagePixelData.h"
#include "Stats/Stats.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Math/NumericLimits.h"
#include "Async/ParallelFor.h"

namespace UE
{
namespace MoviePipeline
{
DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_ImageQuantization"), STAT_ImageQuantization, STATGROUP_MoviePipeline);


// @todo Oodle : can get rid of this whole file
//		use FImage, just call CopyImage


template<class TToColorBitDepthType, typename TFromColorBitDepthType, typename TColorChannelNumericType>
TArray<TToColorBitDepthType> ConvertLinearToLinearBitDepth(TFromColorBitDepthType* InColor, const int64 InCount)
{
	if constexpr (std::is_same_v<TToColorBitDepthType, TFromColorBitDepthType>)
	{
		return TArray(InColor, InCount);
	}

	// Convert all of our pixels.
	TArray<TToColorBitDepthType> OutsRGBData;
	OutsRGBData.SetNumUninitialized(InCount);
	// TColorChannelNumericType = uint8 or uint16
	// do clamp in int in case input float is out of [0,1] range
	int MaxValueInt = TNumericLimits<TColorChannelNumericType>::Max();
	float MaxValue = static_cast<float>(MaxValueInt);
	for (int64 PixelIndex = 0; PixelIndex < InCount; PixelIndex++)
	{
		// Avoid the bounds checking of TArray[]
		TToColorBitDepthType* OutColor = &OutsRGBData.GetData()[PixelIndex];

		// We don't need sRGB color conversion. Avoid FMath::Floor or Round
		OutColor->R = (TColorChannelNumericType) FMath::Clamp( (int)((InColor[PixelIndex].R * MaxValue) + 0.5f), 0, MaxValueInt);
		OutColor->G = (TColorChannelNumericType) FMath::Clamp( (int)((InColor[PixelIndex].G * MaxValue) + 0.5f), 0, MaxValueInt);
		OutColor->B = (TColorChannelNumericType) FMath::Clamp( (int)((InColor[PixelIndex].B * MaxValue) + 0.5f), 0, MaxValueInt);
		OutColor->A = (TColorChannelNumericType) FMath::Clamp( (int)((InColor[PixelIndex].A * MaxValue) + 0.5f), 0, MaxValueInt);
	}

	return OutsRGBData;
}

static TArray<uint8> GenerateSRGBTable(uint32 InPrecision)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ImageQuant_TableGeneration);
	TArray<uint8> OutsRGBTable;
	OutsRGBTable.SetNumUninitialized(InPrecision);
	for (int32 TableIndex = 0; TableIndex < OutsRGBTable.Num(); TableIndex++)
	{
		float ValueAsLinear = (float)TableIndex / (OutsRGBTable.Num() - 1);

		// sRGB is linear under 0.0031308 and Pow(1/2.4) above that.
		if (ValueAsLinear <= 0.0031308f)
		{
			ValueAsLinear = ValueAsLinear * 12.92f;
		}
		else
		{
			ValueAsLinear = FMath::Pow(ValueAsLinear, 1.0f / 2.4f) * 1.055f - 0.055f;
		}

		// Flooring avoids an extra branch for Round. Using [] on GetData() avoids bounds checking cost.
		check( ValueAsLinear >= 0.f && ValueAsLinear <= 1.f );
		OutsRGBTable.GetData()[TableIndex] = (uint8) ( ValueAsLinear * 255.f + 0.5f );
	}
	
	return OutsRGBTable;
}

static TArray<float> GenerateInverseSRGBTable(uint32 InPrecision)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ImageQuant_TableGeneration);
	TArray<float> OutLinearTable;
	OutLinearTable.SetNumUninitialized(InPrecision);
	for (int32 TableIndex = 0; TableIndex < OutLinearTable.Num(); TableIndex++)
	{
		float ValueAsLinear = (float)TableIndex / (OutLinearTable.Num() - 1);
		if (ValueAsLinear <= 0.04045)
		{
			ValueAsLinear = ValueAsLinear / 12.92f;
		}
		else
		{
			ValueAsLinear = FMath::Pow((ValueAsLinear + 0.055f) / 1.055f, 2.4f);
		}

		OutLinearTable.GetData()[TableIndex] = ValueAsLinear;
	}

	return OutLinearTable;
}

static TArray<float> GenerateSRGBTableFloat16toFloat()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ImageQuant_TableGeneration);
	TArray<float> OutsRGBTable;
	OutsRGBTable.SetNumUninitialized(65536);
	float* OutsRGBTableData = OutsRGBTable.GetData();

	FFloat16 OnePointZero = 1.0f;
	for (int32 TableIndex = OnePointZero.Encoded; TableIndex < 32768; TableIndex++)
	{
		// Fill table for values >= 1.0 to be 255
		OutsRGBTableData[TableIndex] = 255;
	}
	for (int32 TableIndex = 32768; TableIndex < 65536; TableIndex++)
	{
		// Fill table for all negative values to be 0
		OutsRGBTableData[TableIndex] = 0;
	}
	for (int32 TableIndex = 0; TableIndex < OnePointZero.Encoded; TableIndex++)
	{
		FFloat16 Value;
		Value.Encoded = TableIndex;
		float ValueAsLinear = (float)Value;
		// sRGB is linear under 0.0031308 and Pow(1/2.4) above that.
		if (ValueAsLinear <= 0.0031308f)
		{
			ValueAsLinear = ValueAsLinear * 12.92f;
		}
		else
		{
			ValueAsLinear = FMath::Pow(ValueAsLinear, 1.0f / 2.4f) * 1.055f - 0.055f;
		}
		// Simply multiply by 255 to get floats in the proper range for quantization after the table is indexed.
		OutsRGBTableData[TableIndex] = (ValueAsLinear * 255.f);
	}

	return OutsRGBTable;
}

static TArray<FFloat16> GenerateSRGBTableFloat16to16()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ImageQuant_TableGeneration);
	TArray<FFloat16> OutsRGBTable;
	OutsRGBTable.SetNumUninitialized(65536);
	FFloat16* OutsRGBTableData = OutsRGBTable.GetData();

	for (int32 TableIndex = 0; TableIndex < 65536; TableIndex++)
	{
		FFloat16 Value;
		Value.Encoded = TableIndex;
		float ValueAsEncodedForsRGB = (float)Value;
		// sRGB is linear under 0.0031308 and Pow(1/2.4) above that.
		if (ValueAsEncodedForsRGB <= 0.0031308f)
		{
			ValueAsEncodedForsRGB = ValueAsEncodedForsRGB * 12.92f;
		}
		else
		{
			ValueAsEncodedForsRGB = FMath::Pow(ValueAsEncodedForsRGB, 1.0f / 2.4f) * 1.055f - 0.055f;
		}

		OutsRGBTableData[TableIndex] = ValueAsEncodedForsRGB;
	}

	return OutsRGBTable;
}

static TArray<FColor> ConvertLinearTosRGB8bppViaLookupTable(FFloat16Color* InColor, const int64 InCount)
{
	TArray<float> sRGBTable = GenerateSRGBTableFloat16toFloat();

	// This quantization uses a table of float values from 0.0-255.0, and does the proper rounding for each pixel.
	// The proper rounding is done by adding a random value from 0.0-1.0 instead of 0.5 to each value returned from the table.
	FRandomStream RandStream(31337);

	// FRandomStream is too slow to call 30,000,000 times (4k image) so instead
	// we pre-calculate a reasonable number of offsets into a table, and then 
	// each pixel in the output image takes a offset. Some care needs to be taken
	// that the table doesn't introduce an obvious repeating pattern in dithering.
	TArray<float> RandStreamTable;
	int32 TableSize = (1024 * 1024) + 7; // Arbitrarily picked for now
	RandStreamTable.SetNumUninitialized(TableSize);
	for (int32 Index = 0; Index < TableSize; Index++)
	{
		// GetFraction() returns [0,1) as desired, but it occasionally contains
		// too much precision - the sRGBTable will return values of 255.0f, and
		// GetFraction() can return values like 0.999996185, which when taken
		// as decimals is a valid value (255.99999996185), and then below we floor it
		// before casting to uint8, which should be fine.

		// In practice however, we can't actually represent 255.999996185 with a 
		// single-point precision float, it gets rounded up to 256 by the hardware,
		// then floored, and then cast to uint8 overflowing to zero.

		// To fix this, we limit this float to 15 bits of mantissa precision.
		float Value = RandStream.GetFraction();
		*(uint32*)&Value &= 0xFFFFFF00U;
		RandStreamTable[Index] = Value;
	}
	// Convert all of our pixels.
	TArray<FColor> OutsRGBData;
	OutsRGBData.SetNumUninitialized(InCount);

	float* sRGBTableData = static_cast<float*>(sRGBTable.GetData());
	FColor* OutData = static_cast<FColor*>(OutsRGBData.GetData());

	QUICK_SCOPE_CYCLE_COUNTER(STAT_ImageQuant_ApplysRGB);

	int32 BatchSize = 64 * 1024 * 8; // Arbitrarily picked for now.
	const uint32 Loops = FMath::CeilToInt((float)InCount / BatchSize);
	ParallelFor(Loops,
		[&](int32 LoopIndex)
		{
			const int64 Start = LoopIndex * BatchSize;
			const int64 End = FMath::Min((LoopIndex + 1) * BatchSize, InCount);


			for (int64 PixelIndex = Start; PixelIndex < End; PixelIndex++)
			{
				// Avoid the bounds checking of TArray[]
				FColor* OutColor = &OutData[PixelIndex];
				OutColor->R = (uint8)FMath::FloorToInt(sRGBTableData[InColor[PixelIndex].R.Encoded] + RandStreamTable[(PixelIndex + 0) % TableSize]);
				OutColor->G = (uint8)FMath::FloorToInt(sRGBTableData[InColor[PixelIndex].G.Encoded] + RandStreamTable[(PixelIndex + 1) % TableSize]);
				OutColor->B = (uint8)FMath::FloorToInt(sRGBTableData[InColor[PixelIndex].B.Encoded] + RandStreamTable[(PixelIndex + 2) % TableSize]);
	
				// Alpha doesn't get sRGB conversion, it just gets linearly converted to 8 bit. Flooring avoids an extra branch for Round.
				OutColor->A = (uint8)FMath::Clamp(FMath::FloorToInt((InColor[PixelIndex].A * 255.f) + RandStreamTable[(PixelIndex + 3) % TableSize]), 0, 255);
			}
		});

	return OutsRGBData;
}

static TArray<FColor> ConvertLinearTosRGB8bppViaLookupTable(FLinearColor* InColor, const int64 InCount)
{
	TArray<uint8> sRGBTable = GenerateSRGBTable(4096); 

	// Convert all of our pixels.
	TArray<FColor> OutsRGBData;
	OutsRGBData.SetNumUninitialized(InCount);

	int32 TableUpperBound = sRGBTable.Num() - 1;

	for (int64 PixelIndex = 0; PixelIndex < InCount; PixelIndex++)
	{
		// Avoid the bounds checking of TArray[]
		FColor* OutColor = &OutsRGBData.GetData()[PixelIndex];
		int32 TableIndexR = int32(InColor[PixelIndex].R * TableUpperBound);
		int32 TableIndexG = int32(InColor[PixelIndex].G * TableUpperBound);
		int32 TableIndexB = int32(InColor[PixelIndex].B * TableUpperBound);
		
		// We clamp the table index because values greater than 1.0 in the color would generate out of bound indexes.
		OutColor->R = sRGBTable.GetData()[FMath::Clamp(TableIndexR, 0, TableUpperBound)];
		OutColor->G = sRGBTable.GetData()[FMath::Clamp(TableIndexG, 0, TableUpperBound)];
		OutColor->B = sRGBTable.GetData()[FMath::Clamp(TableIndexB, 0, TableUpperBound)];
		
		// Alpha doesn't get sRGB conversion, it just gets linearly converted to 8 bit.
		OutColor->A = FColor::QuantizeUNormFloatTo8( InColor[PixelIndex].A );
	}

	return OutsRGBData;
}

static TArray<FFloat16Color> ConvertLinearTosRGB16bppViaLookupTable(FFloat16Color* InColor, const int64 InCount)
{
	TArray<FFloat16> sRGBTable = GenerateSRGBTableFloat16to16();

	// Convert all of our pixels.
	TArray<FFloat16Color> OutsRGBData;
	OutsRGBData.SetNumUninitialized(InCount);

	FFloat16* sRGBTableData = static_cast<FFloat16*>(sRGBTable.GetData());
	FFloat16Color* OutData = static_cast<FFloat16Color*>(OutsRGBData.GetData());

	QUICK_SCOPE_CYCLE_COUNTER(STAT_ImageQuant_ApplysRGB);
	for (int64 PixelIndex = 0; PixelIndex < InCount; PixelIndex++)
	{
		// Avoid the bounds checking of TArray[]
		FFloat16Color* OutColor = &OutData[PixelIndex];
		OutColor->R = sRGBTableData[InColor[PixelIndex].R.Encoded];
		OutColor->G = sRGBTableData[InColor[PixelIndex].G.Encoded];
		OutColor->B = sRGBTableData[InColor[PixelIndex].B.Encoded];

		// Alpha doesn't get sRGB conversion, it just gets linearly converted
		OutColor->A = InColor[PixelIndex].A;
	}

	return OutsRGBData;
}

static TUniquePtr<FImagePixelData> QuantizePixelDataTo8bpp(const FImagePixelData* InPixelData, FImagePixelPayloadPtr InPayload, bool bConvertToSrgb)
{
	TUniquePtr<FImagePixelData> QuantizedPixelData = nullptr;

	FIntPoint RawSize = InPixelData->GetSize();
	int32 RawNumChannels = InPixelData->GetNumChannels();

	// Look at our incoming bit depth
	switch (InPixelData->GetBitDepth())
	{
	case 8:
	{
		// No work actually needs to be done, hooray! We return a copy of the data for consistency
		QuantizedPixelData = InPixelData->CopyImageData();
		break;
	}
	case 16:
	{
		int64 SizeInBytes = 0;
		const void* SrcRawDataPtr = nullptr;
		InPixelData->GetRawData(SrcRawDataPtr, SizeInBytes);

		if (bConvertToSrgb)
		{
			TArray<FColor> sRGBEncoded = ConvertLinearTosRGB8bppViaLookupTable((FFloat16Color*)SrcRawDataPtr, RawSize.X * (int64)RawSize.Y);
			QuantizedPixelData = MakeUnique<TImagePixelData<FColor>>(RawSize, TArray64<FColor>(MoveTemp(sRGBEncoded)), InPayload);
		}
		else
		{
			TArray<FColor> sRGBEncoded = ConvertLinearToLinearBitDepth<FColor, FFloat16Color, uint8>((FFloat16Color*)SrcRawDataPtr, RawSize.X * (int64)RawSize.Y);
			QuantizedPixelData = MakeUnique<TImagePixelData<FColor>>(RawSize, TArray64<FColor>(MoveTemp(sRGBEncoded)), InPayload);
			
		}
		break;
	}
	case 32:
	{
		int64 SizeInBytes = 0;
		const void* SrcRawDataPtr = nullptr;
		InPixelData->GetRawData(SrcRawDataPtr, SizeInBytes);
		
		if (bConvertToSrgb)
		{
			TArray<FColor> sRGBEncoded = ConvertLinearTosRGB8bppViaLookupTable((FLinearColor*)SrcRawDataPtr, RawSize.X * (int64)RawSize.Y);
			QuantizedPixelData = MakeUnique<TImagePixelData<FColor>>(RawSize, TArray64<FColor>(MoveTemp(sRGBEncoded)), InPayload);
		}
		else
		{
			TArray<FColor> sRGBEncoded = ConvertLinearToLinearBitDepth<FColor, FLinearColor, uint8>((FLinearColor*)SrcRawDataPtr, RawSize.X * (int64)RawSize.Y);
			QuantizedPixelData = MakeUnique<TImagePixelData<FColor>>(RawSize, TArray64<FColor>(MoveTemp(sRGBEncoded)), InPayload);
		}
		break;
	}

	default:
		// Unsupported source bit-depth, consider adding it!
		check(false);
	}

	return QuantizedPixelData;
}

static TUniquePtr<FImagePixelData> QuantizePixelDataTo16bpp(const FImagePixelData* InPixelData, FImagePixelPayloadPtr InPayload, bool bConvertToSrgb)
{
	TUniquePtr<FImagePixelData> QuantizedPixelData = nullptr;

	FIntPoint RawSize = InPixelData->GetSize();
	int32 RawNumChannels = InPixelData->GetNumChannels();

	// Look at our incoming bit depth
	switch (InPixelData->GetBitDepth())
	{
	case 16:
	{
		int64 SizeInBytes = 0;
		const void* SrcRawDataPtr = nullptr;
		InPixelData->GetRawData(SrcRawDataPtr, SizeInBytes);

		if (bConvertToSrgb)
		{
			TArray<FFloat16Color> sRGBEncoded = ConvertLinearTosRGB16bppViaLookupTable((FFloat16Color*)SrcRawDataPtr, RawSize.X * (int64)RawSize.Y);
			QuantizedPixelData = MakeUnique<TImagePixelData<FFloat16Color>>(RawSize, TArray64<FFloat16Color>(MoveTemp(sRGBEncoded)), InPayload);
		}
		else
		{
			QuantizedPixelData = MakeUnique<TImagePixelData<FFloat16Color>>(RawSize, TArray64<FFloat16Color>((FFloat16Color*)SrcRawDataPtr, RawSize.X * (int64)RawSize.Y), InPayload);
		}
		break;
	}
	case 8:
	{
		int64 SizeInBytes = 0;
		const void* SrcRawDataPtr = nullptr;
		InPixelData->GetRawData(SrcRawDataPtr, SizeInBytes);

		// FColor is assumed to be in sRGB while FFloat16Color is assumed to be linear so we need to convert back.
		TArray64<FFloat16Color> OutColors;
		OutColors.SetNumUninitialized(RawSize.X * (int64)RawSize.Y);

		TArray<float> LookupTable = GenerateInverseSRGBTable(256);
		const float* LookupTablePtr = LookupTable.GetData();
		
		const FColor* DataPtr = (FColor*)(SrcRawDataPtr);
		if (bConvertToSrgb)
		{
			for (int64 Index = 0; Index < (RawSize.X * (int64)RawSize.Y); Index++)
			{
				OutColors[Index].R = LookupTablePtr[DataPtr[Index].R];
				OutColors[Index].G = LookupTablePtr[DataPtr[Index].G];
				OutColors[Index].B = LookupTablePtr[DataPtr[Index].B];

				// Alpha is linear and doesn't get converted.
				OutColors[Index].A = DataPtr[Index].A / 255.f;
			}
		}
		else
		{
			for (int64 Index = 0; Index < (RawSize.X * (int64)RawSize.Y); Index++)
			{
				OutColors[Index].R = DataPtr[Index].R / 255.f;
				OutColors[Index].G = DataPtr[Index].G / 255.f;
				OutColors[Index].B = DataPtr[Index].B / 255.f;
				OutColors[Index].A = DataPtr[Index].A / 255.f;
			}
		}

		QuantizedPixelData = MakeUnique<TImagePixelData<FFloat16Color>>(RawSize, TArray64<FFloat16Color>(MoveTemp(OutColors)), InPayload);
		break;
	}
	case 32:
	{
		int64 SizeInBytes = 0;
		const void* SrcRawDataPtr = nullptr;
		InPixelData->GetRawData(SrcRawDataPtr, SizeInBytes);

		TArray64<FFloat16Color> OutColors;
		OutColors.SetNumUninitialized(RawSize.X * (int64)RawSize.Y);

		// This doesn't support the bConvertToSrgb flag because both FLinearColor and FFloat16Color should
		// be linear.
		const FLinearColor* DataPtr = (FLinearColor*)(SrcRawDataPtr);

		for (int64 Index = 0; Index < (RawSize.X * (int64)RawSize.Y); Index++)
		{
			OutColors[Index] = FFloat16Color(DataPtr[Index]);
		}
		QuantizedPixelData = MakeUnique<TImagePixelData<FFloat16Color>>(RawSize, TArray64<FFloat16Color>(MoveTemp(OutColors)), InPayload);
		break;
	}
	default:
		// Unsupported source bit-depth, consider adding it!
		check(false);
	}

	return QuantizedPixelData;
}

TUniquePtr<FImagePixelData> QuantizeImagePixelDataToBitDepth(const FImagePixelData* InData, const int32 TargetBitDepth, FImagePixelPayloadPtr InPayload, bool bConvertToSrgb)
{
	SCOPE_CYCLE_COUNTER(STAT_ImageQuantization);
	TUniquePtr<FImagePixelData> QuantizedPixelData = nullptr;
	switch (TargetBitDepth)
	{
	case 8:
		// Convert to 8 bit FColor
		QuantizedPixelData = UE::MoviePipeline::QuantizePixelDataTo8bpp(InData, InPayload, bConvertToSrgb);
		break;
	case 16:
		// Convert to 16 bit FFloat16Color
		QuantizedPixelData = UE::MoviePipeline::QuantizePixelDataTo16bpp(InData, InPayload, bConvertToSrgb);
		break;
	case 32:
		// Convert to 32 bit FLinearColor

	default:
		// Unsupported bit-depth to convert from, please consider implementing!
		check(false);
	}

	return QuantizedPixelData;
}
}
}
