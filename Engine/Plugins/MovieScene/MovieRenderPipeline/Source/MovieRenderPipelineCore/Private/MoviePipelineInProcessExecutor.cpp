// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineInProcessExecutor.h"
#include "Engine/GameInstance.h"
#include "MoviePipeline.h"
#include "Engine/World.h"
#include "LevelSequence.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineInProcessExecutorSettings.h"
#include "MoviePipelineGameOverrideSetting.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet/GameplayStatics.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Misc/PackageName.h"
#include "Engine/GameEngine.h"
#include "MoviePipelineUtils.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphGlobalGameOverrides.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineInProcessExecutor)

#define LOCTEXT_NAMESPACE "MoviePipelineInProcessExecutor"

void UMoviePipelineInProcessExecutor::Start(const UMoviePipelineExecutorJob* InJob)
{
	Super::Start(InJob);
	UWorld* World = MoviePipeline::FindCurrentWorld();

	if (bUseCurrentLevel)
	{		
		if (!World)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Unable to start movie pipeline job. No current map.");
			OnIndividualPipelineFinished(nullptr);
			return;
		}

		if (World != InJob->Map.ResolveObject())
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Current map '%ls' does not match job's map: '%ls'. Rendering may happen on a different map than expected and produce incorrect results.", *GetNameSafe(World), *InJob->Map.GetAssetPath().ToString());
		}
		
		UE_LOGF(LogMovieRenderPipeline, Log, "Starting %ls", *GetNameSafe(World));
	}

	// Check for null sequence
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(InJob->Sequence.TryLoad());
	if (!LevelSequence)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to load Sequence specified by job: %ls", *InJob->Sequence.ToString());

		FText FailureReason = LOCTEXT("InvalidSequenceFailureDialog", "One or more jobs in the queue has an invalid/null sequence. See log for details.");
		OnPipelineErrored(nullptr, true, FailureReason);
		OnExecutorFinishedImpl();
		return;
	}

	BackupState();
	
	// Initialize the transient settings so that they will exist in time for the GameOverrides check.
	InJob->GetConfiguration()->InitializeTransientSettings();

	ModifyState(InJob);

	if (bUseCurrentLevel)
	{
		OnMapLoadFinished(World);
	}
	else
	{
		// We were launched into an empty map so we'll look at our job and figure out which map we should load.
		// Get the next job in the queue
		FString MapOptions;
		
		if (const TSubclassOf<AGameModeBase> GameModeOverride = UMovieGraphGlobalGameOverridesNode::GetGameModeOverride(InJob))
		{
			MapOptions = TEXT("?game=") + GameModeOverride->GetPathName();
		}

		FString LevelPath = InJob->Map.GetLongPackageName();
		UE_LOGF(LogMovieRenderPipeline, Log, "About to load target map %ls", *LevelPath);
		FCoreUObjectDelegates::PostLoadMapWithWorld.AddUObject(this, &UMoviePipelineInProcessExecutor::OnMapLoadFinished);
		UGameplayStatics::OpenLevel(World, FName(*LevelPath), true, MapOptions);
	}
}

void UMoviePipelineInProcessExecutor::OnMapLoadFinished(UWorld* NewWorld)
{
	UE_LOGF(LogMovieRenderPipeline, Log, "Finished map load %ls", *GetNameSafe(NewWorld));

	// NewWorld can be null if a world is being destroyed.
	if (!NewWorld)
	{
		FCoreDelegates::OnBeginFrame.RemoveAll(this);
		return;
	}
	
	// Stop listening for map load until we're done and know we want to start the next config.
	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

	if (CurrentPipelineIndex >= Queue->GetJobs().Num())
	{
		FCoreDelegates::OnBeginFrame.RemoveAll(this);
		UE_LOGF(LogMovieRenderPipeline, Error, "Out of bounds Job Index (%d) in Queue (Length: %d)!", CurrentPipelineIndex, Queue->GetJobs().Num());

		FText FailureReason = LOCTEXT("OutOfBoundsFailureDialog", "The job index is out of bounds, did you clear the Queue while rendering was happening? See log for details.");
		OnPipelineErrored(nullptr, true, FailureReason);
		OnExecutorFinishedImpl();
		return;
	}
	
	UMoviePipelineExecutorJob* CurrentJob = Queue->GetJobs()[CurrentPipelineIndex];

	UClass* MoviePipelineClass = TargetPipelineClass.Get();
	if (MoviePipelineClass == nullptr)
	{
		MoviePipelineClass = UMoviePipeline::StaticClass();
	}

	// Force the pipeline to the default graph pipeline if we're using a graph or basic configuration and the pipeline isn't graph-compatible
	if ((CurrentJob->IsUsingGraphConfiguration() || CurrentJob->IsUsingBasicConfiguration())
		&& !MoviePipelineClass->IsChildOf(UMovieGraphPipeline::StaticClass()))
	{
		MoviePipelineClass = UMovieGraphPipeline::StaticClass();
	}

	ActiveMoviePipeline = NewObject<UMoviePipelineBase>(NewWorld, MoviePipelineClass);
	UMoviePipeline* PipelineAsLegacy = Cast<UMoviePipeline>(ActiveMoviePipeline);
	if (PipelineAsLegacy)
	{
		PipelineAsLegacy->SetViewportInitArgs(ViewportInitArgs);
	}

	// We allow users to set a multi-frame delay before we actually run the Initialization function and start thinking.
	// This solves cases where there are engine systems that need to finish loading before we do anything.
	const UMoviePipelineInProcessExecutorSettings* ExecutorSettings = GetDefault<UMoviePipelineInProcessExecutorSettings>();

	// We tick each frame to update the Window Title, and kick off latent pipeling initialization.
	FCoreDelegates::OnBeginFrame.AddUObject(this, &UMoviePipelineInProcessExecutor::OnTick);
	
	// Listen for when the pipeline thinks it has finished.
	ActiveMoviePipeline->OnMoviePipelineWorkFinished().AddUObject(this, &UMoviePipelineInProcessExecutor::OnMoviePipelineFinished);

	// Give the pipeline a chance to shut down properly if the render is interrupted (Alt+F4, closing the preview window, etc). When the render is
	// stopped like this, the engine's shutdown process may kill the player before more common delegates like FCoreDelegates::OnEnginePreExit are
	// called. ApplicationWillTerminateDelegate will be called immediately after an engine shutdown request is received though, and lets the pipeline
	// shut down before trying to access the (now non-existent) player.
	FCoreDelegates::GetApplicationWillTerminateDelegate().AddUObject(this, &UMoviePipelineInProcessExecutor::OnApplicationQuit);

	// Wait until we actually received the right map and created the pipeline before saying that we're actively rendering
	bIsRendering = true;
	
	if (ExecutorSettings->InitialDelayFrameCount == 0)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Zero Initial Delay, initializing...");
		if (PipelineAsLegacy)
		{
			PipelineAsLegacy->Initialize(Queue->GetJobs()[CurrentPipelineIndex]);
		}
		else if (UMovieGraphPipeline* PipelineAsGraph = Cast<UMovieGraphPipeline>(ActiveMoviePipeline))
		{
			PipelineAsGraph->Initialize(Queue->GetJobs()[CurrentPipelineIndex], FMovieGraphInitConfig());
		}
		RemainingInitializationFrames = -1;
	}
	else
	{
		RemainingInitializationFrames = ExecutorSettings->InitialDelayFrameCount;	
	}
}

void UMoviePipelineInProcessExecutor::OnTick()
{
	if (RemainingInitializationFrames >= 0)
	{
		if (RemainingInitializationFrames == 0)
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "Delay finished, initializing...");
			UMoviePipeline* PipelineAsLegacy = Cast<UMoviePipeline>(ActiveMoviePipeline);
			if (PipelineAsLegacy)
			{
				PipelineAsLegacy->Initialize(Queue->GetJobs()[CurrentPipelineIndex]);
			}
			else if (UMovieGraphPipeline* PipelineAsGraph = Cast<UMovieGraphPipeline>(ActiveMoviePipeline))
			{
				// PipelineAsGraph->Initialize(Queue->GetJobs()[CurrentPipelineIndex], FMovieGraphInitConfig());
			}
		}

		RemainingInitializationFrames--;
	}

	FText WindowTitle = GetWindowTitle();
	UKismetSystemLibrary::SetWindowTitle(WindowTitle);

	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		if (TSharedPtr<SWindow> GameViewportWindow = GameEngine->GameViewportWindow.Pin())
		{
			GameViewportWindow->SetAcceptsInput(false);
		}
	}
}

void UMoviePipelineInProcessExecutor::OnApplicationQuit()
{
	// Only call Shutdown if the pipeline hasn't been finished.
	if (ActiveMoviePipeline && ActiveMoviePipeline->GetPipelineState() != EMovieRenderPipelineState::Finished)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "MoviePipelineInProcessExecutor: Application quit while Movie Pipeline was still active. Stalling to do full shutdown.");

		// This will flush any outstanding work on the movie pipeline (file writes) immediately
		ActiveMoviePipeline->RequestShutdown(); // Set the Shutdown Requested flag.
		ActiveMoviePipeline->Shutdown(); // Flush the shutdown.

		UE_LOGF(LogMovieRenderPipeline, Log, "MoviePipelineInProcessExecutor: Stalling finished, pipeline has shut down.");
	}
}

void UMoviePipelineInProcessExecutor::OnMoviePipelineFinished(FMoviePipelineOutputData InOutputData)
{
	FCoreDelegates::OnBeginFrame.RemoveAll(this);
	OnIndividualJobFinishedDelegateNative.Broadcast(InOutputData);

	if (ActiveMoviePipeline)
	{
		// Unsubscribe in the event that it gets called twice we don't have issues.
		ActiveMoviePipeline->OnMoviePipelineWorkFinished().RemoveAll(this);
	}

	// Null these out now since OnIndividualPipelineFinished might invoke something that causes a GC
	// and we want them to go away with the GC.
	ActiveMoviePipeline = nullptr;
	
	RestoreState();

	// Now that another frame has passed and we should be OK to start another PIE session, notify our owner.
	// We do the cast to UMoviePipeline (which will return nullptr during a graph render) because the functions
	// all take a pointer, but it's never actually used, and you can only have one pipeline at once so the API
	// doesn't really need to pass back the pointer to a soon-to-be-destroyed object.
	OnIndividualPipelineFinished(Cast<UMoviePipeline>(InOutputData.Pipeline));
}

void UMoviePipelineInProcessExecutor::BackupState()
{
	SavedState.bBackedUp = true;
	SavedState.bUseFixedTimeStep = FApp::UseFixedTimeStep();
	SavedState.FixedDeltaTime = FApp::GetFixedDeltaTime();

	UWorld* World = MoviePipeline::FindCurrentWorld();
	if (World && World->GetGameInstance())
	{
		if (APlayerController* PlayerController = World->GetGameInstance()->GetFirstLocalPlayerController())
		{
			SavedState.bCinematicMode = PlayerController->bCinematicMode;
			SavedState.bHidePlayer = PlayerController->bHidePawnInCinematicMode;
		}
	}

	SavedState.WindowTitle.Reset();
	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		TSharedPtr<SWindow> GameViewportWindow = GameEngine->GameViewportWindow.Pin();
		if (GameViewportWindow.IsValid())
		{
			SavedState.WindowTitle = GameViewportWindow->GetTitle();
		}
	}
}

void UMoviePipelineInProcessExecutor::ModifyState(const UMoviePipelineExecutorJob* InJob)
{
	UWorld* World = MoviePipeline::FindCurrentWorld();
	if (World && World->GetGameInstance())
	{
		if (APlayerController* PlayerController = World->GetGameInstance()->GetFirstLocalPlayerController())
		{
			const bool bCinematicMode = true;
			const bool bHidePlayer = true;
			const bool bHideHUD = true;
			const bool bPreventMovement = true;
			const bool bPreventTurning = true;
			PlayerController->SetCinematicMode(bCinematicMode, bHidePlayer, bHideHUD, bPreventMovement, bPreventTurning);
		}
	}

	// Force the engine into fixed timestep mode. There may be a global delay on the job that passes a fixed
	// number of frames, so we want those frames to always pass the same amount of time for determinism. 
	if (ULevelSequence* LevelSequence = CastChecked<ULevelSequence>(InJob->Sequence.TryLoad()))
	{
		FApp::SetUseFixedTimeStep(true);
		FApp::SetFixedDeltaTime(InJob->GetConfiguration()->GetEffectiveFrameRate(LevelSequence).AsInterval());
	}
}

void UMoviePipelineInProcessExecutor::RestoreState()
{
	if (SavedState.bBackedUp)
	{
		SavedState.bBackedUp = false;
		FApp::SetUseFixedTimeStep(SavedState.bUseFixedTimeStep);
		FApp::SetFixedDeltaTime(SavedState.FixedDeltaTime);

		UWorld* World = MoviePipeline::FindCurrentWorld();
		if (World && World->GetGameInstance())
		{
			if (APlayerController* PlayerController = World->GetGameInstance()->GetFirstLocalPlayerController())
			{
				PlayerController->SetCinematicMode(SavedState.bCinematicMode, SavedState.bHidePlayer, true, true, true);
				PlayerController->ResetIgnoreInputFlags();
			}
		}

		if (SavedState.WindowTitle.IsSet())
		{
			UKismetSystemLibrary::SetWindowTitle(SavedState.WindowTitle.GetValue());
			SavedState.WindowTitle.Reset();
		}
	}
	
	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		if (TSharedPtr<SWindow> GameViewportWindow = GameEngine->GameViewportWindow.Pin())
		{
			GameViewportWindow->SetAcceptsInput(true);
		}
	}
}
void UMoviePipelineInProcessExecutor::CancelAllJobs_Implementation()
{
	// Setting this flag will make the completion of the current job skip the remaining jobs
	bIsCanceling = true;
	CancelCurrentJob();
}

void UMoviePipelineInProcessExecutor::CancelCurrentJob_Implementation()
{
	if (ActiveMoviePipeline && ActiveMoviePipeline->GetPipelineState() != EMovieRenderPipelineState::Finished)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "MoviePipelineInProcessExecutor: Application quit while Movie Pipeline was still active. Stalling to do full shutdown.");

		// This will flush any outstanding work on the movie pipeline (file writes) immediately
		ActiveMoviePipeline->RequestShutdown(); // Set the Shutdown Requested flag.
		ActiveMoviePipeline->Shutdown(); // Flush the shutdown.
	}
}
#undef LOCTEXT_NAMESPACE // "MoviePipelineInProcessExecutor"

