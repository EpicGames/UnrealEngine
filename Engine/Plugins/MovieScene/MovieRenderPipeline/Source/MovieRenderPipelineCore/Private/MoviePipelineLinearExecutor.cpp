// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineLinearExecutor.h"

#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"
#include "MoviePipeline.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelineQueue.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineLinearExecutor)

#define LOCTEXT_NAMESPACE "MoviePipelineLinearExecutorBase"

void UMoviePipelineLinearExecutorBase::Execute_Implementation(UMoviePipelineQueue* InPipelineQueue)
{
	if (!InPipelineQueue)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Executor asked to execute a null queue, not supported!");
		OnExecutorErroredImpl(nullptr, true, LOCTEXT("NullPipelineError", "Executor asked to execute a null queue, not supported!"));
		OnExecutorFinishedImpl();
		return;
	}

	if (InPipelineQueue->GetJobs().Num() == 0)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Executor asked to execute on empty list of pipelines.");
		OnExecutorErroredImpl(nullptr, true, LOCTEXT("EmptyPipelineError", "Executor asked to execute empty list of jobs. This was probably not intended!"));
		OnExecutorFinishedImpl();
		return;
	}

	// We'll process them in linear fashion and wait until each one is canceled or finishes on its own
	// before moving onto the next one. This may be parallelizable in the future (either multiple PIE
	// sessions, or multiple external processes) but ideally one render would maximize resource usage anyways...
	Queue = InPipelineQueue;
	InitializationTime = FDateTime::UtcNow();
	JobsStarted = 0;

	UE_LOGF(LogMovieRenderPipeline, Log, "MoviePipelineLinearExecutorBase starting %d jobs.", InPipelineQueue->GetJobs().Num());

	StartPipelineByIndex(0);
}

void UMoviePipelineLinearExecutorBase::StartPipelineByIndex(int32 InPipelineIndex)
{
	check(InPipelineIndex >= 0 && InPipelineIndex < Queue->GetJobs().Num());
	
	CurrentPipelineIndex = InPipelineIndex;

	if (!Queue->GetJobs()[CurrentPipelineIndex]->IsEnabled())
	{
		OnIndividualPipelineFinished(nullptr);
		return;
	}

	if (Queue->GetJobs()[CurrentPipelineIndex]->IsConsumed())
	{
		// We skip working on consumed jobs. Jobs submitted to remote renders might already be consumed and
		// we don't want to submit them (or override them) so that status updates continue to work.
		OnIndividualPipelineFinished(nullptr);
		return;
	}

	if (!Queue->GetJobs()[CurrentPipelineIndex]->GetConfiguration())
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Found null config in list of configs to render. Aborting the pipeline processing!");
		OnExecutorErroredImpl(nullptr, true, LOCTEXT("NullConfigError", "Found null config in list of configs to render with. Does your config have the wrong outer?"));
		OnExecutorFinishedImpl();
		return;
	}
	
	UE_LOGF(LogMovieRenderPipeline, Log, "MoviePipelineLinearExecutorBase starting job [%d/%d]", CurrentPipelineIndex + 1, Queue->GetJobs().Num());
	Start(Queue->GetJobs()[CurrentPipelineIndex]);
	++JobsStarted;
}

void UMoviePipelineLinearExecutorBase::OnIndividualPipelineFinished(UMoviePipeline* /* FinishedPipeline */)
{
	const bool bNoMoreJobs = CurrentPipelineIndex >= Queue->GetJobs().Num() - 1;
	if (bNoMoreJobs || bIsCanceling)
	{
		OnExecutorFinishedImpl();
	}
	else
	{
		// Onto the next one!
		StartPipelineByIndex(CurrentPipelineIndex + 1);
	}
}

void UMoviePipelineLinearExecutorBase::OnPipelineErrored(UMoviePipeline* InPipeline, bool bIsFatal, FText ErrorText)
{
	OnExecutorErroredImpl(InPipeline, bIsFatal, ErrorText);
}

void UMoviePipelineLinearExecutorBase::OnExecutorFinishedImpl()
{
	UE_LOGF(LogMovieRenderPipeline, Log, "MoviePipelineLinearExecutorBase finished %d jobs in %ls.", JobsStarted, *(FDateTime::UtcNow() - InitializationTime).ToString());

	// Only say that we're no longer rendering once we've finished all jobs in the executor so the UI doesn't flicker while switching over between jobs.
	bIsRendering = false;
	Super::OnExecutorFinishedImpl();
}

FText UMoviePipelineLinearExecutorBase::GetWindowTitle()
{
	FNumberFormattingOptions PercentFormatOptions;
	PercentFormatOptions.MinimumIntegralDigits = 1;
	PercentFormatOptions.MaximumIntegralDigits = 3;
	PercentFormatOptions.MaximumFractionalDigits = 0;
	PercentFormatOptions.RoundingMode = ERoundingMode::HalfFromZero;

	const float CompletionPercentage = GetCompletionPercentageFromActivePipeline() * 100.f;

	const FText TitleFormatString = LOCTEXT("MoviePreviewWindowTitleFormat", "Movie Pipeline Render (Preview) [Job {CurrentCount}/{TotalCount} Total] Current Job: {PercentComplete}% Completed.");
	return FText::FormatNamed(TitleFormatString, TEXT("CurrentCount"), FText::AsNumber(CurrentPipelineIndex + 1), TEXT("TotalCount"), FText::AsNumber(Queue->GetJobs().Num()), TEXT("PercentComplete"), FText::AsNumber(CompletionPercentage, &PercentFormatOptions));
}

float UMoviePipelineLinearExecutorBase::GetCompletionPercentageFromActivePipeline()
{
	float CompletionPercentage = 0.f;
	if (const UMoviePipeline* AsLegacyPipeline = Cast<UMoviePipeline>(ActiveMoviePipeline))
	{
		CompletionPercentage = UMoviePipelineBlueprintLibrary::GetCompletionPercentage(AsLegacyPipeline);
	}
	else if (const UMovieGraphPipeline* AsGraphPipeline = Cast<UMovieGraphPipeline>(ActiveMoviePipeline))
	{
		CompletionPercentage = UMovieGraphBlueprintLibrary::GetCompletionPercentage(AsGraphPipeline);
	}

	return CompletionPercentage;
}


void UMoviePipelineLinearExecutorBase::CancelCurrentJob_Implementation()
{
	OnExecutorErroredImpl(nullptr, true, LOCTEXT("CancelingExecutorJob", "Executor canceling current job."));
	if (ActiveMoviePipeline)
	{
		ActiveMoviePipeline->Shutdown(true);
	}	
}

void UMoviePipelineLinearExecutorBase::CancelAllJobs_Implementation()
{
	bIsCanceling = true;
	OnExecutorErroredImpl(nullptr, true, LOCTEXT("CancelingExecutorQueue", "Executor asked to cancel all jobs."));
	if (ActiveMoviePipeline)
	{
		ActiveMoviePipeline->Shutdown(true);
	}
}


#undef LOCTEXT_NAMESPACE // "MoviePipelineLinearExecutorBase"

