// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineOutputBuilder.h"
#include "MoviePipeline.h"
#include "Async/Async.h"


// When the Movie Pipeline Render goes to produce a frame, it will push an expected output frame into
// this list. This gets handed all of the component parts of that frame in an arbitrary order/threads
// so we just wait and gather everything that comes in until a frame is complete and then we can kick
// it off to the output step. Because there's potentially a long delay between asking a frame to be
// produced, and it actually being produced, this is responsible for notifying the output containers
// about shot changes, etc. - This has to be pushed back to the Game Thread though because those want to
// live in UObject land for settings.

FMoviePipelineMergerOutputFrame& FMoviePipelineOutputMerger::QueueOutputFrame_GameThread(const FMoviePipelineFrameOutputState& CachedOutputState)
{
	// Lock the ActiveData while we get a new output frame.
	FScopeLock ScopeLock(&ActiveDataMutex);

	// Ensure this frame hasn't already been entered somehow.
	check(!PendingData.Find(CachedOutputState));

	FMoviePipelineMergerOutputFrame& NewFrame = PendingData.Add(CachedOutputState);
	NewFrame.FrameOutputState = CachedOutputState;

	return NewFrame;
}

void FMoviePipelineOutputMerger::OnSingleSampleDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData)
{
	// This is to support outputting individual samples (skipping accumulation) for debug reasons,
	// or because you want to post-process them yourself. We just forward this directly on for output to disk.

	TWeakObjectPtr<UMoviePipeline> LocalWeakPipeline = WeakMoviePipeline;

	AsyncTask(ENamedThreads::GameThread, [LocalData = MoveTemp(InData), LocalWeakPipeline]() mutable
	{
		if (ensureAlwaysMsgf(LocalWeakPipeline.IsValid(), TEXT("A memory lifespan issue has left an output builder alive without an owning Movie Pipeline.")))
		{
			LocalWeakPipeline->OnSampleRendered(MoveTemp(LocalData));
		}
	}
	);
}

void FMoviePipelineOutputMerger::OnCompleteRenderPassDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData)
{
	// Lock the ActiveData when we're updating what data has been gathered.
	FScopeLock ScopeLock(&ActiveDataMutex);

	// See if we can find the frame this data is for. This should always be valid, if it's not
	// valid it means they either forgot to declare they were going to produce it, or this is
	// coming in after the system already thinks it's finished that frame.
	FMoviePipelineMergerOutputFrame* OutputFrame = nullptr;
	
	// Instead of just finding the result in the TMap with the equality operator, we find it by hand so that we can
	// ignore certain parts of equality (such as Temporal Sample, as the last sample has a temporal index different
	// than the first sample!)
	FImagePixelDataPayload* Payload = InData->GetPayload<FImagePixelDataPayload>();

	// If you're hitting this check then your FImagePixelData was not initialized with a payload with the  required data.
	check(Payload);

	for (TPair<FMoviePipelineFrameOutputState, FMoviePipelineMergerOutputFrame>& KVP : PendingData)
	{
		if (KVP.Key.OutputFrameNumber == Payload->SampleState.OutputState.OutputFrameNumber)
	 	{
	 		OutputFrame = &KVP.Value;
	 		break;
	 	}
	}
	if (!ensureAlwaysMsgf(OutputFrame, TEXT("Received data for unknown frame. Frame was either already processed or not queued yet!")))
	{
		return;
	}

	// Ensure this pass is expected as well...
	if (!ensureAlwaysMsgf(OutputFrame->ExpectedRenderPasses.Contains(Payload->PassIdentifier), TEXT("Received data for unexpected render pass: %s"), *Payload->PassIdentifier.Name))
	{
		return;
	}

	// Merge the metadata from each output state. Metadata is part of the output state but gets forked when
	// we submit different render passes, so we need to merge it again. Doesn't handle conflicts.
	for (const TPair<FString, FString>& KVP : Payload->SampleState.OutputState.FileMetadata)
	{
		OutputFrame->FrameOutputState.FileMetadata.Add(KVP.Key, KVP.Value);
	}

	// If this data was expected and this frame is still in progress, pass the data to the frame.
	OutputFrame->ImageOutputData.FindOrAdd(Payload->PassIdentifier) = MoveTemp(InData);
	
	// Check to see if this was the last piece of data needed for this frame.
	const int32 TotalPasses = OutputFrame->ExpectedRenderPasses.Num();
	const int32 SucceededPasses = OutputFrame->ImageOutputData.Num();

	if (SucceededPasses == TotalPasses)
	{
		// Sort the output frames. This is only really important for multi-channel formats like EXR, but it lets passes
		// specify which one should be the thumbnail/default rgba channels instead of a first-come-first-serve.
		OutputFrame->ImageOutputData.ValueStableSort([](const TUniquePtr<FImagePixelData>& First, const TUniquePtr<FImagePixelData>& Second) -> bool
				{
					FImagePixelDataPayload* FirstPayload = First->GetPayload<FImagePixelDataPayload>();
					FImagePixelDataPayload* SecondPayload = Second->GetPayload<FImagePixelDataPayload>();

					return FirstPayload->SortingOrder < SecondPayload->SortingOrder;
				}
		);
	}

	// Frames can arrive out-of-order in some cases. We should process them in order though, which is especially important for movies (since
	// those write to one file, vs. many individual files for image sequences, so it's important to provide movie writers the frames in order).
	// Note: PendingData is kept sorted by frame number.
	TArray<FMoviePipelineFrameOutputState> PendingFrameKeys;
	PendingData.GetKeys(PendingFrameKeys);
	for (const FMoviePipelineFrameOutputState& PendingFrameKey : PendingFrameKeys)
	{
		const FMoviePipelineMergerOutputFrame& OutputMergerFrame = PendingData[PendingFrameKey];
		const bool bFrameIsComplete = OutputMergerFrame.ExpectedRenderPasses.Num() == OutputMergerFrame.ImageOutputData.Num();
		if (!bFrameIsComplete)
		{
			// This frame hasn't finished yet, so delay pushing it (and any other out-of-order frames) to FinishedFrames until it's finished.
			break;
		}

		// Transfer ownership from the map to here; It's important that we use the manually looked up OutputFrame from PendingData
		// as PendingData uses the equality operator. Some combinations of temporal sampling + slowmo tracks results in different
		// original source frame numbers, which would cause the tmap lookup to fail and thus returning an empty frame.
		FMoviePipelineMergerOutputFrame FinalFrame;
		ensureMsgf(PendingData.RemoveAndCopyValue(PendingFrameKey, FinalFrame), TEXT("Could not find frame in pending data, output will be skipped!"));
		FinishedFrames.Enqueue(MoveTemp(FinalFrame));
	}
}

void FMoviePipelineOutputMerger::AbandonOutstandingWork()
{
	FScopeLock ScopeLock(&ActiveDataMutex);
	FinishedFrames.Empty();
	PendingData.Empty();
}
