// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineOutputSetting.h"
#include "Misc/Paths.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineQueue.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipeline.h"
#include "MoviePipelineTelemetry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineOutputSetting)

UMoviePipelineOutputSetting::UMoviePipelineOutputSetting()
	: OutputResolution(FIntPoint(1920, 1080))
	, bUseCustomFrameRate(false)
	, OutputFrameRate(FFrameRate(24, 1))
	, DEBUG_OutputFrameStepOffset(0)
	, bOverrideExistingOutput(true)
	, HandleFrameCount(0)
	, OutputFrameStep(1)
	, bUseCustomPlaybackRange(false)
	, CustomStartFrame(0)
	, CustomEndFrame(0)
	, VersionNumber(1)
	, bAutoVersion(true)
	, ZeroPadFrameNumbers(4)
	, FrameNumberOffset(0)
	, bFlushDiskWritesPerShot(false)
{
	FileNameFormat = TEXT("{sequence_name}.{frame_number}");
	OutputDirectory.Path = TEXT("{project_dir}/Saved/MovieRenders/");
}

void UMoviePipelineOutputSetting::PostLoad()
{
	Super::PostLoad();

	// In order to ship presets that work with any project, we can't use a relative path because it is
	// relative to the executable and thus has the project name embedded in. To solve this we will save
	// an empty string into the Output Directory and convert it to their relative directory in Post Load.
	// This isn't done in the CDO so that resetting to default value works as expected.
	if (OutputDirectory.Path.Len() == 0)
	{
		OutputDirectory.Path = TEXT("{project_dir}/Saved/MovieRenders/");
	}
}

#if WITH_EDITOR
FText UMoviePipelineOutputSetting::GetFooterText(UMoviePipelineExecutorJob* InJob) const 
{
	FTextBuilder TextBuilder;
	TextBuilder.AppendLine(NSLOCTEXT("MovieRenderPipeline", "OutputSettingFooterText_Fmt", "A list of {format_strings} and example values that are valid to use in the File Name Format:\n"));

	FMoviePipelineFormatArgs FormatArgs;
	FormatArgs.InJob = InJob;
	
	// Find the primary configuration that owns us
	UMoviePipelinePrimaryConfig* PrimaryConfig = GetTypedOuter<UMoviePipelinePrimaryConfig>();
	if (PrimaryConfig)
	{
		PrimaryConfig->GetFormatArguments(FormatArgs);
	}

	for (const TPair<FString, FString>& KVP : FormatArgs.FilenameArguments)
	{
		FStringFormatOrderedArguments OrderedArgs = { KVP.Key, KVP.Value };
		FString FormattedArgs = FString::Format(TEXT("{0} => {1}"), OrderedArgs);

		TextBuilder.AppendLine(FText::FromString(FormattedArgs));
	}
	
	return TextBuilder.ToText();
}
#endif

void UMoviePipelineOutputSetting::GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const
{
	const FString ResolvedProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
	InOutFormatArgs.FilenameArguments.Add(TEXT("project_dir"), ResolvedProjectDir);
	InOutFormatArgs.FileMetadata.Add(TEXT("unreal/project_dir"), ResolvedProjectDir);

	// Resolution Arguments
	{
		FString Resolution = FString::Printf(TEXT("%d_%d"), OutputResolution.X, OutputResolution.Y);
		InOutFormatArgs.FilenameArguments.Add(TEXT("output_resolution"), Resolution);
		InOutFormatArgs.FilenameArguments.Add(TEXT("output_width"), FString::FromInt(OutputResolution.X));
		InOutFormatArgs.FilenameArguments.Add(TEXT("output_height"), FString::FromInt(OutputResolution.Y));
	}

	if (bAutoVersion)
	{
		InOutFormatArgs.FilenameArguments.Add(TEXT("version"), TEXT("v00x"));
	}
	else
	{
		FString VersionText = FString::Printf(TEXT("v%0*d"), 3, VersionNumber);
		InOutFormatArgs.FilenameArguments.Add(TEXT("version"), VersionText);
	}
}

void UMoviePipelineOutputSetting::SetupForPipelineImpl(UMoviePipeline* InPipeline)
{
	if (InPipeline)
	{
		InPipeline->SetFlushDiskWritesPerShot(InPipeline->IsFlushDiskWritesPerShot() || bFlushDiskWritesPerShot);
	}
}

void UMoviePipelineOutputSetting::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->HandleFrameCount = FMath::Max(InTelemetry->HandleFrameCount, HandleFrameCount);
}
