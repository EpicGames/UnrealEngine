// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelinePostRenderSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelinePostRenderSettings)

FMovieGraphPostRenderVideoPlayOptions::FMovieGraphPostRenderVideoPlayOptions()
	: PlaybackMethod(EMovieGraphPlaybackMethod::OperatingSystem)
	, JobPlayback(EMovieGraphJobPlaybackRange::FirstJobOnly)
	, RenderLayerPlayback(EMovieGraphRenderLayerPlaybackRange::FirstRenderLayerOnly)
{
	
}

FMovieGraphPostRenderImageSequencePlayOptions::FMovieGraphPostRenderImageSequencePlayOptions()
	: FrameRangeNotation(EMovieGraphFrameRangeNotation::HashWithStartEndFrame)
	, PlaybackRange(EMovieGraphImageSequencePlaybackRange::FirstFrameOnly)
{
	
}

FMovieGraphPostRenderSettings::FMovieGraphPostRenderSettings()
	: OutputTypePriorityOrder({TEXT("EXR"), TEXT("PNG"), TEXT("JPEG"), TEXT("MP4"), TEXT("BMP"), TEXT("MOV"), TEXT("MXF")})
	, OutputTypePlayback(EMovieGraphOutputTypePlayback::UsePriorityOrder)
	, DirectoryOpenBehavior(EMovieGraphDirectoryOpenBehavior::OpenDirectoryForAllJobs)
{
	
}
