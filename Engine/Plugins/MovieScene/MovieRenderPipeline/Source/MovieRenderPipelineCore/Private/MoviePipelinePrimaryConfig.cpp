// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelinePrimaryConfig.h"
#include "MovieScene.h"
#include "LevelSequence.h"
#include "MoviePipelineShotConfig.h"
#include "MoviePipelineOutputBase.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineUtils.h"
#include "MoviePipelineSetting.h"
#include "MovieRenderPipelineCoreModule.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelinePrimaryConfig)

#define LOCTEXT_NAMESPACE "MoviePipelinePrimaryConfig"


void UMoviePipelinePrimaryConfig::AddTransientSettingByClass(const UClass* InSettingClass)
{
	// We do this directly because the FindOrAddSetting API adds them to the user-defined settings array.
	UMoviePipelineSetting* NewSetting = NewObject<UMoviePipelineSetting>(this, InSettingClass);

	// Don't add settings that don't belong on this type of config
	if (CanSettingBeAdded(NewSetting))
	{
		TransientSettings.Add(NewSetting);
	}
}

void UMoviePipelinePrimaryConfig::InitializeTransientSettings()
{
	if (TransientSettings.Num() > 0)
	{
		return;
	}

	// Reflect over all possible settings that could be added to this class.
	TArray<UClass*> AllSettingClasses = UE::MovieRenderPipeline::FindMoviePipelineSettingClasses(UMoviePipelineSetting::StaticClass());

	// Now remove any from the list that we already have a user setting for
	for (const UMoviePipelineSetting* ExistingSetting : GetUserSettings())
	{
		if (!ExistingSetting)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Null setting found in config: %ls - Did you disable a plugin that contained this setting?", *GetNameSafe(this));
			continue;
		}

		AllSettingClasses.RemoveSwap(ExistingSetting->GetClass());
	}

	// Now we can initialize an instance of every remaining setting that we don't have.
	for (const UClass* SettingClass : AllSettingClasses)
	{
		AddTransientSettingByClass(SettingClass);
	}
}

void UMoviePipelinePrimaryConfig::OnSettingAdded(UMoviePipelineSetting* InSetting) 
{
	Super::OnSettingAdded(InSetting);

	// When the setting is added, we need to remove the class type from the transient array (if any).
	for (UMoviePipelineSetting* TransientSetting : TransientSettings)
	{
		if (TransientSetting->GetClass() == InSetting->GetClass())
		{
			TransientSettings.Remove(TransientSetting);
			break;
		}
	}
}

void UMoviePipelinePrimaryConfig::OnSettingRemoved(UMoviePipelineSetting* InSetting)
{
	Super::OnSettingRemoved(InSetting);

	// Only add to the transient settings array if there's something already in it,
	// otherwise we could miss transient settings when InitializeTransientSettings
	// is called later, since it early outs if there's any transient settings.
	if (TransientSettings.Num() > 0)
	{
		AddTransientSettingByClass(InSetting->GetClass());
	}
}


UMoviePipelineShotConfig* UMoviePipelinePrimaryConfig::GetConfigForShot(const FString& ShotName) const
{
	UMoviePipelineShotConfig* OutConfig = PerShotConfigMapping.FindRef(ShotName);

	return OutConfig;
}

void UMoviePipelinePrimaryConfig::GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs, const bool bIncludeAllSettings) const
{
	// Add "global" ones not specific to a setting.
	{
		FString LevelName = TEXT("Level Name");
		FString SequenceName = TEXT("Sequence Name");
		FString JobName = TEXT("Job Name");
		FString JobComment = TEXT("Job Comment");
		double FrameRate = 0.0;

		if (InOutFormatArgs.InJob)
		{
			LevelName = InOutFormatArgs.InJob->Map.GetAssetName();
			SequenceName = InOutFormatArgs.InJob->Sequence.GetAssetName();
			JobName = InOutFormatArgs.InJob->JobName;
			JobComment = InOutFormatArgs.InJob->Comment;

			// FrameRate is a combination of Output Settings and Sequence so we do it here instead of in OutputSetting
			FrameRate = GetEffectiveFrameRate(Cast<ULevelSequence>(InOutFormatArgs.InJob->Sequence.TryLoad())).AsDecimal();
		}


		InOutFormatArgs.FilenameArguments.Add(TEXT("level_name"), LevelName);
		InOutFormatArgs.FilenameArguments.Add(TEXT("sequence_name"), SequenceName);
		InOutFormatArgs.FilenameArguments.Add(TEXT("job_name"), JobName);
		InOutFormatArgs.FilenameArguments.Add(TEXT("frame_rate"), FString::SanitizeFloat(FrameRate));

		InOutFormatArgs.FileMetadata.Add(TEXT("unreal/levelName"), LevelName);
		InOutFormatArgs.FileMetadata.Add(TEXT("unreal/sequenceName"), SequenceName);
		InOutFormatArgs.FileMetadata.Add(TEXT("unreal/jobName"), JobName);
		InOutFormatArgs.FileMetadata.Add(TEXT("unreal/jobComment"), JobComment);
		InOutFormatArgs.FileMetadata.Add(TEXT("unreal/frameRate"), FString::SanitizeFloat(FrameRate));

		// Normally these are filled when resolving the file name by the job (so that the time is shared), but stub them in here so
		// they show up in the UI with a value.
		FDateTime CurrentTime = FDateTime::UtcNow();
		int32 DummyVersionNumber = 1;
		const FTimespan InitializationTimeOffset = FDateTime::Now() - FDateTime::UtcNow();
		UE::MoviePipeline::GetSharedFormatArguments(InOutFormatArgs.FilenameArguments, InOutFormatArgs.FileMetadata, CurrentTime, DummyVersionNumber, InOutFormatArgs.InJob, InitializationTimeOffset);

		// Let the output state fill out some too, since its the keeper of the information.
		UMoviePipelineOutputSetting* OutputSettings = FindSetting<UMoviePipelineOutputSetting>();
		check(OutputSettings);

		// We use the FrameNumberOffset as the number for all of these so they can see it changing in the UI.
		FString FramePlaceholderNumber = FString::Printf(TEXT("%0*d"), OutputSettings->ZeroPadFrameNumbers, OutputSetting->FrameNumberOffset);
		MoviePipeline::GetOutputStateFormatArgs(InOutFormatArgs.FilenameArguments, InOutFormatArgs.FileMetadata, FramePlaceholderNumber, FramePlaceholderNumber, FramePlaceholderNumber, FramePlaceholderNumber, TEXT("CameraName"), TEXT("ShotName"));
	}

	// Let each setting provide its own set of key/value pairs.
	{
		// The UI will only show user customized settings but actually writing to disk should use all.
		const bool bIncludeDisabledSettings = false;
		const bool bIncludeTransientSettings = bIncludeAllSettings;
		TArray<UMoviePipelineSetting*> TargetSettings = GetAllSettings( bIncludeDisabledSettings, bIncludeTransientSettings);

		for (UMoviePipelineSetting* Setting : TargetSettings)
		{
			Setting->GetFormatArguments(InOutFormatArgs);
		}
	}
}

FFrameRate UMoviePipelinePrimaryConfig::GetEffectiveFrameRate(const ULevelSequence* InSequence) const
{
	UMoviePipelineOutputSetting* OutputSettings = FindSetting<UMoviePipelineOutputSetting>();
	check(OutputSettings);

	// Check to see if they overrode the frame rate.
	if (OutputSettings->bUseCustomFrameRate)
	{
		return OutputSettings->OutputFrameRate;
	}

	// Pull it from the sequence if they didn't.
	if (InSequence)
	{
		return InSequence->GetMovieScene()->GetDisplayRate();
	}

	return FFrameRate();
}


TRange<FFrameNumber> UMoviePipelinePrimaryConfig::GetEffectivePlaybackRange(const ULevelSequence* InSequence) const
{
	UMoviePipelineOutputSetting* OutputSettings = FindSetting<UMoviePipelineOutputSetting>();
	check(OutputSettings);
	check(InSequence);

	// Check to see if they overrode the frame rate.
	if (OutputSettings->bUseCustomPlaybackRange)
	{
		// Convert the custom playback range from frames to ticks.
		FFrameNumber StartTick = FFrameRate::TransformTime(FFrameTime(FFrameNumber(OutputSettings->CustomStartFrame)), InSequence->GetMovieScene()->GetDisplayRate(), InSequence->GetMovieScene()->GetTickResolution()).FloorToFrame();
		FFrameNumber EndTick = FFrameRate::TransformTime(FFrameTime(FFrameNumber(OutputSettings->CustomEndFrame)), InSequence->GetMovieScene()->GetDisplayRate(), InSequence->GetMovieScene()->GetTickResolution()).FloorToFrame();

		// [Inclusive, Exclusive)
		return TRange<FFrameNumber>(StartTick, EndTick);
	}

	// Pull it from the sequence if they didn't.
	return InSequence->GetMovieScene()->GetPlaybackRange();
}

UMoviePipelinePrimaryConfig::UMoviePipelinePrimaryConfig()
{
	// Always add at least the output settings block since having a framerate/directory/etc. is critical.
	OutputSetting = CreateDefaultSubobject<UMoviePipelineOutputSetting>("DefaultOutputSetting");
}

TArray<UMoviePipelineSetting*> UMoviePipelinePrimaryConfig::GetUserSettings() const
{
	TArray<UMoviePipelineSetting*> BaseSettings = Super::GetUserSettings();
	BaseSettings.Add(OutputSetting);

	return BaseSettings;
}

TArray<UMoviePipelineSetting*> UMoviePipelinePrimaryConfig::GetAllSettings(const bool bIncludeDisabledSettings, const bool bIncludeTransientSettings) const
{
	TArray<UMoviePipelineSetting*> OutSettings;

	// User settings were explicitly added by the user via the UI or scripting.
	for (UMoviePipelineSetting* Setting : GetUserSettings())
	{
		if (!Setting)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Null setting found in config: %ls - Did you disable a plugin that contained this setting?", *GetNameSafe(this));
			continue;
		}

		if (bIncludeDisabledSettings || Setting->IsEnabled())
		{
			OutSettings.Add(Setting);
		}
	}

	// Transient settings are initialized when we're about to render so that classes
	// that the user didn't add to the UI still have a chance to apply sane default settings
	for (UMoviePipelineSetting* Setting : GetTransientSettings())
	{
		if (bIncludeTransientSettings || Setting->IgnoreTransientFilters())
		{
			if (bIncludeDisabledSettings || Setting->IsEnabled())
			{
				OutSettings.Add(Setting);
			}
		}
	}

	return OutSettings;
}

void UMoviePipelinePrimaryConfig::CopyFrom(UMoviePipelineConfigBase* InConfig)
{
	Super::CopyFrom(InConfig);

	if (InConfig == this)
	{
		return;
	}

	if (InConfig->IsA<UMoviePipelinePrimaryConfig>())
	{
		UMoviePipelinePrimaryConfig* PrimaryConfig = CastChecked<UMoviePipelinePrimaryConfig>(InConfig);

		// Rename our current default subobject so we can duplicate the incoming config ontop the existing
		// name which is required for it to be loaded correctly.
		if (UMoviePipelineOutputSetting* ExistingObj = FindObject<UMoviePipelineOutputSetting>(this, TEXT("DefaultOutputSetting")))
		{

			FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), UMoviePipelinePrimaryConfig::StaticClass(), "DEAD_MoviePipelineConfig_DefaultOutputSetting");
			ExistingObj->Rename(*UniqueName.ToString(), GetTransientPackage(), REN_DontCreateRedirectors);
		}

		OutputSetting = Cast<UMoviePipelineOutputSetting>(StaticDuplicateObject(PrimaryConfig->OutputSetting, this, FName("DefaultOutputSetting")));
	}
}


TArray<UMoviePipelineOutputBase*> UMoviePipelinePrimaryConfig::GetOutputContainers() const
{
	TArray<UMoviePipelineOutputBase*> OutputContainers;

	// Don't want transient settings trying to write out files 
	for (UMoviePipelineSetting* Setting : FindSettings<UMoviePipelineOutputBase>())
	{
		UMoviePipelineOutputBase* Output = Cast<UMoviePipelineOutputBase>(Setting);
		if (Output)
		{
			OutputContainers.Add(Output);
		}
	}

	return OutputContainers;
}

#undef LOCTEXT_NAMESPACE // "MovieRenderPipelineConfig"

