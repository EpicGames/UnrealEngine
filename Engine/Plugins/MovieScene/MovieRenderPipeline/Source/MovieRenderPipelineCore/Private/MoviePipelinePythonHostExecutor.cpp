// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelinePythonHostExecutor.h"
#include "Misc/CoreDelegates.h"
#include "SocketSubsystem.h"
#include "Sockets.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelinePythonHostExecutor)

void UMoviePipelinePythonHostExecutor::Execute_Implementation(UMoviePipelineQueue* InPipelineQueue)
{
	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (WorldContext.WorldType == EWorldType::Game)
		{
			LastLoadedWorld = WorldContext.World();
		}
	}

	PipelineQueue = InPipelineQueue;
	
	// Register C++ only callbacks that we will forward onto BP/Python
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddUObject(this, &UMoviePipelinePythonHostExecutor::OnMapLoadFinished);

	// Now that we've done some C++ only things, call the Python version of this.
	ExecuteDelayed(InPipelineQueue);
}

void UMoviePipelinePythonHostExecutor::CancelAllJobs_Implementation()
{
	UE_LOGF(LogMovieRenderPipeline, Error, "Attempting to cancel jobs on a UMoviePipelinePythonHostExecutor with a python executor that has not implemented canceling jobs.");
}

void UMoviePipelinePythonHostExecutor::OnMapLoadFinished(UWorld* NewWorld)
{
	LastLoadedWorld = NewWorld;

	// This executor is only created after the world is loaded
	OnMapLoad(NewWorld);
}
