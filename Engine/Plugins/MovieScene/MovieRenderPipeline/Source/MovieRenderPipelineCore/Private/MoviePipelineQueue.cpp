// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineQueue.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineSetting.h"
#include "LevelSequence.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieRenderPipelineCoreObjectVersion.h"

#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineQueue)

UMoviePipelineQueue::UMoviePipelineQueue()
	: QueueSerialNumber(0)
	, bIsDirty(false)
{
	// Ensure instances are always transactional
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		SetFlags(RF_Transactional);
	}

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectModified.AddUObject(this, &UMoviePipelineQueue::OnAnyObjectModified);
#endif
}

UMoviePipelineExecutorJob* UMoviePipelineQueue::AllocateNewJob(TSubclassOf<UMoviePipelineExecutorJob> InJobType)
{
	if (!ensureAlwaysMsgf(InJobType, TEXT("Failed to specify a Job Type. Use the default in project setting or UMoviePipelineExecutorJob.")))
	{
		InJobType = UMoviePipelineExecutorJob::StaticClass();
	}

#if WITH_EDITOR
	Modify();
#endif

	UMoviePipelineExecutorJob* NewJob = NewObject<UMoviePipelineExecutorJob>(this, InJobType);
	NewJob->SetFlags(RF_Transactional);

	Jobs.Add(NewJob);
	QueueSerialNumber++;

	return NewJob;
}

void UMoviePipelineQueue::DeleteJob(UMoviePipelineExecutorJob* InJob)
{
	if (!InJob)
	{
		return;
	}

#if WITH_EDITOR
	Modify();
#endif

	Jobs.Remove(InJob);
	QueueSerialNumber++;
}

void UMoviePipelineQueue::DeleteAllJobs()
{
#if WITH_EDITOR
	Modify();
#endif

	Jobs.Empty();
	QueueSerialNumber++;
}

UMoviePipelineExecutorJob* UMoviePipelineQueue::DuplicateJob(UMoviePipelineExecutorJob* InJob)
{
	if (!InJob)
	{
		return nullptr;
	}

#if WITH_EDITOR
	Modify();
#endif

	UMoviePipelineExecutorJob* NewJob = CastChecked<UMoviePipelineExecutorJob>(StaticDuplicateObject(InJob, this));

	// Duplication causes the DisplayNames to get renamed to the asset name (required to support both duplicating
	// and renaming objects in the Content Browser). So after using the normal duplication, we run our custom
	// CopyFrom code to transfer the configurations over (which do a display name fixup).
	NewJob->GetConfiguration()->CopyFrom(InJob->GetConfiguration());
	for (int32 Index = 0; Index < NewJob->ShotInfo.Num(); Index++)
	{
		if (InJob->ShotInfo[Index]->GetShotOverrideConfiguration())
		{
			NewJob->ShotInfo[Index]->GetShotOverrideConfiguration()->CopyFrom(InJob->ShotInfo[Index]->GetShotOverrideConfiguration());
		}
	}

	NewJob->OnDuplicated();
	Jobs.Add(NewJob);

	QueueSerialNumber++;
	return NewJob;
}

UMoviePipelineQueue* UMoviePipelineQueue::CopyFrom(UMoviePipelineQueue* InQueue)
{
	if (!InQueue)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Cannot copy the contents of a null queue.");
		return nullptr;
	}

	// The copy should reflect the input queue's origin (ie, the queue asset it was originally based off of). Setting
	// the origin to the input queue would change the meaning of what the origin is.
	QueueOrigin = InQueue->QueueOrigin;

#if WITH_EDITOR
	Modify();
#endif

	Jobs.Empty();
	for (UMoviePipelineExecutorJob* Job : InQueue->GetJobs())
	{
		DuplicateJob(Job);
	}

	// Ensure the serial number gets bumped at least once so the UI refreshes in case
	// the queue we are copying from was empty.
	QueueSerialNumber++;
	return this;
}

void UMoviePipelineQueue::SetJobIndex(UMoviePipelineExecutorJob* InJob, int32 Index)
{
#if WITH_EDITOR
	Modify();
#endif

	int32 CurrentIndex = INDEX_NONE;
	Jobs.Find(InJob, CurrentIndex);

	if (CurrentIndex == INDEX_NONE)
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Cannot find Job %s in queue"), *InJob->JobName), ELogVerbosity::Error);
		return;
	}

	if (Index > CurrentIndex)
	{
		--Index;
	}

	Jobs.Remove(InJob);
	Jobs.Insert(InJob, Index);

	// Ensure the serial number gets bumped at least once so the UI refreshes in case
	// the queue we are copying from was empty.
	QueueSerialNumber++;
}

void UMoviePipelineQueue::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Reset dirty state after saving
	bIsDirty = false;
}

#if WITH_EDITOR
void UMoviePipelineQueue::OnAnyObjectModified(UObject* InModifiedObject)
{
	// Mark as dirty if this queue or any of its owned objects have been modified
	if (InModifiedObject)
	{
		if (InModifiedObject->IsIn(this) || (InModifiedObject == this))
		{
			bIsDirty = true;
		}
	}
}

void UMoviePipelineExecutorJob::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMoviePipelineExecutorJob, Sequence))
	{
		// Call our Set function so that we rebuild the shot mask.
		SetSequence(Sequence);
	}

	// We save the config on this object after each property change. This makes the variables flagged as config
	// save even though we're editing them through a normal details panel. This is a nicer user experience for
	// fields that don't change often but do need to be per job.
	SaveConfig();
}
#endif

void UMoviePipelineExecutorJob::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FMovieRenderPipelineCoreObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FMovieRenderPipelineCoreObjectVersion::GUID) < FMovieRenderPipelineCoreObjectVersion::JobConfigModeAdded)
	{
		// Derive ConfigMode from pointer state for assets saved before the enum was introduced.
		if (BasicConfig != nullptr)
		{
			ConfigMode = EMoviePipelineConfigMode::Basic;
		}
		else if (!GraphPreset.IsNull())
		{
			ConfigMode = EMoviePipelineConfigMode::Graph;
		}
		else
		{
			ConfigMode = EMoviePipelineConfigMode::Preset;
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FMovieRenderPipelineCoreObjectVersion::GUID) < FMovieRenderPipelineCoreObjectVersion::JobInitializedLegacyDefaultsTrackingAdded)
	{
		// Backfill bHasInitializedLegacyPresetDefaults for jobs saved before this flag existed. A job is
		// assumed to have had its legacy Configuration populated if this is the case.
		bHasInitializedLegacyPresetDefaults = true;
	}
}

void UMoviePipelineExecutorJob::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	RefreshAllVariableAssignments();
}

void UMoviePipelineExecutorJob::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Listen for any graph/subgraph saves. It's difficult/error-prone to track which graphs to monitor because the subgraph hierarchy can
	// constantly change. Instead, just run any logic needed when any graph is saved.
	FCoreUObjectDelegates::OnObjectPreSave.AddUObject(this, &UMoviePipelineExecutorJob::OnGraphPreSave);
#endif

	// Validate that Graph mode actually has a resolvable graph. The Serialize migration sets ConfigMode
	// based on !GraphPreset.IsNull() (non-empty path), but the referenced asset may have been deleted.
	if (ConfigMode == EMoviePipelineConfigMode::Graph && GetGraphPreset() == nullptr)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning,
			"Job '%ls' was in Graph mode but the graph preset could not be loaded. Falling back to Preset mode.",
			*GetName());
		ConfigMode = EMoviePipelineConfigMode::Preset;
	}

	// Ideally the variable assignments would be updated here, but currently GraphPreset.LoadSynchronous() does not provide a fully-loaded
	// object (GraphPreset will only be partially loaded at this point and its variables will be empty).
}

void UMoviePipelineExecutorJob::BeginDestroy()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPreSave.RemoveAll(this);
#endif

	Super::BeginDestroy();
}

TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& UMoviePipelineExecutorJob::GetGraphVariableAssignments(const bool bUpdateAssignments)
{
	if (bUpdateAssignments)
	{
		RefreshAllVariableAssignments();
	}
	
	return GraphVariableAssignments;
}

void UMoviePipelineExecutorJob::RefreshAllVariableAssignments()
{
	MoviePipeline::RefreshVariableAssignments(GraphPreset.LoadSynchronous(), GraphVariableAssignments, this);

	// Notify all shots that they should refresh their assignments as well (this could be needed, for example, when the primary graph preset changes)
	for (TObjectPtr<UMoviePipelineExecutorShot>& ShotJob : ShotInfo)
	{
		if (ShotJob)
		{
			ShotJob->RefreshAllVariableAssignments();
		}
	}
}

void UMoviePipelineExecutorJob::SetGraphPreset(const UMovieGraphConfig* InGraphPreset, const bool bUpdateVariableAssignments)
{
#if WITH_EDITOR
	Modify();
#endif

	GraphPreset = const_cast<UMovieGraphConfig*>(InGraphPreset);

	const EMoviePipelineConfigMode PreviousMode = ConfigMode;

	if (InGraphPreset != nullptr)
	{
		ConfigMode = EMoviePipelineConfigMode::Graph;
	}
	else if (ConfigMode == EMoviePipelineConfigMode::Graph)
	{
		// Clearing the graph from Graph mode switches back to Preset mode and cascades to all shots. This is mostly done
		// for backwards-compatibility purposes.
		ConfigMode = EMoviePipelineConfigMode::Preset;

		for (const TObjectPtr<UMoviePipelineExecutorShot>& Shot : ShotInfo)
		{
			Shot->SetGraphPreset(nullptr);
		}
	}

	if (bUpdateVariableAssignments)
	{
		RefreshAllVariableAssignments();
	}

	OnJobGraphPresetChanged.Broadcast(this, GraphPreset.Get());

	if (PreviousMode != ConfigMode)
	{
		OnJobConfigModeChanged.Broadcast(this, ConfigMode);
	}
}

void UMoviePipelineExecutorJob::SetSequence(FSoftObjectPath InSequence)
{
	Sequence = InSequence;

	// Rebuild our shot mask.
	ShotInfo.Reset();

	ULevelSequence* LoadedSequence = Cast<ULevelSequence>(Sequence.TryLoad());
	if (!LoadedSequence)
	{
		return;
	}

	// Allow external tools to set job properties based on the current sequence.
	OnSequenceSetEvent().Broadcast(this, LoadedSequence);

	bool bShotsChanged = false;
	UMoviePipelineBlueprintLibrary::UpdateJobShotListFromSequence(LoadedSequence, this, bShotsChanged);
	
	if (UMoviePipelineQueue* OwningQueue = GetTypedOuter<UMoviePipelineQueue>())
	{
		OwningQueue->InvalidateSerialNumber();
	}
}

UMoviePipelineExecutorJob::FOnSequenceSet& UMoviePipelineExecutorJob::OnSequenceSetEvent()
{
	static FOnSequenceSet OnSequenceSetEvent;
	return OnSequenceSetEvent;
}

UMovieJobVariableAssignmentContainer* UMoviePipelineExecutorJob::GetOrCreateJobVariableAssignmentsForGraph(const UMovieGraphConfig* InGraph)
{
	if (InGraph)
	{
		return MoviePipeline::GetOrCreateJobVariableAssignmentsForGraph(InGraph, GraphVariableAssignments, this);
	}

	return nullptr;
}

void UMoviePipelineExecutorJob::SetConfiguration(UMoviePipelinePrimaryConfig* InPreset)
{
	if (InPreset)
	{
		Configuration->CopyFrom(InPreset);
		PresetOrigin = nullptr;

		const EMoviePipelineConfigMode PreviousMode = ConfigMode;
		ConfigMode = EMoviePipelineConfigMode::Preset;

		if (PreviousMode != ConfigMode)
		{
			OnJobConfigModeChanged.Broadcast(this, ConfigMode);
		}
	}
}

void UMoviePipelineExecutorJob::SetPresetOrigin(UMoviePipelinePrimaryConfig* InPreset)
{
	if (InPreset)
	{
		Configuration->CopyFrom(InPreset);
		PresetOrigin = TSoftObjectPtr<UMoviePipelinePrimaryConfig>(InPreset);
		
		const EMoviePipelineConfigMode PreviousMode = ConfigMode;
		ConfigMode = EMoviePipelineConfigMode::Preset;
		
		if (PreviousMode != ConfigMode)
		{
			OnJobConfigModeChanged.Broadcast(this, ConfigMode);
		}
	}
}

void UMoviePipelineExecutorJob::OnDuplicated_Implementation()
{
	UserData = FString();
	StatusMessage = FString();
	StatusProgress = 0.f;
	SetConsumed(false);
}

void UMoviePipelineExecutorJob::OnGraphPreSave(UObject* InObject, FObjectPreSaveContext InObjectPreSaveContext)
{
	const UMovieGraphConfig* SavedGraph = Cast<UMovieGraphConfig>(InObject);
	if (!SavedGraph || InObjectPreSaveContext.IsProceduralSave())
	{
		return;
	}

	RefreshAllVariableAssignments();
}

UMoviePipelineBasicConfig* UMoviePipelineExecutorJob::SetupBasicConfiguration()
{
#if WITH_EDITOR
	Modify();
#endif

	// If the user has saved a default Basic config, duplicate it; otherwise create a fresh one.
	if (!BasicConfig)
	{
		if (const UMoviePipelineBasicConfig* SavedDefault = UMoviePipelineBasicConfig::GetSavedDefault())
		{
			BasicConfig = DuplicateObject(SavedDefault, this);
		}
		else
		{
			BasicConfig = NewObject<UMoviePipelineBasicConfig>(this);
		}
	}

	const EMoviePipelineConfigMode PreviousMode = ConfigMode;
	ConfigMode = EMoviePipelineConfigMode::Basic;

	// Ensure all child shots have a BasicShotConfig so they can display override controls.
	// SetupBasicShotOverrides() is idempotent — safe to call if already initialized.
	for (const TObjectPtr<UMoviePipelineExecutorShot>& Shot : ShotInfo)
	{
		if (Shot)
		{
			Shot->SetupBasicShotOverrides();
		}
	}

	if (PreviousMode != ConfigMode)
	{
		OnJobConfigModeChanged.Broadcast(this, ConfigMode);
	}

	return BasicConfig;
}

bool UMoviePipelineExecutorJob::SetConfigMode(EMoviePipelineConfigMode InConfigMode)
{
	if (InConfigMode == ConfigMode)
	{
		return true;
	}

	// Switching to Graph mode requires a graph to already be assigned. Callers are responsible for assigning a graph
	// first (e.g. via SetGraphPreset) if one doesn't exist, since loading the default graph requires editor-only
	// project settings.
	if (InConfigMode == EMoviePipelineConfigMode::Graph && GraphPreset.IsNull())
	{
		return false;
	}

#if WITH_EDITOR
	Modify();
#endif

	const EMoviePipelineConfigMode PreviousMode = ConfigMode;
	ConfigMode = InConfigMode;

	// Lazily initialize Basic config and shot overrides when entering Basic mode for the first time.
	// SetupBasicConfiguration handles creating the config from saved defaults and setting up shot overrides.
	if (ConfigMode == EMoviePipelineConfigMode::Basic)
	{
		SetupBasicConfiguration();
	}

	if (PreviousMode != ConfigMode)
	{
		OnJobConfigModeChanged.Broadcast(this, ConfigMode);
	}

	return true;
}

bool UMoviePipelineExecutorJob::CanUseConsoleVariableOverrides() const
{
	return IsUsingGraphConfiguration() || IsUsingBasicConfiguration();
}

void UMoviePipelineExecutorShot::SetGraphPreset(const UMovieGraphConfig* InGraphPreset, const bool bUpdateVariableAssignments)
{
#if WITH_EDITOR
	Modify();
#endif
	
	GraphPreset = const_cast<UMovieGraphConfig*>(InGraphPreset);

	if (bUpdateVariableAssignments)
	{
		RefreshAllVariableAssignments();
	}

	OnShotGraphPresetChanged.Broadcast(this, GraphPreset.Get());
}

UMovieJobVariableAssignmentContainer* UMoviePipelineExecutorShot::GetOrCreateJobVariableAssignmentsForGraph(const UMovieGraphConfig* InGraph, const bool bIsForPrimaryOverrides)
{
	if (InGraph)
	{
		return MoviePipeline::GetOrCreateJobVariableAssignmentsForGraph(InGraph, bIsForPrimaryOverrides ? PrimaryGraphVariableAssignments : GraphVariableAssignments, this);
	}

	return nullptr;
}

#if WITH_EDITOR
void UMoviePipelineQueueProjectSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Save the settings as soon as they're changed.
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		SaveConfig();
	}
}
#endif	// WITH_EDITOR

UMoviePipelineShotConfig* UMoviePipelineExecutorShot::AllocateNewShotOverrideConfig(TSubclassOf<UMoviePipelineShotConfig> InConfigType)
{
	if (!ensureAlwaysMsgf(InConfigType, TEXT("Failed to specify a Config Type. Use the default in project setting or UMoviePipelineShotConfig.")))
	{
		InConfigType = UMoviePipelineShotConfig::StaticClass();
	}

#if WITH_EDITOR
	Modify();
#endif

	ShotOverrideConfig = NewObject<UMoviePipelineShotConfig>(this, InConfigType);
	ShotOverrideConfig->SetFlags(RF_Transactional);

	return ShotOverrideConfig;
}

void UMoviePipelineExecutorShot::SetShotOverrideConfiguration(UMoviePipelineShotConfig* InPreset)
{
	if (InPreset)
	{
		if (ShotOverrideConfig == nullptr)
		{
			AllocateNewShotOverrideConfig(UMoviePipelineShotConfig::StaticClass());
		}

		ShotOverrideConfig->CopyFrom(InPreset);
	}
	else
	{
		ShotOverrideConfig = nullptr;
	}

	ShotOverridePresetOrigin = nullptr;
}

void UMoviePipelineExecutorShot::SetShotOverridePresetOrigin(UMoviePipelineShotConfig* InPreset)
{
	if (InPreset)
	{
		if (ShotOverrideConfig == nullptr)
		{
			AllocateNewShotOverrideConfig(UMoviePipelineShotConfig::StaticClass());
		}

		ShotOverrideConfig->CopyFrom(InPreset);
		ShotOverridePresetOrigin = TSoftObjectPtr<UMoviePipelineShotConfig>(InPreset);
	}
	else
	{
		ShotOverridePresetOrigin.Reset();
	}
}

bool UMoviePipelineExecutorShot::IsUsingGraphConfiguration() const
{
	const UMoviePipelineExecutorJob* PrimaryJob = GetTypedOuter<UMoviePipelineExecutorJob>();

	// Calling GetGraphPreset() here is important, rather than just referencing GraphPreset (to ensure that the soft ptr has a chance to load)
	return (GetGraphPreset() != nullptr) || (PrimaryJob && PrimaryJob->IsUsingGraphConfiguration());
}

bool UMoviePipelineExecutorShot::IsUsingBasicConfiguration() const
{
	const UMoviePipelineExecutorJob* PrimaryJob = GetTypedOuter<UMoviePipelineExecutorJob>();
	return PrimaryJob && PrimaryJob->IsUsingBasicConfiguration();
}

UMoviePipelineBasicConfig* UMoviePipelineExecutorShot::SetupBasicShotOverrides()
{
#if WITH_EDITOR
	Modify();
#endif

	if (!BasicShotConfig)
	{
		BasicShotConfig = NewObject<UMoviePipelineBasicConfig>(this);
	}
	return BasicShotConfig;
}

bool UMoviePipelineExecutorShot::CanUseConsoleVariableOverrides() const
{
	return IsUsingGraphConfiguration() || IsUsingBasicConfiguration();
}

void UMoviePipelineExecutorShot::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	RefreshAllVariableAssignments();
}

void UMoviePipelineExecutorShot::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Listen for any graph/subgraph saves. It's difficult/error-prone to track which graphs to monitor because the subgraph hierarchy can
	// constantly change. Instead, just run any logic needed when any graph is saved.
	FCoreUObjectDelegates::OnObjectPreSave.AddUObject(this, &UMoviePipelineExecutorShot::OnGraphPreSave);
#endif

	// Ideally the variable assignments would be updated here, but currently GraphPreset.LoadSynchronous() does not provide a fully-loaded
	// object (GraphPreset will only be partially loaded at this point and its variables will be empty).
}

void UMoviePipelineExecutorShot::BeginDestroy()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPreSave.RemoveAll(this);
#endif

	Super::BeginDestroy();
}

TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& UMoviePipelineExecutorShot::GetGraphVariableAssignments(const bool bUpdateAssignments)
{
	if (bUpdateAssignments)
	{
		RefreshAllVariableAssignments();
	}
	
	return GraphVariableAssignments;
}

TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& UMoviePipelineExecutorShot::GetPrimaryGraphVariableAssignments(const bool bUpdateAssignments)
{
	if (bUpdateAssignments)
	{
		RefreshAllVariableAssignments();
	}
	
	return PrimaryGraphVariableAssignments;
}

void UMoviePipelineExecutorShot::RefreshAllVariableAssignments()
{
	MoviePipeline::RefreshVariableAssignments(GraphPreset.LoadSynchronous(), GraphVariableAssignments, this);

	// Refresh the associated primary graph variable override assignments as well
	if (const UMoviePipelineExecutorJob* PrimaryJob = GetTypedOuter<UMoviePipelineExecutorJob>())
	{
		UMovieGraphConfig* PrimaryGraph = PrimaryJob->GetGraphPreset();
		MoviePipeline::RefreshVariableAssignments(PrimaryGraph, PrimaryGraphVariableAssignments, this);
	}
}

void UMoviePipelineExecutorShot::OnGraphPreSave(UObject* InObject, FObjectPreSaveContext InObjectPreSaveContext)
{
	const UMovieGraphConfig* SavedGraph = Cast<UMovieGraphConfig>(InObject);
	if (!SavedGraph || InObjectPreSaveContext.IsProceduralSave())
	{
		return;
	}

	RefreshAllVariableAssignments();
}
