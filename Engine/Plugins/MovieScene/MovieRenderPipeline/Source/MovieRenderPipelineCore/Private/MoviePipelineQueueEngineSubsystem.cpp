// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineQueueEngineSubsystem.h"
#include "Modules/ModuleManager.h"
#include "MoviePipeline.h"
#include "MoviePipelineInProcessExecutor.h"
#include "LevelSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineQueueEngineSubsystem)

UMoviePipelineExecutorBase* UMoviePipelineQueueEngineSubsystem::RenderQueueWithExecutor(TSubclassOf<UMoviePipelineExecutorBase> InExecutorType)
{
	if (!ensureMsgf(InExecutorType.Get(), TEXT("RenderQueueWithExecutor cannot be called with a null class type!")))
	{
		FFrame::KismetExecutionMessage(TEXT("RenderQueueWithExecutor cannot be called with a null class type!"), ELogVerbosity::Error);
		return nullptr;
	}

	RenderQueueWithExecutorInstance(NewObject<UMoviePipelineExecutorBase>(this, InExecutorType));
	return ActiveExecutor;
}

void UMoviePipelineQueueEngineSubsystem::RenderQueueWithExecutorInstance(UMoviePipelineExecutorBase* InExecutor)
{
	if(!ensureMsgf(!IsRendering(), TEXT("RenderQueueWithExecutorInstance cannot be called while already rendering!")))
	{
		FFrame::KismetExecutionMessage(TEXT("Render already in progress."), ELogVerbosity::Error);
		return;
	}

	if (!InExecutor)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid executor supplied."), ELogVerbosity::Error);
		return;
	}

	ActiveExecutor = InExecutor;
	UE::MoviePipeline::FViewportArgs InitArgs;
	InitArgs.bRenderViewport = bCachedRenderPlayerViewport;
	InitArgs.DebugWidgetClass = CachedProgressWidgetClass;
	ActiveExecutor->SetViewportInitArgs(InitArgs);

	ActiveExecutor->OnExecutorFinished().AddUObject(this, &UMoviePipelineQueueEngineSubsystem::OnExecutorFinished);
	ActiveExecutor->Execute(GetQueue());
}

void UMoviePipelineQueueEngineSubsystem::OnExecutorFinished(UMoviePipelineExecutorBase* InPipelineExecutor, bool bSuccess)
{
	ActiveExecutor = nullptr;
	
	// Wait until ActiveExecutor has been nulled so that if they immediately try to allocate new jobs as a result of
	// the callback it doesn't throw an exception about a render being in progress.
	if(CachedRenderJobResults.IsSet())
	{
		OnRenderFinished.Broadcast(CachedRenderJobResults.GetValue());
		CachedRenderJobResults.Reset();
	}
}

UMoviePipelineExecutorJob* UMoviePipelineQueueEngineSubsystem::AllocateJob(ULevelSequence* InSequence)
{
	if(!InSequence)
	{
		FFrame::KismetExecutionMessage(TEXT("AllocateJob cannot be called with a null sequence!"), ELogVerbosity::Error);
		return nullptr;
	}
	
	if(IsRendering())
	{
		FFrame::KismetExecutionMessage(TEXT("AllocateJob cannot be called while rendering!"), ELogVerbosity::Error);
		return nullptr;
	}
	
	CurrentQueue->DeleteAllJobs();
	
	UMoviePipelineExecutorJob* NewJob = CurrentQueue->AllocateNewJob(UMoviePipelineExecutorJob::StaticClass());
	NewJob->SetSequence(FSoftObjectPath(InSequence));
	NewJob->Map = FSoftObjectPath(GetWorld());
	NewJob->JobName = NewJob->Sequence.GetAssetName();
	
	
	return NewJob;
}

void UMoviePipelineQueueEngineSubsystem::RenderJob(UMoviePipelineExecutorJob* InJob)
{
	if(!CurrentQueue->GetJobs().Contains(InJob))
	{
		FFrame::KismetExecutionMessage(TEXT("RenderJob cannot be called with a job not created by AllocateJob!"), ELogVerbosity::Error);
		return;
	}
	
	if(IsRendering())
	{
		FFrame::KismetExecutionMessage(TEXT("RenderJob cannot be called while rendering!"), ELogVerbosity::Error);
		return;
	}
	
	UMoviePipelineInProcessExecutor* InProcessExecutor = NewObject<UMoviePipelineInProcessExecutor>(this);
	InProcessExecutor->bUseCurrentLevel = true;
	InProcessExecutor->OnIndividualJobFinished().AddUObject(this, &UMoviePipelineQueueEngineSubsystem::OnIndividualJobFinished);
	RenderQueueWithExecutorInstance(InProcessExecutor);
}

void UMoviePipelineQueueEngineSubsystem::OnIndividualJobFinished(FMoviePipelineOutputData Results)
{
	// OnExecutorFinished will be called after this (since this callback is per-job), but we want users
	// to be able to queue up another job immediately without needing to wait 1 tick for ActiveExecutor to
	// be set to nullptr. If they queue up a new job right now, the executor will get nulled out. So instead
	// we cache the result of this individual job, and then broadcast the actual Blueprint event from OnExecutorFinished.
	CachedRenderJobResults = Results;
}

void UMoviePipelineQueueEngineSubsystem::SetConfiguration(TSubclassOf<UMovieRenderDebugWidget> InProgressWidgetClass, const bool bRenderPlayerViewport)
{
	CachedProgressWidgetClass = InProgressWidgetClass;
	bCachedRenderPlayerViewport = bRenderPlayerViewport;
}


