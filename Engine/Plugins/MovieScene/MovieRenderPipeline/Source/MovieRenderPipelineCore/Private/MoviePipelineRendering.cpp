// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipeline.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Misc/FrameRate.h"
#include "MoviePipelineOutputBase.h"
#include "MoviePipelineAntiAliasingSetting.h"
#include "MoviePipelineShotConfig.h"
#include "MoviePipelineRenderPass.h"
#include "MoviePipelineOutputBuilder.h"
#include "RenderingThread.h"
#include "MoviePipelineDebugSettings.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineConfigBase.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "Math/Halton.h"
#include "ImageWriteTask.h"
#include "ImageWriteQueue.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "MoviePipelineHighResSetting.h"
#include "Modules/ModuleManager.h"
#include "MoviePipelineCameraSetting.h"
#include "Engine/GameViewportClient.h"
#include "LegacyScreenPercentageDriver.h"
#include "RenderCaptureInterface.h"
#include "MoviePipelineGameOverrideSetting.h"

// For flushing async systems
#include "RendererInterface.h"
#include "LandscapeSubsystem.h"
#include "EngineModule.h"
#include "DistanceFieldAtlas.h"
#include "MeshCardRepresentation.h"
#include "AssetCompilingManager.h"
#include "ShaderCompiler.h"
#include "EngineUtils.h"
#include "Materials/MaterialInterface.h"
#include "ContentStreaming.h"

static TAutoConsoleVariable<bool> CVarMoviePipelineDisableShaderFlushing(
	TEXT("MoviePipeline.DisableShaderFlushingDebug"), false,
	TEXT("If true, the Movie Pipeline won't wait for any outstanding shader or asset compilation.")
	TEXT("If false (default), any outstanding shaders and assets will be flushed each frame before rendering.")
	TEXT("If true, rendered frames may be missing objects (meshes, particles, etc.) or objects may show the default checkerboard material."),
	ECVF_Default);

static TAutoConsoleVariable<int> CVarMoviePipelineThrottleFrameCount(
	TEXT("MoviePipeline.ThrottleFrameCount"), 2,
	TEXT("Number of rendered frames that can be submitted to the rendering thread before waiting. A value of 0 will allow the CPU to submit all work without waiting on the GPU.\n")
	TEXT("The default value of 2 tries to balance between performance and memory usage. The maximum value is 4.\n")
	TEXT("This option only applies to path traced renders, as deferred rendering is synchronized through pixel readbacks.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarMoviePipelineDisableMaxResolutionCheck(
	TEXT("MoviePipeline.DisableMaxResolutionCheck"),
	false,
	TEXT("When true, the Movie Pipeline will not consider an output resolution higher than the maximum texture size an error. "
		"When using high resolution tiles the system can calculate the size of an individual tile to correctly detect an overly "
		"large render, but when using panoramic renders the system cannot, artificially limiting the resolution to 16k. Set this "
		"true to disable the check, with the understanding that user settings can then choose crash-inducing values."),
	ECVF_Default
);


#define LOCTEXT_NAMESPACE "MoviePipeline"

void UMoviePipeline::SetupRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot)
{
	/*
	* To support tiled rendering we take the final effective resolution and divide
	* it by the number of tiles to find the resolution of each render target. To 
	* handle non-evenly divisible numbers/resolutions we may oversize the targets
	* by a few pixels and then take the center of the resulting image when interlacing
	* to produce the final image at the right resolution. For example:
	*
	* 1920x1080 in 7x7 tiles gives you 274.29x154.29. We ceiling this to set the resolution
	* of the render pass to 275x155 which will give us a final interleaved image size of
	* 1925x1085. To ensure that the image matches a non-scaled one we take the center out.
	* LeftOffset = floor((1925-1920)/2) = 2
	* RightOffset = (1925-1920-LeftOffset)
	*/
	UMoviePipelineAntiAliasingSetting* AccumulationSettings = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(InShot);
	UMoviePipelineHighResSetting* HighResSettings = FindOrAddSettingForShot<UMoviePipelineHighResSetting>(InShot);
	UMoviePipelineOutputSetting* OutputSettings = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	check(OutputSettings);

	// Reset cached camera overscan
	CameraOverscanCache.Empty();
	bHasWarnedAboutAnimatedOverscan = false;
	
	// TODO: Not much support here for multi-camera, so simply get the player controller camera and use its overscan value
	float CameraOverscan = 0.0f;
	if (UCameraComponent* BoundCamera = MovieSceneHelpers::CameraComponentFromRuntimeObject(GetWorld()->GetFirstPlayerController()->PlayerCameraManager->GetViewTarget()))
	{
		FMinimalViewInfo CameraViewInfo;
		BoundCamera->GetCameraView(GetWorld()->GetDeltaSeconds(), CameraViewInfo);
		CameraOverscan = CameraViewInfo.GetOverscan();
	}

	// Cache the default camera overscan at INDEX_NONE to ensure anything that doesn't have multi-camera support still has an overscan value to utilize
	CameraOverscanCache.Add(INDEX_NONE, CameraOverscan);
	
	FIntPoint BackbufferTileCount = FIntPoint(HighResSettings->TileCount, HighResSettings->TileCount);
	FIntPoint TotalResolution = UMoviePipelineBlueprintLibrary::GetOverscannedResolution(GetPipelinePrimaryConfig(), InShot, CameraOverscan);

	// Figure out how big each sub-region (tile) is.
	FIntPoint BackbufferResolution = UMoviePipelineBlueprintLibrary::GetBackbufferResolution(GetPipelinePrimaryConfig(), InShot, CameraOverscan);

	{
		int32 MaxResolution = GetMax2DTextureDimension();
		if (BackbufferResolution.X > MaxResolution || BackbufferResolution.Y > MaxResolution)
		{
			// Panoramic Tiling doesn't correctly pass this check so the system unnecessarily prevents users
			// from making large panoramic renders. This cvar allows the user to disable this check, but we
			// keep the log above as it contains good information to have.
			if (!CVarMoviePipelineDisableMaxResolutionCheck.GetValueOnGameThread())
			{
				UE_LOGF(LogMovieRenderPipeline, Error, "Resolution %dx%d exceeds maximum allowed by GPU (%dx%d). Consider using the HighRes setting and increasing the tile count. If using a tiled render pass that does not use HighRes tiles (such as the Panoramic Pass), set MoviePipeline.DisableMaxResolutionCheck to true to bypass this and allow rendering.", BackbufferResolution.X, BackbufferResolution.Y, MaxResolution, MaxResolution);
				Shutdown(true);
				return;
			}
		}
		if (BackbufferResolution.X <= 0 || BackbufferResolution.Y <= 0)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Resolution %dx%d must be greater than zero in both dimensions.", BackbufferResolution.X, BackbufferResolution.Y);
			Shutdown(true);
			return;
		}
	}

	const ERHIFeatureLevel::Type FeatureLevel = GetWorld()->GetFeatureLevel();

	// Initialize our render pass. This is a copy of the settings to make this less coupled to the Settings UI.
	const MoviePipeline::FMoviePipelineRenderPassInitSettings RenderPassInitSettings(FeatureLevel, BackbufferResolution, BackbufferTileCount);

	// Code expects at least a 1x1 tile.
	ensure(RenderPassInitSettings.TileCount.X > 0 && RenderPassInitSettings.TileCount.Y > 0);

	// Initialize out output passes
	int32 NumOutputPasses = 0;
	for (UMoviePipelineRenderPass* RenderPass : FindSettingsForShot<UMoviePipelineRenderPass>(InShot))
	{
		RenderPass->Setup(RenderPassInitSettings);
		NumOutputPasses++;
	}

	UE_LOGF(LogMovieRenderPipeline, Log, "Finished setting up rendering for shot. Shot has %d Passes. Total resolution: (%dx%d) Individual tile resolution: (%dx%d). Tile count: (%dx%d)", NumOutputPasses, TotalResolution.X, TotalResolution.Y, BackbufferResolution.X, BackbufferResolution.Y, BackbufferTileCount.X, BackbufferTileCount.Y);
}

void UMoviePipeline::TeardownRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot)
{
	for (UMoviePipelineRenderPass* RenderPass : FindSettingsForShot<UMoviePipelineRenderPass>(InShot))
	{
		RenderPass->Teardown();
	}

	if (OutputBuilder->GetNumOutstandingFrames() > 1)
	{
		// The intention behind this warning is to catch when you've created a render pass that doesn't submit as many render passes as you expect. Unfortunately,
		// it also catches the fact that temporal sampling tends to render an extra frame. When we are submitting frames we only check if the actual evaluation point
		// surpasses the upper bound, at which point we don't submit anything more. We could check a whole frame in advance and never submit any temporal samples for
		// the extra frame, but then this would not work with slow-motion. Instead, we will just comprimise here and only warn if there's multiple frames that are missing.
		// This is going to be true if you have set up your rendering wrong (and are rendering more than one frame) so it will catch enough of the cases to be worth it.
		UE_LOGF(LogMovieRenderPipeline, Error, "Not all frames were fully submitted by the time rendering was torn down! Frames will be missing from output!");
	}
}

void UMoviePipeline::RenderFrame()
{
	// Flush built in systems before we render anything. This maximizes the likelihood that the data is prepared for when
	// the render thread uses it.
	FlushAsyncEngineSystems();

	// Send any output frames that have been completed since the last render.
	ProcessOutstandingFinishedFrames();

	if (CachedOutputState.IsFirstTemporalSample())
	{
		CameraOverscanCache.Empty();
	}
	
	FMoviePipelineCameraCutInfo& CurrentCameraCut = ActiveShotList[CurrentShotIndex]->ShotInfo;
	APlayerController* LocalPlayerController = GetWorld()->GetFirstPlayerController();


	// If we don't want to render this frame, then we will skip processing - engine warmup frames,
	// render every nTh frame, etc. In other cases, we may wish to render the frame but discard the
	// result and not send it to the output merger (motion blur frames, gpu feedback loops, etc.)
	if (CachedOutputState.bSkipRendering)
	{
		return;
	}
	
	// Hide the progress widget before we render anything. This allows widget captures to not include the progress bar.
	SetProgressWidgetVisible(false);

	// To produce a frame from the movie pipeline we may render many frames over a period of time, additively collecting the results
	// together before submitting it for writing on the last result - this is referred to as an "output frame". The 1 (or more) samples
	// that make up each output frame are referred to as "sample frames". Within each sample frame, we may need to render the scene many
	// times. In order to support ultra-high-resolution rendering (>16k) movie pipelines support building an output frame out of 'tiles'. 
	// Each tile renders the entire viewport with a small offset which causes different samples to be picked for each final pixel. These
	// 'tiles' are then interleaved together (on the CPU) to produce a higher resolution result. For each tile, we can render a number
	// of jitters that get added together to produce a higher quality single frame. This is useful for cases where you may not want any 
	// motion (such as trees fluttering in the wind) but you do want high quality anti-aliasing on the edges of the pixels. Finally,
	// the outermost loop (which is not represented here) is accumulation over time which happens over multiple engine ticks.
	// 
	// In short, for each output frame, for each accumulation frame, for each tile X/Y, for each jitter, we render a pass. This setup is
	// designed to maximize the likely hood of deterministic rendering and that different passes line up with each other.
	UMoviePipelineAntiAliasingSetting* AntiAliasingSettings = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(ActiveShotList[CurrentShotIndex]);
	UMoviePipelineCameraSetting* CameraSettings = FindOrAddSettingForShot<UMoviePipelineCameraSetting>(ActiveShotList[CurrentShotIndex]);
	UMoviePipelineHighResSetting* HighResSettings = FindOrAddSettingForShot<UMoviePipelineHighResSetting>(ActiveShotList[CurrentShotIndex]);
	UMoviePipelineOutputSetting* OutputSettings = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	UMoviePipelineDebugSettings* DebugSettings = FindOrAddSettingForShot<UMoviePipelineDebugSettings>(ActiveShotList[CurrentShotIndex]);
	
	// Color settings are optional, so we don't need to do any assertion checks.
	UMoviePipelineColorSetting* ColorSettings = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineColorSetting>();
	check(AntiAliasingSettings);
	check(CameraSettings);
	check(HighResSettings);
	check(OutputSettings);

	float CameraOverscan = 0.0f;
	if (!CameraOverscanCache.Contains(INDEX_NONE))
	{
		// If the cache does not contain a default overscan (likely because it has been cleared as the start of a new frame),
		// cache the default camera overscan at INDEX_NONE to ensure anything that doesn't have multi-camera support still has an overscan value to utilize
		// TODO: Not much support here for multi-camera, so simply get the player controller camera and use its overscan value
		if (UCameraComponent* BoundCamera = MovieSceneHelpers::CameraComponentFromRuntimeObject(LocalPlayerController->PlayerCameraManager->GetViewTarget()))
		{
			FMinimalViewInfo CameraViewInfo;
			BoundCamera->GetCameraView(GetWorld()->GetDeltaSeconds(), CameraViewInfo);
			CameraOverscan = CameraViewInfo.GetOverscan();
		}
		
		CameraOverscanCache.Add(INDEX_NONE, CameraOverscan);
	}
	else
	{
		// Use the cache to get the overscan value for resolution scaling so that it doesn't vary between subsamples
		CameraOverscan = CameraOverscanCache[INDEX_NONE];
	}
	
	FIntPoint TileCount = FIntPoint(HighResSettings->TileCount, HighResSettings->TileCount);
	FIntPoint OriginalTileCount = TileCount;
	FIntPoint OverscannedResolution = UMoviePipelineBlueprintLibrary::GetOverscannedResolution(GetPipelinePrimaryConfig(), ActiveShotList[CurrentShotIndex], CameraOverscan);

	int32 NumSpatialSamples = AntiAliasingSettings->SpatialSampleCount;
	int32 NumTemporalSamples = AntiAliasingSettings->TemporalSampleCount;
	if (!ensureAlways(TileCount.X > 0 && TileCount.Y > 0 && NumSpatialSamples > 0 && NumTemporalSamples > 0))
	{
		return;
	}
	
	{
		// Sidecar Cameras get updated below after rendering, they're still separate for backwards compat reasons
		FrameInfo.PrevViewLocation = FrameInfo.CurrViewLocation;
		FrameInfo.PrevViewRotation = FrameInfo.CurrViewRotation;

		// Update the Sidecar Cameras
		FrameInfo.PrevSidecarViewLocations = FrameInfo.CurrSidecarViewLocations;
		FrameInfo.PrevSidecarViewRotations = FrameInfo.CurrSidecarViewRotations;

		// Update our current view location
		LocalPlayerController->GetPlayerViewPoint(FrameInfo.CurrViewLocation, FrameInfo.CurrViewRotation);
		GetSidecarCameraViewPoints(ActiveShotList[CurrentShotIndex], FrameInfo.CurrSidecarViewLocations, FrameInfo.CurrSidecarViewRotations);
	}

	bool bWriteAllSamples = DebugSettings ? DebugSettings->bWriteAllSamples : false;

	// Add appropriate metadata here that is shared by all passes.
	{
		// Add hardware stats such as total memory, cpu vendor, etc.
		FString ResolvedOutputDirectory;
		TMap<FString, FString> FormatOverrides;
		FMoviePipelineFormatArgs FinalFormatArgs;

		// We really only need the output disk path for disk size info, but we'll try to resolve as much as possible anyways
		ResolveFilenameFormatArguments(OutputSettings->OutputDirectory.Path, FormatOverrides, ResolvedOutputDirectory, FinalFormatArgs);
		// Strip .{ext}
		ResolvedOutputDirectory.LeftChopInline(6);

		UE::MoviePipeline::GetHardwareUsageMetadata(CachedOutputState.FileMetadata, ResolvedOutputDirectory);

		// Add in additional diagnostic information (engine version, etc)
		constexpr bool bIsGraph = false;
		UE::MoviePipeline::GetDiagnosticMetadata(CachedOutputState.FileMetadata, bIsGraph);

		// We'll leave these in for legacy, when this tracks the 'Main' camera (of the player), render passes that support
		// multiple cameras will have to write each camera name into their metadata.
		UE::MoviePipeline::GetMetadataFromCameraLocRot(TEXT("camera"), TEXT(""), FrameInfo.CurrViewLocation, FrameInfo.CurrViewRotation, FrameInfo.PrevViewLocation, FrameInfo.PrevViewRotation, CachedOutputState.FileMetadata);

		// This is still global regardless, individual cameras don't get their own motion blur amount because the engine tick is tied to it.
		CachedOutputState.FileMetadata.Add(TEXT("unreal/camera/shutterAngle"), FString::SanitizeFloat(CachedOutputState.TimeData.MotionBlurFraction * 360.0f));
	}

	if (CurrentCameraCut.State != EMovieRenderShotState::Rendering)
	{
		// We can optimize some of the settings for 'special' frames we may be rendering, ie: we render once for motion vectors, but
		// we don't need that per-tile so we can set the tile count to 1, and spatial sample count to 1 for that particular frame.
		{
			// Spatial Samples aren't needed when not producing frames (caveat: Render Warmup Frame, handled below)
			NumSpatialSamples = 1;
		}
	}

	int32 NumWarmupSamples = 0;
	if (CurrentCameraCut.State == EMovieRenderShotState::WarmingUp)
	{
		// We sometimes render the actual warmup frames, and in this case we only want to render one warmup sample each frame,
		// and save any RenderWarmUp frames until the last one.
		if (CurrentCameraCut.NumEngineWarmUpFramesRemaining > 0)
		{
			NumWarmupSamples = 1;
		}
		else
		{
			NumWarmupSamples = AntiAliasingSettings->RenderWarmUpCount;
		}
	}

	TArray<UMoviePipelineRenderPass*> InputBuffers = FindSettingsForShot<UMoviePipelineRenderPass>(ActiveShotList[CurrentShotIndex]);

	// Reset our flag for this frame.
	bHasRenderedFirstViewThisFrame = false;

	for (UMoviePipelineRenderPass* RenderPass : InputBuffers)
	{
		RenderPass->OnFrameStart();
	}
	
	// If this is the first sample for a new frame, update per-output-frame bookkeeping.
	if (CachedOutputState.IsFirstTemporalSample())
	{
		FRenderTimeStatistics& TimeStats = RenderTimeFrameStatistics.FindOrAdd(CachedOutputState.OutputFrameNumber);
		TimeStats.StartTime = FDateTime::UtcNow();
	}

	// Only register expected output once the frame can submit completed data.
	if (CachedOutputState.IsLastTemporalSample())
	{
		CurrentShotRenderedFrameCount++;

		// This happens before any data is queued for this frame.
		FMoviePipelineMergerOutputFrame& OutputFrame = OutputBuilder->QueueOutputFrame_GameThread(CachedOutputState);

		// Now we need to go through all passes and get any identifiers from them of what this output frame should expect.
		for (UMoviePipelineRenderPass* RenderPass : InputBuffers)
		{
			RenderPass->GatherOutputPasses(OutputFrame.ExpectedRenderPasses);
		}
	}

	// Support for RenderDoc captures of just the MRQ work
#if WITH_EDITOR && !UE_BUILD_SHIPPING
	TUniquePtr<RenderCaptureInterface::FScopedCapture> ScopedGPUCapture;
	if (CachedOutputState.bCaptureRendering)
	{
		ScopedGPUCapture = MakeUnique<RenderCaptureInterface::FScopedCapture>(true, *FString::Printf(TEXT("MRQ Frame: %d"), CachedOutputState.SourceFrameNumber));
	}
#endif

	constexpr int FenceBufferMax = 4;
	int FrameThrotteCount = 0;
	for (const UMoviePipelineRenderPass* RenderPass : InputBuffers)
	{
		if (RenderPass->NeedsFrameThrottle())
		{
			FrameThrotteCount = FMath::Clamp(CVarMoviePipelineThrottleFrameCount.GetValueOnGameThread(), 0, FenceBufferMax);
			break;
		}
	}
	FGPUFenceRHIRef MRQThrottleFence[FenceBufferMax];
	int FenceIndex = 0;
	
	for (int32 TileY = 0; TileY < TileCount.Y; TileY++)
	{
		for (int32 TileX = 0; TileX < TileCount.X; TileX++)
		{
			for (UMoviePipelineRenderPass* RenderPass : InputBuffers)
			{
				RenderPass->OnTileStart(FIntPoint(TileX, TileY));
			}

			int NumSamplesToRender = (CurrentCameraCut.State == EMovieRenderShotState::WarmingUp) ? NumWarmupSamples : NumSpatialSamples;

			// Now we want to render a user-configured number of spatial jitters to come up with the final output for this tile. 
			for (int32 RenderSampleIndex = 0; RenderSampleIndex < NumSamplesToRender; RenderSampleIndex++)
			{
				int32 SpatialSampleIndex = (CurrentCameraCut.State == EMovieRenderShotState::WarmingUp) ? 0 : RenderSampleIndex;

				if (CurrentCameraCut.State == EMovieRenderShotState::Rendering)
				{
					// Count this as a sample rendered for the current work.
					CurrentCameraCut.WorkMetrics.OutputSubSampleIndex++;
				}

				// We freeze views for all spatial samples except the last so that nothing in the FSceneView tries to update.
				// Our spatial samples need to be different positional takes on the same world, thus pausing it.
				const bool bAllowPause = CurrentCameraCut.State == EMovieRenderShotState::Rendering;
				const bool bIsLastTile = FIntPoint(TileX, TileY) == FIntPoint(TileCount.X - 1, TileCount.Y - 1);
				const bool bWorldIsPaused = bAllowPause && !(bIsLastTile && (RenderSampleIndex == (NumSamplesToRender - 1)));

				// We need to pass camera cut flag on the first sample that gets rendered for a given camera cut. If you don't have any render
				// warm up frames, we do this on the first render sample because we no longer render the motion blur frame (just evaluate it).
				const bool bCameraCut = CachedOutputState.ShotSamplesRendered == 0;
				CachedOutputState.ShotSamplesRendered++;

				EAntiAliasingMethod AntiAliasingMethod = UE::MovieRenderPipeline::GetEffectiveAntiAliasingMethod(AntiAliasingSettings);

				// Now to check if we have to force it off (at which point we warn the user).
				bool bMultipleTiles = (TileCount.X > 1) || (TileCount.Y > 1);
				if (bMultipleTiles && IsTemporalAccumulationBasedMethod(AntiAliasingMethod))
				{
					// Temporal Anti-Aliasing isn't supported when using tiled rendering because it relies on having history, and
					// the tiles use the previous tile as the history which is incorrect. 
					UE_LOGF(LogMovieRenderPipeline, Warning, "Temporal AntiAliasing is not supported when using tiling!");
					AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
				}


				// We Abs this so that negative numbers on the first frame of a cut (warm ups) don't go into Halton which will assign 0.
				int32 ClampedFrameNumber = FMath::Max(0, CachedOutputState.OutputFrameNumber);
				int32 ClampedTemporalSampleIndex = FMath::Max(0, CachedOutputState.TemporalSampleIndex);
				int32 FrameIndex = FMath::Abs((ClampedFrameNumber * (NumTemporalSamples * NumSpatialSamples)) + (ClampedTemporalSampleIndex * NumSpatialSamples) + SpatialSampleIndex);

				// if we are warming up, we will just use the RenderSampleIndex as the FrameIndex so the samples jump around a bit.
				if (CurrentCameraCut.State == EMovieRenderShotState::WarmingUp)
				{
					FrameIndex = RenderSampleIndex;
				}
				
				// only allow a spatial jitter if we have more than one sample
				bool bAllowSpatialJitter = !(NumSpatialSamples == 1 && NumTemporalSamples == 1);

				float SpatialShiftX = 0.0f;
				float SpatialShiftY = 0.0f;

				if (bAllowSpatialJitter)
				{
					FVector2f SubPixelJitter = UE::MoviePipeline::GetSubPixelJitter(FrameIndex, NumSpatialSamples * NumTemporalSamples);
					SpatialShiftX = SubPixelJitter.X;
					SpatialShiftY = SubPixelJitter.Y;
				}

				// We take all of the information needed to render a single sample and package it into a struct.
				FMoviePipelineRenderPassMetrics SampleState;
				SampleState.FrameIndex = FrameIndex;
				SampleState.bWorldIsPaused = bWorldIsPaused;
				SampleState.bCameraCut = bCameraCut;
				SampleState.AntiAliasingMethod = AntiAliasingMethod;
				SampleState.SceneCaptureSource = (ColorSettings && ColorSettings->bDisableToneCurve) ? ESceneCaptureSource::SCS_FinalColorHDR : ESceneCaptureSource::SCS_FinalToneCurveHDR;
				SampleState.OutputState = CachedOutputState;
				SampleState.OutputState.CameraIndex = 0; // Initialize to a sane default for non multi-cam passes.
				SampleState.TileIndexes = FIntPoint(TileX, TileY);
				SampleState.TileCounts = TileCount;
				SampleState.OriginalTileCounts = OriginalTileCount;
				SampleState.SpatialShiftX = SpatialShiftX;
				SampleState.SpatialShiftY = SpatialShiftY;
				SampleState.bDiscardResult = CachedOutputState.bDiscardRenderResult;
				SampleState.SpatialSampleIndex = SpatialSampleIndex;
				SampleState.SpatialSampleCount = NumSpatialSamples;
				SampleState.TemporalSampleIndex = CachedOutputState.TemporalSampleIndex;
				SampleState.TemporalSampleCount = AntiAliasingSettings->TemporalSampleCount;
				SampleState.AccumulationGamma = AntiAliasingSettings->AccumulationGamma;
				SampleState.FrameInfo = FrameInfo;
				SampleState.bWriteSampleToDisk = bWriteAllSamples;
				SampleState.TextureSharpnessBias = HighResSettings->TextureSharpnessBias;
				SampleState.OCIOConfiguration = ColorSettings ? &ColorSettings->OCIOConfiguration : nullptr;
				SampleState.GlobalScreenPercentageFraction = FLegacyScreenPercentageDriver::GetCVarResolutionFraction();
				SampleState.bAutoExposureCubePass = false;
				SampleState.AutoExposureCubeFace = 0;
				SampleState.bOverrideCameraOverscan = CameraSettings->bOverrideCameraOverscan;
				SampleState.OverscanPercentage = FMath::Clamp(CameraSettings->OverscanPercentage, 0.0f, 1.0f);

				if (FrameThrotteCount > 0)
				{
					// Before we render, wait for previous samples to have completed so the GPU command list doesn't get too far behind
					if (MRQThrottleFence[FenceIndex] && !MRQThrottleFence[FenceIndex]->Poll())
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(MRQFrameThrottle);
						for (;;)
						{
							FPlatformProcess::SleepNoStats(0.001f);
							if (MRQThrottleFence[FenceIndex]->Poll())
							{
								break;
							}
						}
					}

					// Create a fence for this frame and insert a signal to it
					MRQThrottleFence[FenceIndex] = RHICreateGPUFence(TEXT("MRQThrottleFence"));
					ENQUEUE_RENDER_COMMAND(MRQFrameThrottle)([Fence = MRQThrottleFence[FenceIndex]]
						(FRHICommandListImmediate& RHICmdList)
						{
							RHICmdList.WriteGPUFence(Fence);
							RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);
						});

					// Switch fences for the next frame (this makes us wait on a different frame than what we just made the signal for)
					FenceIndex = (FenceIndex + 1) % FrameThrotteCount;
				}

				// Render each output pass
				FMoviePipelineRenderPassMetrics SampleStateForCurrentResolution = UE::MoviePipeline::GetRenderPassMetrics(GetPipelinePrimaryConfig(), ActiveShotList[CurrentShotIndex], SampleState, OverscannedResolution);
				for (UMoviePipelineRenderPass* RenderPass : InputBuffers)
				{
					RenderPass->RenderSample_GameThread(SampleStateForCurrentResolution);
				}
			}

			for (UMoviePipelineRenderPass* RenderPass : InputBuffers)
			{
				RenderPass->OnTileEnd(FIntPoint(TileX, TileY));
			}
		}
	}

	// Re-enable the progress widget so when the player viewport is drawn to the preview window, it shows.
	SetProgressWidgetVisible(true);
}

#if WITH_EDITOR
void UMoviePipeline::AddFrameToOutputMetadata(const FString& ClipName, const FString& ImageSequenceFileName, const FMoviePipelineFrameOutputState& FrameOutputState, const FString& Extension, const bool bHasAlpha)
{
	if (FrameOutputState.ShotIndex < 0 || FrameOutputState.ShotIndex >= ActiveShotList.Num())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "ShotIndex %d out of range", FrameOutputState.ShotIndex);
		return;
	}

	FMovieSceneExportMetadataShot& ShotMetadata = OutputMetadata.Shots[FrameOutputState.ShotIndex];
	FMovieSceneExportMetadataClip& ClipMetadata = ShotMetadata.Clips.FindOrAdd(ClipName).FindOrAdd(Extension.ToUpper());

	if (!ClipMetadata.IsValid())
	{
		ClipMetadata.FileName = ImageSequenceFileName;
		ClipMetadata.bHasAlpha = bHasAlpha;
	}

	if (FrameOutputState.OutputFrameNumber < ClipMetadata.StartFrame)
	{
		ClipMetadata.StartFrame = FrameOutputState.OutputFrameNumber;
	}

	if (FrameOutputState.OutputFrameNumber > ClipMetadata.EndFrame)
	{
		ClipMetadata.EndFrame = FrameOutputState.OutputFrameNumber;
	}
}
#endif

void UMoviePipeline::AddOutputFuture(TFuture<bool>&& OutputFuture, const MoviePipeline::FMoviePipelineOutputFutureData& InOutputData)
{
	OutputFutures.Add(
		TTuple<TFuture<bool>, MoviePipeline::FMoviePipelineOutputFutureData>(MoveTemp(OutputFuture), InOutputData)
	);
}

void UMoviePipeline::ProcessOutstandingFinishedFrames()
{
	while (!OutputBuilder->FinishedFrames.IsEmpty())
	{
		FMoviePipelineMergerOutputFrame OutputFrame;
		OutputBuilder->FinishedFrames.Dequeue(OutputFrame);

		FRenderTimeStatistics& TimeStats = RenderTimeFrameStatistics.FindOrAdd(OutputFrame.FrameOutputState.OutputFrameNumber);
		TimeStats.EndTime = FDateTime::UtcNow();
	
		for (UMoviePipelineOutputBase* OutputContainer : GetPipelinePrimaryConfig()->GetOutputContainers())
		{
			OutputContainer->OnReceiveImageData(&OutputFrame);
		}
	}
}


void UMoviePipeline::OnSampleRendered(TUniquePtr<FImagePixelData>&& OutputSample)
{
	// This function handles the "Write all Samples" feature which lets you inspect data
	// pre-accumulation.
	UMoviePipelineOutputSetting* OutputSettings = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	check(OutputSettings);

	// This is for debug output, writing every individual sample to disk that comes off of the GPU (that isn't discarded).
	TUniquePtr<FImageWriteTask> TileImageTask = MakeUnique<FImageWriteTask>();

	FImagePixelDataPayload* InFrameData = OutputSample->GetPayload<FImagePixelDataPayload>();
	TileImageTask->Format = EImageFormat::EXR;
	TileImageTask->CompressionQuality = (int32)EImageCompressionQuality::Default;

	FString OutputName = InFrameData->Debug_OverrideFilename.IsEmpty() ?
		FString::Printf(TEXT("/%s_SS_%d_TS_%d_TileX_%d_TileY_%d.%d"),
			*InFrameData->PassIdentifier.Name, InFrameData->SampleState.SpatialSampleIndex, InFrameData->SampleState.TemporalSampleIndex,
			InFrameData->SampleState.TileIndexes.X, InFrameData->SampleState.TileIndexes.Y, InFrameData->SampleState.OutputState.OutputFrameNumber)
		: InFrameData->Debug_OverrideFilename;
	
	FString OutputDirectory = OutputSettings->OutputDirectory.Path;
	FString FileNameFormatString = OutputDirectory + OutputName;

	TMap<FString, FString> FormatOverrides;
	FormatOverrides.Add(TEXT("ext"), TEXT("exr"));
	UMoviePipelineExecutorShot* Shot = nullptr;
	if (InFrameData->SampleState.OutputState.ShotIndex >= 0 && InFrameData->SampleState.OutputState.ShotIndex < ActiveShotList.Num())
	{
		Shot = ActiveShotList[InFrameData->SampleState.OutputState.ShotIndex];
	}

	if (Shot)
	{
		FormatOverrides.Add(TEXT("shot_name"), Shot->OuterName);
		FormatOverrides.Add(TEXT("camera_name"), Shot->GetCameraName(InFrameData->SampleState.OutputState.CameraIndex));
	}
	FMoviePipelineFormatArgs FinalFormatArgs;

	FString FinalFilePath;
	ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, FinalFilePath, FinalFormatArgs);

	TileImageTask->Filename = FinalFilePath;

	// Duplicate the data so that the Image Task can own it.
	TileImageTask->PixelData = MoveTemp(OutputSample);
	ImageWriteQueue->Enqueue(MoveTemp(TileImageTask));
}



void UMoviePipeline::FlushAsyncEngineSystems()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_MoviePipelineFlushAsyncEngineSystems);

	// Flush Block until Level Streaming completes. This solves the problem where levels that are not controlled
	// by the Sequencer Level Visibility track are marked for Async Load by a gameplay system.
	// This will register any new actors/components that were spawned during this frame. This needs 
	// to be done before the shader compiler is flushed so that we compile shaders for any newly
	// spawned component materials.
	if (GetWorld())
	{
		GetWorld()->BlockTillLevelStreamingCompleted();
	}

	const bool bDisableShaderFlushing = CVarMoviePipelineDisableShaderFlushing.GetValueOnGameThread();
	if (!bDisableShaderFlushing)
	{
		// Ensure we have complete shader maps for all materials used by primitives in the world.
		// This way we will never render with the default material.
		UMaterialInterface::SubmitRemainingJobsForWorld(GetWorld());

		// Flush all assets still being compiled asynchronously.
		// A progressbar is already in place so the user can get feedback while waiting for everything to settle.
		FAssetCompilingManager::Get().FinishAllCompilation();
	}

	// Flush streaming managers
	{
		UMoviePipelineGameOverrideSetting* GameOverrideSettings = FindOrAddSettingForShot<UMoviePipelineGameOverrideSetting>(ActiveShotList[CurrentShotIndex]);
		if (GameOverrideSettings && GameOverrideSettings->bFlushStreamingManagers)
		{
			FStreamingManagerCollection& StreamingManagers = IStreamingManager::Get();
			StreamingManagers.UpdateResourceStreaming(GetWorld()->GetDeltaSeconds(), /* bProcessEverything */ true);
			StreamingManagers.BlockTillAllRequestsFinished();
		}
	}

	// Flush grass
	if (CurrentShotIndex < ActiveShotList.Num())
	{
		UMoviePipelineGameOverrideSetting* GameOverrides = FindOrAddSettingForShot<UMoviePipelineGameOverrideSetting>(ActiveShotList[CurrentShotIndex]);
		if (GameOverrides && GameOverrides->bFlushGrassStreaming)
		{
			if (ULandscapeSubsystem* LandscapeSubsystem = GetWorld()->GetSubsystem<ULandscapeSubsystem>())
			{
				LandscapeSubsystem->RegenerateGrass(/*bInFlushGrass = */false, /*bInForceSync = */true);
			}
		}
	}

	UE::RenderCommandPipe::FSyncScope SyncScope;

	// Flush virtual texture tile calculations
	ERHIFeatureLevel::Type FeatureLevel = GetWorld()->GetFeatureLevel();
	ENQUEUE_RENDER_COMMAND(VirtualTextureSystemFlushCommand)(
		[FeatureLevel](FRHICommandListImmediate& RHICmdList)
	{
		GetRendererModule().LoadPendingVirtualTextureTiles(RHICmdList, FeatureLevel);
	});
}

#undef LOCTEXT_NAMESPACE // "MoviePipeline"
