// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineSetting.h"
#include "MoviePipeline.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineSetting)

UMoviePipelineSetting::UMoviePipelineSetting()
	: bEnabled(true)
{
}
void UMoviePipelineSetting::OnMoviePipelineInitialized(UMoviePipeline* InPipeline)
{
	CachedPipeline = MakeWeakObjectPtr(InPipeline);
	SetupForPipelineImpl(InPipeline);
}

UMoviePipeline* UMoviePipelineSetting::GetPipeline() const
{
	// Can be null if we're not in the middle of a render (ie in the UI)
	return CachedPipeline.Get();
}

UWorld* UMoviePipelineSetting::GetWorld() const
{
	// GetWorld is occasionally called by other systems (to check if the object is 
	// in the correct world to be relevant) which can be before a pipeline is made.
	if (CachedPipeline.Get())
	{
		return GetPipeline()->GetWorld();
	}

	return Super::GetWorld();
}

#if WITH_EDITOR
void UMoviePipelineSetting::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	ValidateState();
}
#endif

TArray<FText> UMoviePipelineSetting::GetValidationResults() const
{
	if (ValidationResults.Num() > 0)
	{
		ensureMsgf(GetValidationState() != EMoviePipelineValidationState::Valid, TEXT("A setting should not provide a validation result without setting the state to Warning/Error."));
	}

	return ValidationResults;
}
