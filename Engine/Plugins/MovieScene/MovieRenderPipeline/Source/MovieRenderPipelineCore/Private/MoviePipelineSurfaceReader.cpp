// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineSurfaceReader.h"
#include "RenderingThread.h"
#include "RHICommandList.h"
#include "RHIStaticStates.h"
#include "CommonRenderResources.h"
#include "RenderTargetPool.h"
#include "ScreenRendering.h"
#include "GlobalShader.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "Modules/ModuleManager.h"
#include "MovieRenderPipelineCoreModule.h"

DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_SurfaceReadback"), STAT_MoviePipeline_SurfaceReadback, STATGROUP_MoviePipeline);

FMoviePipelineSurfaceReader::FMoviePipelineSurfaceReader(EPixelFormat InPixelFormat, bool bInInvertAlpha)
{
	AvailableEvent = nullptr;
	ReadbackTexture = nullptr;
	PixelFormat = InPixelFormat;
	bQueuedForCapture = false;
	bInvertAlpha = bInInvertAlpha;
}

FMoviePipelineSurfaceReader::~FMoviePipelineSurfaceReader()
{
	BlockUntilAvailable();

	ReadbackTexture = nullptr;
}

void FMoviePipelineSurfaceReader::Initialize()
{
	// Initialize shouldn't be called if we're currently not available.
	check(!AvailableEvent);
	AvailableEvent = FPlatformProcess::GetSynchEventFromPool();
}

void FMoviePipelineSurfaceReader::ResizeImpl(uint32 Width, uint32 Height)
{
	ReadbackTexture = nullptr;
	Size = FIntPoint((int32)Width, (int32)Height);

	// TexCreate_CPUReadback is important to make this texture available for reading later.
	ENQUEUE_RENDER_COMMAND(CreateCaptureFrameTexture)(
		[SurfaceReader = SharedThis(this)](FRHICommandListImmediate& RHICmdList)
		{
			SurfaceReader->ReadbackTexture = MakeUnique<FRHIGPUTextureReadback>(TEXT("FMoviePipelineSurfaceReader"));
		});
}

void FMoviePipelineSurfaceReader::BlockUntilAvailable()
{
	if (AvailableEvent)
	{
		// Make this thread wait until another thread (ie: Rendering Thread) calls Trigger on the event.
		AvailableEvent->Wait(~0);

		// Return the sync event until we call Initialize again.
		FPlatformProcess::ReturnSynchEventToPool(AvailableEvent);
		AvailableEvent = nullptr;
	}
}

void FMoviePipelineSurfaceReader::Reset()
{
	if (AvailableEvent)
	{
		AvailableEvent->Trigger();
	}
	BlockUntilAvailable();
	bQueuedForCapture = false;
}

void FMoviePipelineSurfaceReader::ResolveSampleToReadbackTexture_RenderThread(FRHICommandListImmediate& RHICmdList, const FTextureRHIRef& SourceSurfaceSample)
{
	// We use GetModuleChecked here to avoid accidentally loading the module from the non-main thread.
	static const FName RendererModuleName("Renderer");
	IRendererModule* RendererModule = &FModuleManager::GetModuleChecked<IRendererModule>(RendererModuleName);

	bQueuedForCapture = true;

	// Retrieve a temporary render target that matches the destination surface type/size.
	const FIntPoint TargetSize(Size);

	FPooledRenderTargetDesc OutputDesc = FPooledRenderTargetDesc::Create2DDesc(
		TargetSize,
		PixelFormat,
		FClearValueBinding::None,
		TexCreate_None,
		TexCreate_RenderTargetable,
		false);
	TRefCountPtr<IPooledRenderTarget> ResampleTexturePooledRenderTarget;
	GRenderTargetPool.FindFreeElement(RHICmdList, OutputDesc, ResampleTexturePooledRenderTarget, TEXT("ResampleTexture"));
	check(ResampleTexturePooledRenderTarget);

	RHICmdList.Transition(FRHITransitionInfo(ResampleTexturePooledRenderTarget->GetRHI(), ERHIAccess::Unknown, ERHIAccess::RTV));

	// Enqueue a render pass which uses a simple shader and a fullscreen quad to copy the SourceSurfaceSample to this SurfaceReader's cpu-readback enabled texture.
	// This RenderPass resolves to our ReadbackTexture 
	FRHIRenderPassInfo RPInfo(ResampleTexturePooledRenderTarget->GetRHI(), ERenderTargetActions::Load_Store);
	RHICmdList.BeginRenderPass(RPInfo, TEXT("MoviePipelineSurfaceResolveRenderTarget"));
	{
		RHICmdList.SetViewport(0, 0, 0.0f, TargetSize.X, TargetSize.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);
		TShaderMapRef<FScreenVS> VertexShader(ShaderMap);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		if (bInvertAlpha)
		{
			TShaderMapRef<FScreenPSInvertAlpha> PixelShader(ShaderMap);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			// Bind the SourceSurfaceSample to our texture sampler with a point sample (since the resolutions match).
			SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), SourceSurfaceSample);
		}
		else
		{
			TShaderMapRef<FScreenPS> PixelShader(ShaderMap);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			// Bind the SourceSurfaceSample to our texture sampler with a point sample (since the resolutions match).
			SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), SourceSurfaceSample);
		}

		RendererModule->DrawRectangle(
			RHICmdList,
			0, 0,									// Dest X, Y
			TargetSize.X,							// Dest Width
			TargetSize.Y,							// Dest Height
			0, 0,									// Source U, V
			1, 1,									// Source USize, VSize
			TargetSize,								// Target buffer size
			FIntPoint(1, 1),						// Source texture size
			VertexShader,
			EDRF_Default);
	}
	RHICmdList.EndRenderPass();

	RHICmdList.Transition(FRHITransitionInfo(ResampleTexturePooledRenderTarget->GetRHI(), ERHIAccess::RTV, ERHIAccess::CopySrc));
	ReadbackTexture->EnqueueCopy(RHICmdList, ResampleTexturePooledRenderTarget->GetRHI());
}

void FMoviePipelineSurfaceReader::CopyReadbackTexture_RenderThread(FRHICommandListImmediate& RHICmdList, TUniqueFunction<void(TUniquePtr<FImagePixelData>&&)>&& InFunctionCallback, TSharedPtr<FImagePixelDataPayload, ESPMode::ThreadSafe> InFramePayload)
{
	// We use GetModuleChecked here to avoid accidentally loading the module from the non-main thread.
	static const FName RendererModuleName("Renderer");
	IRendererModule* RendererModule = &FModuleManager::GetModuleChecked<IRendererModule>(RendererModuleName);

	{
#if WITH_MGPU
		FRHIGPUMask GPUMask = RHICmdList.GetGPUMask();

		// If GPUMask is not set to a specific GPU we and since we are reading back the texture, it shouldn't matter which GPU we do this on.
		if (!GPUMask.HasSingleIndex())
		{
			GPUMask = FRHIGPUMask::FromIndex(GPUMask.GetFirstIndex());
		}

		SCOPED_GPU_MASK(RHICmdList, GPUMask);
#endif

		int32 ActualSizeX = 0, ActualSizeY = 0;
		void* ColorDataBuffer = ReadbackTexture->Lock(ActualSizeX, &ActualSizeY);

		int32 ExpectedSizeX = Size.X;
		int32 ExpectedSizeY = Size.Y;

		TUniquePtr<FImagePixelData> PixelData;
		uint8* TypeErasedPixels = nullptr;
		int32 SizeOfColor = 0;
		switch (PixelFormat)
		{
		case EPixelFormat::PF_FloatRGBA:
		{
			TUniquePtr<TImagePixelData<FFloat16Color>> NewPixelData = MakeUnique < TImagePixelData<FFloat16Color>>(FIntPoint(Size.X, Size.Y), InFramePayload);
			NewPixelData->Pixels.SetNumUninitialized(ExpectedSizeX * ExpectedSizeY);
			SizeOfColor = sizeof(FFloat16Color);
			TypeErasedPixels = reinterpret_cast<uint8*>(NewPixelData->Pixels.GetData());
			PixelData = MoveTemp(NewPixelData);
			break;
		}
		case EPixelFormat::PF_B8G8R8A8:
		{
			TUniquePtr<TImagePixelData<FColor>> NewPixelData = MakeUnique < TImagePixelData<FColor>>(FIntPoint(Size.X, Size.Y), InFramePayload);
			NewPixelData->Pixels.SetNumUninitialized(ExpectedSizeX * ExpectedSizeY);
			SizeOfColor = sizeof(FColor);
			TypeErasedPixels = reinterpret_cast<uint8*>(NewPixelData->Pixels.GetData());
			PixelData = MoveTemp(NewPixelData);
			break;
		}
		default:
			check(0); // Unsupported, add a new switch statement.
		}


		// Due to padding, the actual size might be larger than the expected size. If they are the same, do a block copy. Otherwise copy
		// line by line.
		if (ExpectedSizeX == ActualSizeX && ExpectedSizeY == ActualSizeY)
		{
			FMemory::BigBlockMemcpy(TypeErasedPixels, ColorDataBuffer, (int64)ExpectedSizeX * (int64)ExpectedSizeY * SizeOfColor);
		}
		else
		{
			// Make sure the target is larger than expected size.
			check(ExpectedSizeX <= ActualSizeX);
			check(ExpectedSizeY <= ActualSizeY);

			int32 SrcPitchElem = ActualSizeX;
			int32 DstPitchElem = ExpectedSizeX;

			const uint8* SrcColorData = reinterpret_cast<const uint8*>(ColorDataBuffer);

			// Copy one line at a time
			for (int32 RowIndex = 0; RowIndex < ExpectedSizeY; RowIndex++)
			{
				const void* SrcPtr = SrcColorData + RowIndex * SrcPitchElem * SizeOfColor;
				void* DstPtr = TypeErasedPixels + RowIndex * DstPitchElem * SizeOfColor;
				FMemory::Memcpy(DstPtr, SrcPtr, DstPitchElem * SizeOfColor);
			}
		}

		// Enqueue the Unmap before we broadcast the resulting pixels, though the broadcast shouldn't do anything blocking.
		ReadbackTexture->Unlock();

		InFunctionCallback(MoveTemp(PixelData));

		// Now that we've successfully used the surface, we trigger the Available event so that we can reuse this surface. This
		// triggers the Available event and then returns it to the pool.
		Reset();
	}
}

FMoviePipelineSurfaceQueue::FMoviePipelineSurfaceQueue(FIntPoint InSurfaceSize, EPixelFormat InPixelFormat, uint32 InNumSurfaces, bool bInInvertAlpha)
{
	// TargetSize = InSurfaceSize;
	CurrentFrameIndex = 0;
	check(InNumSurfaces > 0);
		
	// The Surface array can't be reallocated as we're storing indexes into it.
	Surfaces.Reserve(InNumSurfaces);
	for (uint32 Index = 0; Index < InNumSurfaces; Index++)
	{
		Surfaces.Emplace(InPixelFormat, InSurfaceSize, bInInvertAlpha);
	}

	FrameResolveLatency = 1;

	UpdateLastUsedFrame();
}

FMoviePipelineSurfaceQueue::~FMoviePipelineSurfaceQueue()
{
}

void FMoviePipelineSurfaceQueue::BlockUntilAnyAvailable()
{
	bool bAnyAvailable = false;
	for(FResolveSurface& ResolveSurface : Surfaces)
	{
		if (ResolveSurface.Surface->IsAvailable())
		{
			bAnyAvailable = true;
			break;
		}
	}

	// We only wait if none of them are available, and the oldest one is the most likely one to
	// be available. 
	if (!bAnyAvailable)
	{
		const int32 OldestIndex = (CurrentFrameIndex + 1) % Surfaces.Num();
		Surfaces[OldestIndex].Surface->BlockUntilAvailable();
	}
}

void FMoviePipelineSurfaceQueue::Shutdown()
{
	for (int32 Index = 0; Index < Surfaces.Num(); Index++)
	{
		FResolveSurface* ResolveSurface = &Surfaces[Index];
		ENQUEUE_RENDER_COMMAND(PerformReadback)(
			[ResolveSurface](FRHICommandListImmediate& RHICmdList)
			{
				// Ensure that all surfaces have queued up their readback.
				if (ResolveSurface->Surface->WasEverQueued())
				{
					ResolveSurface->Surface->CopyReadbackTexture_RenderThread(RHICmdList, MoveTemp(ResolveSurface->FunctionCallback), ResolveSurface->FunctionPayload);
				}
			});
	}

	// Flush the rendering thread so that the readbacks happen and trigger AvailableEvents.
	FlushRenderingCommands();

	for (FResolveSurface& ResolveSurface : Surfaces)
	{
		// These should now all be available.
		ensureMsgf(ResolveSurface.Surface->IsAvailable(), TEXT("Flushed rendering commands but surface reader didn't perform the readback on a surface!"));
	}
}

void FMoviePipelineSurfaceQueue::UpdateLastUsedFrame()
{
	LastUsedFrame = GFrameCounter;
}

bool FMoviePipelineSurfaceQueue::IsStale() const
{
	constexpr int32 StaleThreshold = 3;
	return GFrameCounter - LastUsedFrame > StaleThreshold;
}

void FMoviePipelineSurfaceQueue::OnRenderTargetReady_RenderThread(const FTextureRHIRef InRenderTarget, TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> InFramePayload, TUniqueFunction<void(TUniquePtr<FImagePixelData>&&)>&& InFunctionCallback)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	// Pick the next destination surface and ensure it's available.
	FResolveSurface* NextResolveTarget = &Surfaces[CurrentFrameIndex];
	if (!NextResolveTarget->Surface->IsAvailable())
	{
		// BlockUntilAnyAvailable should have been called before submitting more work. We can't block
		// until a surface is available in this callback because we'd be waiting on the render thread
		// from the render thread.
		check(false);
	}

	NextResolveTarget->Surface->Initialize();
	NextResolveTarget->FunctionCallback = MoveTemp(InFunctionCallback);
	NextResolveTarget->FunctionPayload = InFramePayload;

	// Queue this sample to be copied to the target surface.
	NextResolveTarget->Surface->ResolveSampleToReadbackTexture_RenderThread(RHICmdList, InRenderTarget);

	// By the time we get to this point, our oldest surface should have successfully been rendered to, and no longer be in use by the GPU.
	// We can now safely map the surface and copy the data out of it without causing a GPU stall.
	{
		const int32 PrevCaptureIndexOffset = FMath::Clamp(FrameResolveLatency, 0, Surfaces.Num() - 1);

		// Get PrevCaptureIndexOffset surfaces back, handling wraparound at 0.
		const int32 PrevCaptureIndex = (CurrentFrameIndex - PrevCaptureIndexOffset) < 0 ? Surfaces.Num() - (PrevCaptureIndexOffset - CurrentFrameIndex) : (CurrentFrameIndex - PrevCaptureIndexOffset);

		FResolveSurface* OldestResolveTarget = &Surfaces[PrevCaptureIndex];

		// Only try to do the readback if the target has ever been written to.
		if (OldestResolveTarget->Surface->WasEverQueued())
		{
			SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_SurfaceReadback);
			OldestResolveTarget->Surface->CopyReadbackTexture_RenderThread(RHICmdList, MoveTemp(OldestResolveTarget->FunctionCallback), OldestResolveTarget->FunctionPayload);
		}
	}


	// Write to the next available surface next time.
	CurrentFrameIndex = (CurrentFrameIndex + 1) % Surfaces.Num();
}
