// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineTelemetry.h"

#include "EngineAnalytics.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphQuickRenderSettings.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelineQueue.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

void FMoviePipelineEndShotRenderTelemetryContext::MergeRenderPassTelemetry(const FMoviePipelineEndShotRenderTelemetryContext& InTelemetry)
{
	bUsesAccumulationDepthOfField |= InTelemetry.bUsesAccumulationDepthOfField;
	AccumulationDepthOfFieldSampleCount = FMath::Max(AccumulationDepthOfFieldSampleCount, InTelemetry.AccumulationDepthOfFieldSampleCount);
	AccumulationDepthOfFieldSplatSize = FMath::Max(AccumulationDepthOfFieldSplatSize, InTelemetry.AccumulationDepthOfFieldSplatSize);
}

void FMoviePipelineTelemetry::SendRendersRequestedTelemetry(const bool bIsLocal, TArray<UMoviePipelineExecutorJob*>&& InJobs)
{
	bIsCurrentRenderRequestQuickRender = false;
	
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	int32 EnabledJobCount = 0;
	int32 DisabledJobCount = 0;
	int32 EnabledShotCount = 0;
	int32 DisabledShotCount = 0;

	for (const UMoviePipelineExecutorJob* Job : InJobs)
	{
		if (Job->IsEnabled())
		{
			++EnabledJobCount;
		}
		else
		{
			++DisabledJobCount;
		}

		for (const TObjectPtr<UMoviePipelineExecutorShot>& Shot : Job->ShotInfo)
		{
			if (Shot->bEnabled && Job->IsEnabled())
			{
				++EnabledShotCount;
			}
			else
			{
				++DisabledShotCount;
			}
		}
	}
	
	TArray<FAnalyticsEventAttribute> EventAttributes;
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("IsLocal"), bIsLocal));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SessionType"), GetSessionType()));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("EnabledJobCount"), EnabledJobCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("DisabledJobCount"), DisabledJobCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("EnabledShotCount"), EnabledShotCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("DisabledShotCount"), DisabledShotCount));

	FEngineAnalytics::GetProvider().RecordEvent(TEXT("MoviePipeline.RendersRequested"), EventAttributes);
}

void FMoviePipelineTelemetry::SendQuickRenderRequestedTelemetry(const EMovieGraphQuickRenderMode QuickRenderMode)
{
	bIsCurrentRenderRequestQuickRender = true;
	
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	UMovieGraphQuickRenderModeSettings* ModeSettings = UMovieGraphQuickRenderSettings::GetSavedQuickRenderModeSettings(QuickRenderMode);

	TArray<FAnalyticsEventAttribute> EventAttributes;
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Mode"),
		StaticEnum<EMovieGraphQuickRenderMode>()->GetNameStringByValue(static_cast<int64>(QuickRenderMode))));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("PlayAfterRenderMode"),
		StaticEnum<EMoviePipelinePostRenderActionType>()->GetNameStringByValue(static_cast<int64>(ModeSettings->PostRenderBehavior))));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ViewportLookFlags"),
		StaticEnum<EMovieGraphQuickRenderViewportLookFlags>()->GetValueOrBitfieldAsString(ModeSettings->ViewportLookFlags)));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SequencerFrameRange"),
		StaticEnum<EMovieGraphQuickRenderFrameRangeType>()->GetNameStringByValue(static_cast<int64>(ModeSettings->FrameRangeType))));

	FEngineAnalytics::GetProvider().RecordEvent(TEXT("MoviePipeline.QuickRenderRequested"), EventAttributes);
}

/** Sends out telemetry that includes information about the shot being rendered (the type of rendering being done, which types of settings/nodes are being used, etc). */
void FMoviePipelineTelemetry::SendBeginShotRenderTelemetry(const FMoviePipelineBeginShotRenderTelemetryContext& InContext)
{
	if (!FEngineAnalytics::IsAvailable())
    {
        return;
    }
	
	const FMoviePipelineShotRenderTelemetry ShotRenderTelemetry = InContext.EvaluatedConfig
		? GatherShotRenderTelemetryForGraph(InContext.Shot, InContext.EvaluatedConfig, InContext.bIsBasicConfig)
		: GatherShotRenderTelemetryForLegacy(InContext.Shot);
	
	TArray<FAnalyticsEventAttribute> EventAttributes;
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("IsGraph"), ShotRenderTelemetry.bIsGraph));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("IsBasicConfig"), ShotRenderTelemetry.bIsBasicConfig));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("IsQuickRender"), ShotRenderTelemetry.bIsQuickRender));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesDeferred"), ShotRenderTelemetry.bUsesDeferred));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesPathTracer"), ShotRenderTelemetry.bUsesPathTracer));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesPanoramic"), ShotRenderTelemetry.bUsesPanoramic));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesHighResTiling"), ShotRenderTelemetry.bUsesHighResTiling));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesNDisplay"), ShotRenderTelemetry.bUsesNDisplay));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesObjectID"), ShotRenderTelemetry.bUsesObjectID));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesUIRenderer"), ShotRenderTelemetry.bUsesUIRenderer));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesMultiCamera"), ShotRenderTelemetry.bUsesMultiCamera));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesScripting"), ShotRenderTelemetry.bUsesScripting));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesSubgraphs"), ShotRenderTelemetry.bUsesSubgraphs));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesPPMs"), ShotRenderTelemetry.bUsesPPMs));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesAudio"), ShotRenderTelemetry.bUsesAudio));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("MovieEncodesAudio"), ShotRenderTelemetry.bMovieEncodesAudio));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesCommandLineEncoder"), ShotRenderTelemetry.bUsesCommandLineEncoder));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SetsConsoleVariables"), ShotRenderTelemetry.bSetsConsoleVariables));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SetsQueueConsoleVariables"), ShotRenderTelemetry.bSetsQueueConsoleVariables));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesConsoleVariablePreset"), ShotRenderTelemetry.bUsesConsoleVariablePreset));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesModifier"), ShotRenderTelemetry.bUsesModifier));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesLightModifier"), ShotRenderTelemetry.bUsesLightModifier));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesMpcModifier"), ShotRenderTelemetry.bUsesMpcModifier));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesAvid"), ShotRenderTelemetry.bUsesAvid));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesProRes"), ShotRenderTelemetry.bUsesProRes));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesMP4"), ShotRenderTelemetry.bUsesMP4));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesJPG"), ShotRenderTelemetry.bUsesJPG));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesPNG"), ShotRenderTelemetry.bUsesPNG));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesBMP"), ShotRenderTelemetry.bUsesBMP));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesEXR"), ShotRenderTelemetry.bUsesEXR));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesMultiEXR"), ShotRenderTelemetry.bUsesMultiEXR));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesTmvMovie"), ShotRenderTelemetry.bUsesTmvMovie));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesTmvImageSequence"), ShotRenderTelemetry.bUsesTmvImageSequence));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesMotionDesignRundown"), ShotRenderTelemetry.bUsesMotionDesignRundown));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionX"), ShotRenderTelemetry.ResolutionX));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionY"), ShotRenderTelemetry.ResolutionY));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("HandleFrameCount"), ShotRenderTelemetry.HandleFrameCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TemporalSampleCount"), ShotRenderTelemetry.TemporalSampleCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SpatialSampleCount"), ShotRenderTelemetry.SpatialSampleCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("WarmUpFrames"), ShotRenderTelemetry.WarmUpFrames));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("LayerWarmUpFrames"), ShotRenderTelemetry.LayerWarmUpFrames));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("RenderLayerCount"), ShotRenderTelemetry.RenderLayerCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("HighResTileCount"), ShotRenderTelemetry.HighResTileCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("HighResOverlap"), ShotRenderTelemetry.HighResOverlap));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("PageToSystemMemory"), ShotRenderTelemetry.bUsesPageToSystemMemory));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ScalabilityQualityLevel"), ShotRenderTelemetry.ScalabilityQualityLevel));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("AntiAliasingType"), ShotRenderTelemetry.AntiAliasingType));

	FEngineAnalytics::GetProvider().RecordEvent(TEXT("MoviePipeline.BeginShotRender"), EventAttributes);
}

/** Sends out telemetry that indicates whether the shot was rendered successfully. */
void FMoviePipelineTelemetry::SendEndShotRenderTelemetry(const FMoviePipelineEndShotRenderTelemetryContext& InContext)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	TArray<FAnalyticsEventAttribute> EventAttributes;
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("IsGraph"), InContext.bIsGraph));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("IsBasicConfig"), InContext.bIsBasicConfig));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("IsQuickRender"), bIsCurrentRenderRequestQuickRender));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("WasSuccessful"), InContext.bWasSuccessful));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("WasCanceled"), InContext.bWasCanceled));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TotalRenderedFrameCount"), InContext.TotalRenderedFrameCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesAccumulationDepthOfField"), InContext.bUsesAccumulationDepthOfField));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("AccumulationDepthOfFieldSampleCount"), InContext.AccumulationDepthOfFieldSampleCount));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("AccumulationDepthOfFieldSplatSize"), InContext.AccumulationDepthOfFieldSplatSize));

	FEngineAnalytics::GetProvider().RecordEvent(TEXT("MoviePipeline.EndShotRender"), EventAttributes);
}

FString FMoviePipelineTelemetry::GetSessionType()
{
	FString SessionType;

#if WITH_EDITOR
	if (GEditor != nullptr)
	{
		SessionType = TEXT("Editor");
	}
	else
	{
		SessionType = TEXT("DashGame");
	}
#else
	SessionType = TEXT("Shipping");
#endif

	return SessionType;
}

FMoviePipelineShotRenderTelemetry FMoviePipelineTelemetry::GatherShotRenderTelemetryForGraph(const UMoviePipelineExecutorShot* InShot, UMovieGraphEvaluatedConfig* InEvaluatedConfig, const bool bIsBasicConfig)
{
	FMoviePipelineShotRenderTelemetry Telemetry;
	Telemetry.bIsGraph = !bIsBasicConfig;	// Basic configs are technically graphs, but only bIsGraph or bIsBasicConfig should be true
	Telemetry.bIsBasicConfig = bIsBasicConfig;
	Telemetry.bIsQuickRender = bIsCurrentRenderRequestQuickRender;

	const FIntPoint Resolution = UMovieGraphBlueprintLibrary::GetOverscannedResolution(InEvaluatedConfig);
	Telemetry.ResolutionX = Resolution.X;
	Telemetry.ResolutionY = Resolution.Y;

	const auto HasEnabledConsoleVariableOverrides = [](const TArray<FMoviePipelineConsoleVariableEntry>& ConsoleVariableOverrides)
	{
		for (const FMoviePipelineConsoleVariableEntry& ConsoleVariableOverride : ConsoleVariableOverrides)
		{
			if (ConsoleVariableOverride.bIsEnabled)
			{
				return true;
			}
		}

		return false;
	};

	// Determine if any queue-level console variable overrides were set on the shot or job
	Telemetry.bSetsQueueConsoleVariables = HasEnabledConsoleVariableOverrides(InShot->ConsoleVariableOverrides);
	if (!Telemetry.bSetsQueueConsoleVariables)
	{
		if (const UMoviePipelineExecutorJob* ParentJob = InShot->GetTypedOuter<UMoviePipelineExecutorJob>())
		{
			Telemetry.bSetsQueueConsoleVariables = HasEnabledConsoleVariableOverrides(ParentJob->ConsoleVariableOverrides);
		}
	}

	// The evaluated graph won't include subgraph nodes, so peek into the non-evaluated graph(s) to see if there are any in there
	{
		TSet<UMovieGraphConfig*> Subgraphs;
		if (const UMovieGraphConfig* ShotGraphConfig = InShot->GetGraphPreset())
		{
			ShotGraphConfig->GetAllContainedSubgraphs(Subgraphs);
		}

		if (Subgraphs.IsEmpty())
		{
			if (const UMoviePipelineExecutorJob* ParentJob = InShot->GetTypedOuter<UMoviePipelineExecutorJob>())
			{
				if (const UMovieGraphConfig* JobGraphConfig = ParentJob->GetGraphPreset())
				{
					JobGraphConfig->GetAllContainedSubgraphs(Subgraphs);
				}
			}
		}

		Telemetry.bUsesSubgraphs = !Subgraphs.IsEmpty();
	}

	// Iterate through all of the nodes on every branch. Ask the nodes to update the telemetry data.
	for (const TPair<FName, FMovieGraphEvaluatedBranchConfig>& BranchMapping : InEvaluatedConfig->BranchConfigMapping)
	{
		for (const TObjectPtr<UMovieGraphNode>& Node : BranchMapping.Value.GetNodes())
		{
			const UMovieGraphSettingNode* SettingNode = Cast<UMovieGraphSettingNode>(Node);
			if (!SettingNode || SettingNode->IsDisabled())
			{
				continue;
			}

			SettingNode->UpdateTelemetry(&Telemetry);
		}
	}

	return Telemetry;
}

FMoviePipelineShotRenderTelemetry FMoviePipelineTelemetry::GatherShotRenderTelemetryForLegacy(UMoviePipelineExecutorShot* InShot)
{
	FMoviePipelineShotRenderTelemetry Telemetry;
	Telemetry.bIsGraph = false;

	UMoviePipelinePrimaryConfig* PrimaryConfig = InShot->GetTypedOuter<UMoviePipelineExecutorJob>()->GetConfiguration();
	const FIntPoint Resolution = UMoviePipelineBlueprintLibrary::GetOverscannedResolution(PrimaryConfig, InShot);
	Telemetry.ResolutionX = Resolution.X;
	Telemetry.ResolutionY = Resolution.Y;
	
	// Merge settings from the primary and shot configs
	TArray<UMoviePipelineSetting*> AllSettings = PrimaryConfig->GetAllSettings();
	if (const UMoviePipelineShotConfig* ShotConfig = InShot->GetShotOverrideConfiguration())
	{
		AllSettings.Append(ShotConfig->GetUserSettings());
	}

	// Iterate through all of the settings. Ask the setting to update the telemetry data.
	for (const UMoviePipelineSetting* Setting : AllSettings)
	{
		if (!Setting || !Setting->IsEnabled())
		{
			continue;
		}

		Setting->UpdateTelemetry(&Telemetry);
	}

	return Telemetry;
}