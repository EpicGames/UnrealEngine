// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipeline.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineAntiAliasingSetting.h"
#include "MoviePipelineShotConfig.h"
#include "MoviePipelineOutputBase.h"
#include "CoreGlobals.h"
#include "Containers/Array.h"
#include "Misc/ScopeExit.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieScene.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/WorldSettings.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineCameraSetting.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineOutputSetting.h"

#if WITH_EDITOR
#include "MoviePipelineDebugSettings.h"
#endif
void UMoviePipeline::ProcessOutstandingFutures()
{
	// Check if any frames failed to output
	TArray<int32> CompletedOutputFutures;
	for (int32 Index = 0; Index < OutputFutures.Num(); ++Index)
	{
		// Output futures are pushed in order to the OutputFutures array. However they are
		// completed asyncronously, so we don't process any futures after a not-yet-ready one
		// otherwise we push into the GeneratedShotOutputData array out of order.
		const FMoviePipelineOutputFuture& OutputFuture = OutputFutures[Index];
		if (!OutputFuture.Get<0>().IsReady())
		{
			break;
		}

		CompletedOutputFutures.Add(Index);

		const MoviePipeline::FMoviePipelineOutputFutureData& FutureData = OutputFuture.Get<1>();

		// The future was completed, time to add it to our shot output data.
		FMoviePipelineShotOutputData* ShotOutputData = nullptr;
		for (int32 OutputDataIndex = 0; OutputDataIndex < GeneratedShotOutputData.Num(); OutputDataIndex++)
		{
			if (FutureData.Shot == GeneratedShotOutputData[OutputDataIndex].Shot)
			{
				ShotOutputData = &GeneratedShotOutputData[OutputDataIndex];
			}
		}

		if (!ShotOutputData)
		{
			GeneratedShotOutputData.Add(FMoviePipelineShotOutputData());
			ShotOutputData = &GeneratedShotOutputData.Last();
			ShotOutputData->Shot = FutureData.Shot;
		}

		// Add the filepath to the renderpass data.
		FMoviePipelineRenderPassOutputData& OutputData = ShotOutputData->RenderPassData.FindOrAdd(FutureData.PassIdentifier);
		OutputData.FilePaths.Add(FutureData.FilePath);
		OutputData.FrameTemplatedFilePaths.AddUnique(FutureData.FrameTemplatedFilePath);

		FMoviePipelineStringArray& FilesFromSettingClass = OutputData.SettingTypeToFilePaths.FindOrAdd(FutureData.OriginSettingClass);
		FilesFromSettingClass.Array.Add(FutureData.FilePath);

		if (!OutputFuture.Get<0>().Get())
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Error exporting frame, canceling movie export.");
			RequestShutdown(true);
			break;
		}
	}

	for (int32 Index = CompletedOutputFutures.Num() - 1; Index >= 0; --Index)
	{
		OutputFutures.RemoveAt(CompletedOutputFutures[Index]);
	}
}

void UMoviePipeline::TickProducingFrames()
{
	ON_SCOPE_EXIT
	{
		FMoviePipelineLightweightTickInfo TickInfo;
		TickInfo.bIsActive = true;
		TickInfo.TemporalSampleCount = CachedOutputState.TemporalSampleCount;
		TickInfo.TemporalSampleIndex = CachedOutputState.TemporalSampleIndex;
		TickInfo.SequenceFPS = GetPipelinePrimaryConfig()->GetEffectiveFrameRate(TargetSequence).AsDecimal();
		FMovieRenderPipelineCoreModule::SetTickInfo(TickInfo);
	};

	// The callback for this function does not get registered until Initialization has been called, which sets
	// the state to Render. If it's not, we have a initialization order/flow issue!
	check(PipelineState == EMovieRenderPipelineState::ProducingFrames);

	// We should not be calling this once we have completed all the shots.
	check(CurrentShotIndex >= 0 && CurrentShotIndex < ActiveShotList.Num());

	ProcessOutstandingFutures();

	if (bShutdownRequested)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "[GFrameCounter: %lld] Async Shutdown Requested, abandoning remaining work and moving to Finalize.", GFrameCounter);
		TransitionToState(EMovieRenderPipelineState::Finalize);
		return;
	}

	// When start up we want to override the engine's Custom Timestep with our own.
	// This gives us the ability to completely control the engine tick/delta time before the frame
	// is started so that we don't have to always be thinking of delta times one frame ahead. We need
	// to do this only once we're ready to set the timestep though, as Initialize can be called as
	// a result of a OnBeginFrame, meaning that Initialize is called on the frame before TickProducingFrames
	// so there would be one frame where it used the custom timestep (after initialize) before TPF was called.
	if (GEngine->GetCustomTimeStep() != CustomTimeStep)
	{
		CachedPrevCustomTimeStep = GEngine->GetCustomTimeStep();
		GEngine->SetCustomTimeStep(CustomTimeStep);
	}

	// If we're debug stepping one frame at a time, this will return true
	// and we early out so that we don't end up advancing the pipeline at all.
	// This handles setting the delta time for us to a fixed number.
	if (DebugFrameStepPreTick())
	{
		return;
	}
	
	// Make sure we're unpaused for the duration of this frame. We could have been paused by the frame stepper
	// on the last Tick.
	GetWorld()->GetFirstPlayerController()->SetPause(false);

	UMoviePipelineExecutorShot* CurrentCameraCut = ActiveShotList[CurrentShotIndex];

	const MoviePipeline::FFrameConstantMetrics FrameMetrics = CalculateShotFrameMetrics(CurrentCameraCut);

	// Reset our Output State so we ensure nothing gets unintentionally persisted between frames.
	CachedOutputState.ResetPerFrameData();

	// Check to see if we need to initialize a new shot. This potentially changes a lot of state.
	if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::Uninitialized)
	{
		// Solo our shot and create rendering resources.
		InitializeShot(CurrentCameraCut);

		CachedOutputState.ResetPerShotData();
		CachedOutputState.ShotIndex = CurrentShotIndex;
		CachedOutputState.ShotSamplesRendered = 0;
		CachedOutputState.ShotOutputFrameNumber = -1;
		CachedOutputState.TemporalSampleCount = CurrentCameraCut->ShotInfo.NumTemporalSamples;

		UE_LOGF(LogMovieRenderPipeline, Log, "[%lld] Initializing Camera Cut [%d/%d] in [%ls] %ls.", GFrameCounter,
			CurrentShotIndex + 1, ActiveShotList.Num(), *CurrentCameraCut->OuterName, *CurrentCameraCut->InnerName);
		
		CurrentCameraCut->ShotInfo.SetNextStateAfter(EMovieRenderShotState::Uninitialized);

		// The shot has been initialized and the new state chosen. Jump the Sequence to the starting time.
		// This means that the camera will be in the correct location for warm-up frames to allow any systems
		// dependent on camera position to properly warm up. If motion blur fixes are enabled, then we'll jump 
		// again after the warm-up frames.
		UMoviePipelineAntiAliasingSetting* AntiAliasingSettings = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(CurrentCameraCut);

		// If we're not emulating the first frame motion blur, we back up by an extra frame so that we can fall through to regular frame-advance logic.
		// However this causes the Sequencer to initially jump to/evaluate the wrong frame (where objects may not be spawned) and we need cameras, etc.
		// spawned so we can cache current viewpoints. So we store how much we're going to go back so that we can temporarily add it to the sequence eval
		// time, and then the actual eval will go forward as usual later.
		FFrameNumber SequencerEvalOffset = FFrameNumber(0);
		if (!CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur)
		{
			FFrameTime TicksToEndOfPreviousFrame;
			float WorldTimeDilation = GetWorld()->GetWorldSettings()->GetEffectiveTimeDilation();

			UMoviePipelineAntiAliasingSetting* AntiAliasing = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(CurrentCameraCut);
			if (AntiAliasing->TemporalSampleCount == 1)
			{
				TicksToEndOfPreviousFrame = FrameMetrics.TicksPerOutputFrame;
			}
			else
			{
				// The first sub-frame accumulation is going to try to go forward
				// by the amount the shutter is closed, plus the duration of one sample
				// because that is the logic we use for every other frame.
				TicksToEndOfPreviousFrame = FrameMetrics.TicksPerSample + FrameMetrics.TicksWhileShutterClosed;
			}

			if (!FMath::IsNearlyEqual(WorldTimeDilation, 1.f))
			{
				TicksToEndOfPreviousFrame = TicksToEndOfPreviousFrame * WorldTimeDilation;
			}

			// The above calculation will offset us by a portion of a frame. We then offset by the number of whole frames we need to back up by.
			// The warmup state (when using camera cut for warmup) will simply advance us an entire frame each time. This means that when we go
			// to render the first frame we will be on the correct offset.
			TicksToEndOfPreviousFrame += FFrameRate::TransformTime(FFrameTime(FFrameNumber(CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining)), TargetSequence->GetMovieScene()->GetDisplayRate(), TargetSequence->GetMovieScene()->GetTickResolution());
			AccumulatedTickSubFrameDeltas -= TicksToEndOfPreviousFrame.GetSubFrame();
			CurrentCameraCut->ShotInfo.CurrentTickInRoot = CurrentCameraCut->ShotInfo.CurrentTickInRoot - TicksToEndOfPreviousFrame.FloorToFrame();
			SequencerEvalOffset = TicksToEndOfPreviousFrame.FloorToFrame();
		}

		// Jump to the first frame of the sequence that we will be playing from. This doesn't take into account camera timing offset or temporal sampling, but that is
		// proably okay as it means the first frame (frame 0) is exactly evaluated which is easier to preview effects in the editor instead of -.25 of a frame.
		FFrameNumber CurrentRootSeqTick = CurrentCameraCut->ShotInfo.CurrentTickInRoot + SequencerEvalOffset;
		FFrameTime TimeInPlayRate = FFrameRate::TransformTime(CurrentRootSeqTick, TargetSequence->GetMovieScene()->GetTickResolution(), TargetSequence->GetMovieScene()->GetDisplayRate());

		// We tell it to jump so that things responding to different scrub-types work correctly.
		LevelSequenceActor->GetSequencePlayer()->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(TimeInPlayRate, EUpdatePositionMethod::Jump));
		CustomSequenceTimeController->SetCachedFrameTiming(FQualifiedFrameTime(CurrentRootSeqTick, TargetSequence->GetMovieScene()->GetTickResolution()));

		if (!CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur)
		{
			// Real warm up frames walk through the Sequence
			LevelSequenceActor->GetSequencePlayer()->Play();
		}
		else
		{
			// Ensure we don't try to evaluate as we want to sit and wait during warm up and motion blur frames.
			LevelSequenceActor->GetSequencePlayer()->Pause();
		}

		// Update the camera's previous/current location. On the first frame there is no previous so we just make them match.
		{
			APlayerController* LocalPlayerController = GetWorld()->GetFirstPlayerController();
			LocalPlayerController->GetPlayerViewPoint(FrameInfo.CurrViewLocation, FrameInfo.CurrViewRotation);
			
			FrameInfo.PrevViewLocation = FrameInfo.CurrViewLocation;
			FrameInfo.PrevViewRotation = FrameInfo.CurrViewRotation;

			// Cache data for all the sidecar cameras too.
			bool bSuccess = GetSidecarCameraViewPoints(CurrentCameraCut, FrameInfo.CurrSidecarViewLocations, FrameInfo.CurrSidecarViewRotations);
			FrameInfo.PrevSidecarViewLocations = FrameInfo.CurrSidecarViewLocations;
			FrameInfo.PrevSidecarViewRotations = FrameInfo.CurrSidecarViewRotations;

			// This warning is only relevant if they actually have a camera in Sequencer. If they don't,
			// it'll be falling back to the LocalPlayerController anyways (above) so it's not actually
			// the error we're trying to catch.
			if (CurrentCameraCut->ShotInfo.SubSectionHierarchy->CameraCutSection.IsValid())
			{
				ensureMsgf(bSuccess, TEXT("Failed to evaluate camera to create cur/prev camera locations. No camera actor found on Eval Tick: %d"), CurrentRootSeqTick.Value);
			}
		}

		// We delay setting up the rendering pipeline for the camera cut until the end of setup. This is because we want the Sequence to have evaluated at least once
		// (via the Jump command earlier) so that spawnable cameras exist during setup.
		SetupRenderingPipelineForShot(CurrentCameraCut);
		
		// We can safely fall through to the below states as they're OK to process the same frame we set up.
		UE_LOGF(LogMovieRenderPipeline, Log, "[%lld] Finished initializing Camera Cut [%d/%d] in [%ls] %ls.", GFrameCounter,
			CurrentShotIndex + 1, ActiveShotList.Num(), *CurrentCameraCut->OuterName, *CurrentCameraCut->InnerName);
	}
	
	// Do the WarmingUp State exit condition before the WarmingUp loop, to allow the correct WarmingUp state to
	// persist through to the post-frame render before switching to the next state.
	if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::WarmingUp && CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining == 0)
	{
		// We can either jump to the MotionBlur state or to the Rendering state, same as with coming from Uninitialized stage. 
		// We just let the shot decide to move on if it's appropriate (to avoid duplicating the code from Uninitialized)
		CurrentCameraCut->ShotInfo.SetNextStateAfter(EMovieRenderShotState::WarmingUp);
	}

	// If the shot is warming up then we will just pass a reasonable amount of time this frame (no output data will be produced).
	if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::WarmingUp)
	{
		UE_LOGF(LogMovieRenderPipeline, Verbose, "[%lld] Evaluating WarmUp frame %d. Remaining: %d", GFrameCounter, CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining, CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining - 1);

		// This if block will get called the same frame that we transition into the Warm Up state. We should not be in the
		// warm up state unless we have at least one frame. 
		CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining--;
		CurrentCameraCut->ShotInfo.WorkMetrics.EngineWarmUpFrameIndex++;

		// On the last non-emulated warm-up frame, flag the shot so the renderer
		// issues a sequencer jump to reset MetaSounds, whose internal state advanced 
		// while the sequence was playing during warm-up.
		// The emulated path already triggers a jump via the existing !IsPlaying() branch.
		if (CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining == 0
			&& !CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur)
		{
			CurrentCameraCut->ShotInfo.bNeedsAudioResetOnRenderStart = true;
		}
		
		// Warm up frames will discard the result and not contribute to output.
		CachedOutputState.bDiscardRenderResult = true;

		// We only render a warm up frame if this is the last engine warmup frame and there are render warmup frames to do.
		UMoviePipelineAntiAliasingSetting* AntiAliasingSettings = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(CurrentCameraCut);
		check(AntiAliasingSettings);
		// Some features (such as GPU particles) need to be rendered to warm up
		const bool bRenderFrame = (CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining == 0 && AntiAliasingSettings->RenderWarmUpCount > 0) || AntiAliasingSettings->bRenderWarmUpFrames;

		CachedOutputState.bSkipRendering = !bRenderFrame;

		// Render the next frame at a reasonable frame rate to pass time in the world.
		double FrameDeltaTime = GetPipelinePrimaryConfig()->GetEffectiveFrameRate(TargetSequence).AsInterval();

		UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "[%lld] Shot WarmUp set engine DeltaTime to %f seconds.", GFrameCounter, FrameDeltaTime);
		CustomTimeStep->SetCachedFrameTiming(MoviePipeline::FFrameTimeStepCache(FrameDeltaTime));

		if (!CurrentCameraCut->ShotInfo.bEmulateFirstFrameMotionBlur)
		{
			// If we're using real warmup frames we need to advance the sequence instead of just sitting there like normal.
			CurrentCameraCut->ShotInfo.CurrentTickInRoot = CurrentCameraCut->ShotInfo.CurrentTickInRoot + FrameMetrics.TicksPerOutputFrame.FloorToFrame();

			// The sequence player will already be playing (due to the Uninitialized block) so we just tell it which frame it should be playing at.
			FFrameTime FinalEvalTime = FrameMetrics.GetFinalEvalTime(CurrentCameraCut->ShotInfo.CurrentTickInRoot);
			CustomSequenceTimeController->SetCachedFrameTiming(FQualifiedFrameTime(FinalEvalTime, FrameMetrics.TickResolution));
		}

		// Apply the cloth fixups during warmup differently. If we don't use the increased number of sub-steps used on the large delta times here,
		// then transitioning from warmup to rendering causes a jump in delta times (from large to small) which makes all the cloth ripple. 
		if (AntiAliasingSettings->TemporalSampleCount > 1)
		{
			double Ratio = FrameMetrics.TicksWhileShutterOpen.FloorToFrame().Value / (double)FrameMetrics.TicksPerSample.FrameNumber.Value;

			// Slomo can end up trying to do less than one iteration, we don't want that.	
			int32 DivisionMultiplier = FMath::Max(FMath::FloorToInt(Ratio), 1);
			UE::MoviePipeline::SetSkeletalMeshClothSubSteps(DivisionMultiplier, GetWorld(), ClothSimCache);
		}

		CachedOutputState.TimeData.FrameDeltaTime = FrameDeltaTime;
		CachedOutputState.TimeData.WorldSeconds = CachedOutputState.TimeData.WorldSeconds + FrameDeltaTime;
		TRACE_BOOKMARK(TEXT("MoviePipeline - WarmingUp %d"), CurrentCameraCut->ShotInfo.NumEngineWarmUpFramesRemaining);
		// We don't want to execute the other possible states until at least the next frame.
		// This ensures that we actually tick the world once for the WarmUp state.
		return;
	}

	// Do the MotionBlur State exit condition before the MotionBlur loop, to allow the correct MotionBlur state to
	// persist through to the post-frame render before switching to the next state.
	if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::MotionBlur && CurrentCameraCut->ShotInfo.bHasEvaluatedMotionBlurFrame)
	{
		// Motion Blur will evaluate further on in the frame than the first output frame. This is so that when
		// we jump to the correct evaluation time for the first frame, the motion vectors (which are bi-directional)
		// are correct and more likely to have valid data than if we started before the frame.


		// The first frame of rendering code always assumes that we're coming from the previous frame. To avoid
		// complicating that code, we'll jump back to where that frame would have been rendered, had it existed.
		FFrameTime TicksToEndOfPreviousFrame;
		float WorldTimeDilation = GetWorld()->GetWorldSettings()->GetEffectiveTimeDilation();

		UMoviePipelineAntiAliasingSetting* AntiAliasing = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(CurrentCameraCut);
		if (AntiAliasing->TemporalSampleCount == 1)
		{
			TicksToEndOfPreviousFrame = FrameMetrics.TicksPerOutputFrame;
		}
		else
		{
			// The first sub-frame accumulation is going to try to go forward
			// by the amount the shutter is closed, plus the duration of one sample
			// because that is the logic we use for every other frame.
			TicksToEndOfPreviousFrame = FrameMetrics.TicksPerSample + FrameMetrics.TicksWhileShutterClosed;
		}

		if (!FMath::IsNearlyEqual(WorldTimeDilation, 1.f))
		{
			TicksToEndOfPreviousFrame = TicksToEndOfPreviousFrame * WorldTimeDilation;
		}
		AccumulatedTickSubFrameDeltas -= TicksToEndOfPreviousFrame.GetSubFrame();
		CurrentCameraCut->ShotInfo.CurrentTickInRoot = CurrentCameraCut->ShotInfo.CurrentTickInRoot - TicksToEndOfPreviousFrame.FloorToFrame();

		// Skip to rendering which will skip the next block.
		CurrentCameraCut->ShotInfo.State = EMovieRenderShotState::Rendering;
	}

	// This block is optional, as they may not want motion blur fixes.
	if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::MotionBlur)
	{
		CachedOutputState.bSkipRendering = false;
		CachedOutputState.bDiscardRenderResult = true;

		CachedOutputState.TimeData.MotionBlurFraction = FrameMetrics.ShutterAnglePercentage;
		
		// For the Motion Blur frame, we evaluate the Sequence past the starting frame so that on the next frame
		// we go backwards, and that gives us an approximation of motion blur for data we don't have. The world moves
		// forward in both cases which is a little odd conceptually but we can't use negative delta times.
		double FrameDeltaTime = FrameMetrics.TickResolution.AsSeconds(FrameMetrics.TicksPerSample);
		CachedOutputState.TimeData.FrameDeltaTime = FrameDeltaTime;
		CachedOutputState.TimeData.WorldSeconds = CachedOutputState.TimeData.WorldSeconds + FrameDeltaTime;
		CustomTimeStep->SetCachedFrameTiming(MoviePipeline::FFrameTimeStepCache(FrameDeltaTime));

		// We subtract a single tick here so that if you're trying to render one frame, and it is the last frame of the
		// camera cut, if we jump forward a whole frame the evaluation will fail (since end frames are exclusive). To
		// work around, we evaluate one tick before, which is indistinguishable in positions but neatly avoids the eval problem.
		FFrameNumber MotionBlurDurationTicks = FrameMetrics.TicksPerOutputFrame.FloorToFrame() - FFrameNumber(1);

		// CurrentShot.CurrentTick should be at the first frame they want to evaluate, without the shutter timing/motion blur
		// offsets taken into account. When we evaluate for this frame, take those into account.
		FFrameTime FinalEvalTime = FrameMetrics.GetFinalEvalTime(CurrentCameraCut->ShotInfo.CurrentTickInRoot + MotionBlurDurationTicks);

		// Jump to the motion blur frame
		FFrameTime TimeInPlayRate = FFrameRate::TransformTime(FinalEvalTime, FrameMetrics.TickResolution, TargetSequence->GetMovieScene()->GetDisplayRate());
		LevelSequenceActor->GetSequencePlayer()->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(TimeInPlayRate, EUpdatePositionMethod::Jump));
		CustomSequenceTimeController->SetCachedFrameTiming(FQualifiedFrameTime(FinalEvalTime, FrameMetrics.TickResolution));
		
		// We early out on this loop so that at least one tick passes for motion blur. We need to leave our state in
		// MotionBlur because the render tick needs to know that it is processing motion blur (it's a special case
		// with many renders for a frame to fill history buffers).
		CurrentCameraCut->ShotInfo.bHasEvaluatedMotionBlurFrame = true;
		UE_LOGF(LogMovieRenderPipeline, Verbose, "[%lld] Shot MotionBlur set engine DeltaTime to %f seconds.", GFrameCounter, FrameDeltaTime);
		TRACE_BOOKMARK(TEXT("MoviePipeline - MotionBlurEmulation"));

		return;
	}

	// Alright we're in the point where we finally want to start rendering out frames, we've finished warming up,
	// we've evaluated the correct frame to give us motion blur vector sources, etc. This will be called each time
	// we advance the world, even when sub-stepping, etc.
	if (CurrentCameraCut->ShotInfo.State == EMovieRenderShotState::Rendering)
	{
		float WorldTimeDilation = GetWorld()->GetWorldSettings()->GetEffectiveTimeDilation();

		// This delta time may get modified to respect slowmo tracks and will be converted to seconds for
		// engine delta time at the end.
		FFrameTime DeltaFrameTime = FFrameTime();

		UMoviePipelineAntiAliasingSetting* AntiAliasingSettings = FindOrAddSettingForShot<UMoviePipelineAntiAliasingSetting>(CurrentCameraCut);
		check(AntiAliasingSettings);

		// If we've rendered the last sample and wrapped around, then we're going to be
		// working on the next output frame for this frame. Since OutputFrameNumber gets
		// initialized to -1, this is ok on the very first frame of rendering as it pushes
		// us to be working on outputting frame zero.
		if (CachedOutputState.TemporalSampleIndex >= AntiAliasingSettings->TemporalSampleCount - 1)
		{
			CachedOutputState.TemporalSampleIndex = -1;
		}

		CachedOutputState.TemporalSampleIndex++;
		const bool bFirstFrame = CachedOutputState.TemporalSampleIndex == 0;
		
		if (AntiAliasingSettings->TemporalSampleCount == 1)
		{
			// It should always be zero when using no sub-sampling.
			check(CachedOutputState.TemporalSampleIndex == 0);

			// When not using any sub-frame accumulation, the delta time is just
			// 1/FrameRate * World Timescale (Slo-mo), which is applied afterwards.
			DeltaFrameTime = FrameMetrics.TicksPerOutputFrame;
		}
		else
		{
			/*
			* Sub-frame accumulation is surprisingly nuanced and complicated. Unreal's real-time motion blur is
			* is an approximation that derives a velocity (per-vertex) by taking the difference in positions per 
			* frame and then simulates a 'centered' motion blur, blurring pixels on either side of the object.
			* The amount of pixels that are blurred is determined by the Shutter Angle ("Motion Blur Amount").
			* Because we are simulating a higher quality version of the existing motion blur, we override the
			* Shutter Angle to 360 during accumulation frames, as they are designed to represent one long blur
			* and we emulate the shutter angle ourself over multiple frames.
			*
			* Different industries have different needs for when data is collected relative to an output frame.
			* Since a "frame" is a span of time, and a non-360 shutter angle determines that no photons would
			* accumulate while it is closed, there are three options for what period of time makes up the frame.
			* In addition to the sampling being shifted, there may also be a brightness curve that gets applied
			* to each shutter type during the accumulation frame, as choosing to give a particular frame more
			* weight during the accumulation will make it brighter and thus appear to have more influence. This
			* is handled during the accumulation/pixel data instead of shifting samples around as that would make
			* the timing on this very complicated.
			*
			* In the diagram below, 'f' marks an output frame, 's' marks the shutter-angle (180 in this case)
			* and it is assumed there are four motion samples ('-')
			*
			* f         f         f         f
			* |    s    |    s    |    s    |
			* |    |    |    |    |    |    |
			*       ----     Frame Close
			*         -- --   Frame Center
			*		     ---- Frame Open
			* Frame Closed - In this scenario, a given frame 'n' is when the shutter should close, meaning that
			*                the accumulation is the motion that happened before the frame.
			* Frame Center - In this scenario, a given frame 'n' is split evenly with half of the motion coming
			*                from before that frame, and half afterwards. In this diagram, the space in the
			*                middle is irrelevant (-- --), and is just there to make the diagram line up evenly.
			* Frame Opened - In this scenario, a given frame 'n' is when the shutter should open, meaning that
			*                the accumulation is the motion that happens after the frame.
			*
			* Depending on the shutter angle the spacing between the frames will change, ie: 270 shutter angle
			* means that the accumulation will happen across 75% of the time. In the example below, this assumes
			* that you are using Frame Close, meaning that the 75% of time before each frame.
			* 
			* f         f         f         f
			* |  s      |  s      |  s      |
			* |  |      |  |      |  |      |
			*     ------    ------    ------ Frame Close accumulation.
			*
			* The above offset (Frame Open/Closed/Centered) is the first timing offset that we apply, which only
			* changes what data is sampled to be considered a given frame n. We need to apply a second offset to
			* our sampling, because motion blur is centered around the object. For our motion blur to accurately
			* represent the period of time denoted on the chart, we need to offset the sample by half of the length
			* of a sub-frame so that when the blur is applied on either side of it, no pixels beyond where the frame
			* lies are affected.
			*
			* These offsets are calculated at the start of the shot and counter-adjusted for when calculating out which
			* frame number the output frames should end up with.
			*
			* In the diagram below, 'f' marks an output frame, 's' marks the shutter angle time, and '-' marks where
			* we actually sample time to generate a sub-frame (for which there are 4 in this example).
			*
			* f               f               f
			* |       s       |       s       |
			* |       |       |       |       |
			* |       |-|-|-|-|               |
			*
			* Finally, the tricky part. Because time in Unreal only goes forward (due to world tick) and renders may
			* be very expensive, we need to minimize any extra, unnecessary renders. To emulate Shutter Angle, there is
			* a period of time where the shutter is closed. We still need this time to pass in the world, but we don't
			* want our camera motion blur to stretch to cover this gap. One solution would be to render an extra frame
			* (with a duration of |-|) right before the first accumulation frame but this would be unnecessarily expensive.
			* 
			* Instead, what we do is we jump from the last accumulation frame to the start of the next frame (keeping in
			* mind that we have a timing offset meaning that we're in the center of the frame), but we override the motion
			* blur for this frame and say that the motion blur only has an effective length matching |-|, even though 
			* |-----| time has passed. Our motion blur amount is a 0-1 fraction, so to calculate the desired motion blur
			* for the first frame we take the length of a sub-frame and divide it by the total number of time that has
			* passed. Because the motion blur is centered and we've landed in the middle of the first accumulation frame
			* this ends up being the correct length and centered at the correct period of time. Yay!
			*
			* f               f               f
			* |       s       |       s       |
			* |       |       |       |       |
			* |       |-|-|-|-|       |-|-|-|-|
			*                ^         ^
			*        Jump From         Jump To
			*
			* The only remaining thing to watch out for is accumulation drift. The Movie Pipeline does not rely on 
			* accumulating delta times via engine-tick (even though we control engine tick) to avoid any floating point
			* issues. Instead, we calculate the exact number of ticks to move each time and store the accumulated time as 
			* ticks. Integers don't drift, but certain tick rates/accumulation sub-frame counts can end up not moving time
			* forward by a whole number.
			*
			* Given a default tick rate of 24,000 ticks per second and a output frame rate of 24fps, we have 1,000 ticks to
			* represent each frame. This is a great deal of precision, but it doesn't divide evenly by the number of samples
			* they may choose, such as three. 1000/3 = 333.3333 ticks per sub-frame. Since ticks are our finest resolution
			* we have to either advance time by 333 or 334. To account for this, we need a running tally of how much we're
			* drifting behind and then add it once it crosses the threshold.
			*
			* The reason we derive the Sequence time from our own time, and we derive our own time from the series of multiple
			* samples, is to handle slo-mo tracks. Slo-mo is not linear and can change each frame (which is why we multiply our
			* final delta amount by it) but to ensure the newly generated frames have the most correct timecode we move time
			* on our own, and find the nearest frame number and timecode from the sequence. When slo-mo tracks are not in play
			* the numbers should always add up evenly and it will be an exact 1-to-1 match.
			*
			* Easy!
			*/

			
			// Shouldn't be in this else branch if they don't have an Accumulation Setting for this shot.
			check(AntiAliasingSettings);
			check(AntiAliasingSettings->TemporalSampleCount > 1);
				
			
			if (bFirstFrame)
			{
				/* The first sub-frame has its timing calculated differently than the rest, because we're trying to skip
				* any unnecessary ticks so our world/engine delta time is long (to cover the gap where the shutter is 
				* closed) but the motion blur shutter angle is overridden to be short to simulate only the portion of time
				* with the shutter open that we care about.
				* f               f               f
				* |       s       |       s       |
				* |       |       |       |       |
				* |       |-|-|-|-|       |-|-|-|-|
				*                ^         ^
				*        Jump From         Jump To
				*
				* The duration between the last accumulation frame and the start of the next accumulation frame is
				* (360-ShutterAngle)/360 * Ticks Per Frame + Ticks Per Subframe.
				* The (360-ShutterAngle)/360 gives us the fraction of the output frame that the shutter was closed,
				* so in a 270 shutter angle (open 75% of the time) at 24000 ticks/24fps (1000 ticks per frame) the
				* shutter is closed for 1000 * 0.25 = 250 ticks. We need to add the duration of one accumulation frame
				* so that we end up in the middle of the first frame, and not before the first frame.
				*
				* We don't actually end up adding half offsets or shutter biases because the entire starting index has
				* been adjusted for those and they are constant throughout the shot.
				*/


				// Finally, we take the time it is closed plus the sample duration to put us into the new sample.
				DeltaFrameTime = FrameMetrics.TicksWhileShutterClosed + FrameMetrics.TicksPerSample;


			}
			else
			{
				// We're moving from one sub-frame to the next. Our calculation is simpler.
				DeltaFrameTime = FrameMetrics.TicksPerSample;
			}
		}
		
		if(bFirstFrame)
		{
			while (AccumulatedTickSubFrameDeltas >= 1.f)
			{
				// We add the deltas on the start of the first sample, since it's the one where 
				// we jump by a non-consistent amount of time anyways.
				DeltaFrameTime += FFrameTime(FFrameNumber(1));
				AccumulatedTickSubFrameDeltas -= 1.f;
			}
		}

		

		// Now that we've calculated our delta ticks, we need to multiply it by
		// time dilation so that we advance through the sequence as slow as we
		// advance through the world.
		FFrameTime UndilatedDeltaFrameTime = DeltaFrameTime;
		if (!FMath::IsNearlyEqual(WorldTimeDilation, 1.f))
		{
			UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "[%lld] Modified FrameDeltaTime by a factor of %f to account for World Time Dilation.", GFrameCounter, WorldTimeDilation);

			// This is technically always a frame behind as this function executes before Sequencer evaluates slow-mo tracks
			// and sets the World Time Dilation, but it is close enough for now.
			DeltaFrameTime = DeltaFrameTime * WorldTimeDilation;
		}

		// Cloth doesn't like temporal sub-sampling due to the wildly varying delta times (due to the 'shutter closed') period.
		// To help account for this, we increase the sub-division count for each skeletal mesh inversely to the deltatime
		// for this frame. That means if a 'short' frame is 2ms, but a long frame is 10ms, we make the long frame 5x as many
		// sub-divisions as the short frame to fake the cloth simulation system into processing each step with the same dt.
		{
			if (AntiAliasingSettings->TemporalSampleCount > 1)
			{
				double Ratio = DeltaFrameTime.FrameNumber.Value / (double)FrameMetrics.TicksPerSample.FrameNumber.Value;
				
				// Slomo can end up trying to do less than one iteration, we don't want that.	
				int32 DivisionMultiplier = FMath::Max(FMath::FloorToInt(Ratio), 1);
				UE::MoviePipeline::SetSkeletalMeshClothSubSteps(DivisionMultiplier, GetWorld(), ClothSimCache);
			}
		}

		/*
		* Now that we know how much we want to advance the sequence by (and the world)
		* We can calculate out the desired frame for the shot to evaluate. This is slightly
		* complicated, as it's not just a simple increment. The actual evaluation position
		* is modified by two things:
		*
		* - Shutter Timing: To affect whether the motion before, during, or after
		*                   the frame count as this frame, we add this offset to
		*                   the sampled range.
		*
		* - Motion Blur:    Unreal Motion blur is centered on the object. To
		*                   do our best to ensure that the written pixels match
		*					the motion that would have occurred during a frame,
		*                   we offset by half of a sample to ensure our samples
		*                   are in the middle and stretch half way to each side.
		*
		* When the shutter timing is Frame Open (meaning the motion all comes from
		* after the shutter opens) we have to add half a sample's duration so that
		* our motion blur is centered.
		* When the shutter timing is Frame Center, we add half a sample's duration
		* which counter-acts the half-sample removed by the shutter timing offset
		* which ends up centering our sample exactly on the frame.
		* When the shutter timing is Frame Close, we subtract an entire sample's
		* duration and add half of it back, meaning we only sample information
		* before the current frame, but stretch the motion to cover up to that.
		*
		* For example, example we can look at the following situations when using
		* a 24fps sequence, 24000 tick resolution (1000 tick per frame), and
		* 270* shutter angle.
		*
		* 270* Shutter Angle means the shutter is open for 750 ticks (1000*0.75)
		* since we want to stretch from the 0 to 750, and our motion blur is bi
		* directional, we offset 350 ticks (750/2) to center it.
		*/

		// Convert any sub-frame time we have into a rolling accumulation
		AccumulatedTickSubFrameDeltas += DeltaFrameTime.GetSubFrame();

		// Increment where we should evaluate on the current camera cut.
		CurrentCameraCut->ShotInfo.CurrentTickInRoot += DeltaFrameTime.GetFrame();

		FFrameTime FinalEvalTime = FrameMetrics.GetFinalEvalTime(CurrentCameraCut->ShotInfo.CurrentTickInRoot);
		UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "[%lld] FinalEvalTime: %ls (Tick: %d SOffset: %d MOffset: %d)", 
			GFrameCounter, *LexToString(FinalEvalTime.FloorToFrame()),
			CurrentCameraCut->ShotInfo.CurrentTickInRoot.Value, FrameMetrics.ShutterOffsetTicks.GetFrame().Value, FrameMetrics.MotionBlurCenteringOffsetTicks.GetFrame().Value);


		if (!LevelSequenceActor->GetSequencePlayer()->IsPlaying()
			|| CurrentCameraCut->ShotInfo.bNeedsAudioResetOnRenderStart)
		{
			// If the level sequence actor isn't playing then this is the first frame that we're trying to render for the shot.
			// We want to trigger a jump and then a play (and then set the SetCachedFrameTiming below as normal). The jump is
			// important because we're going backwards in time from the motion blur frame, and some tracks need to be notified
			// of a jump when time goes backwards.
			//
			// bNeedsAudioResetOnRenderStart covers the non-emulated warm-up path where the sequence was already playing:
			// the jump causes the audio system to issue StopAndPlay for all sounds, resetting procedural generators
			// (e.g. MetaSound) whose state advanced during warm-up, before recording opens.
			CurrentCameraCut->ShotInfo.bNeedsAudioResetOnRenderStart = false;

			FFrameTime TimeInPlayRate = FFrameRate::TransformTime(FinalEvalTime, FrameMetrics.TickResolution, TargetSequence->GetMovieScene()->GetDisplayRate());
			LevelSequenceActor->GetSequencePlayer()->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(TimeInPlayRate, EUpdatePositionMethod::Jump));
			if (!LevelSequenceActor->GetSequencePlayer()->IsPlaying())
			{
				LevelSequenceActor->GetSequencePlayer()->Play();
			}
		}

		if (DeltaFrameTime.GetFrame() == FFrameNumber(0))
		{
			// Too many temporal samples for the given shutter angle.
			UE_LOGF(LogMovieRenderPipeline, Error, "Too many temporal samples for the given shutter angle/tick rate combination. Temporal Samples: %d Shutter Angle: %f TicksPerOutputFrame: %ls TicksPerSample: %ls. Consider converting to Spatial Samples instead!",
				AntiAliasingSettings->TemporalSampleCount, FrameMetrics.ShutterAnglePercentage, *LexToString(FrameMetrics.TicksPerOutputFrame), *LexToString(FrameMetrics.TicksPerSample));
			DeltaFrameTime = FFrameTime(FFrameNumber(1));
		}

		double FrameDeltaTime = FrameMetrics.TickResolution.AsSeconds(FFrameTime(DeltaFrameTime.GetFrame()));
		CachedOutputState.TimeData.FrameDeltaTime = FrameDeltaTime;
		CachedOutputState.TimeData.WorldSeconds = CachedOutputState.TimeData.WorldSeconds + FrameDeltaTime;
		CachedOutputState.TimeData.TimeDilation = WorldTimeDilation;

		// The combination of shutter angle percentage, non-uniform render frame delta times and dividing by sample
		// count produce the correct length for motion blur in all cases.
		CachedOutputState.TimeData.MotionBlurFraction = FrameMetrics.ShutterAnglePercentage / AntiAliasingSettings->TemporalSampleCount;

		// Update our Output State with any data we need to derive based on our current time.
		CalculateFrameNumbersForOutputState(FrameMetrics, CurrentCameraCut, CachedOutputState);

		// Now that we know the delta time for the upcoming frame, we can see if this time would extend past the end of our
		// camera cut. If it does, we don't want to produce this frame as that would produce an extra frame at the end!
		if (CurrentCameraCut->ShotInfo.CurrentTickInRoot >= CurrentCameraCut->ShotInfo.TotalOutputRangeRoot.GetUpperBoundValue())
		{
			// If this output frame started but cannot reach its final temporal sample, don't let it consume an output frame number.
			if (CachedOutputState.TemporalSampleIndex > 0 && CachedOutputState.ShotOutputFrameNumber >= 0)
			{
				CurrentCameraCut->ShotInfo.WorkMetrics.OutputFrameIndex--;
				CachedOutputState.OutputFrameNumber--;
				CachedOutputState.ShotOutputFrameNumber--;
			}

			// If this isn't the last shot, we'll immediately call this function again to just
			// determine a new setup/seek/etc. on the same engine tick. Otherwise we would have
			// used a random delta time that didn't actually correlate to anything.
			ProcessEndOfCameraCut(CurrentCameraCut);
			if (PipelineState == EMovieRenderPipelineState::ProducingFrames)
			{
				CachedOutputState.TemporalSampleIndex = -1;
				TickProducingFrames();
			}

			return;
		}

		// If we've made it this far, we're going to produce a frame so we can check to see if we want to increment the output
		// number. If we do it before the above block, then the last frame of each shot increments this global state incorrectly.
		if (CachedOutputState.TemporalSampleIndex == 0)
		{
			CurrentCameraCut->ShotInfo.WorkMetrics.OutputFrameIndex++;
			CurrentCameraCut->ShotInfo.WorkMetrics.OutputSubSampleIndex = -1;
			CachedOutputState.OutputFrameNumber++;
			CachedOutputState.ShotOutputFrameNumber++;
		}
	
#if WITH_EDITOR && !UE_BUILD_SHIPPING
		{
			UMoviePipelineDebugSettings* DebugSettings = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineDebugSettings>();
			if (IsValid(DebugSettings) && DebugSettings->IsRenderDocEnabled())
			{
				if (CachedOutputState.SourceFrameNumber == DebugSettings->CaptureFrame)
				{
					CachedOutputState.bCaptureRendering = true;
				}
			}
		}
#endif

		// Check to see if we should be rendering this frame. If they are frame stepping (rendering every Nth frame) then we
		// just disable rendering but otherwise let the game logic remain the same for evaluation consistency.
		{
			UMoviePipelineOutputSetting* OutputSettings = GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
			int32 OutputFrameStep = FMath::Max(1, OutputSettings->OutputFrameStep);
			CachedOutputState.bSkipRendering = ((CachedOutputState.OutputFrameNumber + OutputSettings->DEBUG_OutputFrameStepOffset) % OutputFrameStep) != 0;
		}
		

		
		// Set our time step for the next frame. We use the undilated delta time for the Custom Timestep as the engine will
		// apply the time dilation to the world tick for us, so we don't want to double up time dilation.
		double UndilatedDeltaTime = FrameMetrics.TickResolution.AsSeconds(FFrameTime(UndilatedDeltaFrameTime.GetFrame()));
		CustomTimeStep->SetCachedFrameTiming(MoviePipeline::FFrameTimeStepCache(UndilatedDeltaTime));
		CustomSequenceTimeController->SetCachedFrameTiming(FQualifiedFrameTime(FinalEvalTime, FrameMetrics.TickResolution));
		TRACE_BOOKMARK(TEXT("MoviePipeline - Rendering Frame %d"), CachedOutputState.EffectiveFrameNumber);

		return;
	}

	// This function should not be called when the shot isn't in one of the above states.
	check(false);
}

void UMoviePipeline::CalculateFrameNumbersForOutputState(const MoviePipeline::FFrameConstantMetrics& InFrameMetrics, const UMoviePipelineExecutorShot* InCameraCut, FMoviePipelineFrameOutputState& InOutOutputState) const
{
	// Find the closest number on the source Sequence. We use the effective framerate and not the original Sequence framerate because had they actually resampled their sequence
	// to the given display rate then the frame numbers would match what they see. This may produce duplicates when using slow-mo tracks, in that case the user should use the _rel version.
	FFrameRate SourceDisplayRate = GetPipelinePrimaryConfig()->GetEffectiveFrameRate(TargetSequence);

	// "Closest" isn't straightforward when using temporal sub-sampling, ie: A large enough shutter angle pushes a sample over the half way point and it rounds to
	// the wrong one. Because temporal sub-sampling isn't centered around a frame (the centering is done via the final eval time) we can just subtract TS*TPS to get our centered value.
	FFrameNumber CenteringOffset = FFrameNumber(InOutOutputState.TemporalSampleIndex * InFrameMetrics.TicksPerSample.RoundToFrame().Value);
	{
		FFrameNumber CenteredTick = InCameraCut->ShotInfo.CurrentTickInRoot - CenteringOffset;

		InOutOutputState.SourceFrameNumber = FFrameRate::TransformTime(CenteredTick, InFrameMetrics.TickResolution, SourceDisplayRate).RoundToFrame().Value;
		InOutOutputState.SourceTimeCode = FTimecode::FromFrameNumber(InOutOutputState.SourceFrameNumber, InFrameMetrics.FrameRate, false);
		InOutOutputState.EffectiveFrameNumber = FFrameRate::TransformTime(CenteredTick, InFrameMetrics.TickResolution, InFrameMetrics.FrameRate).RoundToFrame().Value;
		InOutOutputState.EffectiveTimeCode = FTimecode::FromFrameNumber(InOutOutputState.SourceFrameNumber, InFrameMetrics.FrameRate, false);
	}

	{
		FFrameNumber CenteredTick = InCameraCut->ShotInfo.CurrentTickInRoot - CenteringOffset;

		// Convert from Root space back into shot space - based on the inner most detected shot.
		CenteredTick = (CenteredTick * InCameraCut->ShotInfo.OuterToInnerTransform).FloorToFrame();

		InOutOutputState.CurrentShotSourceFrameNumber = FFrameRate::TransformTime(CenteredTick, InFrameMetrics.ShotTickResolution, SourceDisplayRate).RoundToFrame().Value;
		InOutOutputState.CurrentShotSourceTimeCode = FTimecode::FromFrameNumber(InOutOutputState.CurrentShotSourceFrameNumber, InFrameMetrics.FrameRate, false);
	}

	InOutOutputState.ShotName = InCameraCut->OuterName;

	// Continue to use the InnerName (which is the Root camera for the shot), individual render passes can override it later if rendering more than one camera.
	InOutOutputState.CameraName = InCameraCut->InnerName;
}