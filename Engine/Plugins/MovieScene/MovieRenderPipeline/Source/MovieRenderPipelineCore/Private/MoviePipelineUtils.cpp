// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineUtils.h"
#include "Engine/Blueprint.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MoviePipelineAntiAliasingSetting.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelineCameraSetting.h"
#include "MoviePipelineHighResSetting.h"
#include "MoviePipelineSetting.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/UObjectIterator.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/Interface_PostProcessVolume.h"
#include "MoviePipelineAntiAliasingSetting.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "SceneView.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "LevelSequence.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Math/Halton.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"
#include "Misc/FrameRate.h"
#include "MoviePipelineQueue.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformMemory.h"
#include "HAL/FileManager.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "MovieRenderPipelineCoreModule.h"
#include "RHI.h"
#include "CineCameraComponent.h"
#include "EngineUtils.h"
#include "ClothingSimulationInstance.h"
#include "ClothingSimulationInteractor.h"
#include "ClothingSimulationInterface.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothAssetInteractor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Graph/Nodes/MovieGraphSubgraphNode.h"

static TAutoConsoleVariable<bool> CVarMoviePipelineAlphaOutputOverride(
	TEXT("MoviePipeline.AlphaOutputOverride"),
	true,
	TEXT("Always overrides the \"Alpha Output\" renderer project setting by controlling r.PostProcessing.PropagateAlpha during movie pipeline renders.\n"),
	ECVF_Default);

namespace UE
{
	namespace MovieRenderPipeline
	{
		TArray<UClass*> FindMoviePipelineSettingClasses(UClass* InBaseClass, const bool bIncludeBlueprints)
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			TArray<FAssetData> ClassList;

			FARFilter Filter;
			Filter.ClassPaths.Add(InBaseClass->GetClassPathName());
			if (bIncludeBlueprints)
			{
				Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
			}

			// Include any Blueprint based objects as well, this includes things like Blutilities, UMG, and GameplayAbility objects
			Filter.bRecursiveClasses = true;
			AssetRegistryModule.Get().GetAssets(Filter, ClassList);

			TArray<UClass*> Classes;

			for (const FAssetData& Data : ClassList)
			{
				// We don't want to load every blueprint class to find out if it's an MRQ related one, so we'll look up their native class
				// based on the asset registry metadata before deciding if it needs to be loaded.
				UClass* ParentClass = nullptr;
				FString ParentClassName;

				if (!Data.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
				{
					Data.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
				}

				if (!ParentClassName.IsEmpty())
				{
					ParentClass = UClass::TryFindTypeSlow<UClass>(FPackageName::ExportTextPathToObjectPath(ParentClassName));

					if (ParentClass == nullptr || !ParentClass->IsChildOf(InBaseClass))
					{
						continue;
					}
				}

				// Now attempt to load it as it's an MRQ related setting. This will cause it to show up in the class iterator
				// below.
				UBlueprint* LoadedBlueprint = Cast<UBlueprint>(Data.ToSoftObjectPath().TryLoad());
			}

			// Now iterate through the loaded classes.
			for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
			{
				if (ClassIterator->IsChildOf(InBaseClass) && !ClassIterator->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
				{
					// While a blueprint is being compiled, there can be some transient instances of the class. We don't want these to show up in the list
					// so just manually skip over them as they will go away next time GC is run anyways.
					if (ClassIterator->GetName().StartsWith(TEXT("SKEL_")) || ClassIterator->GetName().StartsWith(TEXT("REINST_")))
					{
						continue;
					}

					Classes.AddUnique(*ClassIterator);
				}
			}

			return Classes;
		}
	
		/**
		* This defaults to the project setting, and then
		* uses the one specified by the setting if overriden.
		*/
		EAntiAliasingMethod GetEffectiveAntiAliasingMethod(bool bOverride, EAntiAliasingMethod OverrideMethod)
		{
			if (bOverride)
			{
				return OverrideMethod;
			}
			else
			{
				IConsoleVariable* AntiAliasingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.AntiAliasingMethod"));
				if (AntiAliasingCVar)
				{
					int32 Value = AntiAliasingCVar->GetInt();
					if (Value >= 0 && Value < AAM_MAX)
					{
						return (EAntiAliasingMethod)Value;
					}
				}
			}

			return EAntiAliasingMethod::AAM_None;
		}

		/**
		* Returns the anti-aliasing setting that we should use. This defaults to the project setting, and then
		* uses the one specified by the setting if overriden.
		*/
		EAntiAliasingMethod GetEffectiveAntiAliasingMethod(const UMoviePipelineAntiAliasingSetting* InSetting)
		{
			check(InSetting);
			return GetEffectiveAntiAliasingMethod(InSetting->bOverrideAntiAliasing, InSetting->AntiAliasingMethod);
		}

		uint64 GetRendererFrameCount()
		{
			// The rendering module relies on GFrameCounter during submission to line several things up that need to happen once per engine-tick.
			// Unfortunately by the time the Movie Render Queue code gets called (in FCoreDelegates::OnEndFrame) GFrameCounter has already been
			// incremented compared to the Tick() that frame that Skeletal Meshes, etc. may have used. To solve this, we manually override which
			// tick the renderer thinks this frame is for.

#if WITH_EDITORONLY_DATA
			if (GIsEditor)
			{
				return GFrameCounter - 1;
			}
#endif
			return GFrameCounter;
		}

		void UpdateSceneViewForShowFlags(FSceneView* View)
		{
			if (View->Family->EngineShowFlags.Wireframe)
			{
				// Wireframe color is emissive-only, and mesh-modifying materials do not use material substitution, hence...
				View->DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
				View->SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
			}
			else if (View->Family->EngineShowFlags.OverrideDiffuseAndSpecular)
			{
				View->DiffuseOverrideParameter = FVector4f(GEngine->LightingOnlyBrightness.R, GEngine->LightingOnlyBrightness.G, GEngine->LightingOnlyBrightness.B, 0.0f);
				View->SpecularOverrideParameter = FVector4f(.1f, .1f, .1f, 0.0f);
			}
			else if (View->Family->EngineShowFlags.LightingOnlyOverride)
			{
				View->DiffuseOverrideParameter = FVector4f(GEngine->LightingOnlyBrightness.R, GEngine->LightingOnlyBrightness.G, GEngine->LightingOnlyBrightness.B, 0.0f);
				View->SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
			}
			else if (View->Family->EngineShowFlags.ReflectionOverride)
			{
				View->DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
				View->SpecularOverrideParameter = FVector4f(1, 1, 1, 0.0f);
				View->NormalOverrideParameter = FVector4f(0, 0, 1, 0.0f);
				View->RoughnessOverrideParameter = FVector2f(0.0f, 0.0f);
			}
	
			if (!View->Family->EngineShowFlags.Diffuse)
			{
				View->DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
			}
	
			if (!View->Family->EngineShowFlags.Specular)
			{
				View->SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
			}

			if (!View->Family->EngineShowFlags.MaterialNormal)
			{
				View->NormalOverrideParameter = FVector4f(0.0f, 0.0f, 1.0f, 0.0f);
			}

			if (!View->Family->EngineShowFlags.MaterialAmbientOcclusion)
			{
				View->AmbientOcclusionOverrideParameter = FVector2f(1.0f, 0.0f);
			}
	
			static const FName BufferVisualizationMode(TEXT("WorldNormal"));
			View->CurrentBufferVisualizationMode = BufferVisualizationMode;
		}
	}
}

namespace MoviePipeline
{
	void GetPassCompositeData(FMoviePipelineMergerOutputFrame* InMergedOutputFrame, TArray<MoviePipeline::FCompositePassInfo>& OutCompositedPasses)
	{
		for (TPair<FMoviePipelinePassIdentifier, TUniquePtr<FImagePixelData>>& RenderPassData : InMergedOutputFrame->ImageOutputData)
		{
			FImagePixelDataPayload* Payload = RenderPassData.Value->GetPayload<FImagePixelDataPayload>();
			if (Payload->bCompositeToFinalImage)
			{
				// Burn in data should always be 8 bit values, this is assumed later when we composite.
				check(RenderPassData.Value->GetType() == EImagePixelType::Color);

				FCompositePassInfo& Info = OutCompositedPasses.AddDefaulted_GetRef();
				Info.PassIdentifier = RenderPassData.Key;
				Info.PixelData = RenderPassData.Value->CopyImageData();
			}
		}
	}

	void GetOutputStateFormatArgs(TMap<FString, FString>& InOutFilenameArguments, TMap<FString, FString>& InOutFileMetadata, const FString FrameNumber, const FString FrameNumberShot, const FString FrameNumberRel, const FString FrameNumberShotRel, const FString CameraName, const FString ShotName)
	{
		InOutFilenameArguments.Add(TEXT("frame_number"), FrameNumber);
		InOutFilenameArguments.Add(TEXT("frame_number_shot"), FrameNumberShot);
		InOutFilenameArguments.Add(TEXT("frame_number_rel"), FrameNumberRel);
		InOutFilenameArguments.Add(TEXT("frame_number_shot_rel"), FrameNumberShotRel);
		InOutFilenameArguments.Add(TEXT("camera_name"), CameraName);
		InOutFilenameArguments.Add(TEXT("shot_name"), ShotName);

		InOutFileMetadata.Add(TEXT("unreal/sequenceFrameNumber"), FrameNumber);
		InOutFileMetadata.Add(TEXT("unreal/shotFrameNumber"), FrameNumberShot);
		InOutFileMetadata.Add(TEXT("unreal/sequenceFrameNumberRelative"), FrameNumberRel);
		InOutFileMetadata.Add(TEXT("unreal/shotFrameNumberRelative"), FrameNumberShotRel);
		InOutFileMetadata.Add(TEXT("unreal/cameraName"), CameraName);
		InOutFileMetadata.Add(TEXT("unreal/shotName"), ShotName);
	}

	FString GetFrameTemplatedFilePath(const FString& InFilePath)
	{
		static const FString FramePlaceholder = TEXT("{frame_placeholder}");

		FString FrameTemplatedFormatString = InFilePath;
		if (FrameTemplatedFormatString.Contains(TEXT("{frame_number}")))
		{
			FrameTemplatedFormatString = InFilePath.Replace(TEXT("{frame_number}"), *FramePlaceholder);
		}
		else if (FrameTemplatedFormatString.Contains(TEXT("{frame_number_rel}")))
		{
			FrameTemplatedFormatString = InFilePath.Replace(TEXT("{frame_number_rel}"), *FramePlaceholder);
		}
		else if (FrameTemplatedFormatString.Contains(TEXT("{frame_number_shot}")))
		{
			FrameTemplatedFormatString = InFilePath.Replace(TEXT("{frame_number_shot}"), *FramePlaceholder);
		}
		else if (FrameTemplatedFormatString.Contains(TEXT("{frame_number_shot_rel}")))
		{
			FrameTemplatedFormatString = InFilePath.Replace(TEXT("{frame_number_shot_rel}"), *FramePlaceholder);
		}

		return FrameTemplatedFormatString;
	}

	TObjectPtr<UMovieJobVariableAssignmentContainer> GetOrCreateJobVariableAssignmentsForGraph(const UMovieGraphConfig* InGraph, TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& InVariableAssignments, UObject* InAssignmentsOwner)
	{
		for (TObjectPtr<UMovieJobVariableAssignmentContainer>& VariableAssignment : InVariableAssignments)
		{
			const TSoftObjectPtr<UMovieGraphConfig> SoftGraphConfig = VariableAssignment->GetGraphConfig();
			if (SoftGraphConfig.Get() == InGraph)
			{
#if WITH_EDITOR
				VariableAssignment->UpdateGraphVariableOverrides();
#endif
				
				return VariableAssignment;
			}
		}

		// Create the variable assignments container if it wasn't found
		TObjectPtr<UMovieJobVariableAssignmentContainer> NewVariableAssignments = NewObject<UMovieJobVariableAssignmentContainer>(InAssignmentsOwner);
		InVariableAssignments.Add(NewVariableAssignments);
		NewVariableAssignments->SetGraphConfig(const_cast<UMovieGraphConfig*>(InGraph));

#if WITH_EDITOR
		NewVariableAssignments->UpdateGraphVariableOverrides();
#endif
	
		return NewVariableAssignments;
	}

	void RefreshVariableAssignments(UMovieGraphConfig* InRootGraph, TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& InVariableAssignments, UObject* InAssignmentsOwner)
	{
		TSet<UMovieGraphConfig*> AllGraphs;
		
		if (InRootGraph)
		{
			AllGraphs = {InRootGraph};
			InRootGraph->GetAllContainedSubgraphs(AllGraphs);
		}
	
		// Add/update variable assignments for the graph on the job and all of its subgraphs
		for (const UMovieGraphConfig* Graph : AllGraphs)
		{
			GetOrCreateJobVariableAssignmentsForGraph(Graph, InVariableAssignments, InAssignmentsOwner);
		}

		// Remove any stale variable assignments for graphs/subgraphs which are no longer part of the job
		InVariableAssignments.RemoveAll([&AllGraphs](const TObjectPtr<UMovieJobVariableAssignmentContainer>& VariableAssignment)
		{
			return !AllGraphs.Contains(VariableAssignment->GetGraphConfig().LoadSynchronous());
		});
	}

	UMovieGraphConfig* DuplicateConfigRecursive(UMovieGraphConfig* InGraphToDuplicate, TMap<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>>& OutDuplicatedGraphs)
	{
		UMovieGraphConfig* DuplicateConfig;

		// Duplicate the graph. If the graph has been duplicated already, don't re-duplicate it, but continue updating variable assignments.
		if (OutDuplicatedGraphs.Contains(InGraphToDuplicate))
		{
			DuplicateConfig = OutDuplicatedGraphs[InGraphToDuplicate];
		}
		else
		{
			// The transient package is used because graphs don't belong to the executor job usually (they belong to an asset package)
			FObjectDuplicationParameters GraphDuplicationParams(InGraphToDuplicate, GetTransientPackage());
			GraphDuplicationParams.DestName = FName(FString::Format(TEXT("{0}_Duplicate"), {InGraphToDuplicate->GetFName().ToString()}));
			GraphDuplicationParams.FlagMask = RF_AllFlags & ~(RF_Standalone | RF_Transactional);
			GraphDuplicationParams.ApplyFlags = RF_Transient;
			DuplicateConfig = Cast<UMovieGraphConfig>(StaticDuplicateObjectEx(GraphDuplicationParams));
			
			OutDuplicatedGraphs.Add(InGraphToDuplicate, DuplicateConfig);
		}

		// Duplicate sub-graphs also.
		for (const TObjectPtr<UMovieGraphNode>& Node : DuplicateConfig->GetNodes())
		{
			UMovieGraphSubgraphNode* SubgraphNode = Cast<UMovieGraphSubgraphNode>(Node);
			if (!SubgraphNode)
			{
				continue;
			}

			// Only duplicate if the subgraph node has a graph asset assigned to it.
			if (UMovieGraphConfig* SubgraphConfig = SubgraphNode->GetSubgraphAsset())
			{
				// Don't recurse into this graph if it was already duplicated. Check BOTH the keys (the original graph) AND value (the duplicated graph)
				// to prevent recursion. Checking the key ensures that we only duplicate if this graph has never been encountered. Checking the value
				// ensures that we don't re-duplicate a graph that has already been duplicated (the subgraph node was already updated).
				bool bHasBeenDuplicated = false;
				for (const TPair<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>>& DuplicateMapping : OutDuplicatedGraphs)
				{
					if ((DuplicateMapping.Key == SubgraphConfig) || (DuplicateMapping.Value == SubgraphConfig))
					{
						bHasBeenDuplicated = true;
						break;
					}
				}
				
				if (!bHasBeenDuplicated)
				{
					DuplicateConfigRecursive(SubgraphConfig, OutDuplicatedGraphs);
				}

				// Update the subgraph node to use the duplicated graph. This should always be done, even if the graph was already duplicated (since
				// a graph can be included as a subgraph in multiple locations).
				if (const TObjectPtr<UMovieGraphConfig>* DuplicatedGraph = OutDuplicatedGraphs.Find(SubgraphConfig))
				{
					SubgraphNode->SetSubGraphAsset(*DuplicatedGraph);
				}
			}
		}
		
		return DuplicateConfig;
	}
}

//namespace UE
//{
namespace MoviePipeline
{
	void GatherLeafNodesRecursive(TSharedPtr<FCameraCutSubSectionHierarchyNode> InNode, TArray<TSharedPtr<FCameraCutSubSectionHierarchyNode>>& OutLeaves)
	{
		for (TSharedPtr<FCameraCutSubSectionHierarchyNode> Child : InNode->GetChildren())
		{
			GatherLeafNodesRecursive(Child, OutLeaves);
		}

		if (InNode->GetChildren().Num() == 0)
		{
			OutLeaves.Add(InNode);
		}
	}

	void BuildCompleteSequenceHierarchyRecursive(UMovieSceneSequence* InSequence, TSharedPtr<FCameraCutSubSectionHierarchyNode> InNode)
	{
		InNode->MovieScene = InSequence->GetMovieScene();

		if (UMovieSceneCameraCutTrack* CameraCutTrack = Cast<UMovieSceneCameraCutTrack>(InSequence->GetMovieScene()->GetCameraCutTrack()))
		{
			// We create leaf nodes for each section. This kind of makes duplicate leafs but since this is separate from the evaluation tree
			// and we only use it to save/restore state at the end, its ok.
			for (UMovieSceneSection* Section : CameraCutTrack->GetAllSections())
			{
				TSharedPtr<FCameraCutSubSectionHierarchyNode> Node = MakeShared<FCameraCutSubSectionHierarchyNode>();
				Node->MovieScene = TWeakObjectPtr<UMovieScene>(InSequence->GetMovieScene());
				Node->CameraCutSection = TWeakObjectPtr<UMovieSceneCameraCutSection>(Cast<UMovieSceneCameraCutSection>(Section));
				InNode->AddChild(Node);
			}
		}

		// The evaluation tree only contains the active bits of the hierarchy (which is what we want). However we disable non-active sections while
		// soloing, and we can't restore them because they weren't part of the shot hierarchy. To resolve this, we build a complete copy of the
		// original for restoration at the end. We'll build our own tree, kept separate from the per-shot ones.
		for (UMovieSceneTrack* Track : InSequence->GetMovieScene()->GetTracks())
		{
			UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track);
			if (!SubTrack)
			{
				continue;
			}

			for (UMovieSceneSection* Section : SubTrack->GetAllSections())
			{
				UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
				if (!SubSection)
				{
					continue;
				}

				TSharedPtr<FCameraCutSubSectionHierarchyNode> Node = MakeShared<FCameraCutSubSectionHierarchyNode>();
				Node->MovieScene = TWeakObjectPtr<UMovieScene>(InSequence->GetMovieScene());
				Node->Section = TWeakObjectPtr<UMovieSceneSubSection>(SubSection);
				InNode->AddChild(Node);

				if (SubSection->GetSequence())
				{
					BuildCompleteSequenceHierarchyRecursive(SubSection->GetSequence(), Node);
				}
			}
		}

		// Cache any sections that we will auto-expand later.
		for (UMovieSceneSection* Section : InSequence->GetMovieScene()->GetAllSections())
		{
			if (Section->GetSupportsInfiniteRange())
			{
				InNode->AdditionalSectionsToExpand.Add(MakeTuple(Section, Section->GetRange()));
			}
		}
	}

	void SaveOrRestoreSubSectionHierarchy(TSharedPtr<FCameraCutSubSectionHierarchyNode> InLeaf, const bool bInSave)
	{
		TSharedPtr<FCameraCutSubSectionHierarchyNode> Node = InLeaf;
		while (Node)
		{
			// Use the playback range as a sentinel: if it has open bounds, save was never called
			// for this node (e.g. the pipeline shut down before hierarchy was saved). Skip the
			// entire restoration to avoid writing default/uninitialized values back.
			if (!bInSave && Node->MovieScene.IsValid()
				&& !ensureMsgf(Node->OriginalMovieScenePlaybackRange.GetLowerBound().IsClosed() && Node->OriginalMovieScenePlaybackRange.GetUpperBound().IsClosed(),
					TEXT("Skipping restoration for MovieScene '%s' — cached range has open bounds (pipeline shut down before hierarchy was saved?)."), *GetNameSafe(Node->MovieScene.Get())))
			{
				Node = Node->GetParent();
				continue;
			}

			if (Node->MovieScene.IsValid())
			{
				if (bInSave)
				{
					Node->OriginalMovieScenePlaybackRange = Node->MovieScene->GetPlaybackRange();
#if WITH_EDITOR
					Node->bOriginalMovieSceneReadOnly = Node->MovieScene->IsReadOnly();
					Node->bOriginalMovieScenePlaybackRangeLocked = Node->MovieScene->IsPlaybackRangeLocked();
#endif
					if (UPackage* OwningPackage = Node->MovieScene->GetPackage())
					{
						Node->bOriginalMovieScenePackageDirty = OwningPackage->IsDirty();
					}

					Node->EvaluationType = Node->MovieScene->GetEvaluationType();

					if (ULevelSequence* OwningSequence = Node->MovieScene->GetTypedOuter<ULevelSequence>())
					{
						Node->OriginalSequenceFlags = OwningSequence->GetFlags();
					}
				}

				// Unlock the movie scene so we can make changes to sections below, it'll get re-locked later if needed.
				// This has to be done each iteration of the loop because we iterate through leaves, so the first leaf 
				// can end up re-locking a MovieScene higher up in the hierarchy, and then when subsequent leaves try
				// to restore their hierarchy the now-locked MovieScene prevents full restoration.
#if WITH_EDITOR
				if (!bInSave)
				{
					Node->MovieScene->SetReadOnly(false);
				}
#endif
			}

			if (Node->Section.IsValid())
			{
				if (bInSave)
				{
					Node->bOriginalShotSectionIsLocked = Node->Section->IsLocked();
					Node->OriginalShotSectionRange = Node->Section->GetRange();
					Node->bOriginalShotSectionIsActive = Node->Section->IsActive();
				}
				else
				{
					Node->Section->SetRange(Node->OriginalShotSectionRange);
					Node->Section->SetIsActive(Node->bOriginalShotSectionIsActive);
					Node->Section->SetIsLocked(Node->bOriginalShotSectionIsLocked);
					Node->Section->MarkAsChanged();
				}
			}

			if (Node->CameraCutSection.IsValid())
			{
				if (bInSave)
				{
					Node->OriginalCameraCutSectionRange = Node->CameraCutSection->GetRange();
					Node->bOriginalCameraCutIsActive = Node->CameraCutSection->IsActive();
				}
				else
				{
					Node->CameraCutSection->SetRange(Node->OriginalCameraCutSectionRange);
					Node->CameraCutSection->SetIsActive(Node->bOriginalCameraCutIsActive);

					Node->CameraCutSection->MarkAsChanged();
				}
			}

			if(!bInSave)
			{
				// These are restored, but they're not saved using this function. This is because they're cached earlier
				for (const TTuple<UMovieSceneSection*, TRange<FFrameNumber>>& Pair : Node->AdditionalSectionsToExpand)
				{
					Pair.Key->SetRange(Pair.Value);
					Pair.Key->MarkAsChanged();
				}
			}

			if (Node->MovieScene.IsValid())
			{
				// Has to come last otherwise calls to MarkAsChanged from children re-dirty the package.
				if (!bInSave)
				{
					Node->MovieScene->SetPlaybackRange(Node->OriginalMovieScenePlaybackRange);
					Node->MovieScene->SetEvaluationType(Node->EvaluationType);
					if (ULevelSequence* OwningSequence = Node->MovieScene->GetTypedOuter<ULevelSequence>())
					{
						OwningSequence->SetSequenceFlags(Node->OriginalSequenceFlags);
					}
#if WITH_EDITOR
					Node->MovieScene->SetReadOnly(Node->bOriginalMovieSceneReadOnly);
					Node->MovieScene->SetPlaybackRangeLocked(Node->bOriginalMovieScenePlaybackRangeLocked);
#endif
					Node->MovieScene->MarkAsChanged();

					if (UPackage* OwningPackage = Node->MovieScene->GetPackage())
					{
						OwningPackage->SetDirtyFlag(Node->bOriginalMovieScenePackageDirty);
					}
				}
			}

			Node = Node->GetParent();
		}
	}

	void SetSubSectionHierarchyActive(TSharedPtr<FCameraCutSubSectionHierarchyNode> InRoot, bool bInActive)
	{
		TSharedPtr<FCameraCutSubSectionHierarchyNode> Node = InRoot;
		while (Node)
		{
			if (Node->MovieScene.IsValid())
			{
#if WITH_EDITOR
				Node->MovieScene->SetReadOnly(false);
				Node->MovieScene->SetPlaybackRangeLocked(false);
#endif
				Node->MovieScene->SetEvaluationType(EMovieSceneEvaluationType::WithSubFrames);
			}

			if (Node->Section.IsValid())
			{
				Node->Section->SetIsLocked(false);
			}

			if (Node->CameraCutSection.IsValid())
			{
				Node->CameraCutSection->SetIsActive(bInActive);
				Node->CameraCutSection->MarkAsChanged();

				UE_LOGF(LogMovieRenderPipeline, Verbose, "Disabled CameraCutSection: %ls while soloing shot.", *GetNameSafe(Node->CameraCutSection.Get()));
			}

			if (Node->Section.IsValid())
			{
				Node->Section->SetIsActive(bInActive);
				Node->Section->MarkAsChanged();

				UE_LOGF(LogMovieRenderPipeline, Verbose, "Disabled SubSequenceSection: %ls while soloing shot.", *GetNameSafe(Node->Section.Get()));
			}

			Node = Node->GetParent();
		}
	}

	void CheckPartialSectionEvaluationAndWarn(const FFrameNumber& LeftDeltaTicks, TSharedPtr<FCameraCutSubSectionHierarchyNode> Node, UMoviePipelineExecutorShot* InShot, const FFrameRate& InRootDisplayRate)
	{
		// For the given movie scene, we want to produce a warning if there is no data to evaluate once we've expanded for 
		// handle frames or temporal sub-sampling. To do our best guess at which tracks are relevant, we can look at the
		// the range between (-HandleFrames+TemporalFrames, PlaybackRangeStart]. We are inclusive for PlaybackRangeStart
		// so that we detect the most common case - all tracks starting on frame 0. We are exclusive of the lower bound
		// so that we don't detect sections that have been correctly expanded. We also produce warnings where we find
		// Shots/Camera cut sections that don't land on whole frames and we warn for those too, as it's often undesired
		// and offsets the output frame number (when rounded back to whole numbers) for output
		if (LeftDeltaTicks > 0 && Node->MovieScene.IsValid())
		{
			FFrameNumber LowerCheckBound = InShot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue() - LeftDeltaTicks;
			FFrameNumber UpperCheckBound = InShot->ShotInfo.TotalOutputRangeRoot.GetLowerBoundValue();

			TRange<FFrameNumber> CheckRange = TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Exclusive(LowerCheckBound), TRangeBound<FFrameNumber>::Inclusive(UpperCheckBound));

			for (const UMovieSceneSection* Section : Node->MovieScene->GetAllSections())
			{
				if (!Section)
				{
					continue;
				}

				// If the section can be made infinite, it will automatically get expanded when the shot is activated, so no need to warn.
				if (Section->GetSupportsInfiniteRange())
				{
					continue;
				}

				// camera cut and sub-sections also will get the expansion manually, no need to warn
				if (Section == Node->Section || Section == Node->CameraCutSection)
				{
					continue;
				}

				if (Section->GetRange().HasLowerBound())
				{
					const bool bOverlaps = CheckRange.Contains(Section->GetRange().GetLowerBoundValue());
					if (bOverlaps)
					{
						FString SectionName = Section->GetName();
						FString BindingName = TEXT("None");

						// Try to find which binding it belongs to as the names will be more useful than section types.
						for (const FMovieSceneBinding& Binding : ((const UMovieScene*)(Node->MovieScene.Get()))->GetBindings())
						{
							for (const UMovieSceneTrack* BindingTrack : Binding.GetTracks())
							{
								if (BindingTrack->HasSection(*Section))
								{
									BindingName = Node->MovieScene.Get()->GetObjectDisplayName(Binding.GetObjectGuid()).ToString();
									break;
								}
							}
						}

						// Convert ticks back to frames for human consumption
						FFrameNumber LowerCheckBoundFrame = FFrameRate::TransformTime(LowerCheckBound, InShot->ShotInfo.CachedTickResolution, InRootDisplayRate).FloorToFrame();
						FFrameNumber UpperCheckBoundFrame = FFrameRate::TransformTime(UpperCheckBound, InShot->ShotInfo.CachedTickResolution, InRootDisplayRate).FloorToFrame();

						UE_LOGF(LogMovieRenderPipeline, Warning, "[%ls %ls] Due to Temporal sub-sampling or handle frames, evaluation will occur outside of shot boundaries (from frame %d to %d). Section %ls (Binding: %ls) starts during this time period and cannot be auto-expanded. Please extend this section to start on frame %d. (All times listed are relative to the root sequence)",
							*InShot->OuterName, *InShot->InnerName, LowerCheckBoundFrame.Value, UpperCheckBoundFrame.Value,
							*SectionName, *BindingName, LowerCheckBoundFrame.Value);
					}
				}
			}
		}
	}

	void CacheCompleteSequenceHierarchy(UMovieSceneSequence* InSequence, TSharedPtr<FCameraCutSubSectionHierarchyNode> InRootNode)
	{
		// The evaluation tree only contains the active bits of the hierarchy (which is what we want). However we disable non-active sections while
		// soloing, and we can't restore them because they weren't part of the shot hierarchy. To resolve this, we build a complete copy of the
		// original for restoration at the end. We'll build our own tree, kept separate from the per-shot ones.
		BuildCompleteSequenceHierarchyRecursive(InSequence, InRootNode);

		TArray<TSharedPtr<FCameraCutSubSectionHierarchyNode>> LeafNodes;
		GatherLeafNodesRecursive(InRootNode, LeafNodes);

		// Now cache the values - playback ranges, section sizes, active states, etc.
		for (TSharedPtr<FCameraCutSubSectionHierarchyNode> Leaf : LeafNodes)
		{
			const bool bInSave = true;

			// This function only takes leaves. Technically we'll end up re-caching the parents multiple times,
			// but the values don't change so it's a non-issue.
			SaveOrRestoreSubSectionHierarchy(Leaf, bInSave);
		}
	}

	void RestoreCompleteSequenceHierarchy(UMovieSceneSequence* InSequence, TSharedPtr<FCameraCutSubSectionHierarchyNode> InRootNode)
	{
		TArray<TSharedPtr<FCameraCutSubSectionHierarchyNode>> LeafNodes;
		GatherLeafNodesRecursive(InRootNode, LeafNodes);
		for (TSharedPtr<FCameraCutSubSectionHierarchyNode> Leaf : LeafNodes)
		{
			const bool bInSave = false;
			SaveOrRestoreSubSectionHierarchy(Leaf, bInSave);
		}
	}

	TTuple<FString, FString> GetNameForShot(const FMovieSceneSequenceHierarchy& InHierarchy, UMovieSceneSequence* InRootSequence, TSharedPtr<FCameraCutSubSectionHierarchyNode> InSubSectionHierarchy)
	{
		FString InnerName;
		FString OuterName;

		// The inner name is the camera cut (if available) otherwise it falls back to the name of the moviescene.
		if (InSubSectionHierarchy->CameraCutSection.IsValid())
		{
			const FMovieSceneObjectBindingID& CameraObjectBindingId = InSubSectionHierarchy->CameraCutSection->GetCameraBindingID();
			if (CameraObjectBindingId.IsValid())
			{
				// Look up the correct sequence from the compiled hierarchy, as bindings can exist in other movie scenes.
				FMovieSceneSequenceID ParentId = InSubSectionHierarchy->NodeID;
				UMovieSceneSequence* OwningSequence = InRootSequence;
				if (ParentId != MovieSceneSequenceID::Invalid)
				{
					FMovieSceneSequenceID ResolvedSequenceId = CameraObjectBindingId.ResolveSequenceID(ParentId, &InHierarchy);
					if (InHierarchy.FindSubSequence(ResolvedSequenceId))
					{
						OwningSequence = InHierarchy.FindSubSequence(ResolvedSequenceId);
					}
				}

				if (OwningSequence && OwningSequence->GetMovieScene())
				{
					InnerName = OwningSequence->GetMovieScene()->GetObjectDisplayName(CameraObjectBindingId.GetGuid()).ToString();
				}
			}
		}
		else if (InSubSectionHierarchy->MovieScene.IsValid())
		{
			InnerName = FPaths::GetBaseFilename(InSubSectionHierarchy->MovieScene->GetPathName());
		}

		// The outer name is a little more complicated. We don't use Outers here because each subscene is outered to its own package.
		TArray<FString> Names;
		TSharedPtr<FCameraCutSubSectionHierarchyNode> CurNode = InSubSectionHierarchy;
		while (CurNode)
		{
			// Camera Cut owned by Track, Track owned by MovieScene, MovieScene by ULevelSequence, Level Sequence by Package
			if (UMovieSceneCinematicShotSection* AsShot = Cast<UMovieSceneCinematicShotSection>(CurNode->Section))
			{
				Names.Add(AsShot->GetShotDisplayName());
			}
			else if (UMovieSceneSubSection* AsSubSequence = Cast<UMovieSceneSubSection>(CurNode->Section))
			{
				// Sub-sequences don't have renameable sections, we just have to use the target sequence name
				if (AsSubSequence->GetSequence())
				{
					Names.Add(FPaths::GetBaseFilename(AsSubSequence->GetSequence()->GetPathName()));

				}
			}

			CurNode = CurNode->GetParent();
		}

		// We built these inner to outer so we need to reverse them
		TStringBuilder<64> StringBuilder;
		for (int32 Index = Names.Num() - 1; Index >= 0; Index--)
		{
			StringBuilder.Append(*Names[Index]);
			if (Index != 0)
			{
				// Separate them by dots, but skip the dot on the last one.
				StringBuilder.Append(TEXT("."));
			}
		}

		// If you don't have any shots, then StringBuilder will be empty.
		if (StringBuilder.Len() == 0)
		{
			StringBuilder.Append(TEXT("no shot"));
		}
		OuterName = StringBuilder.ToString();

		return TTuple<FString, FString>(InnerName, OuterName);
	}

	void BuildSectionHierarchyRecursive(const FMovieSceneSequenceHierarchy& InHierarchy, UMovieSceneSequence* InRootSequence, const FMovieSceneSequenceID InSequenceId, const FMovieSceneSequenceID InChildId, TSharedPtr<FCameraCutSubSectionHierarchyNode> OutSubsectionHierarchy)
	{
		const FMovieSceneSequenceHierarchyNode* SequenceNode = InHierarchy.FindNode(InSequenceId);
		const UMovieSceneSequence* Sequence = (InSequenceId == MovieSceneSequenceID::Root) ? InRootSequence : InHierarchy.FindSubSequence(InSequenceId);
		const FMovieSceneSubSequenceData* ChildSubSectionData = InHierarchy.FindSubData(InChildId);

		if (!SequenceNode || !Sequence || !Sequence->GetMovieScene())
		{
			return;
		}


		if (ChildSubSectionData)
		{
			const TArray<UMovieSceneTrack*> Tracks = Sequence->GetMovieScene()->GetTracks();
			for (const UMovieSceneTrack* Track : Tracks)
			{
				// SubTracks encompass both Cinematic Shot sections and Sub-Sequences
				if (Track && Track->IsA<UMovieSceneSubTrack>())
				{
					const UMovieSceneSubTrack* SubTrack = CastChecked<UMovieSceneSubTrack>(Track);
					for (UMovieSceneSection* Section : SubTrack->GetAllSections())
					{
						UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
						if (SubSection)
						{
							const bool bMatch = SubSection->GetSignature() == ChildSubSectionData->GetSubSectionSignature();
							if (bMatch)
							{
								// This sub-section is the owner of our child. Push it into our tree.
								OutSubsectionHierarchy->Section = TWeakObjectPtr<UMovieSceneSubSection>(SubSection);
								break;
							}
						}

					}
				}
			}
		}

		OutSubsectionHierarchy->MovieScene = TWeakObjectPtr<UMovieScene>(Sequence->GetMovieScene());
		OutSubsectionHierarchy->NodeID = InSequenceId;

		// Only try assigning the parent and diving in if this node isn't already the root, roots have no parents.
		if (InSequenceId != MovieSceneSequenceID::Root)
		{
			TSharedPtr<FCameraCutSubSectionHierarchyNode> ParentNode = MakeShared<FCameraCutSubSectionHierarchyNode>();
			OutSubsectionHierarchy->SetParent(ParentNode);

			BuildSectionHierarchyRecursive(InHierarchy, InRootSequence, SequenceNode->ParentID, InSequenceId, ParentNode);
		}
	}


} // MoviePipeline
//} // UE

namespace UE
{
	namespace MoviePipeline
	{
		DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_HardwareMetadata"), STAT_HardwareMetadata, STATGROUP_MoviePipeline);
		DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_ClothAdjust"), STAT_ClothSubstepAdjust, STATGROUP_MoviePipeline);

		bool GetAlphaOutputOverride()
		{
			return CVarMoviePipelineAlphaOutputOverride.GetValueOnAnyThread();
		}

		void ConformOutputFormatStringToken(FString& InOutFilenameFormatString, const FStringView InToken, const FName& InNodeName, const FName& InBranchName)
		{
			static const FString FrameNumberIdentifiers[] = { TEXT("{frame_number}"), TEXT("{frame_number_shot}"), TEXT("{frame_number_rel}"), TEXT("{frame_number_shot_rel}") };

			if (!InOutFilenameFormatString.Contains(InToken, ESearchCase::IgnoreCase))
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "Missing expected %ls format token on node '%ls' in branch '%ls'. Automatically adding!", InToken.GetData(), *InNodeName.ToString(), *InBranchName.ToString());

				// Search for a frame number in the output string
				int32 FrameNumberIndex = INDEX_NONE;
				for (const FString& Identifier : FrameNumberIdentifiers)
				{
					FrameNumberIndex = InOutFilenameFormatString.Find(Identifier, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
					if (FrameNumberIndex != INDEX_NONE)
					{
						break;
					}
				}

				if (FrameNumberIndex == INDEX_NONE)
				{
					// No frame number found, so just append the token
					InOutFilenameFormatString += InToken;
				}
				else
				{
					// If a frame number is found, we need to insert the token first before it, so various editing
					// software will still be able to identify if this is an image sequence
					InOutFilenameFormatString.InsertAt(FrameNumberIndex, FString(InToken) + TEXT("."));
				}
			}
		}

		void ValidateOutputFormatString(FString& InOutFilenameFormatString, const bool bTestRenderPass, const bool bTestFrameNumber, const bool bTestCameraName)
		{
			const FString FrameNumberIdentifiers[] = { TEXT("{frame_number}"), TEXT("{frame_number_shot}"), TEXT("{frame_number_rel}"), TEXT("{frame_number_shot_rel}") };

			// If there is more than one file being written for this frame, make sure they uniquely identify.
			if (bTestRenderPass)
			{
				if (!InOutFilenameFormatString.Contains(TEXT("{render_pass}"), ESearchCase::IgnoreCase))
				{
					UE_LOGF(LogMovieRenderPipeline, Warning, "Multiple render passes exported but no {render_pass} format found. Automatically adding!");

					// Search for a frame number in the output string
					int32 FrameNumberIndex = INDEX_NONE;
					for (const FString& Identifier : FrameNumberIdentifiers)
					{
						FrameNumberIndex = InOutFilenameFormatString.Find(Identifier, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
						if (FrameNumberIndex != INDEX_NONE)
						{
							break;
						}
					}

					if (FrameNumberIndex == INDEX_NONE)
					{
						// No frame number found, so just append render_pass
						InOutFilenameFormatString += TEXT("{render_pass}");
					}
					else
					{
						// If a frame number is found, we need to insert render_pass first before it, so various editing
						// software will still be able to identify if this is an image sequence
						InOutFilenameFormatString.InsertAt(FrameNumberIndex, TEXT("{render_pass}."));
					}
				}
			}

			// Could be more than one camera being written to, make sure there's {camera_name} so it doesn't stomp over each other.
			if (bTestCameraName)
			{
				if (!InOutFilenameFormatString.Contains(TEXT("{camera_name}"), ESearchCase::IgnoreCase))
				{
					UE_LOGF(LogMovieRenderPipeline, Warning, "Multiple cameras exported but no {camera_name} format found. Automatically adding!");

					// Search for a frame number in the output string
					int32 FrameNumberIndex = INDEX_NONE;
					for (const FString& Identifier : FrameNumberIdentifiers)
					{
						FrameNumberIndex = InOutFilenameFormatString.Find(Identifier, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
						if (FrameNumberIndex != INDEX_NONE)
						{
							break;
						}
					}

					if (FrameNumberIndex == INDEX_NONE)
					{
						// No frame number found, so just append render_pass
						InOutFilenameFormatString += TEXT("{camera_name}");
					}
					else
					{
						// If a frame number is found, we need to insert render_pass first before it, so various editing
						// software will still be able to identify if this is an image sequence
						InOutFilenameFormatString.InsertAt(FrameNumberIndex, TEXT("{camera_name}."));
					}
				}
			}

			if (bTestFrameNumber)
			{
				// Ensure there is a frame number in the output string somewhere to uniquely identify individual files in an image sequence.
				int32 FrameNumberIndex = INDEX_NONE;
				for (const FString& Identifier : FrameNumberIdentifiers)
				{
					FrameNumberIndex = InOutFilenameFormatString.Find(Identifier, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
					if (FrameNumberIndex != INDEX_NONE)
					{
						break;
					}
				}

				// We want to insert a {file_dup} before the frame number. This instructs the name resolver to put the (2) before
				// the frame number, so that they're still properly recognized as image sequences by other software. It will resolve
				// to "" if not needed.
				if (FrameNumberIndex == INDEX_NONE)
				{
					// Previously, the frame number identifier would be inserted so that files would not be overwritten. However, users prefer to have exact control over the filename.
					//InOutFilenameFormatString.Append(TEXT("{file_dup}.{frame_number}"));
					UE_LOGF(LogMovieRenderPipeline, Warning, "Frame number identifier not found. Files may be overwritten.");
				}
				else
				{
					// The user had already specified a frame number identifier, so we need to insert the
					// file_dup tag before it.
					InOutFilenameFormatString.InsertAt(FrameNumberIndex, TEXT("{file_dup}"));
				}
			}

			if (!InOutFilenameFormatString.Contains(TEXT("{file_dup}"), ESearchCase::IgnoreCase))
			{
				InOutFilenameFormatString.Append(TEXT("{file_dup}"));
			}
		}

		void DeduplicateNameArray(TArray<FString>& InOutNames)
		{
			TMap<FString, int32> NameUseCount;
			for (FString& Name : InOutNames)
			{
				int32& Count = NameUseCount.FindOrAdd(Name, 0);
				if (++Count > 1)
				{
					Name.Append(FString::Format(TEXT("({0})"), { NameUseCount[Name] }));
				}
			}

			// For any names we found duplicates of, append (1) to the first to keep naming consistent
			for (TPair<FString, int32>& Pair : NameUseCount)
			{
				if (Pair.Value > 1)
				{
					for (FString& Name : InOutNames)
					{
						if (Name.Equals(Pair.Key))
						{
							Name.Append(TEXT("(1)"));
							break;
						}
					}
				}
			}
		}

		void RemoveFrameNumberFormatStrings(FString& InOutFilenameFormatString, const bool bIncludeShots)
		{
			// Strip {frame_number} related separators from their file name, otherwise it will create one output file per frame.
			InOutFilenameFormatString.ReplaceInline(TEXT("{frame_number}"), TEXT(""));
			InOutFilenameFormatString.ReplaceInline(TEXT("{frame_number_rel}"), TEXT(""));

			if (bIncludeShots)
			{
				InOutFilenameFormatString.ReplaceInline(TEXT("{frame_number_shot}"), TEXT(""));
				InOutFilenameFormatString.ReplaceInline(TEXT("{frame_number_shot_rel}"), TEXT(""));
			}
		}

		FString GetJobAuthor(const UMoviePipelineExecutorJob* InJob)
		{
			if (InJob && InJob->Author.Len() > 0)
			{
				return InJob->Author;
			}

			// If they didn't specify an author in the job, default to the local username.
			return FPlatformProcess::UserName(false);
		}

		void GetSharedFormatArguments(TMap<FString, FString>& InFilenameArguments, TMap<FString, FString>& InFileMetadata, const FDateTime& InDateTime, const int32 InVersionNumber, const UMoviePipelineExecutorJob* InJob, const FTimespan& InInitializationTimeOffset)
		{
			FDateTime DateTimeLocal = InDateTime + InInitializationTimeOffset; 

			FString LocalDateStr = DateTimeLocal.ToString(TEXT("%Y.%m.%d"));
			FString LocalTimeStr = DateTimeLocal.ToString(TEXT("%H.%M.%S"));
			FString LocalYearStr = DateTimeLocal.ToString(TEXT("%Y"));
			FString LocalMonthStr = DateTimeLocal.ToString(TEXT("%m"));
			FString LocalDayStr = DateTimeLocal.ToString(TEXT("%d"));


			InFilenameArguments.Add(TEXT("date"), LocalDateStr);
			InFilenameArguments.Add(TEXT("time"), LocalTimeStr);
			InFilenameArguments.Add(TEXT("year"), LocalYearStr);
			InFilenameArguments.Add(TEXT("month"), LocalMonthStr);
			InFilenameArguments.Add(TEXT("day"), LocalDayStr);


			FString VersionText = FString::Printf(TEXT("v%0*d"), 3, InVersionNumber);

			InFilenameArguments.Add(TEXT("version"), VersionText);

			InFileMetadata.Add(TEXT("unreal/jobDate"), LocalDateStr);
			InFileMetadata.Add(TEXT("unreal/jobTime"), LocalTimeStr);
			InFileMetadata.Add(TEXT("unreal/jobYear"), LocalYearStr);
			InFileMetadata.Add(TEXT("unreal/jobMonth"), LocalMonthStr);
			InFileMetadata.Add(TEXT("unreal/jobDay"), LocalDayStr);

			InFileMetadata.Add(TEXT("unreal/jobVersion"), FString::FromInt(InVersionNumber));

			const FString JobAuthor = InJob ? GetJobAuthor(InJob) : FString();
			const FString JobName = InJob ? InJob->JobName : FString();
			const FString JobComment = InJob ? InJob->Comment : FString();

			InFilenameArguments.Add(TEXT("job_author"), JobAuthor);
			InFilenameArguments.Add(TEXT("job_name"), JobName);
			InFileMetadata.Add(TEXT("unreal/jobName"), JobName);
			InFileMetadata.Add(TEXT("unreal/jobAuthor"), JobAuthor);
			InFileMetadata.Add(TEXT("unreal/jobComment"), JobComment);
		}

		void GetDiagnosticMetadata(TMap<FString, FString>& InFileMetadata, const bool bIsGraph)
		{
			InFileMetadata.Add(TEXT("unreal/build"), FEngineVersion::Current().ToString());
			InFileMetadata.Add(TEXT("unreal/usesGraph"), LexToString(bIsGraph));
		}

		void GetCachedGPUDriverInfo(TMap<FString, FString>& InFileMetadata)
		{
			static TMap<FString, FString> CachedInfo;
			if (CachedInfo.Num() > 0)
			{
				InFileMetadata.Append(CachedInfo);
				return;
			}

			FGPUDriverInfo DriverInfo = FPlatformMisc::GetGPUDriverInfo(GRHIAdapterName);
			CachedInfo.Add(TEXT("unreal/system/gpu/vendorId"), LexToString(DriverInfo.VendorId));
			CachedInfo.Add(TEXT("unreal/system/gpu/deviceDescription"), DriverInfo.DeviceDescription);
			CachedInfo.Add(TEXT("unreal/system/gpu/providerName"), DriverInfo.ProviderName);
			CachedInfo.Add(TEXT("unreal/system/gpu/internalDriverVersion"), DriverInfo.InternalDriverVersion);
			CachedInfo.Add(TEXT("unreal/system/gpu/userDriverVersion"), DriverInfo.UserDriverVersion);
			CachedInfo.Add(TEXT("unreal/system/gpu/driverDate"), DriverInfo.DriverDate);
			CachedInfo.Add(TEXT("unreal/system/gpu/rhiName"), DriverInfo.RHIName);

			InFileMetadata.Append(CachedInfo);
		}

		void GetHardwareUsageMetadata(TMap<FString, FString>& InFileMetadata, const FString& InOutputDir)
		{
			SCOPE_CYCLE_COUNTER(STAT_HardwareMetadata);
			
			// Stats that change per-frame go in /stats/, consistent things go in /system/ though logical grouping
			// overrides some things such as total vs. current memory.
			static const uint64 MBDivider = 1024 * 1024;

			InFileMetadata.Add(TEXT("unreal/system/cpuVendor"), FPlatformMisc::GetCPUVendor());
			InFileMetadata.Add(TEXT("unreal/system/cpuChipset"), FPlatformMisc::GetCPUChipset());
			InFileMetadata.Add(TEXT("unreal/system/cpuBrand"), FPlatformMisc::GetCPUBrand());
			
			FString OSVersionLabel, OSSubVersionLabel;
			FPlatformMisc::GetOSVersions(OSVersionLabel, OSSubVersionLabel);
			InFileMetadata.Add(TEXT("unreal/system/osVersion"), OSVersionLabel);
			InFileMetadata.Add(TEXT("unreal/system/osSubVersion"), OSSubVersionLabel);
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			InFileMetadata.Add(TEXT("unreal/system/deviceTemp"), LexToString(FPlatformMisc::GetDeviceTemperatureLevel())); // Deprecated in 5.8 in favor of the two separate stats below.
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			InFileMetadata.Add(TEXT("unreal/system/deviceTemperature"), LexToString(FPlatformMisc::GetDeviceTemperature()));
			InFileMetadata.Add(TEXT("unreal/system/deviceThermalState"), LexToString(static_cast<int>(FPlatformMisc::GetDeviceThermalState())));
			InFileMetadata.Add(TEXT("unreal/system/deviceMakeAndModel"), FPlatformMisc::GetDeviceMakeAndModel());
			
			if (InOutputDir.Len() > 0)
			{
				uint64 TotalNumBytes, NumFreeBytes;
				FPlatformMisc::GetDiskTotalAndFreeSpace(InOutputDir, TotalNumBytes, NumFreeBytes);
				InFileMetadata.Add(TEXT("unreal/stats/outputDirectoryTotalSizeMB"), LexToString(TotalNumBytes / MBDivider));
				InFileMetadata.Add(TEXT("unreal/stats/outputDirectoryTotalFreeMB"), LexToString(NumFreeBytes / MBDivider));
			}

			// We cache this as each time we fetch it from FPlatformMisc it prints to the log, and
			// these values aren't going to change anyways during runtime.
			GetCachedGPUDriverInfo(InFileMetadata);

			FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
			InFileMetadata.Add(TEXT("unreal/stats/memory/availablePhysicalMB"), LexToString(MemoryStats.AvailablePhysical / MBDivider));
			InFileMetadata.Add(TEXT("unreal/stats/memory/availableVirtualMB"), LexToString(MemoryStats.AvailableVirtual / MBDivider));
			InFileMetadata.Add(TEXT("unreal/stats/memory/totalPhysicalMB"), LexToString(MemoryStats.TotalPhysical / MBDivider));
			InFileMetadata.Add(TEXT("unreal/stats/memory/totalVirtualMB"), LexToString(MemoryStats.TotalVirtual / MBDivider));
			InFileMetadata.Add(TEXT("unreal/stats/memory/peakUsedPhysicalMB"), LexToString(MemoryStats.PeakUsedPhysical / MBDivider));
			InFileMetadata.Add(TEXT("unreal/stats/memory/peakUsedVirtualMB"), LexToString(MemoryStats.PeakUsedVirtual / MBDivider));
		}

		void GetMetadataFromCineCamera(UCineCameraComponent* InComponent, const FString& InCameraName, const FString& InRenderPassName, TMap<FString, FString>& InOutMetadata)
		{
			FString Prefix = FString::Printf(TEXT("unreal/%s/%s"), *InCameraName, *InRenderPassName);
			GetMetadataFromCineCamera(InComponent, Prefix, InOutMetadata);
		}
		
		void GetMetadataFromCineCamera(UCineCameraComponent* InComponent, const FString& InPrefix, TMap<FString, FString>& InOutMetadata)
		{
			if (!InComponent) 
			{ 
				return; 
			}

			InOutMetadata.Add(FString::Printf(TEXT("%s/sensorWidth"), *InPrefix), FString::SanitizeFloat(InComponent->Filmback.SensorWidth));
			InOutMetadata.Add(FString::Printf(TEXT("%s/sensorHeight"), *InPrefix), FString::SanitizeFloat(InComponent->Filmback.SensorHeight));
			InOutMetadata.Add(FString::Printf(TEXT("%s/sensorAspectRatio"), *InPrefix), FString::SanitizeFloat(InComponent->Filmback.SensorAspectRatio));
			InOutMetadata.Add(FString::Printf(TEXT("%s/minFocalLength"), *InPrefix), FString::SanitizeFloat(InComponent->LensSettings.MinFocalLength));
			InOutMetadata.Add(FString::Printf(TEXT("%s/maxFocalLength"), *InPrefix), FString::SanitizeFloat(InComponent->LensSettings.MaxFocalLength));
			InOutMetadata.Add(FString::Printf(TEXT("%s/minFStop"), *InPrefix), FString::SanitizeFloat(InComponent->LensSettings.MinFStop));
			InOutMetadata.Add(FString::Printf(TEXT("%s/maxFStop"), *InPrefix), FString::SanitizeFloat(InComponent->LensSettings.MaxFStop));
			InOutMetadata.Add(FString::Printf(TEXT("%s/dofDiaphragmBladeCount"), *InPrefix), FString::FromInt(InComponent->LensSettings.DiaphragmBladeCount));
			InOutMetadata.Add(FString::Printf(TEXT("%s/focalLength"), *InPrefix), FString::SanitizeFloat(InComponent->CurrentFocalLength));
			InOutMetadata.Add(FString::Printf(TEXT("%s/squeezeFactor"), *InPrefix), FString::SanitizeFloat(InComponent->LensSettings.SqueezeFactor));
		}

		void GetMetadataFromCameraLocRot(const FString& InCameraName, const FString& InRenderPassName, const FVector& InCurLoc, const FRotator& InCurRot, const FVector& InPrevLoc, const FRotator& InPrevRot, TMap<FString, FString>& InOutMetadata)
		{
			// InRenderPassName could be empty (for global camera stuff), and to support proper backwards compat metadata
			// we need to not end up with an extra "/" in it, so we pre-format it.
			FString CamName = FString::Printf(TEXT("%s/%s"), *InCameraName, *InRenderPassName);
			if (InRenderPassName.IsEmpty())
			{
				CamName = InCameraName;
			}

			FString Prefix = FString::Printf(TEXT("unreal/%s"), *CamName);
			GetMetadataFromCameraLocRot(InCurLoc, InCurRot, InPrevLoc, InPrevRot, Prefix, InOutMetadata);
		}
		
		void GetMetadataFromCameraLocRot(const FVector& InCurLoc, const FRotator& InCurRot, const FVector& InPrevLoc, const FRotator& InPrevRot, const FString& InPrefix, TMap<FString, FString>& InOutMetadata)
		{
			InOutMetadata.Add(FString::Printf(TEXT("%s/curPos/x"), *InPrefix), FString::SanitizeFloat(InCurLoc.X));
			InOutMetadata.Add(FString::Printf(TEXT("%s/curPos/y"), *InPrefix), FString::SanitizeFloat(InCurLoc.Y));
			InOutMetadata.Add(FString::Printf(TEXT("%s/curPos/z"), *InPrefix), FString::SanitizeFloat(InCurLoc.Z));
			InOutMetadata.Add(FString::Printf(TEXT("%s/curRot/pitch"), *InPrefix),  FString::SanitizeFloat(InCurRot.Pitch));
			InOutMetadata.Add(FString::Printf(TEXT("%s/curRot/yaw"), *InPrefix), FString::SanitizeFloat(InCurRot.Yaw));
			InOutMetadata.Add(FString::Printf(TEXT("%s/curRot/roll"), *InPrefix), FString::SanitizeFloat(InCurRot.Roll));

			InOutMetadata.Add(FString::Printf(TEXT("%s/prevPos/x"), *InPrefix), FString::SanitizeFloat(InPrevLoc.X));
			InOutMetadata.Add(FString::Printf(TEXT("%s/prevPos/y"), *InPrefix), FString::SanitizeFloat(InPrevLoc.Y));
			InOutMetadata.Add(FString::Printf(TEXT("%s/prevPos/z"), *InPrefix), FString::SanitizeFloat(InPrevLoc.Z));
			InOutMetadata.Add(FString::Printf(TEXT("%s/prevRot/pitch"), *InPrefix), FString::SanitizeFloat(InPrevRot.Pitch));
			InOutMetadata.Add(FString::Printf(TEXT("%s/prevRot/yaw"), *InPrefix), FString::SanitizeFloat(InPrevRot.Yaw));
			InOutMetadata.Add(FString::Printf(TEXT("%s/prevRot/roll"), *InPrefix), FString::SanitizeFloat(InPrevRot.Roll));
		}


		bool CanWriteToFile(const TCHAR* InFilename, bool bOverwriteExisting)
		{
			// Check if there is space on the output disk.
			bool bIsFreeSpace = true;

			uint64 TotalNumberOfBytes, NumberOfFreeBytes;
			if (FPlatformMisc::GetDiskTotalAndFreeSpace(InFilename, TotalNumberOfBytes, NumberOfFreeBytes))
			{
				bIsFreeSpace = NumberOfFreeBytes > 64 * 1024 * 1024; // 64mb minimum
			}
			// ToDO: Infinite loop possible.
			return bIsFreeSpace && (bOverwriteExisting || IFileManager::Get().FileSize(InFilename) == -1);
		}

		FString GetPaddingFormatString(int32 InZeroPadCount, const int32 InFrameNumber)
		{
			// Printf takes the - sign into account when you specify the number of digits to pad. This
			// means padding "3" to 4 digits becomes "0003", while "-3" becomes "-003". We combat this by
			// incrementing the pad count for negative numbers by 1 so that you end up with "0003" and "-0003".
			if (InFrameNumber < 0)
			{
				InZeroPadCount++;
			}

			return FString::Printf(TEXT("%0*d"), InZeroPadCount, InFrameNumber);
		}

		void DoPostProcessBlend(const FVector& InViewLocation, const UWorld* InWorld, const FMinimalViewInfo& InViewInfo, FSceneView* InOutView)
		{
			for (IInterface_PostProcessVolume& PPVolume : InWorld->GetPostProcessVolumeIterator())
			{
				const FPostProcessVolumeProperties VolumeProperties = PPVolume.GetProperties();

				// Skip any volumes which are disabled
				if (!VolumeProperties.bIsEnabled)
				{
					continue;
				}

				float LocalWeight = FMath::Clamp(VolumeProperties.BlendWeight, 0.0f, 1.0f);

				if (!VolumeProperties.bIsUnbound)
				{
					float DistanceToPoint = 0.0f;
					PPVolume.EncompassesPoint(InViewLocation, 0.0f, &DistanceToPoint);

					if (DistanceToPoint >= 0 && DistanceToPoint < VolumeProperties.BlendRadius)
					{
						LocalWeight *= FMath::Clamp(1.0f - DistanceToPoint / VolumeProperties.BlendRadius, 0.0f, 1.0f);
					}
					else
					{
						LocalWeight = 0.0f;
					}
				}

				InOutView->OverridePostProcessSettings(*VolumeProperties.Settings, LocalWeight);
			}

			// After blending all post processing volumes, blend the camera's post process settings too
			InOutView->OverridePostProcessSettings(InViewInfo.PostProcessSettings, InViewInfo.PostProcessBlendWeight);
		}

		void SetSkeletalMeshClothSubSteps(const int32 InSubdivisionCount, UWorld* InWorld, TMap<TWeakObjectPtr<UObject>, TArray<::MoviePipeline::FClothSimSettingsCache>>& InClothSimCache)
		{
			SCOPE_CYCLE_COUNTER(STAT_ClothSubstepAdjust);
			for (TActorIterator<AActor> ActorIt(InWorld); ActorIt; ++ActorIt)
			{
				AActor* FoundActor = *ActorIt;
				if (FoundActor)
				{
					TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
					FoundActor->GetComponents(SkeletalMeshComponents);

					for (const USkeletalMeshComponent* const Component : SkeletalMeshComponents)
					{
						for (const FClothingSimulationInstance& ClothingSimulationInstance : Component->GetClothingSimulationInstances())
						{
							const IClothingSimulationInterface* const ClothingSimulation = ClothingSimulationInstance.GetClothingSimulation();
							UClothingSimulationInteractor* const ClothingSimulationInteractor = ClothingSimulationInstance.GetClothingSimulationInteractor();
							if (ClothingSimulation && ClothingSimulationInteractor)
							{
								constexpr int32 LODIndex = 0;  // There is only a NumSubSteps for LOD 0 in the Skeletal Mesh clothing system
								const TWeakObjectPtr<UObject> WeakPtr(ClothingSimulationInteractor);

								TArray<::MoviePipeline::FClothSimSettingsCache>* ExistingCacheEntry = InClothSimCache.Find(WeakPtr);
								if (!ExistingCacheEntry)
								{
									ExistingCacheEntry = &InClothSimCache.Add(WeakPtr);
									ExistingCacheEntry->SetNumZeroed(1);  // Only store LOD 0
									(*ExistingCacheEntry)[LODIndex] = { ClothingSimulation->GetNumSubsteps(), 0.f };
								}
								ClothingSimulationInteractor->SetNumSubsteps((*ExistingCacheEntry)[LODIndex].NumSubSteps * InSubdivisionCount);
							}
						}
					}

					TArray<UChaosClothComponent*> ChaosClothComponents;
					FoundActor->GetComponents(ChaosClothComponents);

					for (UChaosClothComponent* Component : ChaosClothComponents)
					{
						UChaosClothAssetInteractor* ClothAssetInteractor = Component->GetClothOutfitInteractor();
						if (ClothAssetInteractor)
						{
							const int32 NumLODs = Component->GetNumLODs();
							TWeakObjectPtr<UObject> WeakPtr = TWeakObjectPtr<UChaosClothAssetInteractor>(ClothAssetInteractor);

							TArray<::MoviePipeline::FClothSimSettingsCache>* ExistingCacheEntry = InClothSimCache.Find(WeakPtr);
							if (!ExistingCacheEntry)
							{
								InClothSimCache.Add(WeakPtr);
								ExistingCacheEntry = InClothSimCache.Find(WeakPtr);
								ExistingCacheEntry->SetNumUninitialized(NumLODs);
								for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
								{
									constexpr int32 MinNumSubsteps = 1;
									const int32 NumSubsteps = FMath::Max(ClothAssetInteractor->GetIntPropertyValue(TEXT("NumSubsteps"), LODIndex, MinNumSubsteps), MinNumSubsteps);
									const float DynamicSubstepDeltaTime = ClothAssetInteractor->GetFloatPropertyValue(TEXT("DynamicSubstepDeltaTime"), LODIndex, MinNumSubsteps);
									(*ExistingCacheEntry)[LODIndex].NumSubSteps = NumSubsteps;
									(*ExistingCacheEntry)[LODIndex].DynamicSubstepDeltaTime = DynamicSubstepDeltaTime;
								}
							}

							for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
							{
								if (ExistingCacheEntry->IsValidIndex(LODIndex))
								{
									ClothAssetInteractor->SetIntPropertyValue(TEXT("NumSubsteps"), LODIndex, (*ExistingCacheEntry)[LODIndex].NumSubSteps * InSubdivisionCount);
									ClothAssetInteractor->SetFloatPropertyValue(TEXT("DynamicSubstepDeltaTime"), LODIndex, 0.f);
								}
							}
						}
					}
				}
			}
		}

		void RestoreSkeletalMeshClothSubSteps(const TMap<TWeakObjectPtr<UObject>, TArray<::MoviePipeline::FClothSimSettingsCache>>& InClothSimCache)
		{
			for (const TPair<TWeakObjectPtr<UObject>, TArray<::MoviePipeline::FClothSimSettingsCache>>& Pair : InClothSimCache)
			{
				if (UObject* Object = Pair.Key.Get())
				{
					if (UClothingSimulationInteractor* ClothInteractor = Cast<UClothingSimulationInteractor>(Object))
					{
						const int32 LODIndex = 0;  // There is only a NumSubSteps for LOD 0 in the Skeletal Mesh clothing system
						ClothInteractor->SetNumSubsteps(Pair.Value[LODIndex].NumSubSteps);
					}
					else if (UChaosClothAssetInteractor* ClothAssetInteractor = Cast<UChaosClothAssetInteractor>(Object))
					{
						for (int32 LODIndex = 0; LODIndex < Pair.Value.Num(); ++LODIndex)
						{
							ClothAssetInteractor->SetIntPropertyValue(TEXT("NumSubsteps"), LODIndex, Pair.Value[LODIndex].NumSubSteps);
							ClothAssetInteractor->SetFloatPropertyValue(TEXT("DynamicSubstepDeltaTime"), LODIndex, Pair.Value[LODIndex].DynamicSubstepDeltaTime);
						}
					}
				}
			}
		}

	}
}
namespace UE
{
	namespace MoviePipeline
	{
		FMoviePipelineRenderPassMetrics GetRenderPassMetrics(UMoviePipelinePrimaryConfig* InPrimaryConfig, UMoviePipelineExecutorShot* InPipelineExecutorShot, const FMoviePipelineRenderPassMetrics& InRenderPassMetrics, const FIntPoint& InEffectiveOutputResolution)
		{
			FMoviePipelineRenderPassMetrics OutRenderPassMetrics = InRenderPassMetrics;

			if (InPrimaryConfig && InPipelineExecutorShot)
			{
				const UMoviePipelineOutputSetting* OutputSettings = InPrimaryConfig->FindSetting<UMoviePipelineOutputSetting>();
				const UMoviePipelineHighResSetting* HighResSettings = Cast<const UMoviePipelineHighResSetting>(UMoviePipelineBlueprintLibrary::FindOrGetDefaultSettingForShot(UMoviePipelineHighResSetting::StaticClass(), InPrimaryConfig, InPipelineExecutorShot));
				check(OutputSettings);
				check(HighResSettings);

				FIntPoint BackbufferResolution = FIntPoint(FMath::CeilToInt((float)InEffectiveOutputResolution.X / (float)OutRenderPassMetrics.OriginalTileCounts.X), FMath::CeilToInt((float)InEffectiveOutputResolution.Y / (float)OutRenderPassMetrics.OriginalTileCounts.Y));
				const FIntPoint TileResolution = BackbufferResolution;

				// Apply size padding.
				BackbufferResolution = HighResSettings->CalculatePaddedBackbufferSize(BackbufferResolution);


				OutRenderPassMetrics.TileSize = TileResolution;
				OutRenderPassMetrics.ProjectionMatrixJitterAmount = FVector2D((float)(OutRenderPassMetrics.SpatialShiftX) * 2.0f / BackbufferResolution.X, (float)OutRenderPassMetrics.SpatialShiftY * -2.0f / BackbufferResolution.Y);

				OutRenderPassMetrics.BackbufferSize = BackbufferResolution;
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				OutRenderPassMetrics.EffectiveOutputResolution = InEffectiveOutputResolution;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				OutRenderPassMetrics.OverscannedResolution = InEffectiveOutputResolution;


				FIntRect CropRect;

				// Overscan is symmetrical, so this difference is always evenly divisible by 2.
				CropRect.Min = (InEffectiveOutputResolution - OutputSettings->OutputResolution) / 2;
				CropRect.Max = CropRect.Min + OutputSettings->OutputResolution;

				OutRenderPassMetrics.CropRectangle = CropRect;
				
				{
					OutRenderPassMetrics.OverlappedPad = FIntPoint(FMath::CeilToInt(TileResolution.X * HighResSettings->OverlapRatio),
						FMath::CeilToInt(TileResolution.Y * HighResSettings->OverlapRatio));
					OutRenderPassMetrics.OverlappedOffset = FIntPoint(OutRenderPassMetrics.TileIndexes.X * TileResolution.X - OutRenderPassMetrics.OverlappedPad.X,
						OutRenderPassMetrics.TileIndexes.Y * TileResolution.Y - OutRenderPassMetrics.OverlappedPad.Y);

					// Move the final render by this much in the accumulator to counteract the offset put into the view matrix.
					// Note that when bAllowSpatialJitter is false, SpatialShiftX/Y will always be zero.
					OutRenderPassMetrics.OverlappedSubpixelShift = FVector2D(0.5f - OutRenderPassMetrics.SpatialShiftX, 0.5f - OutRenderPassMetrics.SpatialShiftY);
				}
			}

			return OutRenderPassMetrics;
		}

		FVector2f GetSubPixelJitter(int32 InFrameIndex, int32 InSamplesPerFrame)
		{
			// Repeat the Halton Offset equally on each output frame so non-moving objects don't have any chance to crawl between frames.
			int32 HaltonIndex = (InFrameIndex % InSamplesPerFrame) + 1;
			float HaltonOffsetX = Halton(HaltonIndex, 2);
			float HaltonOffsetY = Halton(HaltonIndex, 3);

			return FVector2f(HaltonOffsetX-0.5, HaltonOffsetY-0.5);
		}

		FIntPoint ScaleResolutionByOverscan(const float OverscanPercentage, const FIntPoint& InOutputResolution)
		{
			const float ClampedOverscanPercentage = FMath::Clamp(OverscanPercentage, 0.0f, 1.0f);
			FIntPoint EffectiveResolution(InOutputResolution);
			if (ClampedOverscanPercentage > 0.f)
			{
				const float Scale = 1.f + ClampedOverscanPercentage;
				EffectiveResolution.X = FMath::CeilToInt(static_cast<float>(EffectiveResolution.X) * Scale);
				EffectiveResolution.Y = FMath::CeilToInt(static_cast<float>(EffectiveResolution.Y) * Scale);
			}

			return EffectiveResolution;
		}
		
		namespace Panoramic
		{
			TArray<float> DistributeValuesInInterval(float InMin, float InMax, int32 InNumDivisions, bool bInInclusiveMax)
			{
				TArray<float> Results;
				Results.Reserve(bInInclusiveMax);

				float Delta = (InMax - InMin) / static_cast<float>(FMath::Max(bInInclusiveMax ? InNumDivisions - 1 : InNumDivisions, 1));
				float CurrentValue = InMin;
				for (int32 Index = 0; Index < InNumDivisions; Index++)
				{
					Results.Add(CurrentValue);

					CurrentValue += Delta;
				}

				return Results;
			};

			void GetCameraOrientationForStereo(FVector& OutLocation, FRotator& OutRotation, FRotator& OutLocalRotation, const UE::MoviePipeline::FPanoramicPane& InPane, const int32 InStereoIndex, const bool bInPrevPosition)
			{
				// ToDo: This 110 (-55, 55) comes from TwinMotion who uses a hard-coded number of v-steps, may need adjusting.
				const TArray<float> PitchValues = DistributeValuesInInterval(-55, 55, InPane.NumVerticalSteps, /*Inclusive Max*/true);
				const TArray<float> YawValues = DistributeValuesInInterval(0, 360, InPane.NumHorizontalSteps, /*Inclusive Max*/false);

				const float HorizontalRotationDeg = YawValues[InPane.HorizontalStepIndex];
				const float VerticalRotationDeg = PitchValues[InPane.VerticalStepIndex];

				const FQuat HorizontalRotQuat = FQuat(FVector::UnitZ(), FMath::DegreesToRadians(HorizontalRotationDeg));
				const FQuat VerticalRotQuat = FQuat(FVector::UnitY(), FMath::DegreesToRadians(VerticalRotationDeg));

				const FRotator SourceRot = bInPrevPosition ? InPane.PrevOriginalCameraRotation : InPane.OriginalCameraRotation;
				FQuat RotationResult = FQuat(SourceRot) * HorizontalRotQuat * VerticalRotQuat;
				OutRotation = FRotator(RotationResult);
				OutLocalRotation = FRotator(HorizontalRotQuat * VerticalRotQuat);

				// If not using stereo rendering then the eye is just the camera location
				if (InStereoIndex < 0)
				{
					OutLocation = bInPrevPosition ? InPane.PrevOriginalCameraLocation : InPane.OriginalCameraLocation;
				}
				else
				{
					check(InStereoIndex == 0 || InStereoIndex == 1);

					float EyeOffset = InStereoIndex == 0 ? (-InPane.EyeSeparation / 2.f) : (InPane.EyeSeparation / 2.f);
					OutLocation = bInPrevPosition ? InPane.PrevOriginalCameraLocation : InPane.OriginalCameraLocation;

					// Translate the eye either left or right of the target rotation.
					OutLocation += RotationResult.RotateVector(FVector(0.f, EyeOffset, 0.f));
				}
			}

			// Cube capture is arranged in 3x2 square tiles, rounded down to a multiple of 8 pixels.
			int32 ComputeAutoExposureCubeCaptureSize(FIntPoint Resolution)
			{
				return AlignDown(FMath::Min(Resolution.X / 3, Resolution.Y / 2), 8);
			}
		}

		FString GetMetadataPrefixPath(const FMovieGraphRenderDataIdentifier& Identifier)
		{
			return FString::Printf(TEXT("unreal/%s/%s/%s"), *Identifier.LayerName, *Identifier.RendererName, *Identifier.CameraName);
		}

	}
}
