// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineVideoOutputBase.h"
#include "MoviePipeline.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineUtils.h"
#include "ImageWriteTask.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformTime.h"
#include "MoviePipelineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineVideoOutputBase)

UMoviePipelineVideoOutputBase::UMoviePipelineVideoOutputBase()
	: bHasError(false)
{
}

void UMoviePipelineVideoOutputBase::OnShotFinishedImpl(const UMoviePipelineExecutorShot* InShot, const bool bFlushToDisk)
{
	if (bFlushToDisk)
	{
		// If they don't have {shot_name} or {camera_name} in their output path, this probably doesn't do what they expect
		// as it will finalize and then overwrite itself.
		UMoviePipelineOutputSetting* OutputSettings = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
		check(OutputSettings);
		FString FullPath = OutputSettings->OutputDirectory.Path / OutputSettings->FileNameFormat;
		if (!(FullPath.Contains(TEXT("{shot_name}")) || FullPath.Contains(TEXT("{camera_name}"))))
		{
			UE_LOGF(LogMovieRenderPipelineIO, Warning, "Asked MoviePipeline to flush file writes to disk after each shot, but filename format doesn't seem to separate video files per shot. This will cause the file to overwrite itself, is this intended?");
		}

		UE_LOGF(LogMovieRenderPipelineIO, Log, "MoviePipelineVideoOutputBase flushing %d tasks to disk...", AllWriters.Num());
		const double FlushBeginTime = FPlatformTime::Seconds();

		// Despite what the comments indicate, there's no actual queue of tasks here, so we can just call BeginFinalize/Finalize.
		// Finalize will clear the AllWriters array, so any subsequent requests to write to that shot will try to generate a new file.
		BeginFinalizeImpl();
		FinalizeImpl();

		const float ElapsedS = float((FPlatformTime::Seconds() - FlushBeginTime));
		UE_LOGF(LogMovieRenderPipelineIO, Log, "Finished flushing tasks to disk after %2.2fs!", ElapsedS);
	}
}


void UMoviePipelineVideoOutputBase::OnReceiveImageDataImpl(FMoviePipelineMergerOutputFrame* InMergedOutputFrame)
{
	// Add an early out if the output encountered an error (such as failure to initialize)
	if (bHasError)
	{
		return;
	}

	UMoviePipelineOutputSetting* OutputSettings = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	check(OutputSettings);

	FString OutputDirectory = OutputSettings->OutputDirectory.Path;

	// Special case for extracting Burn Ins and Widget Renderer 
	TArray<MoviePipeline::FCompositePassInfo> CompositedPasses;
	MoviePipeline::GetPassCompositeData(InMergedOutputFrame, CompositedPasses);

	// Get Camera Names
	TArray<FString> UsedCameraNames;
	for (TPair<FMoviePipelinePassIdentifier, TUniquePtr<FImagePixelData>>& RenderPassData : InMergedOutputFrame->ImageOutputData)
	{
		UsedCameraNames.AddUnique(RenderPassData.Key.CameraName);
	}
	// Support per-camera video output
	const bool bIncludeCameraName = UsedCameraNames.Num() > 1;

	for (TPair<FMoviePipelinePassIdentifier, TUniquePtr<FImagePixelData>>& RenderPassData : InMergedOutputFrame->ImageOutputData)
	{
		// Don't write out a composited pass in this loop, as it will be merged with the Final Image and not written separately. 
		bool bSkip = false;
		for (const MoviePipeline::FCompositePassInfo& CompositePass : CompositedPasses)
		{
			if (CompositePass.PassIdentifier == RenderPassData.Key)
			{
				bSkip = true;
				break;
			}
		}

		if (bSkip)
		{
			continue;
		}

		FImagePixelDataPayload* Payload = RenderPassData.Value->GetPayload<FImagePixelDataPayload>();

		// We need to resolve the filename format string. We combine the folder and file name into one long string first
		FMoviePipelineFormatArgs FinalFormatArgs;
		FString FinalFilePath;
		FString StableFilePath;
		FString FinalVideoFileName;
		FString ClipName;
		{
			FString FileNameFormatString = OutputSettings->FileNameFormat;

			// If we're writing more than one render pass out, we need to ensure the file name has the format string in it so we don't
			// overwrite the same file multiple times. Burn In overlays don't count because they get composited on top of an existing file.
			const bool bIncludeRenderPass = InMergedOutputFrame->ImageOutputData.Num() - CompositedPasses.Num() > 1;
			const bool bTestFrameNumber = false;

			UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber, bIncludeCameraName);

			// Strip any frame number tags so we don't get one video file per frame.
			UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);

			// Create specific data that needs to override 
			TMap<FString, FString> FormatOverrides;
			FormatOverrides.Add(TEXT("camera_name"), RenderPassData.Key.CameraName);
			FormatOverrides.Add(TEXT("render_pass"), RenderPassData.Key.Name);
			FormatOverrides.Add(TEXT("ext"), GetFilenameExtension());

			// This is a bit crummy but we don't have a good way to implement bOverwriteFiles = False for video files. When you don't have the
			// override file flag set, we need to increment the number by 1, and write to that. This works fine for image sequences as image
			// sequences are also differentiated by their {frame_number}, but for video files (which strip it all out), instead we end up
			// with each incoming image sample trying to generate the same filename, finding a file that already exists (from the previous
			// frame) and incrementing again, giving us 1 video file per frame. To work around this, we're going to force bOverwriteExisting
			// off so we can generate a "stable" filename for the incoming image sample, then compare it against existing writers, and
			// if we don't find a writer, then generate a new one (respecting Overwrite Existing) to add the sample to.
			{
				UMoviePipelineOutputSetting* OutputSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
				const bool bPreviousOverwriteExisting = OutputSetting->bOverrideExistingOutput;
				OutputSetting->bOverrideExistingOutput = true;
				
				GetPipeline()->ResolveFilenameFormatArguments(OutputDirectory / FileNameFormatString, FormatOverrides, StableFilePath, FinalFormatArgs, &InMergedOutputFrame->FrameOutputState);
				
				// Restore user setting
				OutputSetting->bOverrideExistingOutput = bPreviousOverwriteExisting;

				if (FPaths::IsRelative(StableFilePath))
				{
					StableFilePath = FPaths::ConvertRelativePathToFull(StableFilePath);
				}
			}

			// The FinalVideoFileName is relative to the output directory (ie: if the user puts folders in to the filename path)
			GetPipeline()->ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, FinalVideoFileName, FinalFormatArgs, &InMergedOutputFrame->FrameOutputState);

			// Then we add the OutputDirectory, and resolve the filename format arguments again so the arguments in the directory get resolved.
			FString FullFilepathFormatString = OutputDirectory / FileNameFormatString;
			GetPipeline()->ResolveFilenameFormatArguments(FullFilepathFormatString, FormatOverrides, FinalFilePath, FinalFormatArgs, &InMergedOutputFrame->FrameOutputState);

			if (FPaths::IsRelative(FinalFilePath))
			{
				FinalFilePath = FPaths::ConvertRelativePathToFull(FinalFilePath);
			}

			// Ensure the directory is created
			{
				FString FolderPath = FPaths::GetPath(FinalFilePath);
				IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

				PlatformFile.CreateDirectoryTree(*FolderPath);
			}

			// Create a deterministic clipname by file extension, and any trailing .'s
			FMoviePipelineFormatArgs TempFormatArgs;
			GetPipeline()->ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, ClipName, TempFormatArgs, &InMergedOutputFrame->FrameOutputState);
			FPaths::NormalizeFilename(ClipName);
			ClipName.RemoveFromEnd(GetFilenameExtension());
			ClipName.RemoveFromEnd(".");
		}


		FMoviePipelineCodecWriter* OutputWriter = nullptr;
		for (int32 Index = 0; Index < AllWriters.Num(); Index++)
		{
			if (AllWriters[Index].Get<0>()->StableFileName == StableFilePath)
			{
				OutputWriter = &AllWriters[Index];
				break;
			}
		}
		
		if(!OutputWriter)
		{
			// If overscan is going to be cropped, use the output resolution from the crop rectangle. Otherwise, use the internal resolution from the
			// raw render pass data
			FIntPoint OutputResolution = RenderPassData.Value->GetSize();
			if (ShouldCropOverscanImpl())
			{
				OutputResolution = Payload->SampleState.CropRectangle.Size();
			}
			
			// Create a new writer for this file name (and output format settings)
			TUniquePtr<MovieRenderPipeline::IVideoCodecWriter> NewWriter = Initialize_GameThread(FinalFilePath,
				OutputResolution, RenderPassData.Value->GetType(), RenderPassData.Value->GetPixelLayout(),
				RenderPassData.Value->GetBitDepth(), RenderPassData.Value->GetNumChannels());

			if (NewWriter)
			{
				// Store the stable filename this was generated with so we can match them up later.
				NewWriter->StableFileName = StableFilePath;

				TPromise<bool> Completed;
				MoviePipeline::FMoviePipelineOutputFutureData OutputData;
				OutputData.Shot = GetPipeline()->GetActiveShotList()[Payload->SampleState.OutputState.ShotIndex];
				OutputData.PassIdentifier = RenderPassData.Key;
				OutputData.FilePath = FinalFilePath;
				OutputData.OriginSettingClass = GetClass();

				GetPipeline()->AddOutputFuture(Completed.GetFuture(), OutputData);

				AllWriters.Add(FMoviePipelineCodecWriter(MoveTemp(NewWriter), MoveTemp(Completed)));
				OutputWriter = &AllWriters.Last();
				OutputWriter->Get<0>().Get()->FormatArgs = FinalFormatArgs;

				// If it fails to initialize, immediately mark the promise as failed so the render queue stops.
				bHasError = !Initialize_EncodeThread(OutputWriter->Get<0>().Get());
				if (bHasError)
				{
					OutputWriter->Get<1>().SetValue(false);
					UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to initialize encoder for FileName: %ls", *FinalFilePath);
					continue;
				}
			}
		}

		if (!OutputWriter)
		{
			UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to generate writer for FileName: %ls", *FinalFilePath);
			continue;
		}

		FMoviePipelineBackgroundMediaTasks Task;
		FImagePixelData* RawRenderPassData = RenderPassData.Value.Get();

		// Pixel data pointer for the cropped result to ensure it stays in memory while the image write is processed
		TUniquePtr<FImagePixelData> CroppedPixelData;
		const bool bIsCropRectValid = !Payload->SampleState.CropRectangle.IsEmpty();
		const bool bCanCropResolution = RenderPassData.Value->GetSize() == Payload->SampleState.OverscannedResolution;
		if (ShouldCropOverscanImpl() && bIsCropRectValid && bCanCropResolution)
		{
			switch (RenderPassData.Value->GetType())
			{
			case EImagePixelType::Color:
				{
					TAsyncCropImage<FColor> CropImagePreprocessor(Payload->SampleState.CropRectangle);
					CropImagePreprocessor(RawRenderPassData);
					CroppedPixelData = MoveTemp(CropImagePreprocessor.OutCroppedImage);
				}
				break;

			case EImagePixelType::Float16:
				{
					TAsyncCropImage<FFloat16Color> CropImagePreprocessor(Payload->SampleState.CropRectangle);
					CropImagePreprocessor(RawRenderPassData);
					CroppedPixelData = MoveTemp(CropImagePreprocessor.OutCroppedImage);
				}
				break;

			case EImagePixelType::Float32:
				{
					TAsyncCropImage<FLinearColor> CropImagePreprocessor(Payload->SampleState.CropRectangle);
					CropImagePreprocessor(RawRenderPassData);
					CroppedPixelData = MoveTemp(CropImagePreprocessor.OutCroppedImage);
				}
				break;
			}
			
			RawRenderPassData = CroppedPixelData.Get();
		}
		
		// Making sure that if OCIO is enabled the Quantization won't do additional color conversion.
		UMoviePipelineColorSetting* ColorSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineColorSetting>();
		OutputWriter->Get<0>()->bConvertToSrgb = !(ColorSetting && ColorSetting->OCIOConfiguration.bIsEnabled);

		// We can have multiple shots in one file, so we find/add the shot in case this is a new shot, and the counter will start over at zero.
		MovieRenderPipeline::IVideoCodecWriter::FLightweightSourceData& LightweightData = OutputWriter->Get<0>()->LightweightSourceData.FindOrAdd(Payload->SampleState.OutputState.ShotIndex);
		LightweightData.SubmittedFrameCount++;
		
		//FGraphEventRef Event = Task.Execute([this, OutputWriter, RawRenderPassData]
		//	{
			if (RenderPassData.Key.Name == TEXT("FinalImage"))
			{
				TArray<MoviePipeline::FCompositePassInfo> CompositesForThisCamera;
				for (MoviePipeline::FCompositePassInfo& CompositePass : CompositedPasses)
				{
					// Match them up by camera name so multiple passes intended for different camera names work.
					if (RenderPassData.Key.CameraName == CompositePass.PassIdentifier.CameraName)
					{
						// Create a new composite pass (but move the actual pixel data), otherwise when the main
						// loop tries to check if we should skip writing out this image pass in the future, it fails
						// because we've MoveTemp'd CompositePass and thus the name checks no longer pass.
						MoviePipeline::FCompositePassInfo NewPassInfo;
						NewPassInfo.PassIdentifier = CompositePass.PassIdentifier;
						NewPassInfo.PixelData = MoveTemp(CompositePass.PixelData);
						CompositesForThisCamera.Add(MoveTemp(NewPassInfo));
					}
				}

				// Enqueue a encode for this frame onto our worker thread.
				this->WriteFrame_EncodeThread(OutputWriter->Get<0>().Get(), RawRenderPassData, MoveTemp(CompositesForThisCamera));
			}
			else
			{
				TArray<MoviePipeline::FCompositePassInfo> Dummy;
				this->WriteFrame_EncodeThread(OutputWriter->Get<0>().Get(), RawRenderPassData, MoveTemp(Dummy));
			}
		//	});
		//OutstandingTasks.Add(Event);
		
#if WITH_EDITOR
		GetPipeline()->AddFrameToOutputMetadata(ClipName, FinalVideoFileName, InMergedOutputFrame->FrameOutputState, GetFilenameExtension(), Payload->bRequireTransparentOutput);
#endif
	}
}

bool UMoviePipelineVideoOutputBase::HasFinishedProcessingImpl()
{
	// for (int32 Index = OutstandingTasks.Num() - 1; Index >= 0; Index--)
	// {
	// 	if (OutstandingTasks[Index].())
	// 	{
	// 		OutstandingTasks.RemoveAt(Index);
	// 	}
	// }

	return OutstandingTasks.Num() == 0;
}

void UMoviePipelineVideoOutputBase::BeginFinalizeImpl()
{
	for (const FMoviePipelineCodecWriter& Writer : AllWriters)
	{
		MovieRenderPipeline::IVideoCodecWriter* RawWriter = Writer.Get<0>().Get();
		FMoviePipelineBackgroundMediaTasks Task;
	
		//OutstandingTasks.Add(Task.Execute([this, RawWriter] {
			this->BeginFinalize_EncodeThread(RawWriter);
		//	}));
	}
}

void UMoviePipelineVideoOutputBase::FinalizeImpl()
{
	FGraphEventRef* LastEvent = nullptr;
	for (FMoviePipelineCodecWriter& Writer : AllWriters)
	{
		MovieRenderPipeline::IVideoCodecWriter* RawWriter = Writer.Get<0>().Get();
		FMoviePipelineBackgroundMediaTasks Task;
	
		//*LastEvent = Task.Execute([this, RawWriter] {
			this->Finalize_EncodeThread(RawWriter);
		//	});
			if (!bHasError)
			{
				Writer.Get<1>().SetValue(true);
			}
	}
	
	// Stall until all of the events are handled so that they still exist when the Task Graph goes to execute them.
	if (LastEvent)
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(*LastEvent);
	}

	AllWriters.Empty();
	bHasError = false;
}

#if WITH_EDITOR
FText UMoviePipelineVideoOutputBase::GetFooterText(UMoviePipelineExecutorJob* InJob) const
{
	if (!IsAudioSupported())
	{
		return NSLOCTEXT("MovieRenderPipeline", "VideoOutputAudioUnsupported", "Audio output is not supported for this video encoder. Please consider using the .wav writer and combining in post.");
	}

	return FText();
}
#endif
