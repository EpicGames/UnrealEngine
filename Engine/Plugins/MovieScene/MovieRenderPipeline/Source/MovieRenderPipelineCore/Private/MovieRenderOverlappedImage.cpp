// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderOverlappedImage.h"

#include "Math/Vector.h"
#include "Math/VectorRegister.h"

#include "MovieRenderPipelineCoreModule.h"
#include "Math/Float16.h"
#include "HAL/PlatformTime.h"

DECLARE_CYCLE_STAT(TEXT("MoviePipeline_SampleReadback"), STAT_SampleReadback, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("OverlapAccumulator_AccumulateSinglePlane"), STAT_AccumulateSinglePlane, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("OverlapAccumulator_InitMemory"), STAT_InitMemory, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("OverlapAccumulator_ZeroPlanes"), STAT_ZeroPlanes, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("OverlapAccumulator_AccumulatePixelData"), STAT_AccumulatePixelData, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("OverlapAccumulator_FetchFull"), STAT_FetchFullImageValue, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("OverlapAccumulator_FetchPixelData8"), STAT_FetchFinalPixelData8bit, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("OverlapAccumulator_FetchPixelData16"), STAT_FetchFinalPixelData16bit, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("OverlapAccumulator_FetchPixelData32"), STAT_FetchFinalPixelData32bit, STATGROUP_MoviePipeline);

void FImageOverlappedPlane::Init(FIntPoint InSize)
{
	Size = InSize;

	// Don't initialize now as we'll call ZeroPlane later when appropriate to reset the values.
	ChannelData.SetNumUninitialized(Size.X * (int64)Size.Y);
}

void FImageOverlappedPlane::ZeroPlane()
{
	FMemory::Memset(ChannelData.GetData(), 0, ChannelData.Num() * sizeof(float));
}

void FImageOverlappedPlane::Reset()
{
	Size.X = 0;
	Size.Y = 0;
	ChannelData.Empty();
}

// a subpixel offset of (0.5,0.5) means that the raw data is exactly centered on destination pixels.
void FImageOverlappedPlane::AccumulateSinglePlane(const TArray64<float>& InRawData, FIntPoint InSize, FIntPoint InOffset,
								float SubpixelOffsetX, float SubpixelOffsetY,
								FIntPoint SubRectOffset,
								FIntPoint SubRectSize,
								const TArray<float> & WeightDataX,
								const TArray<float> & WeightDataY)
{
	SCOPE_CYCLE_COUNTER(STAT_AccumulateSinglePlane);

	check(InRawData.Num() == InSize.X * (int64)InSize.Y);
	check(WeightDataX.Num() == InSize.X);
	check(WeightDataY.Num() == InSize.Y);

	check(SubpixelOffsetX >= 0.0f);
	check(SubpixelOffsetX <= 1.0f);
	check(SubpixelOffsetY >= 0.0f);
	check(SubpixelOffsetY <= 1.0f);

	check(SubRectSize.X >= 1);
	check(SubRectSize.Y >= 1);


	// if the subpixel offset is less than 0.5, go back one pixel
	int32 StartX = (SubpixelOffsetX >= 0.5 ? InOffset.X : InOffset.X - 1);
	int32 StartY = (SubpixelOffsetY >= 0.5 ? InOffset.Y : InOffset.Y - 1);

	float PixelWeight[2][2];
	{
		// make sure that the equal sign is correct, if the subpixel offset is 0.5, the weightX is 0 and it starts on the center pixel
		float WeightX = FMath::Frac(SubpixelOffsetX + 0.5f);
		float WeightY = FMath::Frac(SubpixelOffsetY + 0.5f);

		// row, column
		PixelWeight[0][0] = (1.0f - WeightX) * (1.0f - WeightY);
		PixelWeight[0][1] = (       WeightX) * (1.0f - WeightY);
		PixelWeight[1][0] = (1.0f - WeightX) * (       WeightY);
		PixelWeight[1][1] = (       WeightX) * (       WeightY);
	}

	static bool bSlowReference = false;
	if (bSlowReference)
	{
		static bool bReversedReference = true;
		if (bReversedReference)
		{
			// This is the reversed reference. Instead of taking a tile and applying it to the accumulation, we start from the
			// accumulation point and figure out the source pixels that affect it. I.e. all the math is backwards, but it should
			// be faster.
			//
			// Given a position on the current tile (x,y), we apply the sum of the 4 points (x+0,y+0), (x+1,y+0), (x+0,y+1), (x+1,y+1) to the destination index.
			// So in reverse, given a destination position, our source sample positions are (x-1,y-1), (x+0,y-1), (x-1,y+0), (x+0,y+0).
			// That's why we make sure to start at a minimum of index 1, instead of 0.
			for (int32 CurrY = 0; CurrY < InSize.Y; CurrY++)
			{
				for (int32 CurrX = 0; CurrX < InSize.X; CurrX++)
				{
					int32 DstY = StartY + CurrY;// +OffsetY;
					int32 DstX = StartX + CurrX;// +OffsetX;

					if (DstX >= 0 && DstY >= 0 &&
						DstX < Size.X && DstY < Size.Y)
					{
						for (int32 OffsetY = 0; OffsetY < 2; OffsetY++)
						{
							for (int32 OffsetX = 0; OffsetX < 2; OffsetX++)
							{
								int64 SrcX = FMath::Max<int64>(CurrX - 1 + OffsetX,0);
								int64 SrcY = FMath::Max<int64>(CurrY - 1 + OffsetY,0);


								float Val = InRawData[SrcY * (int64)InSize.X + SrcX];
								
								float WX = WeightDataX[SrcX];
								float WY = WeightDataY[SrcY];
								float BaseWeight = WX * WY;

								float Weight = BaseWeight * PixelWeight[1-OffsetY][1-OffsetX];

								ChannelData[DstY * (int64)Size.X + DstX] += Weight * Val;
							}
						}
					}
				}
			}
		}
		else
		{
			// Slow, reference version. This is the main one.
			for (int64 CurrY = 0; CurrY < InSize.Y; CurrY++)
			{
				for (int64 CurrX = 0; CurrX < InSize.X; CurrX++)
				{
					float Val = InRawData[CurrY * (int64)InSize.X + CurrX];
					float WX = WeightDataX[CurrX];
					float WY = WeightDataX[CurrY];

					float BaseWeight = WX * WY;

					for (int32 OffsetY = 0; OffsetY < 2; OffsetY++)
					{
						for (int32 OffsetX = 0; OffsetX < 2; OffsetX++)
						{
							int64 DstY = StartY + CurrY + OffsetY;
							int64 DstX = StartX + CurrX + OffsetX;

							float Weight = BaseWeight * PixelWeight[OffsetY][OffsetX];

							if (DstX >= 0 && DstY >= 0 &&
								DstX < Size.X && DstY < Size.Y)
							{
								ChannelData[DstY * (int64)Size.X + DstX] += Weight * Val;
							}
						}
					}
				}
			}
		}
	}
	else
	{
		// The initial destination corners, may go off the destination window.
		int32 ActualDstX0 = StartX + SubRectOffset.X;
		int32 ActualDstY0 = StartY + SubRectOffset.Y;
		int32 ActualDstX1 = StartX + SubRectOffset.X + SubRectSize.X;
		int32 ActualDstY1 = StartY + SubRectOffset.Y + SubRectSize.Y;

		ActualDstX0 = FMath::Clamp<int32>(ActualDstX0, 0, Size.X);
		ActualDstX1 = FMath::Clamp<int32>(ActualDstX1, 0, Size.X);

		ActualDstY0 = FMath::Clamp<int32>(ActualDstY0, 0, Size.Y);
		ActualDstY1 = FMath::Clamp<int32>(ActualDstY1, 0, Size.Y);

		const float PixelWeight00 = PixelWeight[0][0];
		const float PixelWeight01 = PixelWeight[0][1];
		const float PixelWeight10 = PixelWeight[1][0];
		const float PixelWeight11 = PixelWeight[1][1];

		static bool bRunVectorized = true;
		if (!bRunVectorized)
		{
			for (int32 DstY = ActualDstY0; DstY < ActualDstY1; DstY++)
			{
				int32 CurrY = DstY - StartY;
				int32 SrcY0 = FMath::Max<int32>(0,CurrY + (0 - 1));
				int32 SrcY1 = FMath::Max<int32>(0,CurrY + (1 - 1));

				const float * SrcLineRaw0 = &InRawData[SrcY0 * (int64)InSize.X];
				const float * SrcLineRaw1 = &InRawData[SrcY1 * (int64)InSize.X];


				const float * SrcLineWeight = WeightDataX.GetData();

				const float RowWeight0 = WeightDataY[SrcY0];
				const float RowWeight1 = WeightDataY[SrcY1];

				float * DstLine = &ChannelData[DstY * (int64)Size.X];
				for (int32 DstX = ActualDstX0; DstX < ActualDstX1; DstX++)
				{
					int32 CurrX = DstX - StartX;
					int32 X0 = FMath::Max<int32>(0,CurrX - 1 + 0);
					int32 X1 = FMath::Max<int32>(0,CurrX - 1 + 1);

					float Sum = DstLine[DstX];

					Sum += RowWeight0 * SrcLineWeight[X0] * PixelWeight[1][1] * SrcLineRaw0[X0];
					Sum += RowWeight0 * SrcLineWeight[X1] * PixelWeight[1][0] * SrcLineRaw0[X1];
					Sum += RowWeight1 * SrcLineWeight[X0] * PixelWeight[0][1] * SrcLineRaw1[X0];
					Sum += RowWeight1 * SrcLineWeight[X1] * PixelWeight[0][0] * SrcLineRaw1[X1];

					DstLine[DstX] = Sum;
				}
			}
		}
		else
		{
			VectorRegister VecPixelWeight00 = VectorSetFloat1(PixelWeight[0][0]);
			VectorRegister VecPixelWeight01 = VectorSetFloat1(PixelWeight[0][1]);
			VectorRegister VecPixelWeight10 = VectorSetFloat1(PixelWeight[1][0]);
			VectorRegister VecPixelWeight11 = VectorSetFloat1(PixelWeight[1][1]);

			int32 Offset = ActualDstY0;
			int32 Count = ActualDstY1 - ActualDstY0;
			ParallelFor(Count, [Offset, StartY, InSize, ActualDstX0, ActualDstX1,StartX, 
				PixelWeight00, PixelWeight01, PixelWeight10, PixelWeight11, 
				VecPixelWeight00, VecPixelWeight01, VecPixelWeight10, VecPixelWeight11,
				Size = this->Size, &ChannelData = this->ChannelData, &InRawData, &WeightDataX, &WeightDataY](int32 InIndexY)
			{
				int32 DstY = InIndexY + Offset;
				int32 CurrY = DstY - StartY;
				int32 SrcY0 = FMath::Max<int32>(CurrY + (0 - 1), 0); // we need the max for the first pixel which could go off the edge
				int32 SrcY1 = FMath::Max<int32>(CurrY + (1 - 1), 0); 

				const float * SrcLineRaw0 = &InRawData[SrcY0 * (int64)InSize.X];
				const float * SrcLineRaw1 = &InRawData[SrcY1 * (int64)InSize.X];

				//const float * SrcLineWeight0 = &InWeightData[SrcY0 * InSize.X];
				//const float * SrcLineWeight1 = &InWeightData[SrcY1 * InSize.X];

				const float * SrcLineWeight = WeightDataX.GetData();

				const float RowWeight0 = WeightDataY[SrcY0];
				const float RowWeight1 = WeightDataY[SrcY1];


				VectorRegister VecRowWeight0 = VectorSetFloat1(RowWeight0);
				VectorRegister VecRowWeight1 = VectorSetFloat1(RowWeight1);

				float * DstLine = &ChannelData[DstY * (int64)Size.X];

				check(ActualDstX0 - StartX >= 0);

				// skip a single pixel to avoid maxes inside the loop
				int32 NumSkipX = (ActualDstX0 - StartX == 0) ? 1 : 0;

				int32 ActualWidth = FMath::Max<int32>(0,(ActualDstX1 - ActualDstX0) - NumSkipX);

				// how many groups of 4
				int32 ActualWidthGroup4 = ActualWidth / 4;

				// extra 0-3
				int32 ActualWidthExtra4 = ActualWidth - 4 * ActualWidthGroup4;


				for (int32 IterStart = 0; IterStart < NumSkipX; IterStart++) //-V1008
				{
					int32 DstX = ActualDstX0 + IterStart;

					int32 CurrX = DstX - StartX;

					int32 X0 = FMath::Max<int32>(0,CurrX - 1 + 0);
					int32 X1 = FMath::Max<int32>(0,CurrX - 1 + 1);

					float Sum = DstLine[DstX];

					Sum += PixelWeight11 * RowWeight0 * SrcLineWeight[X0] * SrcLineRaw0[X0];
					Sum += PixelWeight10 * RowWeight0 * SrcLineWeight[X1] * SrcLineRaw0[X1];
					Sum += PixelWeight01 * RowWeight1 * SrcLineWeight[X0] * SrcLineRaw1[X0];
					Sum += PixelWeight00 * RowWeight1 * SrcLineWeight[X1] * SrcLineRaw1[X1];

					DstLine[DstX] = Sum;
				}

				for (int32 GroupIter = 0; GroupIter < ActualWidthGroup4; GroupIter++)
				{
					{
						int32 BaseDstX = ActualDstX0 + NumSkipX + GroupIter * 4;

						int32 BaseCurrX = BaseDstX - StartX;

						int32 BaseX0 = BaseCurrX - 1 + 0;
						int32 BaseX1 = BaseCurrX - 1 + 1;

						VectorRegister Sum = VectorLoad(&DstLine[BaseDstX]);
						
						Sum = VectorMultiplyAdd(VecPixelWeight11, VectorMultiply(VecRowWeight0,VectorMultiply(VectorLoad(&SrcLineWeight[BaseX0]), VectorLoad(&SrcLineRaw0[BaseX0]))), Sum);
						Sum = VectorMultiplyAdd(VecPixelWeight10, VectorMultiply(VecRowWeight0,VectorMultiply(VectorLoad(&SrcLineWeight[BaseX1]), VectorLoad(&SrcLineRaw0[BaseX1]))), Sum);
						Sum = VectorMultiplyAdd(VecPixelWeight01, VectorMultiply(VecRowWeight1,VectorMultiply(VectorLoad(&SrcLineWeight[BaseX0]), VectorLoad(&SrcLineRaw1[BaseX0]))), Sum);
						Sum = VectorMultiplyAdd(VecPixelWeight00, VectorMultiply(VecRowWeight1,VectorMultiply(VectorLoad(&SrcLineWeight[BaseX1]), VectorLoad(&SrcLineRaw1[BaseX1]))), Sum);

						VectorStore(Sum, &DstLine[BaseDstX]);
					}
#if 0
					// Scalar version of the same code for reference
					for (int32 Iter = 0; Iter < 4; Iter++)
					{
						int32 DstX = ActualDstX0 + GroupIter * 4 + Iter;

						int32 CurrX = DstX - StartX;
						int32 X0 = CurrX - 1 + 0;
						int32 X1 = CurrX - 1 + 1;

						float Sum = DstLine[DstX];

						Sum += PixelWeight11 * SrcLineWeight0[X0] * SrcLineRaw0[X0];
						Sum += PixelWeight10 * SrcLineWeight0[X1] * SrcLineRaw0[X1];
						Sum += PixelWeight01 * SrcLineWeight1[X0] * SrcLineRaw1[X0];
						Sum += PixelWeight00 * SrcLineWeight1[X1] * SrcLineRaw1[X1];

						DstLine[DstX] = Sum;
					}
#endif
				}

				for (int32 IterExtra = 0; IterExtra < ActualWidthExtra4; IterExtra++)
				{
					int64 DstX = ActualDstX0 + NumSkipX + (int64)ActualWidthGroup4 * 4 + IterExtra;

					int32 CurrX = DstX - StartX;

					int32 X0 = FMath::Max<int32>(0,CurrX - 1 + 0);
					int32 X1 = FMath::Max<int32>(0,CurrX - 1 + 1);

					float Sum = DstLine[DstX];

					Sum += PixelWeight11 * RowWeight0 * SrcLineWeight[X0] * SrcLineRaw0[X0];
					Sum += PixelWeight10 * RowWeight0 * SrcLineWeight[X1] * SrcLineRaw0[X1];
					Sum += PixelWeight01 * RowWeight1 * SrcLineWeight[X0] * SrcLineRaw1[X0];
					Sum += PixelWeight00 * RowWeight1 * SrcLineWeight[X1] * SrcLineRaw1[X1];

					DstLine[DstX] = Sum;
				}

			});
		}
	}
}

void FImageOverlappedAccumulator::InitMemory(FIntPoint InPlaneSize, int32 InNumChannels)
{
	SCOPE_CYCLE_COUNTER(STAT_InitMemory);
	PlaneSize.X = InPlaneSize.X;
	PlaneSize.Y = InPlaneSize.Y;
	NumChannels = InNumChannels;

	ChannelPlanes.SetNum(NumChannels);

	// Initialize the RGBA channels and the weight plane all at the same time.
	ParallelFor(NumChannels + 1, [&](int32 InChannelIndex)
	{
		if (InChannelIndex < NumChannels)
		{
			ChannelPlanes[InChannelIndex].Init(PlaneSize);
		}
		else
		{
			WeightPlane.Init(PlaneSize);
		}
	});
}

void FImageOverlappedAccumulator::ZeroPlanes()
{
	SCOPE_CYCLE_COUNTER(STAT_ZeroPlanes);
	check(ChannelPlanes.Num() == NumChannels);

	// Zero our RGBA channels and the weight plane all at the same time.
	ParallelFor(NumChannels + 1, [&](int32 InChannelIndex)
	{
		if (InChannelIndex < NumChannels)
		{
			ChannelPlanes[InChannelIndex].ZeroPlane();
		}
		else
		{
			WeightPlane.ZeroPlane();
		}
	});

}

void FImageOverlappedAccumulator::Reset()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline::ResetAccumulator);
	
	// We don't free the memory on the individual channels because
	// Reset is called at the end of each output frame and the next
	// output frame is likely to be the same resolution and need the
	// same memory. The individual channels gracefully handle resizing.
	PlaneSize.X = 0;
	PlaneSize.Y = 0;
	NumChannels = 0;
}

#if 0

void FImageOverlappedAccumulator::GenerateTileWeight(TArray64<float>& OutWeights, FIntPoint Size)
{
	OutWeights.SetNum(Size.X * Size.Y);

	// we'll use a simple triangle filter, which goes from 1.0 at the center to 0.0 at 3/4 of the way to the tile edge.

	float ScaleX = 1.0f / (float(Size.X / 2) * .75f);
	float ScaleY = 1.0f / (float(Size.Y / 2) * .75f);

	for (int32 PixY = 0; PixY < Size.Y; PixY++)
	{
		for (int32 PixX = 0; PixX < Size.X; PixX++)
		{
			float Y = float(PixY) + .5f;
			float X = float(PixX) + .5f;

			float DistX = FMath::Abs(Size.X / 2 - X);
			float DistY = FMath::Abs(Size.Y / 2 - Y);

			float WeightX = FMath::Clamp(1.0f - DistX * ScaleX, 0.0f, 1.0f);
			float WeightY = FMath::Clamp(1.0f - DistY * ScaleY, 0.0f, 1.0f);

			OutWeights[PixY * Size.X + PixX] = WeightX * WeightY;
		}
	}
}

// Any changes to GenerateTileWeight() need to propagate to this function. The subrect should be large enough to incorporate all nonzero values
void FImageOverlappedAccumulator::GetTileSubRect(FIntPoint & OutSubRectOffset, FIntPoint & OutSubRectSize, const TArray64<float>& Weights, const FIntPoint Size)
{
	static bool bBruteForce = false;
	if (bBruteForce)
	{
		int32 MinX = Size.X-1;
		int32 MinY = Size.Y-1;

		int32 MaxX = 0;
		int32 MaxY = 0;

		for (int32 Y = 0; Y < Size.Y; Y++)
		{
			for (int32 X = 0; X < Size.X; X++)
			{
				const float Val = Weights[Y*Size.X + X];
				if (Val > 0.0f)
				{
					MinX = FMath::Min<int32>(MinX, X);
					MinY = FMath::Min<int32>(MinY, Y);

					MaxX = FMath::Max<int32>(MaxX, X);
					MaxY = FMath::Max<int32>(MaxY, Y);
				}
			}
		}

		OutSubRectOffset.X = MinX;
		OutSubRectOffset.Y = MinY;
		OutSubRectSize.X = (MaxX - MinX) + 1;
		OutSubRectSize.Y = (MaxY - MinY) + 1;
	}
	else
	{
		// We're using a traingle filter using the middle 75% of the screen. The filter starts at 12.5%, and ends at 87.5%. Then add one pixel
		// just in case for roundoff/precision error.
		int32 MinX = FMath::FloorToInt(0.125f * Size.X) - 1;
		int32 MinY = FMath::FloorToInt(0.125f * Size.Y) - 1;
		int32 MaxX = FMath::CeilToInt(0.875f * Size.X) + 1;
		int32 MaxY = FMath::CeilToInt(0.875f * Size.Y) + 1;

		OutSubRectOffset.X = MinX;
		OutSubRectOffset.Y = MinY;
		OutSubRectSize.X = (MaxX - MinX) + 1;
		OutSubRectSize.Y = (MaxY - MinY) + 1;
	}
}

void FImageOverlappedAccumulator::CheckTileSubRect(const TArray64<float>& OutWeights, const FIntPoint Size, FIntPoint SubRectOffset, FIntPoint SubRectSize)
{
	for (int32 Y = 0; Y < Size.Y; Y++)
	{
		for (int32 X = 0; X < Size.X; X++)
		{
			const float Val = OutWeights[Y*Size.X + X];
			if (Val > 0.0f)
			{
				check(SubRectOffset.X <= X);
				check(SubRectOffset.Y <= Y);
				check(X < SubRectOffset.X + SubRectSize.X);
				check(Y < SubRectOffset.Y + SubRectSize.Y);
			}
		}
	}
}
#endif

void FImageOverlappedAccumulator::AccumulatePixelData(const FImagePixelData& InPixelData, FIntPoint InTileOffset, FVector2D InSubpixelOffset, const MoviePipeline::FTileWeight1D & WeightX, const MoviePipeline::FTileWeight1D & WeightY)
{
	SCOPE_CYCLE_COUNTER(STAT_AccumulatePixelData);
	EImagePixelType Fmt = InPixelData.GetType();

	int32 RawNumChan = InPixelData.GetNumChannels();
	int32 RawBitDepth = InPixelData.GetBitDepth();
	FIntPoint RawSize = InPixelData.GetSize();

	int64 SizeInBytes = 0;
	const void* SrcRawDataPtr = nullptr;

	bool IsFetchOk = InPixelData.GetRawData(SrcRawDataPtr, SizeInBytes);
	check(IsFetchOk); // keeping the if below for now, but this really should always succeed

	if (IsFetchOk)
	{

		TArray<float> WeightDataX;
		TArray<float> WeightDataY;
		WeightX.CalculateArrayWeight(WeightDataX, RawSize.X);
		WeightY.CalculateArrayWeight(WeightDataY, RawSize.Y);

		FIntPoint SubRectOffset(WeightX.X0, WeightY.X0);
		FIntPoint SubRectSize(WeightX.X3 - WeightX.X0, WeightY.X3 - WeightY.X0);


		check(NumChannels >= 1);
		check(NumChannels <= 4);

		{
			// We always make room for all 4 channels (RGBA) even if we only accumulate 3 of them.
			if (UnpackedDataStorage[0].Num() != RawSize.X * (int64)RawSize.Y)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(AllocDataStorage);
				ParallelFor(4, [&](int32 InChannelIndex)
				{
					UnpackedDataStorage[InChannelIndex].SetNumUninitialized(RawSize.X * RawSize.Y);
				});
			}

			// Initialize the weight channel with 1's so that we can accumulate a bunch of 1 values together
			if (WeightChannelStorage.Num() != RawSize.X * (int64)RawSize.Y)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(AllocWeightChannelStorage);
				WeightChannelStorage.SetNumUninitialized(RawSize.X * (int64)RawSize.Y);
				ParallelFor(RawSize.Y, [&](int32 InIndexY)
				{
					for (int32 IndexX = 0; IndexX < RawSize.X; IndexX++)
					{
						WeightChannelStorage[(InIndexY * (int64)RawSize.X) + IndexX] = 1.f;
					}
				});
			}
		}

		const double AccumulateBeginTime = FPlatformTime::Seconds();
			
		// for now, only handle rgba8
		if (Fmt == EImagePixelType::Color && RawNumChan == 4 && RawBitDepth == 8)
		{
			const uint8* RawDataPtr = static_cast<const uint8*>(SrcRawDataPtr);

			static bool bIsReferenceUnpack = false;
			if (bIsReferenceUnpack)
			{
				// simple, slow unpack, takes about 10-15ms on a 1080p image
				for (int32 Y = 0; Y < RawSize.Y; Y++)
				{
					for (int32 X = 0; X < RawSize.X; X++)
					{
						int32 ChanReorder[4] = { 2, 1, 0, 3 };
						for (int32 ChanIter = 0; ChanIter < 4; ChanIter++)
						{
							int32 RawValue = RawDataPtr[(Y*(int64)RawSize.X + X)*RawNumChan + ChanIter];
							float Value = float(RawValue) / 255.0f;
							int32 Reorder = ChanReorder[ChanIter];
							UnpackedDataStorage[Reorder][Y*(int64)RawSize.X + X] = Value;
						}
					}
				}
			}
			else
			{
				// slightly optimized, takes about 3-7ms on a 1080p image
				for (int32 Y = 0; Y < RawSize.Y; Y++)
				{
					const uint8* SrcRowDataPtr = &RawDataPtr[Y*(int64)RawSize.X*RawNumChan];

					float* DstRowDataR = &UnpackedDataStorage[0][Y*(int64)RawSize.X];
					float* DstRowDataG = &UnpackedDataStorage[1][Y*(int64)RawSize.X];
					float* DstRowDataB = &UnpackedDataStorage[2][Y*(int64)RawSize.X];
					float* DstRowDataA = &UnpackedDataStorage[3][Y*(int64)RawSize.X];

					VectorRegister ColorScale = MakeVectorRegister(1.0f / 255.0f, 1.0f / 255.0f, 1.0f / 255.0f, 1.0f / 255.0f);

					// simple, one pixel at a time vectorized version, we could do better
					for (int32 X = 0; X < (int64)RawSize.X; X++)
					{
						VectorRegister Color = VectorLoadByte4(&SrcRowDataPtr[X*RawNumChan]);
						Color = VectorMultiply(Color, ColorScale); // multiply by 1/255

						const float* RawColorVec = reinterpret_cast<const float *>(&Color);
						DstRowDataR[X] = RawColorVec[2];
						DstRowDataG[X] = RawColorVec[1];
						DstRowDataB[X] = RawColorVec[0];
						DstRowDataA[X] = RawColorVec[3];
					}
				}
			}
		}
		else if (Fmt == EImagePixelType::Float16 && RawNumChan == 4 && RawBitDepth == 16)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline::Float16Unpack);
			const uint16* RawDataPtr = static_cast<const uint16*>(SrcRawDataPtr);

			ParallelFor(RawSize.Y, [&](int32 Y)
			{
				for (int32 X = 0; X < (int64)RawSize.X; X++)
				{
					float UnpackedColors[4];
					uint16_t* HalfColor = (uint16_t*)&RawDataPtr[(Y * (int64)RawSize.X + X) * RawNumChan];
					FPlatformMath::VectorLoadHalf(&UnpackedColors[0], HalfColor);

					for (int32 ChanIter = 0; ChanIter < 4; ChanIter++)
					{
						UnpackedDataStorage[ChanIter][Y * (int64)RawSize.X + X] = UnpackedColors[ChanIter];
					}
				}
			});
		}
		else if (Fmt == EImagePixelType::Float32 && RawNumChan == 4 && RawBitDepth == 32)
		{

			static bool IsReferenceUnpack = false;
			if (IsReferenceUnpack)
			{
				const float* RawDataPtr = static_cast<const float*>(SrcRawDataPtr);

				// reference version for float
				for (int32 Y = 0; Y < RawSize.Y; Y++)
				{
					for (int32 X = 0; X < (int64)RawSize.X; X++)
					{
						for (int32 ChanIter = 0; ChanIter < 4; ChanIter++)
						{
							float Value = RawDataPtr[(Y*(int64)RawSize.X + X)*RawNumChan + ChanIter];
							UnpackedDataStorage[ChanIter][Y*(int64)RawSize.X + X] = Value;
						}
					}
				}
			}
			else
			{
				const float* RawDataPtr = static_cast<const float*>(SrcRawDataPtr);

				// slightly optimized, takes about 3-7ms on a 1080p image
				for (int32 Y = 0; Y < RawSize.Y; Y++)
				{
					const float* SrcRowDataPtr = &RawDataPtr[Y*(int64)RawSize.X*RawNumChan];

					float* DstRowDataR = &UnpackedDataStorage[0][Y*(int64)RawSize.X];
					float* DstRowDataG = &UnpackedDataStorage[1][Y*(int64)RawSize.X];
					float* DstRowDataB = &UnpackedDataStorage[2][Y*(int64)RawSize.X];
					float* DstRowDataA = &UnpackedDataStorage[3][Y*(int64)RawSize.X];

					// simple, one pixel at a time vectorized version, we could do better
					for (int32 X = 0; X < (int64)RawSize.X; X++)
					{
						DstRowDataR[X] = SrcRowDataPtr[X*4+0];
						DstRowDataG[X] = SrcRowDataPtr[X*4+1];
						DstRowDataB[X] = SrcRowDataPtr[X*4+2];
						DstRowDataA[X] = SrcRowDataPtr[X*4+3];
					}
				}
			}
		}
		else
		{
			check(0);
		}

		static bool bLogTiming = false;

		const double AccumulateEndTime = FPlatformTime::Seconds();
		if(bLogTiming)
		{
			const float ElapsedMs = float((AccumulateEndTime - AccumulateBeginTime)*1000.0f);
			UE_LOGF(LogTemp, Log, "    [%8.2f] Unpack time.", ElapsedMs);
		}
	
		if (AccumulationGamma != 1.0f)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline::ApplyAccumulationGamma);
			// Unfortunately, we don't have an SSE optimized pow function. This function is quite slow (about 30-40ms).
			float Gamma = AccumulationGamma;

			for (int32 ChanIter = 0; ChanIter < NumChannels; ChanIter++)
			{
				float* DstData = UnpackedDataStorage[ChanIter].GetData();

				int32 DstSize = (int64)RawSize.X * RawSize.Y;

				static bool EnableVectorizedPow = true;
				if (EnableVectorizedPow)
				{
					int32 VectorGroups = DstSize / 4;
					int32 VectorExtra = DstSize - 4 * VectorGroups;
					
					VectorRegister PowExponent = VectorSetFloat1(Gamma);

					for (int32 Iter = 0; Iter < VectorGroups; Iter++)
					{
						VectorRegister Value = VectorLoad(&DstData[4*Iter]);
						Value = VectorPow(Value, PowExponent);
						VectorStore(Value, &DstData[4*Iter]);
					}

					for (int32 Iter = 0; Iter < VectorExtra; Iter++)
					{
						int32 Index = 4 * VectorGroups + Iter;
						DstData[Index] = FMath::Pow(DstData[Index], Gamma);
					}
				}
				else
				{
					for (int32 Iter = 0; Iter < DstSize; Iter++)
					{
						DstData[Iter] = FMath::Pow(DstData[Iter], Gamma);
					}
				}
			}
		} 

		const double GammaEndTime = FPlatformTime::Seconds();
		if (bLogTiming)
		{
			const float ElapsedMs = float((GammaEndTime - AccumulateEndTime)*1000.0f);
			UE_LOGF(LogTemp, Log, "    [%8.2f] Gamma time.", ElapsedMs);
		}

		const double TileEndTime = FPlatformTime::Seconds();
		if (bLogTiming)
		{
			const float ElapsedMs = float((TileEndTime - GammaEndTime)*1000.0f);
			UE_LOGF(LogTemp, Log, "    [%8.2f] Tile time.", ElapsedMs);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline::PlaneAccumulation);

			// We accumulate an RGBA plane, and an additional weight plane (for dividing per-pixel later)
			// but we do them all in the same ParallelFor through a slightly confusing bit of logic since
			// they're stored in separate arrays. This allows us to go wider than doing the RGBA, waiting
			// for them to finish, and then the weight plane.
			ParallelFor(NumChannels+1, [&](int32 InChannelIndex)
				{
					if (InChannelIndex < NumChannels)
					{
						ChannelPlanes[InChannelIndex].AccumulateSinglePlane(UnpackedDataStorage[InChannelIndex], RawSize, InTileOffset, InSubpixelOffset.X, InSubpixelOffset.Y,
							SubRectOffset, SubRectSize, WeightDataX, WeightDataY);
					}
					else
					{
						WeightPlane.AccumulateSinglePlane(WeightChannelStorage, RawSize, InTileOffset, InSubpixelOffset.X, InSubpixelOffset.Y,
							SubRectOffset, SubRectSize, WeightDataX, WeightDataY);

					}
				});

		}

		const double PlaneEndTime = FPlatformTime::Seconds();
		if (bLogTiming)
		{
			const float ElapsedMs = float((PlaneEndTime - TileEndTime)*1000.0f);
			UE_LOGF(LogTemp, Log, "    [%8.2f] Plane time.", ElapsedMs);
		}
	}
}

void FImageOverlappedAccumulator::FetchFullImageValue(float Rgba[4], int32 FullX, int32 FullY) const
{

	Rgba[0] = 0.0f;
	Rgba[1] = 0.0f;
	Rgba[2] = 0.0f;
	Rgba[3] = 1.0f;

	float RawWeight = WeightPlane.ChannelData[FullY * (int64)PlaneSize.X + FullX];

	float Scale = 1.0f / FMath::Max<float>(RawWeight, 0.0001f);

	for (int64 Chan = 0; Chan < NumChannels; Chan++)
	{
		float Val = ChannelPlanes[Chan].ChannelData[FullY * (int64)PlaneSize.X + FullX];

		Rgba[Chan] = Val * Scale;
	}

	if (AccumulationGamma != 1.0f)
	{
		float InvGamma = 1.0f / AccumulationGamma;
		Rgba[0] = FMath::Pow(Rgba[0], InvGamma);
		Rgba[1] = FMath::Pow(Rgba[1], InvGamma);
		Rgba[2] = FMath::Pow(Rgba[2], InvGamma);
		Rgba[3] = FMath::Pow(Rgba[3], InvGamma);
	}
}

void FImageOverlappedAccumulator::FetchFinalPixelDataByte(TArray64<FColor> & OutPixelData) const
{
	SCOPE_CYCLE_COUNTER(STAT_FetchFinalPixelData8bit);
	int32 FullSizeX = PlaneSize.X;
	int32 FullSizeY = PlaneSize.Y;
	OutPixelData.SetNumUninitialized(FullSizeX * FullSizeY);

	for (int32 FullY = 0L; FullY < FullSizeY; FullY++)
	{
		for (int32 FullX = 0L; FullX < FullSizeX; FullX++)
		{
			float Rgba[4] = { 0.0f, 0.0f, 0.0f, 1.0f };

			FetchFullImageValue(Rgba, FullX, FullY);
			FColor Color = FColor(Rgba[0]*255.0f,Rgba[1]*255.0f,Rgba[2]*255.0f,Rgba[3]*255.0f);

			// be careful with this index, make sure to use 64bit math, not 32bit
			int64 DstIndex = int64(FullY) * int64(FullSizeX) + int64(FullX);
			OutPixelData[DstIndex] = Color;
		}
	}
}

void FImageOverlappedAccumulator::FetchFinalPixelDataHalfFloat(TArray64<FFloat16Color>& OutPixelData) const
{
	SCOPE_CYCLE_COUNTER(STAT_FetchFinalPixelData16bit);
	int64 FullSizeX = PlaneSize.X;
	int64 FullSizeY = PlaneSize.Y;
	OutPixelData.SetNumUninitialized(FullSizeX * FullSizeY);

	ParallelFor(FullSizeY, [&](int32 InIndexY)
	{
		for (int32 FullX = 0L; FullX < FullSizeX; FullX++)
		{
			float Rgba[4] = { 0.0f, 0.0f, 0.0f, 1.0f };

			FetchFullImageValue(Rgba, FullX, InIndexY);
			FLinearColor Color = FLinearColor(Rgba[0],Rgba[1],Rgba[2],Rgba[3]);

			// be careful with this index, make sure to use 64bit math, not 32bit
			int64 DstIndex = int64(InIndexY) * int64(FullSizeX) + int64(FullX);
			OutPixelData[DstIndex] = FFloat16Color(Color);
		}
	});
}

void FImageOverlappedAccumulator::FetchFinalPixelDataLinearColor(TArray64<FLinearColor> & OutPixelData) const
{
	SCOPE_CYCLE_COUNTER(STAT_FetchFinalPixelData32bit);
	int32 FullSizeX = PlaneSize.X;
	int32 FullSizeY = PlaneSize.Y;
	OutPixelData.SetNumUninitialized(FullSizeX * FullSizeY);

	for (int32 FullY = 0L; FullY < FullSizeY; FullY++)
	{
		for (int32 FullX = 0L; FullX < FullSizeX; FullX++)
		{
			float Rgba[4] = { 0.0f, 0.0f, 0.0f, 1.0f };

			FetchFullImageValue(Rgba, FullX, FullY);
			FLinearColor Color = FLinearColor(Rgba[0],Rgba[1],Rgba[2],Rgba[3]);

			// be careful with this index, make sure to use 64bit math, not 32bit
			int64 DstIndex = int64(FullY) * int64(FullSizeX) + int64(FullX);
			OutPixelData[DstIndex] = Color;
		}
	}
}

