// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderPipelineCoreModule.h"
#include "Modules/ModuleInterface.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "MoviePipelineExecutor.h"
#include "MoviePipeline.h"
#include "MoviePipelineInProcessExecutor.h"
#include "LevelSequence.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineQueue.h"
#include "Misc/PackageName.h"
#include "Misc/FileHelper.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "MoviePipelinePythonHostExecutor.h"
#include "MoviePipelineUtils.h"
#include "MoviePipelineBlueprintLibrary.h"

#if WITH_EDITOR
//#include "Editor.h"
#include "ObjectTools.h"
#endif

void FMovieRenderPipelineCoreModule::InitializeCommandLineMovieRender()
{
#if WITH_EDITOR
	//const bool bIsGameMode = !GEditor;
	//if (!bIsGameMode)
	//{
	//	UE_LOGF(LogMovieRenderPipeline, Fatal, "Command Line Renders must be performed in -game mode, otherwise use the editor ui/python and PIE. Add -game to your command line arguments.");
	//	FPlatformMisc::RequestExitWithStatus(false, MoviePipelineErrorCodes::Critical);
	//	return;
	//}
#endif

	// Attempt to convert their command line arguments into the required objects.
	UMoviePipelineExecutorBase* ExecutorBase = nullptr;
	UMoviePipelineQueue* Queue = nullptr;

	uint8 ReturnCode = ParseMovieRenderData(SequenceAssetValue, SettingsAssetValue, MoviePipelineLocalExecutorClassType, MoviePipelineClassType, 
		/*Out*/ Queue, /*Out*/ ExecutorBase);
	if (!ensureMsgf(ExecutorBase, TEXT("There was a failure parsing the command line and a movie render cannot be started. Check the log for more details.")))
	{
		// Take the failure return code from the detection of our command line arguments.
		FPlatformMisc::RequestExitWithStatus(/*Force*/ false, /*ReturnCode*/ ReturnCode);
		return;
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Successfully detected and loaded required movie arguments. Rendering will begin once the map is loaded.");
		if (Queue)
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "NumJobs: %d ExecutorClass: %ls", Queue->GetJobs().Num(), *ExecutorBase->GetClass()->GetName());
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "ExecutorClass: %ls", *ExecutorBase->GetClass()->GetName());
		}

	}

	// We add the Executor to the root set. It will own all of the configuration data so this keeps it nicely in memory until finished,
	// and means we only have to add/remove one thing from the root set, everything else uses normal outer ownership.
	ExecutorBase->AddToRoot();
	ExecutorBase->OnExecutorFinished().AddRaw(this, &FMovieRenderPipelineCoreModule::OnCommandLineMovieRenderCompleted);
	ExecutorBase->OnExecutorErrored().AddRaw(this, &FMovieRenderPipelineCoreModule::OnCommandLineMovieRenderErrored);

	ExecutorBase->Execute(Queue);
}

void FMovieRenderPipelineCoreModule::OnCommandLineMovieRenderCompleted(UMoviePipelineExecutorBase* InExecutor, bool bSuccess)
{
	if (InExecutor)
	{
		InExecutor->RemoveFromRoot();
	}

	// Return a success code. Ideally any errors detected during rendering will return different codes.
	FPlatformMisc::RequestExitWithStatus(/*Force*/ false, /*ReturnCode*/ 0);
}

void FMovieRenderPipelineCoreModule::OnCommandLineMovieRenderErrored(UMoviePipelineExecutorBase* InExecutor, UMoviePipeline* InPipelineWithError, bool bIsFatal, FText ErrorText)
{
	UE_LOGF(LogMovieRenderPipeline, Error, "Error caught in Executor. Error: %ls", *ErrorText.ToString());
}


UClass* GetLocalExecutorClass(const FString& MoviePipelineLocalExecutorClassType, const FString ExecutorClassFormatString)
{
	if (MoviePipelineLocalExecutorClassType.Len() > 0)
	{
		UClass* LoadedExecutorClass = LoadClass<UMoviePipelineExecutorBase>(GetTransientPackage(), *MoviePipelineLocalExecutorClassType);
		if (LoadedExecutorClass)
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "Loaded explicitly specified Movie Pipeline Executor %ls.", *MoviePipelineLocalExecutorClassType);
			return LoadedExecutorClass;
		}
		else
		{
			// They explicitly specified an object, but that couldn't be loaded so it's an error.
			UE_LOGF(LogMovieRenderPipeline, Fatal, "Failed to load specified Local Executor class. Executor Class: %ls", *MoviePipelineLocalExecutorClassType);
			UE_LOGF(LogMovieRenderPipeline, Warning, "%ls", *ExecutorClassFormatString);
			return nullptr;
		}
	}
	else
	{
		// Fall back to our provided one. This is okay because it doesn't come from a user preference,
		// since it needs an different executor than the editor provided New Process
		UE_LOGF(LogMovieRenderPipeline, Log, "Using default Movie Pipeline Executor %ls. See '-MoviePipelineLocalExecutorClass' if you need to override this.", *MoviePipelineLocalExecutorClassType);
		return UMoviePipelineInProcessExecutor::StaticClass();
	}
}

UClass* GetMoviePipelineClass(const FString& MoviePipelineClassType, const FString ExecutorClassFormatString)
{
	if (MoviePipelineClassType.Len() > 0)
	{
		UClass* LoadedMoviePipelineClass = LoadClass<UMoviePipeline>(GetTransientPackage(), *MoviePipelineClassType);
		if (LoadedMoviePipelineClass)
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "Loaded explicitly specified Movie Pipeline %ls.", *MoviePipelineClassType);
			return LoadedMoviePipelineClass;
		}
		else
		{
			// They explicitly specified an object, but that couldn't be loaded so it's an error.
			UE_LOGF(LogMovieRenderPipeline, Fatal, "Failed to load specified Movie Pipeline class. Pipeline Class: %ls", *MoviePipelineClassType);
			UE_LOGF(LogMovieRenderPipeline, Warning, "%ls", *ExecutorClassFormatString);
			return nullptr;
		}
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Using default Movie Pipeline %ls. See '-MoviePipelineClass' if you need to override this.", *MoviePipelineClassType);
		return UMoviePipeline::StaticClass();
	}
}


bool FMovieRenderPipelineCoreModule::IsTryingToRenderMovieFromCommandLine(FString& OutSequenceAssetPath, FString& OutConfigAssetPath, FString& OutExecutorType, FString& OutPipelineType) const
{
	// Look to see if they've specified a Level Sequence to render. This should be in the format:
	// "/Game/MySequences/MySequence.MySequence"
	FParse::Value(FCommandLine::Get(), TEXT("-LevelSequence="), OutSequenceAssetPath);

	// Look to see if they've specified a configuration to use. This should be in the format:
	// "/Game/MyRenderSettings/MyHighQualitySetting.MyHighQualitySetting" or an absolute path 
	// to a exported json file.
	FParse::Value(FCommandLine::Get(), TEXT("-MoviePipelineConfig="), OutConfigAssetPath);

	// The user may want to override the executor. By default, we use the one specified in the Project
	// Settings, but we allow them to override it on the command line (for render farms, etc.) in case
	// the one used when a human is running the box isn't appropriate. This should be in the format:
	// "/Script/ModuleName.ClassNameNoUPrefix"
	FParse::Value(FCommandLine::Get(), TEXT("-MoviePipelineLocalExecutorClass="), OutExecutorType);

	// The user may want to override the Movie Pipeline itself. By default we use the one specified in the Project
	// Setting, but we allow overriding it for consistency's sake really. This should be in the format:
	// "/Script/ModuleName.ClassNameNoUPrefix"
	FParse::Value(FCommandLine::Get(), TEXT("-MoviePipelineClass="), OutPipelineType);

	// If they've specified any of them we'll assume they're trying to start - generous here to give people more flexibility
	// with what they are trying to do.
	bool bValidRenderCommands = OutSequenceAssetPath.Len() > 0 || OutConfigAssetPath.Len() > 0 || OutExecutorType.Len() > 0 || OutPipelineType.Len() > 0;

	// This can be removed when MovieSceneCapture is removed
	if (bValidRenderCommands)
	{
		FString MovieSceneCaptureType;
		FParse::Value(FCommandLine::Get(), TEXT("-MovieSceneCaptureType="), MovieSceneCaptureType);

		if (!MovieSceneCaptureType.IsEmpty())
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Detected a legacy MovieSceneCaptureType %ls along with valid render commands for Movie Render Queue. Skipping Movie Render Queue", *MovieSceneCaptureType);
			return false;
		}

		FString MovieSceneCaptureManifest;
		FParse::Value(FCommandLine::Get(), TEXT("-MovieSceneCaptureManifest="), MovieSceneCaptureManifest);

		if (!MovieSceneCaptureManifest.IsEmpty())
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Detected a legacy MovieSceneCaptureManifest %ls along with valid render commands for Movie Render Queue. Skipping Movie Render Queue", *MovieSceneCaptureManifest);
			return false;
		}
	}

	return bValidRenderCommands;
}

/**
* Command Line rendering supports two options for rendering things. We support a very simple 'provide the level sequence/settings to use'
* option which mimicks the old command line support. Option two is more advanced where you provide your own Executor. To allow for flexibility,
* the only thing required here is the executor path, a queue/level sequence are optional.
*
*
*	Option 1: Simple Command Line Render.
*		- Level Sequence (Required unless you pass an entire Queue asset, passed via -LevelSequence="/Game/...")
*		- Preset or Queue to use (A preset is required if using a Level Sequence above, passed via -MoviePipelineConfig="/Game/...")
*		- Will render on current map
*		ie: 
*			"E:\SubwaySequencer\SubwaySequencer.uproject" subwaySequencer_P -game -LevelSequence="/Game/Sequencer/SubwaySequencer.SubwaySequencer"
				-MoviePipelineConfig="/Game/Cinematics/MoviePipeline/Presets/SmallTestPreset.SmallTestPreset" -windowed -resx=1280 -resy=720 -log -notexturestreaming
*		or:
*			ie: "E:\SubwaySequencer\SubwaySequencer.uproject" subwaySequencer_P -game -MoviePipelineConfig="/Game/Cinematics/MoviePipeline/Presets/BigTestQueue.BigTestQueue" 
				-windowed -resx=1280 -resy=720 -log -notexturestreaming
*
*	Option 2: Advanced Custom Executor. 
*		- Executor Class (Required, pass via -MoviePipelineLocalExecutorClass=/Script/MovieRenderPipelineCore.MoviePipelinePythonHostExecutor)
*		- Level Sequence or Queue (Optional, if passed will be available to Executor)
*		- Python Class Override (Optional, requires using MoviePipelinePythonHostExecutor above, pass via -ExecutorPythonClass=/Engine/PythonTypes.MoviePipelineExampleRuntimeExecutor
*/
uint8 FMovieRenderPipelineCoreModule::ParseMovieRenderData(const FString& InSequenceAssetPath, const FString& InConfigAssetPath, const FString& InExecutorType, const FString& InPipelineType,
	UMoviePipelineQueue*& OutQueue, UMoviePipelineExecutorBase*& OutExecutor) const
{
	// Store off the messages that print the expected format since they're printed in a couple places.
	const FString SequenceAssetFormatString = TEXT("Level Sequence/Queue Asset should be specified in the format '-LevelSequence=\"/Game/MySequences/MySequence.MySequence\"");
	const FString ConfigAssetFormatString	= TEXT("Pipeline Config Asset should be specified in the format '-MoviePipelineConfig=\"/Game/MyRenderSettings/MyHighQualitySetting.MyHighQualitySetting\"'");
	const FString PipelineClassFormatString = TEXT("Movie Pipeline Class should be specified in the format '-MoviePipelineClass=\"/Script/ModuleName.ClassNameNoUPrefix\"'");
	const FString ExecutorClassFormatString = TEXT("Pipeline Executor Class should be specified in the format '-MoviePipelineLocalExecutorClass=\"/Script/ModuleName.ClassNameNoUPrefix\"'");

	OutQueue = nullptr;
	OutExecutor = nullptr;

	// Try to detect executor/class overrides 
	UClass* ExecutorClass = GetLocalExecutorClass(MoviePipelineLocalExecutorClassType, ExecutorClassFormatString);
	UClass* PipelineClass = GetMoviePipelineClass(MoviePipelineClassType, PipelineClassFormatString);

	if (!ensureMsgf(PipelineClass && ExecutorClass, TEXT("Attempted to render a movie pipeline but could not load executor or pipeline class, nor fall back to defaults.")))
	{
		return MoviePipelineErrorCodes::Critical;
	}

	// If they're just trying to render a specific sequence, parse that now.
	ULevelSequence* TargetSequence = nullptr;
	if(InSequenceAssetPath.Len() > 0)
	// Locate and load the level sequence they wish to render.
	{
		// Convert it to a soft object path and use that load to ensure it follows redirectors, etc.
		FSoftObjectPath AssetPath = FSoftObjectPath(InSequenceAssetPath);
		TargetSequence = Cast<ULevelSequence>(AssetPath.TryLoad());

		if (!TargetSequence)
		{
			UE_LOGF(LogMovieRenderPipeline, Fatal, "Failed to find Movie Pipeline sequence asset to render. Please note that the /Content/ part of the on-disk structure is omitted. Looked for: %ls", *InSequenceAssetPath);
			UE_LOGF(LogMovieRenderPipeline, Warning, "%ls", *SequenceAssetFormatString);
			return MoviePipelineErrorCodes::NoAsset;
		}
	}

	// Now look for the configuration file to see how to render it.
	if (InConfigAssetPath.Len() > 0)
	{
		if (InConfigAssetPath.EndsWith(FPackageName::GetTextAssetPackageExtension()))
		{
			OutQueue = UMoviePipelineBlueprintLibrary::LoadManifestFileFromString(InConfigAssetPath);

			if (!OutQueue)
			{
				UE_LOGF(LogMovieRenderPipeline, Fatal, "Could not parse text asset package.");
				return MoviePipelineErrorCodes::NoConfig;
			}
		}
		else
		{
			// Convert it to a soft object path and use that load to ensure it follows redirectors, etc.
			FSoftObjectPath AssetPath = FSoftObjectPath(InConfigAssetPath);
			if (UMoviePipelineQueue* AssetAsQueue = Cast<UMoviePipelineQueue>(AssetPath.TryLoad()))
			{
				OutQueue = AssetAsQueue;
			}
			else if (UMoviePipelinePrimaryConfig* AssetAsConfig = Cast<UMoviePipelinePrimaryConfig>(AssetPath.TryLoad()))
			{
				OutQueue = NewObject<UMoviePipelineQueue>();
				UMoviePipelineExecutorJob* NewJob = OutQueue->AllocateNewJob(UMoviePipelineExecutorJob::StaticClass()); // Only the default job type is supported right now.
				NewJob->Sequence = FSoftObjectPath(InSequenceAssetPath);
				NewJob->SetConfiguration(AssetAsConfig);
				UWorld* CurrentWorld = MoviePipeline::FindCurrentWorld();
				if (CurrentWorld)
				{
					NewJob->Map = FSoftObjectPath(CurrentWorld);
				}
			}
		}

		if (!OutQueue)
		{
			UE_LOGF(LogMovieRenderPipeline, Fatal, "Failed to find Pipeline Configuration asset to render. Please note that the /Content/ part of the on-disk structure should be omitted. Looked for: %ls", *InConfigAssetPath);
			UE_LOGF(LogMovieRenderPipeline, Warning, "%ls", *ConfigAssetFormatString);
			return MoviePipelineErrorCodes::NoConfig;
		}
	}

	// We have a special edge case for our Python Host class. It relies on Python modifying the CDO to point to a specific class, so
	// if we detect that their executor is of that type we use the class type specified in its CDO instead. Note that this only works
	// in editor builds (-game mode).
	if (ExecutorClass == UMoviePipelinePythonHostExecutor::StaticClass())
	{
		const UMoviePipelinePythonHostExecutor* CDO = GetDefault<UMoviePipelinePythonHostExecutor>();
		ExecutorClass = CDO->ExecutorClass;

		// If they didn't set it on the CDO, see if they passed it on the command line.
		if (!ExecutorClass)
		{
			FString PythonClassName;
			FParse::Value(FCommandLine::Get(), TEXT("-ExecutorPythonClass="), PythonClassName);

			if (PythonClassName.Len() > 0)
			{
				ExecutorClass = LoadClass<UMoviePipelineExecutorBase>(GetTransientPackage(), *PythonClassName);
				UE_LOGF(LogMovieRenderPipeline, Log, "Loaded executor from Python class: %ls", *PythonClassName);
			}
		}

		if (!ExecutorClass)
		{
			UE_LOGF(LogMovieRenderPipeline, Fatal, "No class set in Python Host Executor. This does nothing without setting a class via \"unreal.get_default_object(unreal.MoviePipelinePythonHostExecutor).executor_class = self.get_class()\", or passing \"-ExecutorPythonClass=/Engine/PythonTypes.MoviePipelineExampleRuntimeExecutor\" on the command line!");
			return MoviePipelineErrorCodes::Critical;
		}
	}

	// By this time, we know what assets you want to render, how to process the array of assets, and what runs an individual render. First we will create an executor.
	OutExecutor = NewObject<UMoviePipelineExecutorBase>(GetTransientPackage(), ExecutorClass);
	OutExecutor->SetMoviePipelineClass(PipelineClass);

	// A queue is optional if they're using an advanced executor
	if (OutQueue)
	{
		// Rename our Queue to belong to the Executor.
		OutQueue->Rename(nullptr, OutExecutor);
	}

	return MoviePipelineErrorCodes::Success;
}
