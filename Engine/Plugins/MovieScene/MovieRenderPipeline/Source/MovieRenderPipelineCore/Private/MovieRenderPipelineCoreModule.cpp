// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderPipelineCoreModule.h"
#include "Modules/ModuleInterface.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "MovieRenderPipelineDataTypes.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "TimerManager.h"
#include "MoviePipeline.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphBurnInNode.h"
#include "NamingTokenData.h"
#include "NamingTokens.h"
#include "NamingTokensEngineSubsystem.h"
#include "UObject/ICookInfo.h"

FGuid FMovieRenderPipelineCoreModule::NamingTokensGuid = FGuid();
TWeakObjectPtr<UNamingTokens> FMovieRenderPipelineCoreModule::NamingTokens = nullptr;

#if WITH_EDITOR
static TAutoConsoleVariable<bool> CVarAddCodeReferencedAssetsToCook(
	TEXT("MoviePipeline.AddCodeReferencedAssetsToCook"), true,
	TEXT("Add code-referenced-only assets to the cook.")
	TEXT("Can be disabled for a project that restricts this plugin's usage to the Editor target."),
	ECVF_Default);
#endif

FName IMoviePipelineBurnInExtension::ModularFeatureName = "ModularFeature_MoviePipelineBurnInExt";

void FMovieRenderPipelineCoreModule::StartupModule()
{
#if WITH_EDITOR
	if (IsRunningCookCommandlet() && CVarAddCodeReferencedAssetsToCook.GetValueOnGameThread())
	{
		UE::Cook::FDelegates::ModifyCook.AddLambda(
			[](UE::Cook::ICookInfo& CookInfo, TArray<UE::Cook::FPackageCookRule>& InOutPackageCookRules)
			{
				// Ensure these assets (which are referenced only by code) get packaged
				const FString* Assets[] =
				{
					&UMoviePipeline::DefaultDebugWidgetAsset,
					&UMovieGraphPipeline::DefaultPreviewWidgetAsset,
					&UMovieGraphBurnInNode::DefaultBurnInWidgetAsset
				};

				for (const FString* Asset : Assets)
				{
					InOutPackageCookRules.Add(
						UE::Cook::FPackageCookRule{
							.PackageName = FName(FSoftObjectPath(*Asset).GetLongPackageName()),
							.InstigatorName = FName("FMovieRenderPipelineCoreModule"),
							.CookRule = UE::Cook::EPackageCookRule::AddToCook
						}
					);
				}
			}
		);
	}
#endif

	// Look to see if they supplied arguments on the command line indicating they wish to render a movie.
	if (IsTryingToRenderMovieFromCommandLine(SequenceAssetValue, SettingsAssetValue, MoviePipelineLocalExecutorClassType, MoviePipelineClassType))
	{

		UE_LOGF(LogMovieRenderPipeline, Log, "Detected that the user intends to render a movie. Waiting until engine loop init is complete to ensure ");

		// Register a hook to wait until the engine has finished loading to increase the likelihood that the desired classes are loaded.
		FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FMovieRenderPipelineCoreModule::OnMapLoadFinished);
	}

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FMovieRenderPipelineCoreModule::BindToNamingTokenEvents);
}

void FMovieRenderPipelineCoreModule::OnMapLoadFinished(UWorld* InWorld)
{
	if (!InWorld)
	{
		return;
	}

	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

	// We have to wait two ticks for Python classes to have a chance to be initialized too. Using a chain of function calls
	// instead of a timer to ensure it is guranteed to be two ticks regardless of how long the first frame takes.
	InWorld->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateRaw(this, &FMovieRenderPipelineCoreModule::QueueInitialize, InWorld));
}

void FMovieRenderPipelineCoreModule::QueueInitialize(UWorld* InWorld)
{
	InWorld->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateRaw(this, &FMovieRenderPipelineCoreModule::InitializeCommandLineMovieRender));
}

void FMovieRenderPipelineCoreModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	if (const TStrongObjectPtr<UNamingTokens> NamingTokensPin = NamingTokens.Pin())
	{
		NamingTokensPin->UnregisterExternalTokens(NamingTokensGuid);
		NamingTokensPin->GetOnPreEvaluateEvent().RemoveAll(this);
	}
}

void FMovieRenderPipelineCoreModule::SetTickInfo(const FMoviePipelineLightweightTickInfo& InTickInfo)
{
	FMovieRenderPipelineCoreModule& MRQModule = FModuleManager::Get().GetModuleChecked<FMovieRenderPipelineCoreModule>("MovieRenderPipelineCore");
	MRQModule.TickInfo = InTickInfo;
}

void FMovieRenderPipelineCoreModule::BindToNamingTokenEvents()
{
	if (GEngine && !NamingTokens.IsValid())
	{
		NamingTokens = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->GetNamingTokens(UMovieRenderPipelineNamingTokens::TokenNamespace);
		if (ensure(NamingTokens.IsValid()))
		{
			const TStrongObjectPtr<UNamingTokens> NamingTokensPin = NamingTokens.Pin();
			NamingTokensPin->RegisterExternalTokens(NamingTokensGuid);
			NamingTokensPin->GetOnPreEvaluateEvent().AddRaw(this, &FMovieRenderPipelineCoreModule::OnMovieRenderPipelineNamingTokensPreEvaluate);
		}
	}
}

void FMovieRenderPipelineCoreModule::OnMovieRenderPipelineNamingTokensPreEvaluate(const FNamingTokensEvaluationData& InEvaluationData)
{
	if (NamingTokens.IsValid())
	{
		TArray<FNamingTokenData>& ExternalTokens = NamingTokens->GetExternalTokensChecked(NamingTokensGuid);

		// Only add tokens if they have not yet been registered with the Naming Tokens system
		if (ExternalTokens.IsEmpty())
		{
			// Just fetch the format arguments, no need to specify a format string; all we care about here is the token list
			FMovieGraphResolveArgs FormatArgs;
			UMovieGraphBlueprintLibrary::ResolveFormatArguments(FString(), FMovieGraphFilenameResolveParams(), FormatArgs);

			// Sort the format tokens alphabetically
			TArray<FString> FormatTokens;
			FormatArgs.FilenameArguments.GetKeys(FormatTokens);
			FormatTokens.Sort();

			// Add all of the tokens. Note that they do not evaluate to a value (empty delegate) because they're evaluated by MRG, not
			// the Naming Tokens system.
			for (const FString& FormatToken : FormatTokens)
			{
				ExternalTokens.Add({
					FormatToken,
					FText::FromString(FormatToken),
					FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([]() { return FText::GetEmpty(); })
				});
			}
		}
	}
}

IMPLEMENT_MODULE(FMovieRenderPipelineCoreModule, MovieRenderPipelineCore);
DEFINE_LOG_CATEGORY(LogMovieRenderPipeline); 
DEFINE_LOG_CATEGORY(LogMovieRenderPipelineIO);
