// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderTileImage.h"

#include "Math/Vector.h"
#include "Math/VectorRegister.h"

#include "MovieRenderPipelineCoreModule.h"
#include "Math/Float16.h"
#include "HAL/PlatformTime.h"

void FImageTilePlane::Init(int32 InSizeX, int32 InSizeY)
{
	SizeX = InSizeX;
	SizeY = InSizeY;

	AccumulationWeight = 0.0f;

	// leaves the data as is
	ChannelData.SetNumUninitialized(SizeX * SizeY);
}

void FImageTilePlane::ZeroPlane()
{
	AccumulationWeight = 0.0f;
	int32 Num = SizeX * SizeY;
	for (int32 Index = 0; Index < Num; Index++)
	{
		ChannelData[Index] = 0.0f;
	}
}

void FImageTilePlane::Reset()
{
	SizeX = 0;
	SizeY = 0;
	AccumulationWeight = 0.0f;
	ChannelData.Empty();
}

void FImageTilePlane::AccumulateSinglePlane(const TArray64<float> & InRawData, int32 InRawSizeX, int32 InRawSizeY, float InSampleWeight, int InSampleOffsetX, int InSampleOffsetY)
{
	check(InRawData.Num() == SizeX * SizeY);
	check(InRawSizeX == SizeX);
	check(InRawSizeY == SizeY);
	check(InRawData.Num() == ChannelData.Num());

	int32 CurrSizeX = SizeX;
	int32 CurrSizeY = SizeY;

	bool bRunUnoptimizedReference = false;
	if (bRunUnoptimizedReference)
	{
		// Here is the reference version. If we see any bugs around borders, just flip the bool above to verify that accumulation is not the issue.

		for (int32 DstY = 0; DstY < CurrSizeY; DstY++)
		{
			int32 SrcY = FMath::Clamp<int32>(DstY + InSampleOffsetY, 0, CurrSizeY - 1);

			const float* SrcRow = &InRawData[SrcY * InRawSizeX];
			float* DstRow = &ChannelData[DstY * CurrSizeX];

			for (int32 DstX = 0; DstX < CurrSizeX; DstX++)
			{
				int32 SrcX = FMath::Clamp<int32>(DstX + InSampleOffsetX, 0, CurrSizeX - 1);

				float SrcVal = SrcRow[SrcX];
				DstRow[DstX] += SrcVal * InSampleWeight;
			}
		}
	}
	else
	{
		// Samples are divided into 3 sections. Left, Middle, and Right (L, M, R)
		// The middle samples are vectorized groups of 4. The left and right samples use the reference clamp function.
		// We need to do that for any samples that are clamped (because they go off the edge) or extras because M must
		// be a multiple of 4.

		// Assume all samples are in the middle
		int32 SamplesL = 0;
		int32 SamplesM = CurrSizeX;
		int32 SamplesR = 0;

		// if InSampleOffset <= -1, then we need (-InSampleOffset) samples to be run on the left.
		if (InSampleOffsetX <= -1)
		{
			SamplesL = (-InSampleOffsetX);
			SamplesM -= SamplesL;
		}

		// if InSampleOffset >= 1, then we need InSampleOffset samples to be run on the right
		if (InSampleOffsetX >= 1)
		{
			SamplesR = InSampleOffsetX;
			SamplesM -= SamplesR;
		}

		// if SamplesM is not divisible by 4, add the extras on the right
		if (SamplesM % 4 != 0)
		{
			int32 ExtraSamples = SamplesM % 4;
			SamplesR += ExtraSamples;
			SamplesM -= ExtraSamples;
		}

		// Add some checks. Also make sure we didn't do something weird like have a 1 pixel sized tiled image (tiles should be large)
		// so the sample offset should never be larger than the width. Unlikely, but let's check just in case.
		check(SamplesL >= 0);
		check(SamplesM >= 0);
		check(SamplesR >= 0);
		check(SamplesM % 4 == 0);

		for (int32 DstY = 0; DstY < CurrSizeY; DstY++)
		{
			int32 SrcY = FMath::Clamp<int32>(DstY + InSampleOffsetY, 0, CurrSizeY - 1);

			const float* SrcRow = &InRawData[SrcY * InRawSizeX];
			float* DstRow = &ChannelData[DstY * CurrSizeX];

			// only need to clamp on the left
			for (int32 DstX = 0; DstX < SamplesL; DstX++)
			{
				int32 SrcX = FMath::Max<int32>(0, CurrSizeX - 1);

				float SrcVal = SrcRow[SrcX];
				DstRow[DstX] += SrcVal * InSampleWeight;
			}

			int32 MidVecSamples = SamplesM / 4;

			VectorRegister4Float VecScale = MakeVectorRegister(InSampleWeight, InSampleWeight, InSampleWeight, InSampleWeight);

			const char* SrcVecRow = reinterpret_cast<const char*>(&SrcRow[SamplesL + InSampleOffsetX]);
			char* DstVecRow = reinterpret_cast<char*>(&DstRow[SamplesL]);

			for (int32 VecIter = 0; VecIter < MidVecSamples; VecIter++)
			{
				VectorRegister4Float SrcVec = VectorLoad((float*)(SrcVecRow + 16 * VecIter));
				VectorRegister4Float DstVec = VectorLoad((float*)(DstVecRow + 16 * VecIter));

				DstVec = VectorMultiplyAdd(SrcVec, VecScale, DstVec);

				VectorStore(DstVec, (float*)(DstVecRow + 16 * VecIter));
			}

			// only need to clamp on the right
			for (int32 DstX = SamplesL + SamplesM; DstX < SamplesL + SamplesM + SamplesR; DstX++)
			{
				int32 SrcX = FMath::Min<int32>(DstX + InSampleOffsetX, CurrSizeX - 1);

				float SrcVal = SrcRow[SrcX];
				DstRow[DstX] += SrcVal * InSampleWeight;
			}
		}
	}
	

	AccumulationWeight += InSampleWeight;
}

void FImageTileAccumulator::InitMemory(int InTileSizeX, int InTileSizeY, int InNumTilesX, int InNumTilesY, int InNumChannels)
{
	TileSizeX = InTileSizeX;
	TileSizeY = InTileSizeY;
	NumTilesX = InNumTilesX;
	NumTilesY = InNumTilesY;
	NumChannels = InNumChannels;

	ImagePlanes.SetNum(NumTilesX * NumTilesY * NumChannels);

	for (int32 TileY = 0; TileY < NumTilesY; TileY++)
	{
		for (int32 TileX = 0; TileX < NumTilesX; TileX++)
		{
			for (int Channel = 0; Channel < NumChannels; Channel++)
			{
				int32 PlaneIndex = GetPlaneIndex(TileX, TileY, Channel);
				ImagePlanes[PlaneIndex].Init(TileSizeX, TileSizeY);
			}
		}
	}
}

void FImageTileAccumulator::ZeroPlanes()
{
	check(ImagePlanes.Num() == NumTilesX * NumTilesY * NumChannels);

	for (int32 TileY = 0; TileY < NumTilesY; TileY++)
	{
		for (int32 TileX = 0; TileX < NumTilesX; TileX++)
		{
			for (int Channel = 0; Channel < NumChannels; Channel++)
			{
				int32 PlaneIndex = GetPlaneIndex(TileX, TileY, Channel);
				ImagePlanes[PlaneIndex].ZeroPlane();
			}
		}
	}
}

void FImageTileAccumulator::Reset()
{
	TileSizeX = 0;
	TileSizeY = 0;
	NumTilesX = 0;
	NumTilesY = 0;
	NumChannels = 0;
	KernelRadius = 1.0;

	// Let the desctructor clean up
	ImagePlanes.Empty();
}

int32 FImageTileAccumulator::GetPlaneIndex(int32 InTileX, int32 InTileY, int32 InChannel) const
{
	return InTileY * (NumTilesX * NumChannels) + InTileX * (NumChannels) + InChannel;
}

FImageTilePlane & FImageTileAccumulator::AtPlaneData(int32 InTileX, int32 InTileY, int32 InChannel)
{
	int32 PlaneIndex = GetPlaneIndex(InTileX, InTileY, InChannel);
	ImagePlanes.RangeCheck(PlaneIndex);
	return ImagePlanes[PlaneIndex];
}

const FImageTilePlane & FImageTileAccumulator::AtPlaneData(int32 InTileX, int32 InTileY, int32 InChannel) const
{
	int32 PlaneIndex = GetPlaneIndex(InTileX, InTileY, InChannel);
	ImagePlanes.RangeCheck(PlaneIndex);
	return ImagePlanes[PlaneIndex];
}

void FImageTileAccumulator::AccumulateTile(const TArray64<float> & InRawData, int32 InRawSizeX, int32 InRawSizeY, int InRawChannel, FVector2D InSubpixelOffset)
{
	// Hardcode to look at a one pixel offset. We are assuming the radius is less than 1.0 in a Tile. For example,
	// if we have a 4x4 tile format, the radius must be less than 4.0.
	int32 Radius = 1;

	FVector2D ActualOffset = InSubpixelOffset * FVector2D(float(NumTilesX), float(NumTilesY));

	for (int32 CurrOffsetY = -Radius; CurrOffsetY <= Radius; CurrOffsetY++)
	{
		for (int32 CurrOffsetX = -Radius; CurrOffsetX <= Radius; CurrOffsetX++)
		{
			for (int32 SampleIterY = 0; SampleIterY < NumTilesY; SampleIterY++)
			{
				for (int32 SampleIterX = 0; SampleIterX < NumTilesX; SampleIterX++)
				{
					FVector2D TileSample;
					TileSample.X = (float(SampleIterX) + .5f) + float(NumTilesX * CurrOffsetX);
					TileSample.Y = (float(SampleIterY) + .5f) + float(NumTilesY * CurrOffsetY);

					float Dist = FVector2D::Distance(ActualOffset, TileSample);
					float Weight = CalcSampleWeight(Dist);

					if (Weight > 0.0f)
					{
						int32 PlaneIndex = GetPlaneIndex(SampleIterX, SampleIterY, InRawChannel);
						ImagePlanes[PlaneIndex].AccumulateSinglePlane(InRawData, InRawSizeX, InRawSizeY, Weight, -CurrOffsetX, -CurrOffsetY);
					}
				}
			}
		}
	}
}

float FImageTileAccumulator::CalcSampleWeight(float InDistance) const
{
	// 1.0 at sample center, 0.0 at KernelRadius distance away.
	float Weight = FMath::Clamp(1.0f - InDistance / KernelRadius, 0.0f, 1.0f);
	return Weight;
}

void FImageTileAccumulator::AccumulatePixelData(const FImagePixelData& InPixelData, int32 InTileX, int32 InTileY, FVector2D InSubpixelOffset)
{
	EImagePixelType Fmt = InPixelData.GetType();

	int32 RawNumChan = InPixelData.GetNumChannels();
	int32 RawBitDepth = InPixelData.GetBitDepth();
	FIntPoint RawSize = InPixelData.GetSize();

	int64 SizeInBytes = 0;
	const void* SrcRawDataPtr = nullptr;

	bool IsFetchOk = InPixelData.GetRawData(SrcRawDataPtr, SizeInBytes);
	check(IsFetchOk); // keeping the if below for now, but this really should always succeed

	if (IsFetchOk)
	{
		// hardcode to 4 channels (RGBA), even if we are only saving fewer channels
		TArray64<float> RawData[4];

		check(NumChannels >= 1);
		check(NumChannels <= 4);

		for (int32 ChanIter = 0; ChanIter < 4; ChanIter++)
		{
			RawData[ChanIter].SetNumUninitialized(RawSize.X * RawSize.Y);
		}

		const double AccumulateBeginTime = FPlatformTime::Seconds();
			
		// for now, only handle rgba8
		if (Fmt == EImagePixelType::Color && RawNumChan == 4 && RawBitDepth == 8)
		{
			const uint8* RawDataPtr = static_cast<const uint8*>(SrcRawDataPtr);

			static bool bIsReferenceUnpack = false;
			if (bIsReferenceUnpack)
			{
				// simple, slow unpack, takes about 10-15ms on a 1080p image
				for (int32 Y = 0; Y < RawSize.Y; Y++)
				{
					for (int32 X = 0; X < RawSize.X; X++)
					{
						int32 ChanReorder[4] = { 2, 1, 0, 3 };
						for (int32 ChanIter = 0; ChanIter < 4; ChanIter++)
						{
							int32 RawValue = RawDataPtr[(Y*RawSize.X + X)*RawNumChan + ChanIter];
							float Value = float(RawValue) / 255.0f;
							int32 Reorder = ChanReorder[ChanIter];
							RawData[Reorder][Y*RawSize.X + X] = Value;
						}
					}
				}
			}
			else
			{
				// slightly optimized, takes about 3-7ms on a 1080p image
				for (int32 Y = 0; Y < RawSize.Y; Y++)
				{
					const uint8* SrcRowDataPtr = &RawDataPtr[Y*RawSize.X*RawNumChan];

					float* DstRowDataR = &RawData[0][Y*RawSize.X];
					float* DstRowDataG = &RawData[1][Y*RawSize.X];
					float* DstRowDataB = &RawData[2][Y*RawSize.X];
					float* DstRowDataA = &RawData[3][Y*RawSize.X];

					VectorRegister4Float ColorScale = MakeVectorRegister(1.0f / 255.0f, 1.0f / 255.0f, 1.0f / 255.0f, 1.0f / 255.0f);

					// simple, one pixel at a time vectorized version, we could do better
					for (int32 X = 0; X < RawSize.X; X++)
					{
						VectorRegister4Float Color = VectorLoadByte4(&SrcRowDataPtr[X*RawNumChan]);
						Color = VectorMultiply(Color, ColorScale); // multiply by 1/255

						const float* RawColorVec = reinterpret_cast<const float *>(&Color);
						DstRowDataR[X] = RawColorVec[2];
						DstRowDataG[X] = RawColorVec[1];
						DstRowDataB[X] = RawColorVec[0];
						DstRowDataA[X] = RawColorVec[3];
					}
				}
			}
		}
		else if (Fmt == EImagePixelType::Float16 && RawNumChan == 4 && RawBitDepth == 16)
		{
			const uint16* RawDataPtr = static_cast<const uint16*>(SrcRawDataPtr);

			// simple, slow unpack version for fp16, could make this faster
			for (int32 Y = 0; Y < RawSize.Y; Y++)
			{
				for (int32 X = 0; X < RawSize.X; X++)
				{
					for (int32 ChanIter = 0; ChanIter < 4; ChanIter++)
					{
						FFloat16 RawValue;
						RawValue.Encoded = RawDataPtr[(Y*RawSize.X + X)*RawNumChan + ChanIter];
						
						// c cast does the conversion from fp16 bits to float
						float Value = float(RawValue);
						RawData[ChanIter][Y*RawSize.X + X] = Value;
					}
				}
			}
		}
		else if (Fmt == EImagePixelType::Float32 && RawNumChan == 4 && RawBitDepth == 32)
		{
			const float* RawDataPtr = static_cast<const float*>(SrcRawDataPtr);

			// reference version for float
			for (int32 Y = 0; Y < RawSize.Y; Y++)
			{
				for (int32 X = 0; X < RawSize.X; X++)
				{
					for (int32 ChanIter = 0; ChanIter < 4; ChanIter++)
					{
						float Value = RawDataPtr[(Y*RawSize.X + X)*RawNumChan + ChanIter];
						RawData[ChanIter][Y*RawSize.X + X] = Value;
					}
				}
			}
		}
		else
		{
			check(0);
		}

		const double AccumulateEndTime = FPlatformTime::Seconds();
		{
			const float ElapsedMs = float((AccumulateEndTime - AccumulateBeginTime)*1000.0f);
			//UE_LOGF(LogTemp, Log, "    [%8.2f] Unpack time.", ElapsedMs);
		}
	
		if (AccumulationGamma != 1.0f)
		{
			// Unfortunately, we don't have an SSE optimized pow function. This function is quite slow (about 30-40ms).
			float Gamma = AccumulationGamma;

			for (int ChanIter = 0; ChanIter < NumChannels; ChanIter++)
			{
				float* DstData = RawData[ChanIter].GetData();
				int32 DstSize = RawSize.X * RawSize.Y;
				for (int32 Iter = 0; Iter < DstSize; Iter++)
				{
					DstData[Iter] = FMath::Pow(DstData[Iter], Gamma);
				}
			}
		} 

		const double GammaEndTime = FPlatformTime::Seconds();
		{
			const float ElapsedMs = float((GammaEndTime - AccumulateEndTime)*1000.0f);
			//UE_LOGF(LogTemp, Log, "        [%8.2f] Gamma time.", ElapsedMs);
		}

		for (int32 ChanIter = 0; ChanIter < NumChannels; ChanIter++)
		{

			float TileScaleX = 1.0f / float(NumTilesX);
			float TileScaleY = 1.0f / float(NumTilesY);

			FVector2D SubpixelOffset;
			SubpixelOffset.X = (float(InTileX) + InSubpixelOffset.X) * TileScaleX;
			SubpixelOffset.Y = (float(InTileY) + InSubpixelOffset.Y) * TileScaleY;

			AccumulateTile(RawData[ChanIter], RawSize.X, RawSize.Y, ChanIter, SubpixelOffset);
		}
	}
}

void FImageTileAccumulator::FetchFullImageValue(float Rgba[4], const TArray<float>& PlaneScale, int32 FullX, int32 FullY) const
{
	int32 TileX = static_cast<int32>(FullX) % NumTilesX;
	int32 TileY = static_cast<int32>(FullY) % NumTilesY;

	int32 IndexX = static_cast<int32>(FullX) / NumTilesX;
	int32 IndexY = static_cast<int32>(FullY) / NumTilesY;

	Rgba[0] = 0.0f;
	Rgba[1] = 0.0f;
	Rgba[2] = 0.0f;
	Rgba[3] = 1.0f;

	for (int64 Chan = 0; Chan < NumChannels; Chan++)
	{
		int32 PlaneId = GetPlaneIndex(TileX, TileY, static_cast<int32>(Chan));

		float Scale = PlaneScale[PlaneId];
		float Val = ImagePlanes[PlaneId].ChannelData[IndexY * TileSizeX + IndexX];

		Rgba[Chan] = Val * Scale;
	}

	if (AccumulationGamma != 1.0f)
	{
		float InvGamma = 1.0f / AccumulationGamma;
		Rgba[0] = FMath::Pow(Rgba[0], InvGamma);
		Rgba[1] = FMath::Pow(Rgba[1], InvGamma);
		Rgba[2] = FMath::Pow(Rgba[2], InvGamma);
		Rgba[3] = FMath::Pow(Rgba[3], InvGamma);
	}
}

void FImageTileAccumulator::FetchFinalPlaneScale(TArray<float>& PlaneScale) const
{
	PlaneScale.SetNum(ImagePlanes.Num());
	for (int32 Index = 0; Index < PlaneScale.Num(); Index++)
	{
		float Weight = ImagePlanes[Index].AccumulationWeight;
		float Scale = (Weight != 0.0f) ? 1.0f / Weight : 0.0f;
		PlaneScale[Index] = Scale;
	}
}

void FImageTileAccumulator::FetchFinalPixelDataByte(TArray64<FColor> & OutPixelData) const
{
	int32 FullSizeX = TileSizeX * NumTilesX;
	int32 FullSizeY = TileSizeY * NumTilesY;
	OutPixelData.SetNumUninitialized(FullSizeX * FullSizeY);

	TArray<float> PlaneScale;
	FetchFinalPlaneScale(PlaneScale);

	for (int32 FullY = 0L; FullY < FullSizeY; FullY++)
	{
		for (int32 FullX = 0L; FullX < FullSizeX; FullX++)
		{
			float Rgba[4] = { 0.0f, 0.0f, 0.0f, 1.0f };

			FetchFullImageValue(Rgba, PlaneScale, FullX, FullY);
			FColor Color = FColor(Rgba[0]*255.0f,Rgba[1]*255.0f,Rgba[2]*255.0f,Rgba[3]*255.0f);

			// be careful with this index, make sure to use 64bit math, not 32bit
			int64 DstIndex = int64(FullY) * int64(FullSizeX) + int64(FullX);
			OutPixelData[DstIndex] = Color;
		}
	}
}

void FImageTileAccumulator::FetchFinalPixelDataLinearColor(TArray64<FLinearColor> & OutPixelData) const
{
	int32 FullSizeX = TileSizeX * NumTilesX;
	int32 FullSizeY = TileSizeY * NumTilesY;
	OutPixelData.SetNumUninitialized(FullSizeX * FullSizeY);

	TArray<float> PlaneScale;
	FetchFinalPlaneScale(PlaneScale);

	for (int32 FullY = 0L; FullY < FullSizeY; FullY++)
	{
		for (int32 FullX = 0L; FullX < FullSizeX; FullX++)
		{
			float Rgba[4] = { 0.0f, 0.0f, 0.0f, 1.0f };

			FetchFullImageValue(Rgba, PlaneScale, FullX, FullY);
			FLinearColor Color = FLinearColor(Rgba[0],Rgba[1],Rgba[2],Rgba[3]);

			// be careful with this index, make sure to use 64bit math, not 32bit
			int64 DstIndex = int64(FullY) * int64(FullSizeX) + int64(FullX);
			OutPixelData[DstIndex] = Color;
		}
	}
}



