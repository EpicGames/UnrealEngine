// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Graph/MovieGraphFilenameResolveParams.h"
#include "Misc/FrameRate.h"
#include "MovieGraphNamedResolution.h"

#include "MovieGraphBlueprintLibrary.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declare
class UMovieGraphGlobalOutputSettingNode;
class UMovieGraphPipeline;

UCLASS(MinimalAPI, meta = (ScriptName = "MovieGraphLibrary"))
class UMovieGraphBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	* If InNode is valid, inspects the provided OutputsettingNode to determine if it wants to override the
	* Frame Rate, and if so, returns the overwritten frame rate. If nullptr, or it does not have the
	* bOverride_bUseCustomFrameRate flag set, then InDefaultrate is returned.
	* @param	InNode			- Optional, setting to inspect for a custom framerate.
	* @param	InDefaultRate	- The frame rate to use if the node is nullptr or doesn't want to override the rate.
	* @return					- The effective frame rate (taking into account the node's desire to override it). 
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	static UE_API FFrameRate GetEffectiveFrameRate(UMovieGraphGlobalOutputSettingNode* InNode, const FFrameRate& InDefaultRate);

	/**
	* Takes a Movie Graph format string (in the form of {token}), a list of parameters (which normally come from the running UMovieGraphPipeline) and
	* resolves them into a file path. Unknown tokens are ignored. Which tokens can be resolved depends on the contents of InParams, tokens from settings
	* rely on a evaluated config being provided, etc.
	* @param	InFormatString		- Format string to attempt to resolve. Leave blank if just the format args should be populated.
	* @param	InParams			- A list of parameters to use as source data for the resolve step. Normally comes from the UMovieGraphPipeline instance,
	*								- but takes (mostly) POD here to allow using this function outside of the render runtime.
	* @param	OutMergedFormatArgs - The set of KVP for both filename formats and file metadata that is generated as a result of this. Provided in case you
	* 								- needed to do your own string resolving with the final dataset.
	* @return						- The resolved file path string. Returns an empty string if InFormatString is blank.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	static UE_API FString ResolveFilenameFormatArguments(const FString& InFormatString, const FMovieGraphFilenameResolveParams& InParams, FMovieGraphResolveArgs& OutMergedFormatArgs);

	/**
	 * Functions identically to ResolveFilenameFormatArguments(), with the exception that this variation does not resolve the format string to a
	 * file path (any non-path format string can be provided).
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	static UE_API FString ResolveFormatArguments(const FString& InFormatString, const FMovieGraphFilenameResolveParams& InParams, FMovieGraphResolveArgs& OutMergedFormatArgs);

	/**
	 * If the version number is explicitly specified on the Output Setting node, returns that. Otherwise searches the
	 * output directory for the highest version that already exists (and increments it by one if bGetNextVersion is
	 * true). Returns -1 if the version could not be resolved.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	static UE_API int32 ResolveVersionNumber(FMovieGraphFilenameResolveParams InParams, const bool bGetNextVersion = true);

	/**
	* Retrieves the cached version number calculated for the current shot, which depends on where the version token was used in the File Name Output
	* ie: If {version} comes before {shot_name} then all shots will use the same version number, but if it comes afterwards then each shot may
	* have a different version (which is the highest number found of that particular shot). This function should retrieve what is used in the
	* filename writing step either way.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API int32 GetCurrentVersionNumber(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* In case of overscan percentage being higher than 0, additional pixels are rendered. This function returns the resolution with overscan taken into account.
	* @param	InEvaluatedGraph	- The evaluated graph that will provide context for resolving the resolution
	* @param	DefaultOverscan		- The default overscan to use if there are no camera settings that provide an overscan override value, from 0.0 to 1.0
	* @return						- The output resolution, taking into account overscan
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph", meta=(DeprecatedFunction, DeprecationMessage = "Use GetOverscannedResolution instead"))
	UE_DEPRECATED(5.6, "GetEffectiveOutputResolution is deprecated. Use GetOverscannedResolution instead")
	static UE_API FIntPoint GetEffectiveOutputResolution(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float DefaultOverscan = 0.0f);

	/**
	 * Gets the desired output resolution for the movie render graph as specified by the user. Does not include overscan, tiling, or aspect ratio constraints,
	 * and is the target resolution that the pipeline will generally output (e.g. when cropping overscan in non-EXR formats). Use GetOverscannedResolution or
	 * GetBackbufferResolution to get resolutions that factor in overscan and tiling, respectively
	 * @param	InEvaluatedGraph	- The evaluated graph that will provide context for resolving the resolution
	 * @param	CameraAspectRatio	- The aspect ratio for the camera. Set to zero if you want to adapt to the output resolution's 
	 *								  aspect ratio (otherwise resolution may be adapted to fit aspect ratio based on config settings).
	 * @return						- The desired output resolution
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	static UE_API FIntPoint GetDesiredOutputResolution(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float CameraAspectRatio = 0.0f);

	/**
	* Gets the desired output resolution scaled by any configured overscan for the movie render graph
	* @param	InEvaluatedGraph	- The evaluated graph that will provide context for resolving the resolution
	* @param	DefaultOverscan		- The default overscan to use if there are no camera settings that provide an overscan override value, from 0.0 to 1.0
	* @param	CameraAspectRatio	- The aspect ratio for the camera. Set to zero if you want to adapt to the output resolution's
	*								  aspect ratio (otherwise resolution may be adapted to fit aspect ratio based on config settings).
	* @return						- The overscan-scaled resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	static UE_API FIntPoint GetOverscannedResolution(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float DefaultOverscan = 0.0f, float CameraAspectRatio = 0.0f);

	/**
	 * Gets the resolution that frames will actually be rendered at in MRG, which includes factors such as overscan and tiling
	 * @param	InEvaluatedGraph	- The evaluated graph that will provide context for resolving the resolution
	 * @param	DefaultOverscan		- The default overscan to use if there are no camera settings that provide an overscan override value, from 0.0 to 1.0
	 * @param	CameraAspectRatio	- The aspect ratio for the camera. Set to zero if you want to adapt to the output resolution's
	 *								  aspect ratio (otherwise resolution may be adapted to fit aspect ratio based on config settings).
	 * @return						- The resolution of a rendered frame
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	static UE_API FIntPoint GetBackbufferResolution(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float DefaultOverscan = 0.0f, float CameraAspectRatio = 0.0f);
	
	/**
	 * Gets a rectangle that will crop out any overscan applied to the scene. If there is no overscan, the rectangle returned will match the backbuffer rectangle
	 * @param InEvaluatedGraph		- The evaluated graph that will provide context for resolving the resolution
	 * @param DefaultOverscan		- The default overscan to use if there are no camera settings that provide an overscan override value, from 0.0 to 1.0
	 * @param	CameraAspectRatio	- The aspect ratio for the camera. Set to zero if you want to adapt to the output resolution's
	 *								  aspect ratio (otherwise resolution may be adapted to fit aspect ratio based on config settings).
	 * @return						- The crop rectangle, which will have a resolution matching the requested output resolution
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	static UE_API FIntRect GetOverscanCropRectangle(UMovieGraphEvaluatedConfig* InEvaluatedGraph, float DefaultOverscan = 0.0f, float CameraAspectRatio = 0.0f);
	
	/**
	* Gets the name of the current job.
	* @param	InMovieGraphPipeline	 - The pipeline to get the job name from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FText GetJobName(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* Gets the author of the current job, or the logged in user's username if the job has no specified author.
	* @param	InMovieGraphPipeline	 - The pipeline to get the job author from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FText GetJobAuthor(const UMovieGraphPipeline* InMovieGraphPipeline);

	/** Gets the completion percent of the Pipeline in 0-1 */
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API float GetCompletionPercentage(const UMovieGraphPipeline* InPipeline);

	/**
	* Determines the overall current frame number and total number of frames.
	* @param	InMovieGraphPipeline	- The pipeline to get the frame information from.
	* @param	OutCurrentIndex			- The current frame number.
	* @param	OutTotalCount			- The total number of frames.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API void GetOverallOutputFrames(const UMovieGraphPipeline* InMovieGraphPipeline, int32& OutCurrentIndex, int32& OutTotalCount);

	/**
	* Gets the time the job was initialized.
	* @param	InMovieGraphPipeline	- The pipeline to get the job initialization time from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FDateTime GetJobInitializationTime(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* Get the estimated amount of time remaining for the current pipeline. Based on looking at the total
	* amount of samples to render vs. how many have been completed so far. Inaccurate when Time Dilation
	* is used, and gets more accurate over the course of the render.
	* @param	InMovieGraphPipeline	- The pipeline to get the time estimate from.
	* @param	OutEstimate				- The resulting estimate, or FTimespan() if estimate is not valid.
	* @return							- True if a valid estimate can be calculated, or false if it is not ready yet (ie: not enough samples rendered)
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API bool GetEstimatedTimeRemaining(const UMovieGraphPipeline* InMovieGraphPipeline, FTimespan& OutEstimate);

	/**
	* Get the current state of the specified pipeline. See EMovieRenderPipelineState for more detail about each state.
	* @param	InMovieGraphPipeline	- The pipeline to get the state for.
	* @return							- The current state.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API EMovieRenderPipelineState GetPipelineState(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* Gets the state of the segment (shot) currently being rendered.
	* @param	InMovieGraphPipeline	- The pipeline to get segment information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API EMovieRenderShotState GetCurrentSegmentState(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* Gets the name of the segment (shot) currently being rendered.
	* @param	InMovieGraphPipeline	- The pipeline to get segment information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API void GetCurrentSegmentName(const UMovieGraphPipeline* InMovieGraphPipeline, FText& OutOuterName, FText& OutInnerName);

	/**
	* Gets the number of segments (shots) that will be rendered.
	* @param	InMovieGraphPipeline	- The pipeline to get segment information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API void GetOverallSegmentCounts(const UMovieGraphPipeline* InMovieGraphPipeline, int32& OutCurrentIndex, int32& OutTotalCount);

	/**
	* Gets the work metrics for the segment (shot) that is currently being rendered.
	* @param	InMovieGraphPipeline	- The pipeline to get segment information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FMoviePipelineSegmentWorkMetrics GetCurrentSegmentWorkMetrics(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* Gets the timecode of the current render at the root (sequence) level.
	* @param	InMovieGraphPipeline	- The pipeline to get timecode information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FTimecode GetRootTimecode(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* Gets the frame number of the current render at the root (sequence) level.
	* @param	InMovieGraphPipeline	- The pipeline to get frame number information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FFrameNumber GetRootFrameNumber(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* Gets the timecode of the current render at the shot level.
	* @param	InMovieGraphPipeline	- The pipeline to get timecode information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FTimecode GetCurrentShotTimecode(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* Gets the frame number of the current render at the shot level.
	* @param	InMovieGraphPipeline	- The pipeline to get frame number information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FFrameNumber GetCurrentShotFrameNumber(const UMovieGraphPipeline* InMovieGraphPipeline);

	/**
	* Gets the focus distance for the camera currently in use.
	* @param	InMovieGraphPipeline	- The pipeline to get the camera information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API float GetCurrentFocusDistance(const UMovieGraphPipeline* InMovieGraphPipeline, int32 InCameraIndex = -1);

	/**
	* Gets the focal length for the camera currently in use.
	* @param	InMovieGraphPipeline	- The pipeline to get the camera information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API float GetCurrentFocalLength(const UMovieGraphPipeline* InMovieGraphPipeline, int32 InCameraIndex = -1);
	
	/**
	* Gets the aperture for the camera currently in use.
	* @param	InMovieGraphPipeline	- The pipeline to get the camera information from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API float GetCurrentAperture(const UMovieGraphPipeline* InMovieGraphPipeline, int32 InCameraIndex = -1);

	/**
	* Gets the currently active cine camera, or nullptr if one was not found.
	* @param	InMovieGraphPipeline	- The pipeline to get the camera from.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API UCineCameraComponent* GetCurrentCineCamera(const UMovieGraphPipeline* InMovieGraphPipeline, int32 InCameraIndex = -1);



	/**
	* Create a Named Resolution from the profile name. Throws a Kismet Exception if the profile name isn't found.
	* The known profiles can be found in UMovieGraphProjectSettings's CDO.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FMovieGraphNamedResolution NamedResolutionFromProfile(const FName& InResolutionProfileName);

	/**
	* Utility function for checking if a given resolution profile name is valid, since NamedResolutionFromProfile
	* will throw a kismet exception, but blueprints can't actually try/catch them.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API bool IsNamedResolutionValid(const FName& InResolutionProfileName);

	/**
	* Create a Named Resolution from the given resolution. Given named resolution will be named "Custom".
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API FMovieGraphNamedResolution NamedResolutionFromSize(const int32 InResX, const int32 InResY);

	/**
	* Gets the current shot being rendered by the graph (could be nullptr if rendering hasn't started or has moved to Finalize!)
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	static UE_API UMoviePipelineExecutorShot* GetCurrentExecutorShot(const UMovieGraphPipeline* InMoviePipeline);

	/**
	 * Returns the current play world (PIE or Game). Useful when a Blueprint does not have a world context object,
	 * such as in MovieGraphScriptBase callbacks. Returns nullptr if no play world is active.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Graph", meta = (DisplayName = "Get Current Play World"))
	static UE_API UWorld* GetCurrentPlayWorld();

};

#undef UE_API
