// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"

#include "MovieGraphBurnInWidget.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Base class for graph-based level sequence burn-ins.
 */
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMovieGraphBurnInWidget : public UUserWidget
{
public:
	GENERATED_BODY()

	/** 
	* Called on the first temporal and first spatial sample of each output frame.
	* @param	InGraphPipeline		The graph pipeline the burn-in is for. This will be consistent throughout a burn-in widget's life.
	* @param	InEvaluatedConfig	The evaluated graph that was used to generate this output frame.
	*/
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void UpdateForGraph(UMovieGraphPipeline* InGraphPipeline, UMovieGraphEvaluatedConfig* InEvaluatedConfig, int32 InCameraIndex, const FString& CameraName);
};

#undef UE_API
