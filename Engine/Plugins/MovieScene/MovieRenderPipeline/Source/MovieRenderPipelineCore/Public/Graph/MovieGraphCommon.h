// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"

#include "MovieGraphCommon.generated.h"

// Note: This is a copy of the property bag's types so the implementation details of the graph
// members don't leak to the external API. Not ideal, but UHT doesn't let us alias types.
/** The type of a graph member's value. */
UENUM()
enum class EMovieGraphValueType : uint8
{
	None UMETA(Hidden),
	Bool,
	Byte,
	Int32,
	Int64,
	Float,
	Double,
	Name,
	String,
	Text,
	Enum,
	Struct,
	Object,
	SoftObject,
	Class,
	SoftClass,
	Int8,	// Type not fully supported at UI, will work with restrictions to type editing
	Int16,	// Type not fully supported at UI, will work with restrictions to type editing
	UInt16,	// Type not fully supported at UI, will work with restrictions to type editing
	UInt32,	// Type not fully supported at UI, will work with restrictions to type editing
	UInt64, // Type not fully supported at UI, will work with restrictions to type editing

	Count UMETA(Hidden)
};

// TODO: We may want a method which converts between these enum types instead
static_assert((uint8)EMovieGraphValueType::Count == (uint8)EPropertyBagPropertyType::Count);

// Note: Also a copy of the property bag's container types for the same reason as EMovieGraphValueType.
/** The container type of a graph member's value. */
UENUM()
enum class EMovieGraphContainerType : uint8
{
	None UMETA(Hidden),
	Array,
	Set,
	Map,

	Count UMETA(Hidden)
};

static_assert((uint8)EMovieGraphContainerType::Count == (uint8)EPropertyBagContainerType::Count);