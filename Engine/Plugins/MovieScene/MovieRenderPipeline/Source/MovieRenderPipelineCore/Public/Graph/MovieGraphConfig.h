// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphTraversalContext.h"
#include "MovieGraphValueContainer.h"
#include "UObject/Interface.h"

#include "MovieGraphConfig.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declare
class UMovieGraphNode;
class UMovieGraphSubgraphNode;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieGraphVariableChanged, class UMovieGraphMember*);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieGraphInputChanged, class UMovieGraphMember*);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieGraphOutputChanged, class UMovieGraphMember*);
#endif

UCLASS(MinimalAPI, Abstract)
class UMovieGraphMember : public UMovieGraphValueContainer
{
	// The graph needs to set private flags during construction time
	friend class UMovieGraphConfig;
	
	GENERATED_BODY()

public:
	UMovieGraphMember() = default;

	/** Gets the graph that owns this member, or nullptr if one was not found. */
	UE_API UMovieGraphConfig* GetOwningGraph() const;

	/** Gets the name of this member. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	FString GetMemberName() const { return Name; }

	/** Sets the name of this member. Returns true if the rename was successful, else false. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API virtual bool SetMemberName(const FString& InNewName);

	/**
	 * Determines if this member can be renamed to the specified name. If the rename is not possible, returns false
	 * and OutError is populated with the reason, else returns true.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	UE_API virtual bool CanRename(const FText& InNewName, FText& OutError) const;

	/** Gets the GUID that uniquely identifies this member. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	const FGuid& GetGuid() const { return Guid; }

	/** Sets the GUID that uniquely identifies this member. */
	void SetGuid(const FGuid& InGuid) { Guid = InGuid; }

	/** Determines if this member can be deleted. */
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	virtual bool IsDeletable() const { return true; }

	/** Gets whether this member is editable via the UI. */
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	virtual bool IsEditable() const { return bIsEditable; }
	
protected:
    /** Determines if InName is a unique name within the members in MemberArray. */
	template<class T>
    bool IsUniqueNameInMemberArray(const FText& InName, const TArray<T*>& InMemberArray) const
	{
		const FString NameString = InName.ToString();
	
		const bool bExists = InMemberArray.ContainsByPredicate([&NameString](const T* Member)
		{
			return Member->GetMemberName() == NameString;
		});

		// Check against the current name as well; this method shouldn't flag the provided name as non-unique if it's
		// the member's current name
		return !bExists || (NameString == Name);
	}

public:
	/** The optional description of this member, which is user-facing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General", meta=(EditCondition="bIsEditable", HideEditConditionToggle))
	FString Description;

protected:
	/** The name of this member, which is user-facing. */
	UPROPERTY()
	FString Name;
	
	/** A GUID that uniquely identifies this member within its graph. */
	UPROPERTY()
	FGuid Guid;

	// Note: This is a bool flag rather than a method (eg, IsEditable()) for now in order to allow it to drive the
	// EditCondition metadata on properties.
	/** Whether this member can be edited in the UI. */
	UPROPERTY()
	bool bIsEditable = true;
};

/**
 * A variable that can be used inside the graph. Most variables are created by the user, and can have their value
 * changed at the job level. Global variables, however, are not user-created and their values are provided when the
 * graph is evaluated. Overriding them at the job level is not possible.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphVariable : public UMovieGraphMember
{
	// The graph needs to set private flags during construction time
	friend class UMovieGraphConfig;
	
	GENERATED_BODY()

public:
	UMovieGraphVariable() = default;

	/** Returns true if this variable is a global variable. */
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	UE_API bool IsGlobal() const;

	/** Gets the category (if any) assigned to this variable. */
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	UE_API const FString& GetCategory() const;

	/**
	 * Sets the variable to the provided category. Be aware that the category provided here may not be the final category set on the
	 * variable (InNewCategory will be put through FName::NameToDisplayString().
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API void SetCategory(const FString& InNewCategory);

	//~ Begin UMovieGraphMember interface
	UE_API virtual bool IsDeletable() const override;
	UE_API virtual bool CanRename(const FText& InNewName, FText& OutError) const override;
	UE_API virtual bool SetMemberName(const FString& InNewName) override;
	//~ End UMovieGraphMember interface

#if WITH_EDITOR
	FOnMovieGraphVariableChanged OnMovieGraphVariableChangedDelegate;

	//~ Begin UObject overrides
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject overrides
#endif // WITH_EDITOR

private:
	/** The category assigned to the variable. Defaults to empty, which means no category. */
	UPROPERTY()
	FString Category;
};

/**
 * Similar to normal UMovieGraphVariable instances. However, their values are provided by the graph, they cannot be
 * edited/deleted, and they cannot be overridden at the job level.
 */
UCLASS(MinimalAPI, Abstract)
class UMovieGraphGlobalVariable : public UMovieGraphVariable
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphGlobalVariable();

	/** Update the internal value of the global variable. */
	virtual void UpdateValue(const FMovieGraphTraversalContext* InTraversalContext, const UMovieGraphPipeline* InPipeline) PURE_VIRTUAL(UMovieGraphGlobalVariable::UpdateValue, );

	//~ Begin UMovieGraphMember interface
	UE_API virtual bool IsDeletable() const override;
	UE_API virtual bool CanRename(const FText& InNewName, FText& OutError) const override;
	//~ End UMovieGraphMember interface
};

UCLASS()
class UMovieGraphGlobalVariable_ShotName final : public UMovieGraphGlobalVariable
{
	GENERATED_BODY()

public:
	UMovieGraphGlobalVariable_ShotName();
	virtual void UpdateValue(const FMovieGraphTraversalContext* InTraversalContext, const UMovieGraphPipeline* InPipeline) override;
};

UCLASS()
class UMovieGraphGlobalVariable_SequenceName final : public UMovieGraphGlobalVariable
{
	GENERATED_BODY()

public:
	UMovieGraphGlobalVariable_SequenceName();
	virtual void UpdateValue(const FMovieGraphTraversalContext* InTraversalContext, const UMovieGraphPipeline* InPipeline) override;
};

UCLASS()
class UMovieGraphGlobalVariable_FrameNumber final : public UMovieGraphGlobalVariable
{
	GENERATED_BODY()

public:
	UMovieGraphGlobalVariable_FrameNumber();
	virtual void UpdateValue(const FMovieGraphTraversalContext* InTraversalContext, const UMovieGraphPipeline* InPipeline) override;
};

UCLASS()
class UMovieGraphGlobalVariable_CameraName final : public UMovieGraphGlobalVariable
{
	GENERATED_BODY()

public:
	UMovieGraphGlobalVariable_CameraName();
	virtual void UpdateValue(const FMovieGraphTraversalContext* InTraversalContext, const UMovieGraphPipeline* InPipeline) override;
};

/**
 * Common base class for input/output members on the graph.
 */
UCLASS(MinimalAPI, Abstract)
class UMovieGraphInterfaceBase : public UMovieGraphMember
{
	GENERATED_BODY()

public:
	/** Whether this interface member represents a branch. If not a branch, then a value is associated with it. */
	UPROPERTY(EditAnywhere, Category = "Value", meta=(EditCondition="bIsEditable", HideEditConditionToggle))
	bool bIsBranch = true;
};

/**
 * An input exposed on the graph that will be available for nodes to connect to.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphInput : public UMovieGraphInterfaceBase
{
	GENERATED_BODY()

public:
	UMovieGraphInput() = default;

	//~ Begin UMovieGraphMember interface
	UE_API virtual bool IsDeletable() const override;
	UE_API virtual bool CanRename(const FText& InNewName, FText& OutError) const override;
	UE_API virtual bool SetMemberName(const FString& InNewName) override;
	//~ End UMovieGraphMember interface

public:
#if WITH_EDITOR
	FOnMovieGraphInputChanged OnMovieGraphInputChangedDelegate;

	//~ Begin UObject overrides
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject overrides
#endif
};

/**
 * An output exposed on the graph that will be available for nodes to connect to.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphOutput : public UMovieGraphInterfaceBase
{
	GENERATED_BODY()

public:
	UMovieGraphOutput() = default;

	//~ Begin UMovieGraphMember interface
	UE_API virtual bool IsDeletable() const override;
	UE_API virtual bool CanRename(const FText& InNewName, FText& OutError) const override;
	UE_API virtual bool SetMemberName(const FString& InNewName) override;
	//~ End UMovieGraphMember interface

public:
#if WITH_EDITOR
	FOnMovieGraphOutputChanged OnMovieGraphOutputChangedDelegate;

	//~ Begin UObject overrides
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject overrides
#endif
};

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnMovieGraphChanged);
	DECLARE_MULTICAST_DELEGATE(FOnMovieGraphVariablesChanged);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieGraphInputAdded, UMovieGraphInput*);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieGraphOutputAdded, UMovieGraphOutput*);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieGraphNodesDeleted, TArray<UMovieGraphNode*>);
#endif // WITH_EDITOR

USTRUCT()
struct FMovieGraphEvaluatedSettingsStack
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMovieGraphNode>> NodeInstances;
};

/**
* A flattened list of configuration values for a given Graph Branch. For named branches, this includes the "Globals"
* branch (for any value not also overridden by the named branch).
* 
*/
USTRUCT()
struct FMovieGraphEvaluatedBranchConfig
{
	GENERATED_BODY()

	UMovieGraphNode* GetNodeByClassExactMatch(const TSubclassOf<UMovieGraphNode>& InClass, const FString& InName) const
	{
		if (const FMovieGraphEvaluatedSettingsStack* FoundStack = NamedNodes.Find(InName))
		{
			for (const TObjectPtr<UMovieGraphNode>& Instance : FoundStack->NodeInstances)
			{
				if (Instance && Instance->GetClass() == InClass)
				{
					return Instance;
				}
			}
		}

		return nullptr;
	}

	TArray<TObjectPtr<UMovieGraphNode>> GetNodes() const
	{
		TArray<TObjectPtr<UMovieGraphNode>> AllNodeInstances;
		for (const TPair<FString, FMovieGraphEvaluatedSettingsStack>& KVP : NamedNodes)
		{
			AllNodeInstances.Append(KVP.Value.NodeInstances);
		}

		return AllNodeInstances;
	}

	/** Removes all nodes that are subclasses of the given type from the evaluated config. */
	void RemoveNodesOfType(const TSubclassOf<UMovieGraphNode>& InClass)
	{
		// Keep track of the instance names (keys) in the map to remove if all node instances under the key are removed
		TArray<FString> InstanceNamesToRemove;

		for (TTuple<FString, FMovieGraphEvaluatedSettingsStack>& SettingsPair : NamedNodes)
		{
			SettingsPair.Value.NodeInstances.RemoveAll([InClass](const TObjectPtr<UMovieGraphNode>& NodeInstance)
			{
				return NodeInstance && (NodeInstance->GetClass() == InClass);
			});

			// Remove this entry in the map if all node instances were removed
			if (SettingsPair.Value.NodeInstances.IsEmpty())
			{
				InstanceNamesToRemove.Add(SettingsPair.Key);
			}
		}

		for (const FString& KeyToRemove : InstanceNamesToRemove)
		{
			NamedNodes.Remove(KeyToRemove);
		}
	}

private:
	// Allow the config to add nodes to this, but otherwise we don't want the public adding nodes to them
	// without going through the graph resolving.
	friend class UMovieGraphConfig;
	
	/**
	* Nodes that have been evaluated in the branch. Key: the node instance name, value: the nodes that share the
	* instance name. For nodes that do not have an instance name, an empty string is the key.
	*/
	UPROPERTY(Transient)
	TMap<FString, FMovieGraphEvaluatedSettingsStack> NamedNodes;
};

// Note: This struct exists purely as a workaround for UHT throwing an error when putting a TSet in a TArray.
/** Information on visited nodes found during traversal. */
USTRUCT()
struct FMovieGraphEvaluationContext_VisitedNodeInfo
{
	GENERATED_BODY()
	
	/** The nodes that were visited during traversal. */
	UPROPERTY()
	TSet<TObjectPtr<UMovieGraphNode>> VisitedNodes;
};

/**
* This stores short-term information needed during traversal of the graph
* such as disabled nodes, already visited nodes, etc. This information is
* discarded after traversal.
*/
USTRUCT()
struct FMovieGraphEvaluationContext
{
	GENERATED_BODY()
public:
	
	/** 
	* This is the user provided traversal context which specifies high level user decisions. This is the calling
	* context such as what frame you're on, or what the shot name is, stuff generally driven by global variables.
	*/
	UPROPERTY()
	FMovieGraphTraversalContext UserContext;

	/**
	* A list of nodes that have been visited, where the key is the graph where the node was found. Used for cycle detection right now.
	*/
	UPROPERTY()
	TMap<TObjectPtr<const UMovieGraphConfig>, FMovieGraphEvaluationContext_VisitedNodeInfo> VisitedNodesByOwningGraph;

	/**
	* The pin that is currently being followed in the traversal process.
	*/
	UPROPERTY()
	TObjectPtr<UMovieGraphPin> PinBeingFollowed;

	/**
	* The current stack of subgraphs that are being visited. The last subgraph in the stack is the one currently being
	* visited. If no subgraphs are in this stack, then the parent-most graph is being traversed currently.
	*/
	UPROPERTY()
	TArray<TObjectPtr<const UMovieGraphSubgraphNode>> SubgraphStack;

	/**
	 * Whether a circular graph reference was found during traversal.
	 */
	UPROPERTY()
	bool bCircularGraphReferenceFound = false;

	/*
	 * The error that was generated during traversal. A non-empty string implies that the traversal did not complete successfully.
	 */
	UPROPERTY()
	FText TraversalError;

	/**
	 * The stack of node types (exact match) that should be removed from the graph while it is being traversed. Each node
	 * which specifies a type to removed adds to the stack.
	 */
	TArray<TSubclassOf<UMovieGraphSettingNode>> NodeTypesToRemoveStack;
};

/**
* An evaluated config for the current frame. Each named branch (including Globals) has its own
* copy of the config, fully resolved (so there is no need to check the Globals branch when
* looking at a named branch). You can use the functions to fetch a node by type from a given
* branch and it will return the right object (or the CDO if the node is NOT in the config).
*/
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphEvaluatedConfig : public UObject
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	const TArray<FName> GetBranchNames() const
	{
		TArray<FName> OutKeys;
		BranchConfigMapping.GenerateKeyArray(OutKeys);
 		return OutKeys;
	}

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UMovieGraphSettingNode* GetSettingForBranch(UClass* InClass, const FName InBranchName, bool bIncludeCDOs = true, bool bExactMatch = false) const
	{
		TArray<UMovieGraphSettingNode*> AllSettings = GetSettingsForBranch(InClass, InBranchName, bIncludeCDOs, bExactMatch);
		if (AllSettings.Num() > 0)
		{
			return AllSettings[0];
		}

		return nullptr;
	}

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	TArray<UMovieGraphSettingNode*> GetSettingsForBranch(UClass* InClass, const FName InBranchName, bool bIncludeCDOs = true, bool bExactMatch = false) const
	{
		const FMovieGraphEvaluatedBranchConfig* BranchConfig = BranchConfigMapping.Find(InBranchName);
		ensureMsgf(BranchConfig, TEXT("Failed to find branch mapping for Branch: %s"), *InBranchName.ToString());

		TArray<UMovieGraphSettingNode*> ResultNodes;
		if (BranchConfig)
		{
			// Check to see if the branch has an instance of this.
			for (const TObjectPtr<UMovieGraphNode>& Node : BranchConfig->GetNodes())
			{
				bool bMatches = bExactMatch ? Node->GetClass() == InClass : Node->IsA(InClass);
				if (bMatches)
				{
					ResultNodes.Add(Cast<UMovieGraphSettingNode>(Node.Get()));
				}
			}
		}

		// If we didn't found results above, then either they specified an invalid branch (for which the ensure tripped)
		// or the config simply didn't override that setting class, at which point we might try to return a default
		if (bIncludeCDOs && ResultNodes.Num() == 0)
		{
			ResultNodes.Add(Cast<UMovieGraphSettingNode>(InClass->GetDefaultObject()));
		}

		return ResultNodes;
	}

	/** Gets settings that implement a specific interface. InInterfaceClass should be the U-prefixed class; InterfaceType should be I-prefixed. */
	template<typename InterfaceType>
	TArray<InterfaceType*> GetSettingsImplementing(const UClass* InInterfaceClass, const FName InBranchName) const
	{
		const FMovieGraphEvaluatedBranchConfig* BranchConfig = BranchConfigMapping.Find(InBranchName);
		ensureMsgf(BranchConfig, TEXT("Failed to find branch mapping for Branch: %s"), *InBranchName.ToString());

		TArray<InterfaceType*> ResultNodes;
		if (BranchConfig)
		{
			for (const TObjectPtr<UMovieGraphNode>& Node : BranchConfig->GetNodes())
			{
				if (Node->GetClass()->ImplementsInterface(InInterfaceClass))
				{
					if (InterfaceType* CastNode = Cast<InterfaceType>(Node.Get()))
					{
						ResultNodes.Add(CastNode);
					}
				}
			}
		}

		return ResultNodes;
	}

	template<typename NodeType>
	NodeType* GetSettingForBranch(const FName InBranchName, bool bIncludeCDOs = true, bool bExactMatch = false) const
	{
		return Cast<NodeType>(GetSettingForBranch(NodeType::StaticClass(), InBranchName, bIncludeCDOs, bExactMatch));
	}

	template<typename NodeType>
	TArray<NodeType*> GetSettingsForBranch(const FName InBranchName, bool bIncludeCDOs = true, bool bExactMatch = false) const
	{
		TArray<UMovieGraphSettingNode*> UntypedResults = GetSettingsForBranch(NodeType::StaticClass(), InBranchName, bIncludeCDOs, bExactMatch);

		TArray<TObjectPtr<NodeType>> ResultNodes;
		ResultNodes.Reserve(UntypedResults.Num());
		for (UMovieGraphSettingNode* UntypedNode : UntypedResults)
		{
			ResultNodes.Add(Cast<NodeType>(UntypedNode));
		}

		return ResultNodes;
	}

	/**
	 * Gets a variable's value via a "value container". This container lets you fetch the variable's value in its native form, rather than being
	 * forced into a string (see GetVariableStringValueByName()). The variable's value is returned from the container via any of its GetValue*()
	 * methods, like GetValueFloat().
	 *
	 * Returns true if a variable with the provided name exists, else false. If the variable does not exist, the value container will be initialized
	 * to an empty one, which defaults to containing a float with value 0. The evaluated graph does not store a reference to this value container, so
	 * if used outside of blueprints, be sure to keep a reference to it in order to prevent it from being garbage collected.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph", meta=(ReturnDisplayName = "Variable Exists"))
	bool GetVariableValueByName(
		UPARAM(DisplayName = "Variable Name") const FString& InVariableName,
		UPARAM(DisplayName = "Value Container") UMovieGraphValueContainer*& OutVariableValue) const;

	/**
	 * Convenience function to get a variable's string value, instead of its value container (see GetVariableValueByName()). Uses the provided
	 * default value if a variable by the given name does not exist. For variables that aren't string-typed, this provides the serialized
	 * representation of the variable value. Returns true if a variable with the provided name exists, else false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph", meta=(ReturnDisplayName = "Variable Exists"))
	bool GetVariableStringValueByName(
		UPARAM(DisplayName = "Variable Name") const FString& InVariableName,
		UPARAM(DisplayName = "Default Value") const FString& InDefaultValue,
		UPARAM(DisplayName = "String Value") FString& OutVariableValue) const;

	/** Gets all of the graphs that were included in the evaluation process. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	const TArray<UMovieGraphConfig*>& GetEvaluatedGraphs() const;
	
public:
	/** Mapping between named branches (at the root of the config) and their evaluated values. */
	UPROPERTY(Transient)
	TMap<FName, FMovieGraphEvaluatedBranchConfig> BranchConfigMapping;

private:
	// Allow the graph traversal/evaluation process to update some private members.
	friend class UMovieGraphConfig;

	/**
	 * The fully-evaluated values of all variables used in all graphs traversed in the evaluated result. The key is the variable name,
	 * and the value is the resolved value of the variable. If there is more than one entry in the array, that means that multiple graphs
	 * used the same variable name. This is not an error, but it makes fetching a variable by name challenging without any additional context.
	 * For future use, all variables with the same name are gathered, where the first in the array is the variable from the parent-most graph,
	 * and so on down the chain of subgraphs.
	 */
	TMap<FString, TArray<UMovieGraphValueContainer*>> VariablesByName;

	/** All of the graphs that were included in the evaluation process. */
	TArray<UMovieGraphConfig*> EvaluatedGraphs;
};

UINTERFACE(MinimalAPI)
class UMovieGraphTraversableObject : public UInterface
{
	GENERATED_BODY()
};

/**
 * Provides a way for objects, which would otherwise not be mergeable during a traversal, to merge in a well-defined way.
 * Also allows objects to expose which properties have been affected by the merge.
 */
class IMovieGraphTraversableObject
{
	GENERATED_BODY()

public:
	/** Merges the contents of InSourceClass into this object. */
	virtual void Merge(const IMovieGraphTraversableObject* InSourceObject) { }

	/**
	 * Gets properties, and their associated values, which have been modified by a merge.
	 * Key = property name, value = stringified value
	 * The stringified value is a representation of the value which will usually be displayed in the UI. It does not need
	 * to be a serialized representation.
	 */
	virtual TArray<TPair<FString, FString>> GetMergedProperties() const { return {}; }
};

/**
* This is the runtime representation of the UMoviePipelineEdGraph which contains the actual strongly
* typed graph network that is read by the MoviePipeline. There is an editor-only representation of
* this graph (UMoviePipelineEdGraph).
*/
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConfig : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphConfig();

	/**
	 * Callback for when a node is visited. The node is the node being visited, and the pin is the pin which the node
	 * was accessed by (eg, if visiting downstream nodes, the pin will be the input pin that connects to the node that
	 * the traversal started from, or the node that was previously visited). Return true to continue traversal, or false
	 * to stop traversal.
	 */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FVisitNodesCallback, UMovieGraphNode*, const UMovieGraphPin*);

	//~ UObject interface
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

	/**
	* Add a connection in the graph between the given nodes and pin names. Pin name may be empty for basic
	* nodes (if no name is displayed in the UI). Can be used for either input or output pins.
	* Returns False if the pin could not be found, or the connection could not be made (type mismatches).
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool AddLabeledEdge(UMovieGraphNode* FromNode, const FName& FromPinLabel, UMovieGraphNode* ToNode, const FName& ToPinLabel);

	/**
	* Like AddLabeledEdge, removes the given connection between Node A and Node B (for the specified pins by name).
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool RemoveLabeledEdge(UMovieGraphNode* FromNode, const FName& FromPinName, UMovieGraphNode* ToNode, const FName& ToPinName);
	
	/**
	* Convenience function which removes all Inbound (pins on the left side of a node) edges for the given node.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool RemoveAllInboundEdges(UMovieGraphNode* InNode);
	
	/**
	* Convenience function which removes all Outbound (pins on the right side of a node) edges for the given node.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool RemoveAllOutboundEdges(UMovieGraphNode* InNode);

	/**
	* Convenience function which removes all Inbound (pins on the left side of a node) edges connected to the given inbound pin by name, for the given node.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool RemoveInboundEdges(UMovieGraphNode* InNode, const FName& InPinName);
	
	/**
	* Convenience function which removes all Outbound (pins on the right side of a node) edges connected to the given outbound pin by name, for the given node.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool RemoveOutboundEdges(UMovieGraphNode* InNode, const FName& InPinName);

	/** 
	 * Creates a new node of the given class and inserts it immediately upstream (to the left) of InTargetNode
	 * on the specified input pin (defaults to the unnamed pin). Any node previously connected to that input
	 * is reconnected upstream of the new node. Returns the newly created node, or nullptr on failure.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API UMovieGraphNode* InsertBefore(UMovieGraphNode* InTargetNode, const TSubclassOf<UMovieGraphNode> InNodeClass, const FName& InTargetPinLabel = FName());

	/**
	 * Creates a new node of the given class and inserts it immediately downstream (to the right) of InTargetNode
	 * on the specified output pin (defaults to the unnamed pin). The first downstream connection on that pin is
	 * reconnected downstream of the new node; if the pin has multiple connections, the caller should manage
	 * additional reconnections manually. Returns the newly created node, or nullptr on failure.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API UMovieGraphNode* InsertAfter(UMovieGraphNode* InTargetNode, const TSubclassOf<UMovieGraphNode> InNodeClass, const FName& InTargetPinLabel = FName());

	/**
	* Add the specified node instance to the graph. This will rename the node to ensure the graph is the outer
	* and then it will add it to the internal list of nodes used by the graph. See ConstructRuntimeNode if you
	* want to construct a node by class and don't already have an instance.
	* 
	* Not currently exposed to the Blueprint API as it's generally internal use only.
	*/
	UE_API void AddNode(UMovieGraphNode* InNode);

	/** Removes the specified node from the graph, disconnecting connected edges as it goes. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool RemoveNode(UMovieGraphNode* InNode);

	/** Like RemoveNode but takes an entire array at once for convenience. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool RemoveNodes(TArray<UMovieGraphNode*> InNodes);

	/** Gets the automatically generated "Inputs" node in the Graph. */
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	UMovieGraphNode* GetInputNode() const { return InputNode; }

	/** Gets the automatically generated "Outputs" node in the Graph. */
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	UMovieGraphNode* GetOutputNode() const { return OutputNode; }

	const TArray<TObjectPtr<UMovieGraphNode>>& GetNodes() const { return AllNodes; }

	/** Returns an array of the branch names for the OutputNode on this Graph. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API TArray<FName> GetBranchNames() const;

	/**
	* Finds a node (by type) for the given branch name (see GetBranchNames()). Returns the first result
	* of that type found, even if there are multiple, following traversal order (ie: right to left).
	* To prevent accidentally editing unrelated assets, does not dive into sub-graphs (but can continue
	* traversal beyond them), so returned results should only exist in current asset.
	* Does not contain the Input or Output nodes, see GetInputNode() and GetOutputNode().
	*
	* @param InClass - The node class type to search for. Use UMovieGraphSettingNode to only include nodes that show up in the final evaluated graph (excludes nodes like the Branch node).
	* @param InBranchName - The branch to search for these nodes on.
	* @param bExactMatch - Defaults false. If true, will not consider inherited classes of InClass.
	* 					   (ie: If requesting a base class and only child classes exist, they will not count when using exact matching).
	* @return First node found (if any) which are of the specified InClass type on the given branch.
	*/
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "InClass"), Category = "Movie Graph")
	UE_API UMovieGraphNode* GetNodeForBranch(UClass* InClass, const FName& InBranchName, bool bExactMatch = false) const;

	/**
	* Finds all nodes (by type) for the given branch name (see GetBranchNames()). Returns all results of that type
	* following traversal order (ie: right to left).
	* To prevent accidentally editing unrelated assets, does not dive into sub-graphs (but can continue
	* traversal beyond them), so returned results should only exist in current asset.
	* Does not contain the Input or Output nodes, see GetInputNode() and GetOutputNode().
	*
	* @param InClass - The node class type to search for. Use UMovieGraphSettingNode to only include nodes that show up in the final evaluated graph (excludes nodes like the Branch node).
	* @param InBranchName - The branch to search for these nodes on.
	* @param bExactMatch - Defaults false. If true, will not consider inherited classes of InClass.
	* 					   (ie: If requesting a base class and only child classes exist, they will not count when using exact matching).
	* @return All nodes found (if any) which are of the specified InClass type on the given branch.
	*/
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "InClass"), Category = "Movie Graph")
	UE_API TArray<UMovieGraphNode*> GetNodesForBranch(UClass* InClass, const FName& InBranchName, bool bExactMatch = false) const;

	/**
	* Finds a node by ScriptTag field. THIS SEARCH IS CASE SENSITIVE. Returns the first result
	* of that tag found, even if there are multiple, following traversal order (ie: right to left).
	* If OptionalClassFilter is specified, only matches against nodes that have the right class type and
	* contain the correct tag.
	* To prevent accidentally editing unrelated assets, does not dive into sub-graphs (but can continue
	* traversal beyond them), so returned results should only exist in current asset.
	* Does not contain the Input or Output nodes, see GetInputNode() and GetOutputNode().
	*
	* @param ScriptTag - Case sensitive name into the "Script Tags" property on Nodes
	* @param OptionalClassFilter - Defaults None. If specified, will only return a node of this class that matches the tag. Use UMovieGraphSettingNode to only include nodes
	*							   that show up in the final evaluated graph (excludes nodes like the Branch node).
	* @param OptionalBranchFilter - Defaults None. If specified, will only search the specified branch, otherwise searches all branches.
	* @param bExactMatch - Defaults false. If true, will not consider inherited classes to pass the OptionalClassFilter.
	* 					   (ie: If requesting a base class and only child classes exist, they will not count when using exact matching).
	* @return The first node found (if any) which contains the case-sensitive tag and is of the correct class type (if using OptionalClassFilter)
	*/
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "OptionalClassFilter"), Category = "Movie Graph")
	UE_API UMovieGraphNode* GetNodeForTag(const FString& ScriptTag, UClass* OptionalClassFilter = nullptr, const FName OptionalBranchFilter = NAME_None, bool bExactMatch = false) const;

	/**
	* Finds nodes by ScriptTag field. THIS SEARCH IS CASE SENSITIVE. Returns all results
	* of that tag found, following traversal order (ie: right to left).
	* If OptionalClassFilter is specified, only matches against nodes that have the right class type and
	* contain the correct tag.
	* To prevent accidentally editing unrelated assets, does not dive into sub-graphs (but can continue
	* traversal beyond them), so returned results should only exist in current asset.
	* Does not contain the Input or Output nodes, see GetInputNode(), GetOutputNode())
	*
	* @param ScriptTag - Case sensitive name into the "Script Tags" property on Nodes
	* @param OptionalClassFilter - Defaults None. If specified, will only return a node of this class that matches the tag. Use UMovieGraphSettingNode to only include nodes
	*							   that show up in the final evaluated graph (excludes nodes like the Branch node).
	* @param OptionalBranchFilter - Defaults None. If specified, will only search the specified branch, otherwise searches all branches.
	* @param bExactMatch - Defaults false. If true, will not consider inherited classes to pass the OptionalClassFilter.
	* 					   (ie: If requesting a base class and only child classes exist, they will not count when using exact matching).
	* @return All nodes found (if any) which contain the case-sensitive tag and are of the correct class type (if using OptionalClassFilter)
	*/
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "OptionalClassFilter"), Category = "Movie Graph")
	UE_API TArray<UMovieGraphNode*> GetNodesForTag(const FString& ScriptTag, UClass* OptionalClassFilter = nullptr, const FName OptionalBranchFilter = NAME_None, bool bExactMatch = false) const;

	/**
	 * Adds a new variable member with default values to the graph. The new variable will have a base name of
	 * "Variable" unless specified in InCustomBaseName. Returns the new variable on success, else nullptr.
	 */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API UMovieGraphVariable* AddVariable(const FName InCustomBaseName = NAME_None);

	/**
	 * Adds a new input member to the graph. Returns the new input on success, else nullptr.
	 *
	 * The default name of the input is "Input". Optionally, InBaseName can be specified to add the input with a specific name. If the name "Input"
	 * (or the custom InBaseName) isn't available, a numerical suffix will be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API UMovieGraphInput* AddInput(const FText& InBaseName = FText::GetEmpty());

	/**
	 * Adds a new output member to the graph. Returns the new output on success, else nullptr.
	 *
	 * The default name of the output is "Output". Optionally, InBaseName can be specified to add the output with a specific name. If the name "Output"
	 * (or the custom InBaseName) isn't available, a numerical suffix will be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API UMovieGraphOutput* AddOutput(const FText& InBaseName = FText::GetEmpty());

	/** Gets the variable in the graph with the specified GUID, else nullptr if one could not be found. */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API UMovieGraphVariable* GetVariableByGuid(const FGuid& InGuid) const;
	
	/** Gets the variable in the graph with the specified name (including global variables), else nullptr if one could not be found. */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API UMovieGraphVariable* GetVariableByName(const FString& InVariableName) const;

	/**
	 * Gets all variables that are available to be used in the graph. Global variables can optionally be included if
	 * bIncludeGlobal is set to true.
	 */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API TArray<UMovieGraphVariable*> GetVariables(const bool bIncludeGlobal = false) const;

	/** Updates the values of all global variables. */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API void UpdateGlobalVariableValues(const UMovieGraphPipeline* InPipeline);

	/** Gets all inputs that have been defined on the graph. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API TArray<UMovieGraphInput*> GetInputs() const;

	/** Gets all outputs that have been defined on the graph. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API TArray<UMovieGraphOutput*> GetOutputs() const;

	/** Remove the specified member (input, output, variable) from the graph. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool DeleteMember(UMovieGraphMember* MemberToDelete);

	/** Duplicates the provided variable. Returns the new variable on success, else nullptr. */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API UMovieGraphVariable* DuplicateVariable(UMovieGraphVariable* InVariableToDuplicate);

#if WITH_EDITOR
	/** Gets the editor-only nodes in this graph. Editor-only nodes do not have an equivalent runtime node. */
	const TArray<TObjectPtr<UObject>>& GetEditorOnlyNodes() const { return EditorOnlyNodes; }

	/** Sets the editor-only nodes in this graph. */
	UE_API void SetEditorOnlyNodes(const TArray<TObjectPtr<const UObject>>& InNodes);
#endif

	/**
	 * Given a user-defined evaluation context, evaluate the graph and build a "flattened" list of settings for each branch discovered.
	 * If there was an error while evaluating the graph, nullptr will be returned and OutError will be populated with a description of the problem.
	 */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API UMovieGraphEvaluatedConfig* CreateFlattenedGraph(const FMovieGraphTraversalContext& InContext, FString& OutError);

	/** Given a class and FProperty that belongs to that class, search for a FBoolProperty that matches the name "bOverride_<name of InRealProperty>. */
	static UE_API FBoolProperty* FindOverridePropertyForRealProperty(UClass* InClass, const FProperty* InRealProperty);

	/**
	 * Visits all nodes upstream from FromNode, running VisitCallback on each one. Note this only follows branch connections,
	 * and does not recurse into subgraphs.
	 */
	UE_API void VisitUpstreamNodes(UMovieGraphNode* FromNode, const FVisitNodesCallback& VisitCallback) const;

	/**
	 * Visits all nodes downstream from FromNode, running VisitCallback on each one. Note this only follows branch connections,
	 * and does not recurse into subgraphs.
	 */
	UE_API void VisitDownstreamNodes(UMovieGraphNode* FromNode, const FVisitNodesCallback& VisitCallback) const;

	/**
	 * Determines the name(s) of the branches downstream from FromNode, starting at FromPin. Optionally, subgraph nodes can halt graph traversal
	 * if bStopAtSubgraph is set to true.
	 */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API TArray<FString> GetDownstreamBranchNames(UMovieGraphNode* FromNode, const UMovieGraphPin* FromPin, const bool bStopAtSubgraph = false) const;

	/**
	 * Determines the name(s) of the branches upstream from FromNode, starting at FromPin. Optionally, subgraph nodes can halt graph traversal
	 * if bStopAtSubgraph is set to true.
	 */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API TArray<FString> GetUpstreamBranchNames(UMovieGraphNode* FromNode, const UMovieGraphPin* FromPin, const bool bStopAtSubgraph = false) const;

	/** Get all subgraphs that this graph contains, recursively (ie, subgraphs of subgraphs are included, etc). */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API void GetAllContainedSubgraphs(TSet<UMovieGraphConfig*>& OutSubgraphs) const;

	/**
	 * Walks the graph backward recursively from the output node searching for a UMovieGraphOutputSettings node. Traverses subgraphs as well.
	 * If a node is not found with an override set, value is taken from the CDO of UMovieGraphOutputSettings.
	 */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API void GetOutputDirectory(FString& OutOutputDirectory) const;

	/**
	 * Moves one variable (InTargetVariable) before another variable (InBeforeVariable). Takes care of ensuring the variable's category is
	 * set properly after the move.
	 */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API void MoveVariableBefore(UMovieGraphVariable* InTargetVariable, UMovieGraphVariable* InBeforeVariable);

	/**
	 * Moves one variable (InTargetVariable) to the specified index among all user graph variables.
	 *
	 * Note that MoveVariableBefore() should be used in almost all cases unless there is very specific use case. This method will not take care of setting
	 * the category for you after the move unlike MoveVariableBefore().
	 */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API void MoveVariableToIndex(UMovieGraphVariable* InTargetVariable, int32 NewIndex);

	/** Moves one category (InCategoryToMove) and its variables before another category (InCategoryBefore). */
	UFUNCTION(BlueprintCallable, Category="Movie Graph")
	UE_API void MoveCategoryBefore(const FString& InCategoryToMove, const FString& InCategoryBefore);

protected:
	/** Look for the output directory in the UMovieGraphOutputSettings nodes found upstream of InNode. */
	UE_API void RecurseUpGlobalsBranchToFindOutputDirectory(const UMovieGraphNode* InNode, FString& OutOutputDirectory, TArray<const UMovieGraphConfig*>& VisitedGraphStack) const;
	
	/** Copies properties in FromNode that are marked for override into ToNode, but only if ToNode doesn't already override that value. */
	UE_API void CopyOverriddenProperties(UMovieGraphNode* FromNode, UMovieGraphNode* ToNode, const FMovieGraphEvaluationContext& InEvaluationContext);
	
	/** Find all "Overrideable" marked properties, then find their edit condition properties, then set those to false. */
	UE_API void InitializeFlattenedNode(UMovieGraphNode* InNode);

	/**
	 * Traverse the graph, generating a combined "flatten" graph as it goes. Returns false if there was an issue (and the evaluation context will be
	 * updated with more details regarding the failure).
	 */
	UE_API bool CreateFlattenedGraph_Recursive(UMovieGraphEvaluatedConfig* InOwningConfig, FMovieGraphEvaluatedBranchConfig& OutBranchConfig,
		FMovieGraphEvaluationContext& InEvaluationContext, UMovieGraphPin* InPinToFollow);

	/** Recursive helper for VisitUpstreamNodes(). */
	UE_API void VisitUpstreamNodes_Recursive(UMovieGraphNode* FromNode, const FVisitNodesCallback& VisitCallback, TSet<UMovieGraphNode*>& VisitedNodes) const;

	/** Recursive helper for VisitDownstreamNodes(). */
	UE_API void VisitDownstreamNodes_Recursive(UMovieGraphNode* FromNode, const FVisitNodesCallback& VisitCallback, TSet<UMovieGraphNode*>& VisitedNodes) const;

public:
#if WITH_EDITOR
	FOnMovieGraphChanged OnGraphChangedDelegate;
	FOnMovieGraphVariablesChanged OnGraphVariablesChangedDelegate;
	FOnMovieGraphInputAdded OnGraphInputAddedDelegate;
	FOnMovieGraphOutputAdded OnGraphOutputAddedDelegate;
	FOnMovieGraphNodesDeleted OnGraphNodesDeletedDelegate;
#endif

protected:	
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphNode>> AllNodes;

	UPROPERTY()
	TObjectPtr<UMovieGraphNode> InputNode;

	UPROPERTY()
	TObjectPtr<UMovieGraphNode> OutputNode;
public:
#if WITH_EDITORONLY_DATA
	// Not strongly typed to avoid a circular dependency between the editor only module
	// and the runtime module, but it should be a UMoviePipelineEdGraph.
	//
	// Note that the editor graph is saved with the runtime graph. This is done to prevent the runtime graph from being dirtied immediately upon loading
	// (because the editor graph would have to be re-created from the runtime graph, thus dirtying the package).
	//
	// If the runtime graph representation is modified outside of the editor (eg, programmatically via the UMovieGraphConfig API
	// directly), this should be nulled out so the editor can rebuild its representation the next time it is loaded. If the editor representation
	// of the graph isn't important (ie, the graph is transient) then this is unnecessary.
	UPROPERTY()
	TObjectPtr<UEdGraph> PipelineEdGraph;
#endif

	/**
	* Creates the given node type in this graph. Does not create any connections, and a node will not be
	* considered during evaluation unless it is connected to other nodes in the graph.
	*/
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "InClass"), Category = "Movie Graph")
	UMovieGraphNode* CreateNodeByClass(const TSubclassOf<UMovieGraphNode> InClass)
	{
		if (!InClass)
		{
			FFrame::KismetExecutionMessage(
				*FString::Printf(
					TEXT("%hs: Invalid PipelineGraphNodeClass. Please specify a valid class."), __FUNCTION__),
				ELogVerbosity::Error);

			return nullptr;
		}

		// Construct a new object with ourselves as the outer, then keep track of it.
		UMovieGraphNode* RuntimeNode = NewObject<UMovieGraphNode>(this, InClass, NAME_None, RF_Transactional);
		RuntimeNode->UpdateDynamicProperties();
		RuntimeNode->UpdatePins();
		RuntimeNode->Guid = FGuid::NewGuid();

		AddNode(RuntimeNode);
		return RuntimeNode;
	}

	template<class T>
	T* ConstructRuntimeNode(TSubclassOf<UMovieGraphNode> PipelineGraphNodeClass = T::StaticClass())
	{
		return Cast<T>(CreateNodeByClass(PipelineGraphNodeClass));
	}

private:
	/** Remove the specified variable member from the graph. */
	UE_API bool DeleteVariableMember(UMovieGraphVariable* VariableMemberToDelete);
	
	/** Remove the specified input member from the graph. */
	UE_API bool DeleteInputMember(UMovieGraphInput* InputMemberToDelete);

	/** Remove the specified output member from the graph. */
	UE_API bool DeleteOutputMember(UMovieGraphOutput* OutputMemberToDelete);
	
	/**
	 * Add a new member of type RetType to MemberArray (ArrType, which RetType must derive from), with a unique name
	 * that includes BaseName in it.
	 */
	template<typename RetType, typename ArrType>
	RetType* AddMember(TArray<TObjectPtr<ArrType>>& InMemberArray, const FName& InBaseName);

	/** Adds a global variable of type T to the graph. */
	template<typename T>
	T* AddGlobalVariable();

	/** Adds members to the graph that should always be available. */
	UE_API void AddDefaultMembers();

private:
	/** All user (not global) variables which are available for use in the graph. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphVariable>> Variables;

	/** All global variables which are available for use in the graph. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphGlobalVariable>> GlobalVariables;

	/** All inputs which have been defined on the graph. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphInput>> Inputs;

	/** All outputs which have been defined on the graph. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphOutput>> Outputs;

#if WITH_EDITORONLY_DATA
	/** Nodes which are only useful in the editor (like comments) and have no runtime equivalent */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> EditorOnlyNodes;
#endif
};

#undef UE_API
