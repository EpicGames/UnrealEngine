// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphConfig.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "UObject/StrongObjectPtr.h"

#include "MovieGraphCoreTimeStep.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
class UMovieGraphEngineTimeStep;

/**
 * Provides common logic for typical time-step functionality.
 * 
 * The number of temporal sub-samples is read from the graph for each output frame, and we then take the time the
 * shutter is open and break it into that many sub-samples. Subclasses must implement GetNextTemporalRangeIndex() to
 * indicate the index of the next temporal sub-sample. 
 */
UCLASS(MinimalAPI, Abstract)
class UMovieGraphCoreTimeStep : public UMovieGraphTimeStepBase
{
	GENERATED_BODY()
public:
	UE_API UMovieGraphCoreTimeStep();

	// UMovieGraphTimeStepBase Interface
	UE_API virtual void Initialize() override;
	UE_API virtual void Shutdown() override;
	UE_API virtual void TickProducingFrames() override;
	virtual FMovieGraphTimeStepData GetCalculatedTimeData() const override { return CurrentTimeStepData; }
	UE_API virtual bool IsExpansionForTSRequired(const TObjectPtr<UMovieGraphEvaluatedConfig>& InConfig) const override;
	// ~UMovieGraphTimeStepBase Interface

protected:
	/** Update CurrentFrameData.RangeShutterOpen and CurrentFrameData.RangeShutterClosed. */
	UE_API virtual void UpdateShutterRanges();
	
	/** Update CurrentFrameData.TemporalRanges. */
	UE_API virtual void UpdateTemporalRanges();

	/**
	 * Gets the index of the next temporal range. The index returned should not be the index returned by the prior call
	 * (this would result in a frame delta time of zero, causing the engine to crash).
	 */
	virtual int32 GetNextTemporalRangeIndex() const PURE_VIRTUAL(UMovieGraphCoreTimeStep::GetNextTemporalRangeIndex, return 0; );

	/** Determines if the current sample being rendered is the last. */
	UE_API virtual bool IsLastTemporalSample() const;

	/** Gets the number of samples that should be used in each frame. */
	virtual int32 GetTemporalSampleCount() const PURE_VIRTUAL(UMovieGraphCoreTimeStep::UpdateTemporalSampleCount, return 0; ); 
	
	UE_API virtual void UpdateFrameMetrics();
	UE_API virtual bool IsFirstTemporalSample() const;
	UE_API virtual void ResetForEndOfOutputFrame();
	UE_API virtual float GetBlendedMotionBlurAmount();
	UE_API virtual int32 GetTemporalSampleCountFromConfig(UMovieGraphEvaluatedConfig* InConfig) const;

protected:
	/** This is the output data needed by the rest of MRQ to produce a frame. */
	UPROPERTY(Transient)
	FMovieGraphTimeStepData CurrentTimeStepData;

	struct FOutputFrameMetrics
	{
		FOutputFrameMetrics()
			: MotionBlurAmount(0.f)
			, TimeDilation(1.f)
		{}

		/** The human readable frame rate, adjusted by the config file (24fps, 30fps, etc.) */
		FFrameRate FrameRate;

		/** The internal resolution data is actually stored at. (24,000, 120,000 etc.) */
		FFrameRate TickResolution;

		/** The amount of time (in Tick Resolution) that represents one output frame (ie: 1/24). */
		FFrameTime FrameTimePerOutputFrame;

		/** The 0-1 Motion Blur amount the camera has. */
		float MotionBlurAmount;
		
		/** The time dilation multiplier set by the world, 1.0 if there is no time dilation being applied. */
		float TimeDilation;

		/** The amount of time (in Tick Resolution) that the shutter is open for. Could be <= FrameTimePerOutputFrame. */
		FFrameTime FrameTimeWhileShutterOpen;

		/** 
		* The amount of time (in Tick Resolution) per temporal sample. The actual time slice for a temporal sub-sample
		* may be different than this (as we have to jump over the time period that the shutter is closed too for one 
		* temporal sub-sample.
		*/
		FFrameTime FrameTimePerTemporalSample;
		
		/** The inverse of FrameTimeWhileShutterOpen. ShutterClosed+ShutterOpen should add up to FrameTimePerOutputFrame. */
		FFrameTime FrameTimeWhileShutterClosed;

		/** A constant offset applied to the final evaluated time to allow users to influence what counts as a frame. (the time before, during, or after a frame) */
		FFrameTime ShutterOffsetFrameTime;

		/** An offset to the evaluated time to get us into the center of the time period (since motion blur is bi-directional and centered) */
		FFrameTime MotionBlurCenteringOffsetTime;
	};

	struct FCurrentFrameData
	{
		FCurrentFrameData()
			: TemporalSampleIndex(0)
			, TemporalSampleCount(0)
		{
		}

		/** Which temporal sub-sample are we working on? [0,TemporalSampleCount) */
		int32 TemporalSampleIndex;
		
		/** How many temporal sub-samples are there for the current output frame? */
		int32 TemporalSampleCount;
		
		/** A range of time (in Tick Resolution) for the last output frame being worked on. Updated before first TS of next frame. */
		TRange<FFrameTime> LastOutputFrameRange;

		/** A range of time (in Tick Resolution) for the current frame being worked on. Represents the whole output frame (shutter closed + open). */
		TRange<FFrameTime> CurrentOutputFrameRange;
		
		/** A range of time (in Tick Resolution) that the shutter is opened. */
		TRange<FFrameTime> RangeShutterOpen;
		
		/** A range of time (in Tick Resolution) that the shutter is closed. Can be an empty range (when using MotionBlurAmount=1.0) */
		TRange<FFrameTime> RangeShutterClosed;

		/**
		* A range of time (in Tick Resolution) for the last temporal sub-sample (ie: what the last tick rendered with). We don't track
		* CurrentSampleRange as it's just TemporalRanges[TemporalSampleIndex].
		*/
		TRange<FFrameTime> LastSampleRange;

		/** A pre-calculated (when the output frame starts) range of time, breaking RangeShutterOpen into smaller chunks, one per temporal sub-sample. */
		TArray<TRange<FFrameTime>> TemporalRanges;

		/** Updated at the start of each Temporal Sample. */
		TStrongObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig;
	};

	/**
	* A set of cached values that are true for the current output frame. They will be recalculated
	* at the start of the next output frame.
	*/
	FOutputFrameMetrics CurrentFrameMetrics;

	/**
	* Frame Metrics are constant data updated once per output frame, while CurrentFrameData is keeping track
	* across engine ticks to work towards producing a single frame.
	*/
	FCurrentFrameData CurrentFrameData;

	/**
	* Was the world allowing time dilation before this render started? We'll cache and then restore this value to
	* avoid changing any user settings.
	*/
	bool bPrevWorldAllowTimeDilation = true;
};

#undef UE_API
