// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/MovieGraphTraversalContext.h"
#include "Graph/MovieGraphRenderDataIdentifier.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"
#include "Templates/SubclassOf.h"
#include "ImagePixelData.h"
#include "Containers/Queue.h"
#include "Containers/Set.h"
#include "Misc/FrameRate.h"
#include "MovieRenderPipelineDataTypes.h"
#include "Engine/EngineCustomTimeStep.h"

#include "MovieGraphDataTypes.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
class UMovieGraphEvaluatedConfig;
class UMovieGraphFileOutputNode;
class UMovieGraphPipeline;
class UMovieGraphTimeStepBase;
class UMovieGraphRendererBase;
class UMovieGraphTimeRangeBuilderBase;
class UMovieGraphDataCachingBase;
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;
struct FImagePixelData;

namespace UE::MovieGraph
{
	struct FRenderTimeStatistics;
}

/** Convenience type for passing around pass data plus its identifier. */
typedef TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>> FMovieGraphPassData;

/**
* A message that a render pass can overlay on its preview image in the Render Preview window.
*/
USTRUCT(BlueprintType)
struct FMovieGraphPreviewBannerMessage
{
	GENERATED_BODY()

	/** The message text to display over the preview tile. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	FText Message;

	/** The color used by the message text. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	FLinearColor TextColor = FLinearColor::White;

	/** The background color of the banner behind the message text. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	FLinearColor BackgroundColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);
};

USTRUCT(BlueprintType)
struct FMovieGraphImagePreviewData
{
	GENERATED_BODY()

	FMovieGraphImagePreviewData()
	: Texture(nullptr)
	{}

	/** The texture this preview image was rendered to. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	TObjectPtr<class UTexture> Texture;
	
	/** The identifier for the image, containing the branch name, renderer, etc. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	FMovieGraphRenderDataIdentifier Identifier;

	/** If true, then there is more than one camera name being used (ie: multi-camera rendering) */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	bool bMultipleCameraNames = false;

	/** Optional banner messages contributed by the render pass that produced this tile. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	TArray<FMovieGraphPreviewBannerMessage> BannerMessages;
};

USTRUCT(BlueprintType)
struct FMovieGraphInitConfig
{
	GENERATED_BODY()
	
	UE_API FMovieGraphInitConfig();
	
	/**
	* Which class should the UMovieGraphPipeline use to look for render layers and
	* request renders from. Defaults to UMovieGraphDefaultRenderer.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TSubclassOf<UMovieGraphRendererBase> RendererClass;

	/**
	* Which class should the UMovieGraphPipeline use to build time ranges from, and
	* during evaluation, send callbacks about the time actually evaluated so you
	* can sync with an external source. Defaults to UMovieGraphSequenceDataSource.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TSubclassOf<UMovieGraphDataSourceBase> DataSourceClass;

	/**
	 * Which class should the UMovieGraphPipeline use to generate audio. Defaults to
	 * UMovieGraphDefaultAudioOutput.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TSubclassOf<UMovieGraphAudioRendererBase> AudioRendererClass;

	/**
	* Should the UMovieGraphPipeline render the full player viewport? Defaults
	* to false (so no 3d content is rendered) so we can display the UMG widgets
	* and MRQ rendering always happens in an off-screen render target.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	bool bRenderViewport;
};

UCLASS(MinimalAPI, BlueprintType, Abstract)
class UMovieGraphTimeStepBase : public UObject
{
	GENERATED_BODY()
public:
	/** Called each frame while the Movie Graph Pipeline is in a producing frames state. */
	virtual void TickProducingFrames() {}

	/** Called when this time step instance becomes active (ie: at the start of a shot). */
	virtual void Initialize() {}
	/** Called when this time step instance is no longer active (ie: at the end of a shot). */
	virtual void Shutdown() {}

	/** 
	* TickProducingFrames will be called for a frame (before the frame starts) and then this will be
	* called at the end of the frame when we kick off the renders for the frame. Should return data
	* needed to calculate the correct rendering timestep.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual FMovieGraphTimeStepData GetCalculatedTimeData() const { return FMovieGraphTimeStepData(); }
	
	/**
	* When expanding shots, do we need to expand the first frame by one extra to account for how
	* temporal sub-sampling can sample outside the first frame (due to centered frame evals).
	*/
	virtual bool IsExpansionForTSRequired(const TObjectPtr<UMovieGraphEvaluatedConfig>& InConfig) const { return false; }

	UE_API UMovieGraphPipeline* GetOwningGraph() const;
};


UCLASS(MinimalAPI)
class UMovieGraphEngineTimeStep : public UEngineCustomTimeStep
{
	GENERATED_BODY()
public:
	UE_API UMovieGraphEngineTimeStep();

	struct FTimeStepCache
	{
		FTimeStepCache()
			: UndilatedDeltaTime(0.0)
		{}

		FTimeStepCache(double InUndilatedDeltaTime)
			: UndilatedDeltaTime(InUndilatedDeltaTime)
		{}

		double UndilatedDeltaTime;
	};

	struct FSharedData
	{
		FSharedData()
			: OutputFrameNumber(0)
			, RenderedFrameNumber(0)
			, bHasTimeDilationBeenUsed(false)
		{}
		
		
		/** Which output frame are we working on, relative to zero.*/
		int32 OutputFrameNumber;

		/** 
		* Index of which frames we've submitted for rendering. Doesn't line up with OutputFrameNumber when using warm-up frames.
		* Used internally by the rendering engine to keep track of which frames need to be read back.
		*/
		int32 RenderedFrameNumber;

		/**
		* If true, a frame rendered for this job used time dilation, and all subsequent frame numbers must be their relative versions
		* to avoid any frame overwriting an already produced one.
		*/
		bool bHasTimeDilationBeenUsed;
	};

public:
	UE_API void SetCachedFrameTiming(const FTimeStepCache& InTimeCache);

	// UEngineCustomTimeStep Interface
	UE_API virtual bool Initialize(UEngine* InEngine) override;
	UE_API virtual void Shutdown(UEngine* InEngine) override;
	UE_API virtual bool UpdateTimeStep(UEngine* InEngine) override;
	virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const override { return ECustomTimeStepSynchronizationState::Synchronized; }
	// ~UEngineCustomTimeStep Interface

	/** We don't do any thinking on our own, instead we just spit out the numbers stored in our time cache. */
	FTimeStepCache TimeCache;

	/** Data that should be shared between all shot time step instances. */
	FSharedData SharedTimeStepData;

	// Not cached in TimeCache as TimeCache is reset every frame.
	float PrevMinUndilatedFrameTime;
	float PrevMaxUndilatedFrameTime;
};

namespace UE::MovieGraph
{
	/** 
	* When fetching multiple cameras from the Data Source, we need to know both
	* the rendering information (ViewInfo), but also who the renderering object
	* is, which isn't stored in the ViewInfo.
	*/
	struct FMinimalCameraInfo
	{
		FMinimalCameraInfo()
		: ViewActor(nullptr)
		{}
		
		TWeakObjectPtr<AActor> ViewActor;
		FMinimalViewInfo ViewInfo;
	};
}

UCLASS(MinimalAPI, BlueprintType, Abstract)
class UMovieGraphRendererBase : public UObject
{
	GENERATED_BODY()
public:

	
	/** Get an array of image previews that are valid this frame. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual TArray<FMovieGraphImagePreviewData> GetPreviewData() const { return TArray<FMovieGraphImagePreviewData>(); }
		
	virtual void Render(const FMovieGraphTimeStepData& InTimeData) {}
	virtual void SetupRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot) {}
	virtual void TeardownRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot) {}
	virtual UE::MovieGraph::FRenderTimeStatistics* GetRenderTimeStatistics(const int32 InFrameNumber) { return nullptr; }

	UE_DEPRECATED(5.5, "Use the version that doesn't take a config instead. The information from the config is now derived from the index value.")
	virtual UE::MovieGraph::FMinimalCameraInfo GetMinimalCameraInfo(UMovieGraphEvaluatedConfig* InConfig, const int32 InCameraIndex) const { return GetMinimalCameraInfo(InCameraIndex); }
	
	/** InCameraIndex can be -1 (for primary camera when not using multi-layer rendering) or 0...n for sidecar cameras. */
	virtual UE::MovieGraph::FMinimalCameraInfo GetMinimalCameraInfo(const int32 InCameraIndex) const { return UE::MovieGraph::FMinimalCameraInfo(); }

	UE_API UMovieGraphPipeline* GetOwningGraph() const;
};

/**
* Movie Graph Pipeline is mostly interested in knowing about ranges of time that
* it should render, and less concerned with the specifics of where that data comes
* from (ie: a Level Sequence). This lets you synchronize with a different data source
* to provide the ranges of time to render, and then the UMovieGraphTimeStepBase class
* figures out how to move around within that time step, before calling some functions
* to synchronize your external data source to actually match the evaluated time.
*/
UCLASS(MinimalAPI, BlueprintType, Abstract)
class UMovieGraphDataSourceBase : public UObject
{
	GENERATED_BODY()
public:
	/** An internal, high resolution framerate that seeks, etc. will be returned in. (ie: 24,000/1) */
	virtual FFrameRate GetTickResolution() const { return FFrameRate(); }

	/** A lower res, human readable Frame Rate. We convert to the Tick Resolution internally. (ie: 24/1) */
	virtual FFrameRate GetDisplayRate() const { return FFrameRate(); }

	/** Called by the Time Step system when it wants the external data source to update. Time is in TickResolution scale. */
	virtual void SyncDataSourceTime(const FFrameTime& InTime) {}
	
	/** Called by the Time Step system when the external data source should start playback (time values will have been set by SyncDataSourceTime */
	virtual void PlayDataSource() {}

	/** Called by the Time Step system when the external data source should pause playback. */
	virtual void PauseDataSource() {}

	/** Called by the Time Step system when the external data source should stop playback (typically at the end of rendering). */
	virtual void StopDataSource() {}

	/** Called by the Time Step system when the external data source should jump to the given time. Time is in TickResolution scale. */
	virtual void JumpDataSource(const FFrameTime& InTimeToJumpTo) {}

	/** 
	* Called when the Movie Graph Pipeline starts before anything has happened, allowing you to 
	* cache your datasource before making any modifications to it as a result of rendering.
	*/
	virtual void CacheDataPreJob(const FMovieGraphInitConfig& InInitConfig) {}
	virtual void RestoreCachedDataPostJob() {}
	virtual void UpdateShotList() {}
	virtual void InitializeShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot, const FFrameTime& InEvalTime) {}
	virtual void CacheHierarchyForShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot) {}
	virtual void RestoreHierarchyForShot(const TObjectPtr<UMoviePipelineExecutorShot> &InShot) {}
	virtual void MuteShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot) {}
	virtual void UnmuteShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot) {}
	virtual void ExpandShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot, const int32 InLeftDeltaFrames, const int32 InLeftDeltaFramesUserPoV,
		const int32 InRightDeltaFrames, const bool bInPrepass) {}

	/**
	 * Returns camera info for the given shot. When bIncludeSidecar is true, results must align with InShot->SidecarCameras:
	 * return one entry per sidecar camera, leaving unresolved sidecar camera entries defaulted.
	 */
	virtual TArray<UE::MovieGraph::FMinimalCameraInfo> GetCameraInformation(UMoviePipelineExecutorShot* InShot, bool bIncludeSidecar) const { return TArray<UE::MovieGraph::FMinimalCameraInfo>(); }

	UE_API UMovieGraphPipeline* GetOwningGraph() const;
};

/** Base class for generating audio while the pipeline is running. */
UCLASS(MinimalAPI, BlueprintType, Abstract)
class UMovieGraphAudioRendererBase : public UObject
{
	GENERATED_BODY()

public:
	/** Tell our submixes to start capturing the data they are generating. Should only be called once output frames are being produced. */
	virtual void StartAudioRecording() {}

	/** Tell our submixes to stop capturing the data, and then store a copy of it. */
	virtual void StopAudioRecording() {}

	/** Attempt to process the audio thread work. This is complicated by non-linear time steps. */
	virtual void ProcessAudioTick() {}

	/** Prepares for audio rendering (ensuring volume is set correctly, the needed cvars are set, etc). */
	virtual void SetupAudioRendering() {}

	/** Undoes the work done in SetupAudioRendering(). */
	virtual void TeardownAudioRendering() const {}

	/** Gets the pipeline that owns this audio output instance. */
	UE_API UMovieGraphPipeline* GetOwningGraph() const;

	/** Gets the current state of the audio renderer. This is the main data source that audio-related nodes can reference. */
	UE_API const MoviePipeline::FAudioState& GetAudioState() const;

protected:
	MoviePipeline::FAudioState AudioState;
};

// ToDo: Both of these can probably go into the Default Renderer implementation.
struct FMovieGraphRenderPassLayerData
{
	FName BranchName;
	FString LayerName;
	int32 CameraIndex;
	FString CameraName;

	// The index of this layer in the list of layers being rendered (0 being the first layer)
	int32 LayerIndex = 0;

	// The number of cameras that are being rendered.
	uint32 NumCameras = 1;

	/**
	 * The render pass node that's associated with this layer data. WARNING: this node will ONLY be valid during initial pass setup! Post-setup, if
	 * a non-CDO node is needed, it needs to be fetched from the evaluated config.
	 */
	TWeakObjectPtr<class UMovieGraphRenderPassNode> RenderPassNode;
};
// ToDo: Both of these can probably go into the Default Renderer implementation.

struct FMovieGraphRenderPassSetupData
{
	TWeakObjectPtr<class UMovieGraphDefaultRenderer> Renderer;
	TArray<FMovieGraphRenderPassLayerData> Layers;
	TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig;
};


namespace UE::MovieGraph
{
	/** Supplemental data to store with a file future so we can retrieve information about what the future was doing once it finishes writing to disk. */
	struct FMovieGraphOutputFutureData
	{
		FMovieGraphOutputFutureData()
			: Shot(nullptr)
			, OriginNodeClass(nullptr)
			, RenderLayerIndex(0)
		{}

		/** Which shot is this for? */
		class UMoviePipelineExecutorShot* Shot;

		/** What was the filepath on disk that was generated by the future? */
		FString FilePath;

		/** The "frame templated" filepath. This groups together a set of output frames by frame number, eg "Seq.Shot.{frame_placeholder}.exr". */
		FString FrameTemplatedFilePath;

		/** Which render pass is this for? */
		FMovieGraphRenderDataIdentifier DataIdentifier;

		/** The node class that this file originated from. */
		UClass* OriginNodeClass;

		/** The index of the render layer this future belongs to. Index 0 is the first render layer rendered, and so on. */
		int32 RenderLayerIndex;
	};
	
	/**
	* Statistics tracked for a given frame during rendering, such as when the
	* render started and when it ended. This can be added as metadata during
	* file output to include useful information. 
	*
	* ToDo: This could potentially be tracked as just regular filemetadata
	* with a rendered sample but requires strong typing support and metadata
	*  merging support.
	*/
	struct FRenderTimeStatistics
	{
		/** Time in UTC that the frame was first started. */
		FDateTime StartTime;

		/** Time in UTC that the frame was fully rendered and merged together. */
		FDateTime EndTime;
	};

	struct FMovieGraphSampleState : IImagePixelDataPayload, public TSharedFromThis<FMovieGraphSampleState>
	{
		FMovieGraphSampleState()
			: bWriteSampleToDisk(false)
			, bRequiresAccumulator(false)
			, bFetchFromAccumulator(false)
			, bCompositeOnOtherRenders(false)
			, bAllowsCompositing(true)
			, bIsBeautyPass(false)
			, OverscanFraction(0.f)
			, CompositingSortOrder(0)
			, RenderLayerIndex(0)
			, bAllowOCIO(true)
			, bForceLosslessCompression(false)
		{}

		virtual TSharedRef<FMovieGraphSampleState> Copy() const
		{
			return MakeShared<FMovieGraphSampleState>(*this);
		}

		/** The traversal context used to read graph values at the time of submission. */
		FMovieGraphTraversalContext TraversalContext;

		/** The desired output resolution scaled by any configured overscan */
		FIntPoint OverscannedResolution;
		
		/** The backbuffer resolution of this render resource (ie: the resolution the sample was rendered at). Includes overscan and tiling */
		FIntPoint BackbufferResolution;

		/** The size of a tile without padding (ie: OverscannedResolution/TileCount) */
		FIntPoint UnpaddedTileSize;

		/** 
		* The resolution the accumulation buffer is accumulating at. This may be different than our 
		* backbuffer resolution (due to tiling), and our final output resolution (due to overscan + crop). 
		*/
		FIntPoint AccumulatorResolution;

		/** Debug feature, should this sample be written to disk, not being accumulated? */
		bool bWriteSampleToDisk;

		/**
		* When bWriteSampleToDisk is true, if this string is specified then the single-sample debug writing will use this filename for the format
		* token instead of the default one. Useful for render passes with complex output cases (like Panoramics) that need pass-specific names.
		*/
		FString Debug_OverrideFilename;

		/** Specifies the filename format that this sample should use when being written to disk. Not used for debug like Debug_OverrideFilename. */
		FString FilenameFormatOverride;

		/** Set this to true for every sample if an accumulator is required (ie: has temporal/spatial sub-samples, or high res tiling). */
		bool bRequiresAccumulator;

		/** Set this to true for the last sample for a given frame (if it requires accumulation) to fetch the accumulated data out of the accumulator. */
		bool bFetchFromAccumulator;

		/** Set this to true if this pass should be composited on top of other renders. */
		bool bCompositeOnOtherRenders;

		/** Set this to false if this pass should not allow other passes to be composited onto it. */
		bool bAllowsCompositing;

		/** Whether this sample represents a sample in the main beauty pass. */
		bool bIsBeautyPass;

		/**
		 * When true, forces alpha to 1.0 in the output. Typically used for Post Process Material passes that don't
		 * explicitly output alpha, where the render target's clear color (alpha=0) would otherwise leak through.
		 */
		bool bForceOpaqueAlpha = false;

		/** When using high-res tiling, how many pixels does each tile overlap the adjacent tiles (on each side)? This should be zero if not using tiling. */
		FIntPoint OverlappedPad;

		/** When using high-res tiling, how many pixels offset into the output image does this accumulation get added to. */
		FIntPoint OverlappedOffset;

		/** When using spatial jitters, how much do we need to shift the output data during accumulation to counter-act the jitter. */
		FVector2D OverlappedSubpixelShift;

		/** When using camera overscan, what fraction (0-1) did this render use? Needed so that exrs can take a center-out crop of the data. */
		float OverscanFraction;

		/**
		 * The rectangle to use to crop the image prior to compositing and output. Used to crop out overscan from appropriate formats, and will
		 * generally have size equal to the originally requested resolution, whereas internal resolutions such as Backbuffer and Accumulator
		 * resolutions will be scaled up to include overscan
		 */
		FIntRect CropRectangle;
		
		/**
		* If multiple passes are composited on top of a render, the sort order determines the order in which they're composited.
		* Passes with a low sort order will composite on top of passes with a higher sort order.
		*/
		int32 CompositingSortOrder;

		/** The index of the render layer this sample belongs to. Index 0 is the first render layer rendered, and so on. */
		int32 RenderLayerIndex;

		/** Allow OpenColorIO transform to be used on this render. */
		bool bAllowOCIO;

		/** Whether this sample must use lossless compression when written to a format that supports it. */
		bool bForceLosslessCompression;

		/** Render scene capture source used for tracking the output color space (without OpenColorIO). */
		ESceneCaptureSource SceneCaptureSource;

		/**
		 * Additional metadata that should be added to the output. In most cases, the node's GetFormatResolveArgs() should be used to provide
		 * metadata unless the metadata is generated at a point in the pipeline where GetFormatResolveArgs() cannot be used.
		 */
		TMap<FString, FString> AdditionalFileMetadata;

		/**
		 * Additional key/value pairs for filename format token substitution. Keys are token names without braces (e.g. "camera_name").
		 * Use this when token values are only known at render time and GetFormatResolveArgs() cannot be used at that point.
		 */
		TMap<FString, FString> AdditionalFormatArgs;

		/**
		 * By default, MRG injects a {camera_name} token before {frame_number} when rendering multiple cameras
		 * (if one is not already present in the filename format string).
		 *
		 * If set, and {camera_name} is not already in the filename format string, these tokens are always injected instead of {camera_name},
		 * even when only one camera is active. Tokens already present in the format string are skipped.
		 * 
		 * If set, forces multi-camera code paths even when only one camera is active. Does not cause additional cameras to render.
		 *
		 * Example with CameraNameTokenOverrides = {t_1}, {t_2}:
		 *   {layer_name}.{frame_number}
		 *     -> {layer_name}.{t_1},{t_2}.{frame_number}
		 *   {t_2}.{layer_name}.{frame_number}
		 *     -> {t_2}.{layer_name}.{t_1}.{frame_number}  ({t_2} already present, skipped)
		 */
		TArray<FString> CameraNameTokenOverrides;

		/**
		 * For multi-layer output formats, setting this allows the layer name to be explicitly specified. Normally the layer name will be
		 * procedurally generated from multiple data sources to avoid collision with other layer names, but there are scenarios where specifying an
		 * exact layer name may be needed.
		 */
		FString LayerNameOverride;

		/**
		 * Specifies the output node class(es) that can output this sample. If this array is empty, this sample will be processed by all
		 * output nodes (which is typical).
		 */
		TArray<FSoftClassPath> OutputTypeRestrictions;
	};

	/**
	* A collection of validation information extracted from the FMovieGraphOutputMergerFrame 
	*/
	struct FMovieGraphRenderDataValidationInfo
	{
		int32 LayerCount = 0;
		int32 BranchCount = 0;
		int32 ActiveBranchRendererCount = 0;
		int32 ActiveRendererSubresourceCount = 0;
		int32 ActiveCameraCount = 0;
	};

	/**
	* A collection of all of the data needed to produce the one output frame on disk. This holds
	* all of the pixel data associated with the output frame at once, so we can pass it to our 
	* image writing/video encoders in one go, to ensure we can do things like Multilayer EXRs which
	* embed all of the render passes inside of one file.
	*/
	struct FMovieGraphOutputMergerFrame
	{
		FMovieGraphOutputMergerFrame() = default;
		
		/**
		* The configuraton file that was being used by this frame during submission. We store it here so that it doesn't have to go through the pipeline
		* (which is difficult becuase it's a Garbage Collected object), and the Output Merger is the "join" step before we go back onto the Game Thread.
		*/
		TStrongObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig;

		/** 
		* Traversal context for the Movie Pipeline that isn't layer specific. Each image layer will have
		* its own traversal context within its payload, specific to itself. */
		FMovieGraphTraversalContext TraversalContext;

		/**
		* An array of expected data identifiers this frame. This is generated when the frame starts, 
		* and this frame won't be complete until ImageOutputData contains an image for each expected pass.
		*/
		TArray<FMovieGraphRenderDataIdentifier> ExpectedRenderPasses;

		/** Stores the actual pixel data for each render pass. */
		TMap<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>> ImageOutputData;

		/** Additional metadata to be added to the output (if supported by the output container). */
		TMap<FString, FString> FileMetadata;

		/**
		 * Specifies which payloads will be written from a specific output node instance.
		 * NOTE: This is only updated immediately prior to filename disambiguation and should not be used outside of that.
		 */
		TMap<const UMovieGraphFileOutputNode*, TArray<UE::MovieGraph::FMovieGraphSampleState*>> NodeInstanceToPayloads;

		/** Get filename token validation info from the (expected) render passes. */
		MOVIERENDERPIPELINECORE_API FMovieGraphRenderDataValidationInfo GetValidationInfo(const FMovieGraphRenderDataIdentifier& InRenderID, bool bInDiscardCompositedRenders = true) const;
	};

	/**
	* Interface for the intermediate threadsafe class that allocates new frames, holds onto the individual
	* pieces as they come in, and eventually moves it back to the game thread once the data is complete.
	*/
	struct IMovieGraphOutputMerger : public TSharedFromThis<IMovieGraphOutputMerger>
	{
		virtual ~IMovieGraphOutputMerger() = default;

		/** Call once per output frame to generate a struct to hold data. */
		virtual FMovieGraphOutputMergerFrame& AllocateNewOutputFrame_GameThread(const int32 InRenderedFrameNumber) = 0;

		/** Getter for the Output Frame. Make sure you call AllocateNewOutputFrame_GameThread first, otherwise this trips a check. */
		virtual FMovieGraphOutputMergerFrame& GetOutputFrame_GameThread(const int32 InRenderedFrameNumber) = 0;

		/** When a final accumulated render pass is available, call this function with the pixel data. */
		virtual void OnCompleteRenderPassDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) = 0;

		/** When a individual sample is read back from the GPU, call this function. Debug Feature, immediately writes to disk. */
		virtual void OnSingleSampleDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) = 0;

		/** Abandon any frames we're waiting on, ie; render stopped in the middle of a many temporal sample count frame. */
		virtual void AbandonOutstandingWork() = 0;

		/** Number of frames that have been started (via AllocateNewOutputFrame_GameThread) but haven't had all of their data come in yet via OnCompleteRenderPassDataAvailable_AnyThread. */
		virtual int32 GetNumOutstandingFrames() const = 0;

		/** Get a thread-safe FIFO queue of finished frame data. */
		virtual TQueue<FMovieGraphOutputMergerFrame>& GetFinishedFrames() = 0;
	};
}

#undef UE_API
