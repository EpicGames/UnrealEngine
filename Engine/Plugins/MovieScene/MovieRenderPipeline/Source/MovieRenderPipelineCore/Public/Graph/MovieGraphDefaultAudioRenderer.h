// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieGraphDataTypes.h"

#include "MovieGraphDefaultAudioRenderer.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Provides default audio rendering for the pipeline.
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Default Audio Renderer"))
class UMovieGraphDefaultAudioRenderer : public UMovieGraphAudioRendererBase
{
	GENERATED_BODY()

public:
	UMovieGraphDefaultAudioRenderer() = default;

protected:
	// UMovieGraphAudioOutputBase interface
	UE_API virtual void StartAudioRecording() override;
	UE_API virtual void StopAudioRecording() override;
	UE_API virtual void ProcessAudioTick() override;
	UE_API virtual void SetupAudioRendering() override;
	UE_API virtual void TeardownAudioRendering() const override;
	// ~UMovieGraphAudioOutputBase
};

#undef UE_API
