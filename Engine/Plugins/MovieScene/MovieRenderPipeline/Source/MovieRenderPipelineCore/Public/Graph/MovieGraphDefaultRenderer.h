// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/MovieGraphDataTypes.h"
#include "PixelFormat.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/ThreadSafeBool.h"
#include "Camera/CameraTypes.h"
#include "MoviePipelineTelemetry.h"
#include "Tasks/Task.h"
#include "MovieGraphDefaultRenderer.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
class UMovieGraphRenderPassNode;
class UTextureRenderTarget2D;
struct FMoviePipelineSurfaceQueue;
struct FMovieGraphRenderCameraSource;
namespace MoviePipeline { struct IMoviePipelineOverlappedAccumulator; }
namespace UE::MovieGraph::DefaultRenderer { struct FSurfaceAccumulatorPool; }

typedef TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> FMoviePipelineSurfaceQueuePtr;
typedef MoviePipeline::IMoviePipelineOverlappedAccumulator MoviePipelineOverlappedAccumulatorInterface;
typedef TSharedPtr<UE::MovieGraph::DefaultRenderer::FSurfaceAccumulatorPool, ESPMode::ThreadSafe> FMoviePipelineAccumulatorPoolPtr;

namespace UE::MovieGraph::DefaultRenderer
{
	struct FMovieGraphTilingParams
	{
		// The number of pixels that the tiles overlap each other.
		FVector2f OverlapPad = FVector2f::ZeroVector;

		// The size of the backbuffer in pixels.
		FIntPoint TileSize = FIntPoint(0, 0);

		// The index of this tile.
		FIntPoint TileIndexes = FIntPoint(0, 0);

		// The number of tiles there are in total.
		FIntPoint TileCount = FIntPoint(1, 1);
	};

	struct FMovieGraphSamplingParams
	{
		int32 TemporalSampleIndex = 0;
		int32 TemporalSampleCount = 1;
		int32 SpatialSampleIndex = 0;
		int32 SpatialSampleCount = 1;
		int32 SeedOffset = 0;
	};

	struct FCameraInfo
	{
		FCameraInfo()
			: ViewActor(nullptr)
			, bAllowCameraAspectRatio(true)
			, DoFSensorScale(1.0f)
			, bUseCameraManagerPostProcess(true)
		{}

		/** Minimal view info describing the position and orientation in the world. */
		FMinimalViewInfo ViewInfo;

		/** The actor that is considered the "owner" of the view. Used by the renderer to respect bOnlyOwnerSee/bOwnerNoSee flags. */
		class AActor* ViewActor;

		/** The name to use for the {camera_name} token. Filled out by the renderer where possible. */
		FString CameraName;

		/** should we respect the camera's aspect ratio settings. */
		bool bAllowCameraAspectRatio;

		// questionable if these are fcamerainfo
		/** When using tiling, we scale the sensor to counteract the view changes. This value comes from modifying the ProjectionMatrix. */
		float DoFSensorScale;

		FMovieGraphTilingParams TilingParams;
		FMovieGraphSamplingParams SamplingParams;

		// Sub-pixel jitter this camera should use. Only applied when using no AA.
		FVector2D ProjectionMatrixJitterAmount;

		// If true, we use the Post Process data from the Camera Manager, instead of deriving it from the camera location. This should be
		// false when rendering multiple cameras.
		bool bUseCameraManagerPostProcess;

	};

	struct FRenderTargetInitParams
	{
		FRenderTargetInitParams()
			: Size(FIntPoint(0, 0))
			, TargetGamma(0.f)
			, bForceLinearGamma(false)
			, PixelFormat(EPixelFormat::PF_Unknown)
			, bIncludeInPreviewWindow(true)
		{
		}

		FIntPoint Size;
		float TargetGamma;
		bool bForceLinearGamma;
		EPixelFormat PixelFormat;

		/** Whether this render target is shown in the "preview" window that displays during a render. */
		bool bIncludeInPreviewWindow;

		bool operator == (const FRenderTargetInitParams& InRHS) const
		{
			return Size == InRHS.Size
				&& TargetGamma == InRHS.TargetGamma
				&& bForceLinearGamma == InRHS.bForceLinearGamma
				&& PixelFormat == InRHS.PixelFormat
				&& bIncludeInPreviewWindow == InRHS.bIncludeInPreviewWindow;
		}

		bool operator != (const FRenderTargetInitParams& InRHS) const
		{
			return !(*this == InRHS);
		}

		friend uint32 GetTypeHash(FRenderTargetInitParams Params)
		{
			return HashCombineFast(GetTypeHash(Params.Size),
				HashCombineFast(GetTypeHash(Params.TargetGamma),
				HashCombineFast(GetTypeHash(Params.bForceLinearGamma),
				HashCombineFast(GetTypeHash(Params.PixelFormat),
				GetTypeHash(Params.bIncludeInPreviewWindow)))));
		}
	};

	struct FMovieGraphImagePreviewDataPoolParams
	{
		/** The size of the render target, and the depth requested. */
		FRenderTargetInitParams RenderInitParams;
		/** Identifier for which render resource this is used by. */
		FMovieGraphRenderDataIdentifier Identifier;

		bool operator == (const FMovieGraphImagePreviewDataPoolParams& InRHS) const
		{
			return RenderInitParams == InRHS.RenderInitParams && Identifier == InRHS.Identifier;
		}

		bool operator != (const FMovieGraphImagePreviewDataPoolParams& InRHS) const
		{
			return !(*this == InRHS);
		}

		friend uint32 GetTypeHash(FMovieGraphImagePreviewDataPoolParams Params)
		{
			return HashCombineFast(GetTypeHash(Params.RenderInitParams), GetTypeHash(Params.Identifier));
		}
	};

	/** State object to keep track of the state of image render targets in the image render target pool */
	struct FMovieGraphImagePreviewPoolState
	{
		FMovieGraphImagePreviewPoolState()
		{
			UpdateLastUsedFrame();
		}
		
		/** The last frame that the render target was used */
		uint64 LastUsedFrame;

		/** Updates the last used frame to the current frame counter */
		void UpdateLastUsedFrame()
		{
			LastUsedFrame = GFrameCounter;
		}

		/** Compares the last used frame to the current frame counter and returns true of the difference is larger than the specified threshold */
		bool IsStale() const
		{
			constexpr int32 StaleThreshold = 3;
			return GFrameCounter - LastUsedFrame > StaleThreshold;
		}
	};
	
	struct FSurfaceAccumulatorPool : public TSharedFromThis<FSurfaceAccumulatorPool>
	{
		struct FInstance
		{
			FInstance(TSharedPtr<MoviePipelineOverlappedAccumulatorInterface, ESPMode::ThreadSafe> InAccumulator)
			{
				Accumulator = InAccumulator;
				SetIsActive(false);
			}

			bool IsActive() const { return bIsActive; }
			
			/** Changing the active state resets the internal state, so set it to active before configuring. */
			void SetIsActive(const bool bInIsActive) 
			{ 
				bIsActive = bInIsActive;
				ActiveFrameNumber = INDEX_NONE;
				TaskPrereq = UE::Tasks::FTask();
			}

			TSharedPtr<MoviePipelineOverlappedAccumulatorInterface, ESPMode::ThreadSafe> Accumulator;
			int32 ActiveFrameNumber;
			FMovieGraphRenderDataIdentifier ActivePassIdentifier;
			UE::Tasks::FTask TaskPrereq;
		private:
			FThreadSafeBool bIsActive;
		};

		typedef TSharedPtr<FInstance, ESPMode::ThreadSafe> FInstancePtr;

		TArray<TSharedPtr<FInstance, ESPMode::ThreadSafe>> Accumulators;
		FCriticalSection CriticalSection;

		template <typename AccumulatorType>
		FSurfaceAccumulatorPool::FInstancePtr GetAccumulatorInstance_GameThread(int32 InFrameNumber, const FMovieGraphRenderDataIdentifier& InPassIdentifier)
		{
			FScopeLock ScopeLock(&CriticalSection);

			// Search for an existing accumulator for the given frame number and render data
			for (int32 Index = 0; Index < Accumulators.Num(); Index++)
			{
				if (InFrameNumber == Accumulators[Index]->ActiveFrameNumber && InPassIdentifier == Accumulators[Index]->ActivePassIdentifier)
				{
					return Accumulators[Index];
				}
			}

			// If we didn't find one already in use for this frame, look to see if there's a previously
			// allocated one which is no longer being used.
			int32 AvailableIndex = INDEX_NONE;
			for (int32 Index = 0; Index < Accumulators.Num(); Index++)
			{
				if (!Accumulators[Index]->IsActive())
				{
					AvailableIndex = Index;
					break;
				}
			}

			// If we still don't have one, just allocate a new entry. The allocations are reasonably light-weight
			// we don't actually allocate storage memory until they're used.
			if (AvailableIndex == INDEX_NONE)
			{
				AvailableIndex = Accumulators.Num();

				TSharedPtr<AccumulatorType> NewAccumulatorInstance = MakeShared<AccumulatorType>();
				Accumulators.Add(MakeShared<UE::MovieGraph::DefaultRenderer::FSurfaceAccumulatorPool::FInstance>(NewAccumulatorInstance));
				UE_LOGF(LogMovieRenderPipeline, Log, "Allocated a Accumulator for Pool %ls, New Pool Count: %d", *AccumulatorType::GetName().ToString(), Accumulators.Num());
			}

			// Ensure we've either updated the reused accumulator to our new data
			// or configured our new accumulator for the first time.
			Accumulators[AvailableIndex]->SetIsActive(true);
			Accumulators[AvailableIndex]->ActiveFrameNumber = InFrameNumber;
			Accumulators[AvailableIndex]->ActivePassIdentifier = InPassIdentifier;
			Accumulators[AvailableIndex]->TaskPrereq = UE::Tasks::FTask();

			return Accumulators[AvailableIndex];
		}
	};

	class FMovieGraphAccumulationTask
	{
	public:
		FGraphEventRef LastCompletionEvent;
		UE::Tasks::FTask TaskPrereq;

	public:
		FGraphEventRef Execute(TUniqueFunction<void()> InFunctor)
		{
			if (LastCompletionEvent)
			{
				LastCompletionEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(InFunctor), GetStatId(), LastCompletionEvent);
			}
			else
			{
				LastCompletionEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(InFunctor), GetStatId());
			}
			return LastCompletionEvent;
		}

		inline TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FMovieGraphAccumulationTask, STATGROUP_ThreadPoolAsyncTasks);
		}
	};
}



/**
* This class is the default implementation for the Movie Graph Pipeline renderer. This
* is split off into a separate class to minimize the complexity of the core UMovieGraphPipeline,
* and to provide a better way to store render-specific data during runtime. It is responsible
* for taking all of the render passes and rendering them, and then moving their rendered
* data back to the main UMoviePipeline OutputMerger once finished.
* 
* It is unlikely you will want to implement your own renderer. If you need to create new render
* passes, inherit from UMovieGraphRenderPassNode and add it to your configuration, at which point
* MRQ will call function on the CDO of it that allow you to set up your own render data.
*/
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphDefaultRenderer : public UMovieGraphRendererBase
{
	GENERATED_BODY()

public:
	// UMovieGraphRendererBase Interface
	UE_API virtual TArray<FMovieGraphImagePreviewData> GetPreviewData() const override;
	UE_API virtual void Render(const FMovieGraphTimeStepData& InTimeData) override;
	UE_API virtual void SetupRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot) override;
	UE_API virtual void TeardownRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot) override;
	UE_API virtual UE::MovieGraph::FRenderTimeStatistics* GetRenderTimeStatistics(const int32 InFrameNumber);
	// ~UMovieGraphRendererBase Interface

	// UObject Interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	// ~UObject Interface

	UE_API void AddOutstandingRenderTask_AnyThread(UE::Tasks::FTask InTask);
	
	/**
	 * Fetches information for the given camera index.
	 * First, if InCameraSource is provided with a non-negative camera index, its information is used.
	 * Otherwise, it is resolved against default sequence camera or the shot's sidecar camera data.
	 * @param InCameraIndex  Pass INDEX_NONE when using default sequence camera.
	 *                       Pass [0..N) for multi-camera rendering (shot's sidecar or custom source).
	 * @param InCameraSource Optional camera source. If provided but the camera index cannot be
	 *                       resolved, falls back to the default sequence camera.
	 */
	UE_API UE::MovieGraph::DefaultRenderer::FCameraInfo GetCameraInfo(const int32 InCameraIndex, const FMovieGraphRenderCameraSource* InCameraSource = nullptr) const;
	void SetHasRenderedFirstViewThisFrame(bool bInValue) { bHasRenderedFirstViewThisFrame = bInValue; }
	bool GetHasRenderedFirstViewThisFrame() const { return bHasRenderedFirstViewThisFrame; }

	/**
	 * Gets the overscan value for the specified camera.
	 * @param InCameraIndex  Pass INDEX_NONE when using default sequence camera.
	 *                       Pass [0..N) for multi-camera rendering (shot's sidecar or custom source).
	 * @param InCameraSource Optional camera source. If provided but the camera index cannot be
	 *                       resolved, falls back to the default sequence camera.
	 */
	UE_API float GetCameraOverscan(int32 InCameraIndex, const FMovieGraphRenderCameraSource* InCameraSource = nullptr);

	/** Outputs a warning message regarding animated overscan to the MRQ log if one has not already been output  */
	UE_DEPRECATED(5.6, "WarnAboutAnimatedOverscan is no longer used; animated overscan is supported in 5.6")
	UE_API void WarnAboutAnimatedOverscan(float InInitialOverscan);
	
public:
	UE_API UTextureRenderTarget2D* GetOrCreateViewRenderTarget(const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InInitParams, const FMovieGraphRenderDataIdentifier& InIdentifier);
	UE_API FMoviePipelineSurfaceQueuePtr GetOrCreateSurfaceQueue(const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InInitParams);
	
	template <typename AccumulatorType>
	FMoviePipelineAccumulatorPoolPtr GetOrCreateAccumulatorPool()
	{
		if (const FMoviePipelineAccumulatorPoolPtr* ExistingAccumulatorPool = PooledAccumulators.Find(AccumulatorType::GetName()))
		{
			return *ExistingAccumulatorPool;
		}

		FMoviePipelineAccumulatorPoolPtr NewAccumulatorPool = MakeShared<UE::MovieGraph::DefaultRenderer::FSurfaceAccumulatorPool, ESPMode::ThreadSafe>();
		UE_LOGF(LogMovieRenderPipeline, Log, "Allocated a Accumulator Pool for AccumulatorType: %ls", *AccumulatorType::GetName().ToString());
		
		PooledAccumulators.Emplace(AccumulatorType::GetName(), NewAccumulatorPool);
		
		return NewAccumulatorPool;
	}

	/** Updates telemetry as needed. Generally should only be called by the pipeline. */
	UE_API void UpdateRenderPassTelemetry(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const;

protected:
	UE_API TObjectPtr<UTextureRenderTarget2D> CreateViewRenderTarget(const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InInitParams) const;
	UE_API FMoviePipelineSurfaceQueuePtr CreateSurfaceQueue(const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InInitParams) const;
	
	/** 
	* Gets the camera locations for the current frame to inform the grass system of where streaming is required. If bIncludeSidecar is false, only the primary camera location is considered,
	* otherwise it'll provide all the camera locations for multi-camera streaming (which can increase resource requirements).
	*/
	UE_API void GetCameraLocationsForFrame(TArray<FVector>& OutLocations, UMoviePipelineExecutorShot* InShot, bool bIncludeSidecar = false) const;
	UE_API void FlushAsyncEngineSystems(const TObjectPtr<UMovieGraphEvaluatedConfig>& InConfig) const;

	/** Fetches information for the given camera index. Should be "-1" when not using multi-camera rendering, or [0, n] based on the shot's sidecar camera data when using multi-camera rendering. */
	UE_API UE::MovieGraph::DefaultRenderer::FCameraInfo GetCameraInfoImpl(const int32 InCameraIndex) const;
	/**
	 * Gets the overscan value for the specified camera. First checks to see if any cached overscan value exists and returning it, and if
	 * no cached value was found, gets the camera's live overscan value and caches it.
	 */
	UE_API float GetCameraOverscanImpl(int32 InCameraIndex);

protected:
	/** A pointer to the CDOs of the Render Pass nodes that are valid for the current shot render. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMovieGraphRenderPassNode>> RenderPassesInUse;

	/** Telemetry gathered from live render pass instances before they are torn down. */
	FMoviePipelineEndShotRenderTelemetryContext CachedRenderPassTelemetry;

	/** For this engine tick, has any view been submitted yet? */
	bool bHasRenderedFirstViewThisFrame = false;

	/** Keep track of some statistics about render frame data for metadata purposes. */
	TMap<int32, UE::MovieGraph::FRenderTimeStatistics> RenderTimeStatistics;

	/*
	* Render Target specify a transient backbuffer to draw the image to.We reuse these
	* because we queue a copy onto a FMoviePipelineSurface so the only reason it is needed
	* is for display in the UI.
	*/
	TMap<UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewDataPoolParams, TObjectPtr<UTextureRenderTarget2D>> PooledViewRenderTargets;

	/** States for the render targets stored in PooledViewRenderTargets */
	TMap<UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewDataPoolParams, UE::MovieGraph::DefaultRenderer::FMovieGraphImagePreviewPoolState> PooledViewRenderTargetStates;
	
	/*
	* This is a list of Surfaces that we can copy our render target to immediately after
	* it is drawn to. This is read back to the CPU a frame later (to avoid blocking the
	* GPU), and it's contents are then copied into a FImagePixelData.
	*/
	TMap<UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams, FMoviePipelineSurfaceQueuePtr> PooledSurfaceQueues;

	/*
	* Once a sample is copied back from the GPU, we need to potentially accumulate it over
	* multiple frames, and to apply high-res tiling. To do this we have a pool of accumulators,
	* where each unique render resource gets one per frame.
	*/
	TMap<FName, TSharedPtr<UE::MovieGraph::DefaultRenderer::FSurfaceAccumulatorPool>> PooledAccumulators;

	/** Accessed by the Render Thread when starting up a new task.Makes sure we don't add tasks to the array while we're removing finished ones. */
	FCriticalSection OutstandingTasksMutex;
	/** Array of outstanding accumulation / blending tasks that are currently being worked on. */
	TArray<UE::Tasks::FTask> OutstandingTasks;
	
	/** Caches the camera overscan used during setup to ensure that overscan-scaled resolution stays constant for every frame during a render */
	TMap<int32, float> CameraOverscanCache;

	/** Indicates if the user has already been warned about animate overscan if it is detected so that logs aren't flooded with warning messages */
	bool bHasWarnedAboutAnimatedOverscan = false;
};

#undef UE_API
