// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MovieGraphEdge.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMovieGraphPin;

UCLASS(MinimalAPI)
class UMovieGraphEdge : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UMovieGraphPin> InputPin;
	
	UPROPERTY()
	TObjectPtr<UMovieGraphPin> OutputPin;

	/** Whether the edge is valid or not. Being valid means it contains a non-null input and output pin. */
	UE_API bool IsValid() const;

	/**
	 * Gets the pin on the other side of the edge. If bFollowRerouteConnections is true, reroute nodes will be passthrough, and this method will
	 * continue traversing edges until a pin on a non-reroute node is found.
	 */
	UE_API UMovieGraphPin* GetOtherPin(const UMovieGraphPin* InFirstPin, const bool bFollowRerouteConnections = false);
};

#undef UE_API
