// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "Graph/MovieGraphRenderDataIdentifier.h"
#include "MovieGraphFilenameResolveParams.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declare
class UMoviePipelineExecutorJob;
class UMovieGraphEvaluatedConfig;
class UMovieGraphPipeline;
class UMoviePipelineExecutorShot;

struct FMovieGraphTraversalContext;

namespace UE::MovieGraph { struct FMovieGraphSampleState; };

/**
* This data structure contains a list of key-value pairs (as strings) for both filename resolving, and file metadata.
* They are stored as two separate arrays (as metadata is often a bit more verbose/human friendly) and it is not required
* that the two arrays be in sync. When resolving final filenames, any {tokens} will be replaced with a matching key-value
* if found in the FilenameArguments variable. Because of this, FilenameArguments should be short and avoid illegal filename chars.
*/
USTRUCT(BlueprintType)
struct FMovieGraphResolveArgs
{
	GENERATED_BODY()

	FMovieGraphResolveArgs() = default;

	/** A set of Key/Value pairs for output filename format strings (without {}) and their values. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	TMap<FString, FString> FilenameArguments;

	/** A set of Key/Value pairs for file metadata for file formats that support metadata. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	TMap<FString, FString> FileMetadata;
};

/*
* This data structure contains all of the information needed to resolve a filename format string from the Movie Graph.
* During runtime the values are generated by the actual rendering and then placed into this data structure before resolving
* an actual {format_string} using the UMovieGraphBlueprintLibrary. You can use this to resolve filenames when editing. If
* you wish to resolve a partial format string (such as {sequence_name}/{frame_number} but have it only resolve {sequence_name}
* you can add ("frame_number", "{frame_number}") to the FileNameFormatOverrides and that portion of the format string will
* be preserved so the resulting string back would say "my_seq_name/{frame_number}".
*/
USTRUCT(BlueprintType)
struct FMovieGraphFilenameResolveParams
{
	GENERATED_BODY()

	FMovieGraphFilenameResolveParams()
		: RootFrameNumber(0)
		, ShotFrameNumber(0)
		, RootFrameNumberRel(0)
		, ShotFrameNumberRel(0)
		, Version(0)
		, bForceRelativeFrameNumbers(false)
		, bEnsureAbsolutePath(true)
		, InitializationTime(0)
		, ZeroPadFrameNumberCount(0)
		, Job(nullptr)
		, Shot(nullptr)
		, EvaluatedConfig(nullptr)
		, FrameNumberOffset(0)
		, DefaultFrameRate(FFrameRate(24, 1))
	{
	}

	/** Convenience function to make filename resolve parameters for output files, or OCIO contexts. */
	static UE_API FMovieGraphFilenameResolveParams MakeResolveParams(
		const FMovieGraphRenderDataIdentifier& InRenderId,
		const UMovieGraphPipeline* InPipeline,
		const TObjectPtr<UMovieGraphEvaluatedConfig>& InEvaluatedConfig,
		const FMovieGraphTraversalContext& InTraversalContext,
		const TMap<FString, FString>& InAdditionalFormatArgs = {},
		const UE::MovieGraph::FMovieGraphSampleState* InSamplePayload = nullptr);
	
	/**
	* This is used to fill out tokens related to the render data that the file represents.
	* Used to fill out the {camera_name}, {layer_name}, {renderer_name}, {renderer_sub_name} tokens.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	FMovieGraphRenderDataIdentifier RenderDataIdentifier;
	
	/**
	* Frame Number for the Root (matching what you see in the Sequencer timeline. ie: If the Sequence PlaybackRange starts on 50, this value would be 50 on the first frame.)
	* Used to fill out the {frame_number} token.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 RootFrameNumber;
	
	/** 
	* Frame Number for the Shot (matching what you would see in Sequencer at the sub-sequence level.) 
	* Used to fill out the {frame_number_shot} token.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 ShotFrameNumber;
	
	/** 
	* Frame Number for the Root (relative to the first RootFrameNumber for this render, ie: if your sequence started at 35, this would be 35 as well, but supports time dilation because it will count up based on the total number of frame rendered, not the closest Sequencer frame (like RootFrameNumber))
	* Used to fill out the {frame_number_rel} token.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 RootFrameNumberRel;
	
	/** 
	* Frame Number for the Shot (relative to the first ShotFrameNumber for this shot, ie: if your shot started at 35, this would be 35 as well, but supports time dilation because it will count up based on the total number of frame rendered, not the closest Sequencer frame (like ShotFrameNumber))
	* Used to fill out the {frame_number_shot_rel} token.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 ShotFrameNumberRel;
	
	/** A key/value pair that maps metadata names to their values. Output is only supported in exr formats at the moment. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TMap<FString, FString> FileMetadata;
	
	/** Used to resolve {version} format arguments. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 Version;
	
	/** If true, force format strings (like {frame_number}) to resolve using the relative version. Used when slow-mo is detected as frame numbers would overlap. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	bool bForceRelativeFrameNumbers;

	/** If specified, this is the filename that will be used instead of building it from the Output Setting node. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	FString FileNameOverride;

	/** 
	* If true, If the format string is a relative path, then the resulting resolved path will be converted to an absolute path.
	* Assumes that the relative path should be resolved relative to the engine/editor executable.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	bool bEnsureAbsolutePath;
	
	/**
	* The initialization time for this job. Used to resolve time-based format arguments. This should be in UTC, if you want filenames in a local timezone
	* then you should set InitializationTimeOffset to your offset from UTC.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	FDateTime InitializationTime;

	/** What offset should be applied to InitializationTime when generating {time} related filename tokens? Likely your offset from UTC if you want local time. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	FTimespan InitializationTimeOffset;
	
	/** When converting frame numbers to strings, how many digits should we pad them up to? ie: 5 => 0005 with a count of 4. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 ZeroPadFrameNumberCount;

	/** 
	* A map between "{format}" tokens and their values. These are applied after the auto-generated ones from the system,
	* which allows the caller to override things like {.ext}
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TMap<FString, FString> FileNameFormatOverrides;

	/** Optional. If specified, used to resolve some tokens related to the job.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TObjectPtr<UMoviePipelineExecutorJob> Job;

	/** Optional. If specified, used to resolve the shot name. No settings from this shot are ever used (they must come form the EvaluatedConfig instead). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TObjectPtr<UMoviePipelineExecutorShot> Shot;

	/** Optional. If specified, used to resolve some tokens related to settings in the shot (such as {ts_count}). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig;

	/** Added onto the various FrameNumber settings automatically. Can be used to offset values (such as starting your Level Sequence on frame 0, but outputting it to disk at 1001). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 FrameNumberOffset;

	/** What is the default Frame Rate? This will be overwritten by the Output Setting from the evaluated config (if specified). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	FFrameRate DefaultFrameRate;
};

#undef UE_API
