// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieGraphCoreTimeStep.h"

#include "MovieGraphLinearTimeStep.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Advances time forward linearly until the end of the range of time that is being rendered is reached. This is useful
 * for deferred rendering (where there's a small number of temporal sub-samples and no feedback mechanism for measuring
 * noise in the final image).
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Linear Time Step"))
class UMovieGraphLinearTimeStep : public UMovieGraphCoreTimeStep
{
	GENERATED_BODY()

public:
	UMovieGraphLinearTimeStep() = default;

protected:
	UE_API virtual int32 GetNextTemporalRangeIndex() const override;
	UE_API virtual int32 GetTemporalSampleCount() const override;
};

#undef UE_API
