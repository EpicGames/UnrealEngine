// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphMultiValueContainer.h"

#include "MovieGraphCommon.h"
#include "MovieGraphValueView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphMultiValueContainer)

UMovieGraphMultiValueContainer::UMovieGraphMultiValueContainer()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		ValuesView = NewObject<UMovieGraphMutableValueView>(this, TEXT("ValuesView"));
		ValuesView->ViewProperties(&Values);
	}
}

bool UMovieGraphMultiValueContainer::AddProperty(const FName& InPropertyName, const EMovieGraphValueType ValueType, UObject* InValueTypeObject, const TMap<FName, FString>& InMetadata)
{
	return ValuesView->AddProperty(InPropertyName, ValueType, InValueTypeObject, InMetadata);
}

bool UMovieGraphMultiValueContainer::RemoveProperty(const FName& InPropertyName)
{
	return ValuesView->RemoveProperty(InPropertyName);
}

bool UMovieGraphMultiValueContainer::HasProperty(const FName& InPropertyName)
{
	return ValuesView->HasProperty(InPropertyName);
}

int32 UMovieGraphMultiValueContainer::GetNumProperties() const
{
	return ValuesView->GetNumProperties();
}

TArray<FName> UMovieGraphMultiValueContainer::GetPropertyNames() const
{
	return ValuesView->GetPropertyNames();
}

#if WITH_EDITOR
TMap<FName, FString> UMovieGraphMultiValueContainer::GetPropertyMetadata(const FName& InPropertyName) const
{
	return ValuesView->GetPropertyMetadata(InPropertyName);
}
#endif	// WITH_EDITOR

bool UMovieGraphMultiValueContainer::GetValueBool(const FName& InPropertyName, bool& bOutValue) const
{
	return ValuesView->GetValueBool(InPropertyName, bOutValue);
}

bool UMovieGraphMultiValueContainer::GetValueByte(const FName& InPropertyName, uint8& OutValue) const
{
	return ValuesView->GetValueByte(InPropertyName, OutValue);
}

bool UMovieGraphMultiValueContainer::GetValueInt32(const FName& InPropertyName, int32& OutValue) const
{
	return ValuesView->GetValueInt32(InPropertyName, OutValue);
}

bool UMovieGraphMultiValueContainer::GetValueInt64(const FName& InPropertyName, int64& OutValue) const
{
	return ValuesView->GetValueInt64(InPropertyName, OutValue);
}

bool UMovieGraphMultiValueContainer::GetValueFloat(const FName& InPropertyName, float& OutValue) const
{
	return ValuesView->GetValueFloat(InPropertyName, OutValue);
}

bool UMovieGraphMultiValueContainer::GetValueDouble(const FName& InPropertyName, double& OutValue) const
{
	return ValuesView->GetValueDouble(InPropertyName, OutValue);
}

bool UMovieGraphMultiValueContainer::GetValueName(const FName& InPropertyName, FName& OutValue) const
{
	return ValuesView->GetValueName(InPropertyName, OutValue);
}

bool UMovieGraphMultiValueContainer::GetValueString(const FName& InPropertyName, FString& OutValue) const
{
	return ValuesView->GetValueString(InPropertyName, OutValue);
}

bool UMovieGraphMultiValueContainer::GetValueText(const FName& InPropertyName, FText& OutValue) const
{
	return ValuesView->GetValueText(InPropertyName, OutValue);
}

bool UMovieGraphMultiValueContainer::GetValueEnum(const FName& InPropertyName, uint8& OutValue, const UEnum* RequestedEnum) const
{
	return ValuesView->GetValueEnum(InPropertyName, OutValue, RequestedEnum);
}

bool UMovieGraphMultiValueContainer::GetValueStruct(const FName& InPropertyName, FStructView& OutValue, const UScriptStruct* RequestedStruct) const
{
	return ValuesView->GetValueStruct(InPropertyName, OutValue, RequestedStruct);
}

bool UMovieGraphMultiValueContainer::GetValueObject(const FName& InPropertyName, UObject*& OutValue, const UClass* RequestedClass) const
{
	return ValuesView->GetValueObject(InPropertyName, OutValue, RequestedClass);
}

bool UMovieGraphMultiValueContainer::GetValueClass(const FName& InPropertyName, UClass*& OutValue) const
{
	return ValuesView->GetValueClass(InPropertyName, OutValue);
}

FString UMovieGraphMultiValueContainer::GetValueSerializedString(const FName& InPropertyName)
{
	return ValuesView->GetValueSerializedString(InPropertyName);
}

bool UMovieGraphMultiValueContainer::SetValueBool(const FName& InPropertyName, const bool bInValue)
{
	return ValuesView->SetValueBool(InPropertyName, bInValue);
}

bool UMovieGraphMultiValueContainer::SetValueByte(const FName& InPropertyName, const uint8 InValue)
{
	return ValuesView->SetValueByte(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueInt32(const FName& InPropertyName, const int32 InValue)
{
	return ValuesView->SetValueInt32(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueInt64(const FName& InPropertyName, const int64 InValue)
{
	return ValuesView->SetValueInt64(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueFloat(const FName& InPropertyName, const float InValue)
{
	return ValuesView->SetValueFloat(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueDouble(const FName& InPropertyName, const double InValue)
{
	return ValuesView->SetValueDouble(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueName(const FName& InPropertyName, const FName InValue)
{
	return ValuesView->SetValueName(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueString(const FName& InPropertyName, const FString& InValue)
{
	return ValuesView->SetValueString(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueText(const FName& InPropertyName, const FText& InValue)
{
	return ValuesView->SetValueText(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueEnum(const FName& InPropertyName, const uint8 InValue, const UEnum* Enum)
{
	return ValuesView->SetValueEnum(InPropertyName, InValue, Enum);
}

bool UMovieGraphMultiValueContainer::SetValueStruct(const FName& InPropertyName, FConstStructView InValue)
{
	return ValuesView->SetValueStruct(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueObject(const FName& InPropertyName, UObject* InValue)
{
	return ValuesView->SetValueObject(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueClass(const FName& InPropertyName, UClass* InValue)
{
	return ValuesView->SetValueClass(InPropertyName, InValue);
}

bool UMovieGraphMultiValueContainer::SetValueSerializedString(const FName& InPropertyName, const FString& NewValue)
{
	return ValuesView->SetValueSerializedString(InPropertyName, NewValue);
}

EMovieGraphValueType UMovieGraphMultiValueContainer::GetValueType(const FName& InPropertyName) const
{
	return ValuesView->GetValueType(InPropertyName);
}

void UMovieGraphMultiValueContainer::SetValueType(const FName& InPropertyName, EMovieGraphValueType ValueType, UObject* InValueTypeObject)
{
	if (ValuesView->HasProperty(InPropertyName))
	{
		// Skip if the type already matches to avoid unnecessary property change events (which can dirty packages)
		if ((ValuesView->GetValueType(InPropertyName) == ValueType) && (ValuesView->GetValueTypeObject(InPropertyName) == InValueTypeObject))
		{
			return;
		}

		ValuesView->SetValueType(InPropertyName, ValueType, InValueTypeObject);

#if WITH_EDITOR
		// Send a property change event manually since the property bag doesn't seem to generate one in this scenario
		FProperty* ValuesProperty = FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(UMovieGraphMultiValueContainer, Values));
		FPropertyChangedEvent PropertyEvent(ValuesProperty, EPropertyChangeType::ValueSet);
		PostEditChangeProperty(PropertyEvent);

		// Forward the PostEditChangeProperty event to outers, as UMovieGraphMultiValueContainer is often held as a data
		// member in other classes.
		if (UObject* Owner = GetOuter())
		{
			Owner->PostEditChangeProperty(PropertyEvent);
		}
#endif // WITH_EDITOR
	}
}

const UObject* UMovieGraphMultiValueContainer::GetValueTypeObject(const FName& InPropertyName) const
{
	return ValuesView->GetValueTypeObject(InPropertyName);
}

void UMovieGraphMultiValueContainer::SetValueTypeObject(const FName& InPropertyName, const UObject* ValueTypeObject)
{
	if (ValuesView->HasProperty(InPropertyName))
    {
    	ValuesView->SetValueTypeObject(InPropertyName, ValueTypeObject);
    }
}

EMovieGraphContainerType UMovieGraphMultiValueContainer::GetValueContainerType(const FName& InPropertyName) const
{
	return ValuesView->GetValueContainerType(InPropertyName);
}

void UMovieGraphMultiValueContainer::SetValueContainerType(const FName& InPropertyName, EMovieGraphContainerType ContainerType)
{
	if (ValuesView->HasProperty(InPropertyName))
	{
		ValuesView->SetValueContainerType(InPropertyName, ContainerType);
	}
}

TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> UMovieGraphMultiValueContainer::GetArrayRef(const FName& InPropertyName)
{
	return ValuesView->GetArrayRef(InPropertyName);
}
