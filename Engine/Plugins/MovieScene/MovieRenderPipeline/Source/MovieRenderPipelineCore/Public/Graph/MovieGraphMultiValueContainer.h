// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieGraphCommon.h"
#include "MovieGraphValueView.h"
#include "StructUtils/PropertyBag.h"
#include "Templates/ValueOrError.h"

#include "MovieGraphMultiValueContainer.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Holds generic values, with an API for getting/setting values, as well as getting/setting the value types
 * and value container types (eg, array).
 *
 * For a container that holds a single value, see UMovieGraphValueContainer.
 */
UCLASS(MinimalAPI)
class UMovieGraphMultiValueContainer final : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphMultiValueContainer();

	/**
	 * Adds a new property to the container. If a property by the given name already exists, no add operation will occur. Returns true
	 * on successful add, otherwise returns false.
	 */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool AddProperty(const FName& InPropertyName, const EMovieGraphValueType ValueType, UObject* InValueTypeObject, const TMap<FName, FString>& InMetadata);

	/** Removes a property by the given name. Returns true on successful remove, otherwise returns false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool RemoveProperty(const FName& InPropertyName);

	/** Returns true if a property by the given name exists in the container, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool HasProperty(const FName& InPropertyName);

	/** Gets the number of properties that this container holds. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API int32 GetNumProperties() const;

	/** Gets the names of properties in this container. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API TArray<FName> GetPropertyNames() const;

#if WITH_EDITOR
	/** Gets the metadata for the given property. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API TMap<FName, FString> GetPropertyMetadata(const FName& InPropertyName) const;
#endif	// WITH_EDITOR

	/** Gets the bool value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueBool(const FName& InPropertyName, bool& bOutValue) const;

	/** Gets the byte value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueByte(const FName& InPropertyName, uint8& OutValue) const;

	/** Gets the int32 value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueInt32(const FName& InPropertyName, int32& OutValue) const;

	/** Gets the int64 value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueInt64(const FName& InPropertyName, int64& OutValue) const;

	/** Gets the float value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueFloat(const FName& InPropertyName, float& OutValue) const;

	/** Gets the double value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueDouble(const FName& InPropertyName, double& OutValue) const;

	/** Gets the FName value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueName(const FName& InPropertyName, FName& OutValue) const;

	/** Gets the FString value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueString(const FName& InPropertyName, FString& OutValue) const;

	/** Gets the FText value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueText(const FName& InPropertyName, FText& OutValue) const;

	/** Gets the enum value (for a specific enum) of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueEnum(const FName& InPropertyName, uint8& OutValue, const UEnum* RequestedEnum = nullptr) const;

	/** Gets the struct value (for a specific struct) of the held data. Returns true on success, else false. */
	UE_API bool GetValueStruct(const FName& InPropertyName, FStructView& OutValue, const UScriptStruct* RequestedStruct = nullptr) const;

	/** Gets the object value (for a specific class) of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueObject(const FName& InPropertyName, UObject*& OutValue, const UClass* RequestedClass = nullptr) const;

	/** Gets the UClass value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueClass(const FName& InPropertyName, UClass*& OutValue) const;

	/** Gets the serialized string value of the held data. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API FString GetValueSerializedString(const FName& InPropertyName);

	/** Gets the enum value of the held data. Returns true on success, else false. */
	template <typename T>
	bool GetValueEnum(const FName& InPropertyName, T& OutValue) const
	{
		return ValuesView->GetValueEnum<T>(InPropertyName, OutValue);
	}

	/** Gets the struct value of the held data. Returns true on success, else false. */
	template <typename T>
	bool GetValueStruct(const FName& InPropertyName, T*& OutValue) const
	{
		return ValuesView->GetValueStruct<T>(InPropertyName, OutValue);
	}

	/** Gets the object value of the held data. Returns true on success, else false. */
	template <typename T>
	bool GetValueObject(const FName& InPropertyName, T*& OutValue) const
	{
		return ValuesView->GetValueObject<T>(InPropertyName, OutValue);
	}

	/** Sets the bool value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueBool(const FName& InPropertyName, const bool bInValue);

	/** Sets the byte value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueByte(const FName& InPropertyName, const uint8 InValue);

	/** Sets the int32 value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueInt32(const FName& InPropertyName, const int32 InValue);

	/** Sets the int64 value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueInt64(const FName& InPropertyName, const int64 InValue);

	/** Sets the float value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueFloat(const FName& InPropertyName, const float InValue);

	/** Sets the double value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueDouble(const FName& InPropertyName, const double InValue);

	/** Sets the FName value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueName(const FName& InPropertyName, const FName InValue);

	/** Sets the FString value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueString(const FName& InPropertyName, const FString& InValue);

	/** Sets the FText value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueText(const FName& InPropertyName, const FText& InValue);

	/** Sets the enum value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueEnum(const FName& InPropertyName, const uint8 InValue, const UEnum* Enum);

	/** Sets the struct value of the held data. Returns true on success, else false. */
	UE_API bool SetValueStruct(const FName& InPropertyName, FConstStructView InValue);

	/** Sets the object value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueObject(const FName& InPropertyName, UObject* InValue);

	/** Sets the class value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueClass(const FName& InPropertyName, UClass* InValue);

	/** Sets the serialized value of the held data. The string should be the serialized representation of the value. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueSerializedString(const FName& InPropertyName, const FString& NewValue);

	/** Sets the enum value of the held data. Returns true on success, else false. */
	template <typename T>
	bool SetValueEnum(const FName& InPropertyName, const T InValue)
	{
		return ValuesView->SetValueEnum<T>(InPropertyName, InValue);
	}

	/** Sets the struct value of the held data. Returns true on success, else false. */
	template <typename T>
	bool SetValueStruct(const FName& InPropertyName, const T& InValue)
	{
		return ValuesView->SetValueStruct<T>(InPropertyName, InValue);
	}

	/** Sets the object value of the held data. Returns true on success, else false. */
	template <typename T>
	bool SetValueObject(const FName& InPropertyName, T* InValue)
	{
		return ValuesView->SetValueObject<T>(InPropertyName, InValue);
	}

	/** Gets the type of the stored data. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API EMovieGraphValueType GetValueType(const FName& InPropertyName) const;

	/** Sets the type of the stored data. Enums, structs, and classes must specify a value type object. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API void SetValueType(const FName& InPropertyName, EMovieGraphValueType ValueType, UObject* InValueTypeObject = nullptr);

	/** Gets the object that defines the enum, struct, or class. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API const UObject* GetValueTypeObject(const FName& InPropertyName) const;

	/** Sets the object that defines the enum, struct, or class. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API void SetValueTypeObject(const FName& InPropertyName, const UObject* ValueTypeObject);

	/** Gets the container type of the stored value. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API EMovieGraphContainerType GetValueContainerType(const FName& InPropertyName) const;

	/** Sets the container type of the stored value. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API void SetValueContainerType(const FName& InPropertyName, EMovieGraphContainerType ContainerType);

	/**
	 * Gets a reference to the array backing the value, if any. GetValueContainerType() will return
	 * EMovieGraphContainerType::Array if the value is holding an array.
	 */
	UE_API TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> GetArrayRef(const FName& InPropertyName);

private:
	// Allow the single value container to directly modify the Values bag so we can keep the public API clean.
	friend class UMovieGraphValueContainer;

	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphMutableValueView> ValuesView = nullptr;

	// Using a property bag solves a number of issues that would be very difficult to solve otherwise. 1) Presentation of
	// a property that can have its type changed within the details pane, 2) data storage of a property which can have
	// its type changed, 3) the ability to set the value of the property from both Python and C++, and 4) the ability to
	// change the property at runtime.
	/** The values held by this container. */
	UPROPERTY(EditAnywhere, meta=(ShowOnlyInnerProperties, FixedLayout), Category = "Value")
	FInstancedPropertyBag Values;
};

#undef UE_API
