// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/EngineTypes.h"

#include "MovieGraphNamedResolution.generated.h"

/**
 * Holds information about a screen resolution to be used for rendering.
 */
USTRUCT(BlueprintType)
struct FMovieGraphNamedResolution
{
	GENERATED_BODY()

public:

	FMovieGraphNamedResolution() = default;

	FMovieGraphNamedResolution(
		const FName& InResolutionProfileName, const FIntPoint InResolution, const FString& InDescription)
		: ProfileName(InResolutionProfileName)
		, Resolution(InResolution)
		, Description(InDescription)
	{}

	
	/** The name of the resolution this links to */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Movie Graph|Resolution")
	FName ProfileName = DefaultResolutionName;

	/** The screen resolution (in pixels). */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Movie Graph|Resolution")
	FIntPoint Resolution = DefaultResolution;
	
	/** The description text for this screen resolution. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Movie Graph|Resolution")
	FString Description = FString();

	bool IsValid() const
	{
		return !ProfileName.IsNone() && Resolution.X > 0 && Resolution.Y > 0;
	}

	/**
	 * The default resolution name to use when one is not defined.
	 */
	inline static FName DefaultResolutionName = TEXT("1080p (FHD)");

	/**
	 * The default resolution name to use when one is not defined.
	 */
	inline static FIntPoint DefaultResolution = FIntPoint(1920, 1080);

	/**
	 * Predefined name for the 'custom' resolution option in the combobox
	 */
	inline static FName CustomEntryName = TEXT("Custom");
};


/** Convert a FMovieGraphNamedResolution into a string */
inline FString LexToString(const FMovieGraphNamedResolution InResolution)
{
	return FString::Printf(TEXT("%s [%d, %d]"), *InResolution.ProfileName.ToString(), InResolution.Resolution.X, InResolution.Resolution.Y);
}
