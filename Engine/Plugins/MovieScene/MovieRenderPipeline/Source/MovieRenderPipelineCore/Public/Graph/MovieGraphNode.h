// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieGraphPin.h"
#include "StructUtils/PropertyBag.h"
#include "Graph/MovieGraphMultiValueContainer.h"
#include "Graph/MovieGraphValueContainer.h"
#include "Graph/MovieGraphFilenameResolveParams.h"
#include "UObject/Interface.h"

#if WITH_EDITOR
#include "Textures/SlateIcon.h"
#include "Math/Color.h"
#endif

#include "MovieGraphNode.generated.h"

// Forward Declares
class UMovieGraphInput;
class UMovieGraphMember;
class UMovieGraphOutput;
class UMovieGraphPin;
class UMovieGraphPipeline;
class UMovieGraphVariable;
struct FMovieGraphEvaluationContext;
struct FMovieGraphTraversalContext;
struct FMoviePipelineShotRenderTelemetry;

#if WITH_EDITOR
class UEdGraphNode;
#endif

DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieGraphNodeChanged, const UMovieGraphNode*);

/**
 * Information about a property that currently is (or can be) exposed on a node.
 */
USTRUCT(BlueprintType)
struct FMovieGraphPropertyInfo
{
	GENERATED_BODY()
	
	/** The name of the property. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	FName Name;

	/** The display name of the property which will be shown in the context menu. If empty, the value from 'Name' will be used. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	FText ContextMenuName;

	/** If this property is promoted, this is the name of the variable that is created. If empty, the value from 'Name' will be used. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	FName PromotionName;

	/** Whether this property is dynamic (ie, it does not correspond to a native UPROPERTY on the node). */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	bool bIsDynamicProperty = false;

	/** The type of the value pointed to by the property. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	EMovieGraphValueType ValueType = EMovieGraphValueType::None;

	/** The associated value type object if the ValueType is an enum, struct, class, or object. */
	UPROPERTY()
	TObjectPtr<const UObject> ValueTypeObject;

	/**
	 * Whether this property is permanently exposed on the node. If true, it cannot be toggled on/off (via Expose Property as Pin).
	 */
	UPROPERTY()
	bool bIsPermanentlyExposed = false;

	/**
	 * Determines if this struct represents the same property as another instance of this struct.
	 *
	 * The equality operator ensures equality of all members of the struct. However, there may be cases where, for
	 * example, a dynamic property changes its value type, and the ValueType member has not been changed yet. Two
	 * structs that have the same Name and bIsDynamicProperty values should still be considered identical for the
	 * purposes of determining if they identify the same property.
	 */
	bool IsSamePropertyAs(const FMovieGraphPropertyInfo& Other) const
	{
		return (Name == Other.Name) && (bIsDynamicProperty == Other.bIsDynamicProperty);
	}

	bool operator==(const FMovieGraphPropertyInfo& Other) const
	{
		return (Name == Other.Name)
			&& (ContextMenuName.EqualTo(Other.ContextMenuName))
			&& (PromotionName == Other.PromotionName)
			&& (bIsDynamicProperty == Other.bIsDynamicProperty)
			&& (ValueType == Other.ValueType)
			&& (ValueTypeObject == Other.ValueTypeObject)
			&& (bIsPermanentlyExposed == Other.bIsPermanentlyExposed);
	}
};

/**
 * Additional context provided to ResolveTokenContainingProperties(). May be helpful if the node needs to implement custom {token} resolve
 * behavior that goes beyond the default resolve behavior that MRG provides.
 */
USTRUCT(BlueprintType)
struct FMovieGraphTokenResolveContext
{
	GENERATED_BODY()
	
	FMovieGraphTokenResolveContext() = default;
	
	const FMovieGraphRenderDataIdentifier* RenderDataIdentifier = nullptr;
	const FMovieGraphTraversalContext* TraversalContext = nullptr;
};

/** Describes a restriction on what kind of branch a node can be created in within the graph. */
UENUM(BlueprintType)
enum class EMovieGraphBranchRestriction : uint8
{
	Any,			///< The node can be created in any type of branch
	Globals,		///< The node must be created in the Globals branch
	RenderLayer		///< The node must be created in a branch representing a render layer
};

/**
* This is a base class for all nodes that can exist in the UMovieGraphConfig network.
* In the editor, each node in the network will have an editor-only representation too 
* which contains data about it's visual position in the graph, comments, etc.
*/
UCLASS(MinimalAPI, Abstract)
class UMovieGraphNode : public UObject
{
	GENERATED_BODY()


	friend class UMovieGraphConfig;
	friend class UMovieGraphEdge;
	
public:
	static MOVIERENDERPIPELINECORE_API FName GlobalsPinName;
	static MOVIERENDERPIPELINECORE_API FString GlobalsPinNameString;
	
	MOVIERENDERPIPELINECORE_API UMovieGraphNode();

	/** Gets all input pins on the node. Note that the returned array is const, so input pins cannot be added/removed from the node via this array. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	const TArray<UMovieGraphPin*>& GetInputPins() const { return InputPins; }
	
	/** Gets all output pins on the node. Note that the returned array is const, so output pins cannot be added/removed from the node via this array. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	const TArray<UMovieGraphPin*>& GetOutputPins() const { return OutputPins; }
	
	/** Gets the properties for all input pins. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual TArray<FMovieGraphPinProperties> GetInputPinProperties() const
	{
		return TArray<FMovieGraphPinProperties>();
	}
	
	/** Gets the properties for all output pins. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const
	{
		return TArray<FMovieGraphPinProperties>();
	}

	/**
	 * Gets the resolved value of a named output pin (one that is returned in GetOutputPinProperties()). If the resolved
	 * value could not be determined, an empty string is returned.
	 */
	virtual FString GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext) const
	{
		return FString();
	}

	/**
	 * The same functionality as the other method that returns a string. However, this version returns true if a
	 * resolved value could be determined, and provides the value via a UMovieGraphValueContainer.
	 */
	virtual bool GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext, TObjectPtr<UMovieGraphValueContainer>& OutValueContainer) const
	{
		return false;
	}

	/**
	 * Gets the descriptions of properties which can be dynamically added to the node. These types of properties
	 * do not correspond to a UPROPERTY defined on the node itself.
	 */
	virtual TArray<FPropertyBagPropertyDesc> GetDynamicPropertyDescriptions() const
	{
		return TArray<FPropertyBagPropertyDesc>();
	}

	/**
	 * Sets the value of the dynamic property with the specified name. Note that the provided value must be the serialized
	 * representation of the value. Returns true upon success, else false.
	 */
	MOVIERENDERPIPELINECORE_API bool SetDynamicPropertyValue(const FName PropertyName, const FString& InNewValue);

	/**
	 * Gets the value of the dynamic property with the specified name. Provides the serialized value of the property in
	 * "OutValue". Returns true if "OutValue" was set and there were no errors, else returns false.
	 */
	MOVIERENDERPIPELINECORE_API virtual bool GetDynamicPropertyValue(const FName PropertyName, FString& OutValue);

	/**
	 * Gets the dynamic properties as a view. The view provides a scripting-friendly way of getting and modifying the values
	 * within the dynamic properties.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API UMovieGraphMutableValueView* GetDynamicPropertiesView() const;

	/** Gets the override property for the specified dynamic property. If one does not exist, returns nullptr. */
	MOVIERENDERPIPELINECORE_API const FBoolProperty* FindOverridePropertyForDynamicProperty(const FName& InPropertyName) const;

	/**
	 * Returns true if the dynamic property with the provided name has been overridden, else false. Note that the name
	 * provided should not be prefixed with "bOverride_".
	 */
	MOVIERENDERPIPELINECORE_API bool IsDynamicPropertyOverridden(const FName& InPropertyName) const;

	/**
	 * Sets the dynamic property with the provided name to the specified override state. Note that the name provided
	 * should not be prefixed with "bOverride_".
	 */
	MOVIERENDERPIPELINECORE_API void SetDynamicPropertyOverridden(const FName& InPropertyName, const bool bIsOverridden);

	/** Gets the information about properties which can be exposed as a pin on the node. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API virtual TArray<FMovieGraphPropertyInfo> GetOverrideablePropertyInfo() const;

	/** Gets the information about properties which are currently exposed as pins on the node. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual TArray<FMovieGraphPropertyInfo> GetExposedProperties() const
	{
		return ExposedPropertyInfo;
	}

	/** 
	* Used to determine which pins we should follow when trying to traverse the graph.
	* By default we will follow any input pin (with Branch type) on the node, but override this in
	* inherited classes and change that if you need custom logic, such as boolean nodes that want 
	* to choose one or the other based on the results of a conditional property.
	*
	* Note that if custom logic is provided, the case where the node is disabled should be handled
	* as well.
	*/
	MOVIERENDERPIPELINECORE_API virtual TArray<UMovieGraphPin*> EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const;

	/**
	* When a non-branch pin type is being evaluated on a node, the calling node will ask this node
	* for the value connected to the given pin name. For example, a Branch node will call this
	* function on whatever node is connected to the Conditional pin, and then will try to get a 
	* Boolean value out of the returned UMovieGraphValueContainer.
	*/
	virtual UMovieGraphValueContainer* GetPropertyValueContainerForPin(const FString& InPinName) const
	{
		return nullptr;
	}

	/** Toggles the promotion of the property with the given name to a pin on the node. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API virtual void TogglePromotePropertyToPin(const FName& PropertyName);
	
	/** Promotes the given PropertyName to a pinned property, if it is found in GetOverrideablePropertyInfo(). Throws a kismet (scripting) exception if it isn't a overridable property. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API virtual void PromotePropertyToPin(const FName& PropertyName);
	
	/** Unpromotes a given PropertyName from being pinned. Throws a kismet (scripting) exception if it's not already exposed. See GetExposedProperties() */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API virtual void UnpromotePropertyFromPin(const FName& PropertyName);
	
	/**
	 * Allows subclasses to provide a custom serialized default value when a property is promoted
	 * to a variable. Called during "Promote to Variable" before writing the variable's default.
	 * Return true and set OutSerializedValue to override; return false for normal behavior.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API virtual bool GetCustomPromotionDefaultValue(const FMovieGraphPropertyInfo& InPropertyInfo, FString& OutSerializedValue) const;
	
	/**
	 * Determines if this node type can be added to the graph interactively by a user or via the API when constructing a graph.
	 * @return true if the object can be added via the API, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual bool CanBeAddedByUser() const { return true; }

	/**
	 * Gets all overrideable properties that are defined on the node. This includes UPROPERTY-defined properties, as
	 * well as dynamic properties. "Overrideable" means that the property has a corresponding property prefixed with
	 * "bOverride_".
	 */
	MOVIERENDERPIPELINECORE_API TArray<const FProperty*> GetAllOverrideableProperties() const;

	MOVIERENDERPIPELINECORE_API void UpdatePins();
	MOVIERENDERPIPELINECORE_API void UpdateDynamicProperties();
	MOVIERENDERPIPELINECORE_API class UMovieGraphConfig* GetGraph() const;

	/**
	 * Gets the property bag instance that backs the dynamic properties on this node.
	 * 
	 * Should not be used except for very specific, niche cases where the property bag needs to be accessed/modified directly, like
	 * for details customizations. For normal uses, the other dynamic property methods should be used.
	 */
	MOVIERENDERPIPELINECORE_API FInstancedPropertyBag* GetMutableDynamicProperties_Unsafe() const;

	/**
	 * Gets the input pin with the specified name, or nullptr if one could not be found. Most pins on a node are
	 * "built-in", meaning they ship with the node. Dynamic pins (pins which are not built-in) can potentially have the
	 * same name as a built-in (eg, the option pins on the Select node). To disambiguate between built-in and dynamic
	 * pins, specify bIsBuiltInPin = false if trying to fetch a pin that is not built-in.  */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API UMovieGraphPin* GetInputPin(const FName& InPinLabel, const EMovieGraphPinQueryRequirement PinRequirement = EMovieGraphPinQueryRequirement::BuiltInOrDynamic) const;

	/** Gets the output pin with the specified name, or nullptr if one could not be found. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API UMovieGraphPin* GetOutputPin(const FName& InPinLabel) const;

	/** Gets the first input pin on the node which has a connection, or nullptr if no pins are connected. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API UMovieGraphPin* GetFirstConnectedInputPin() const;
	
	/** Gets the first output pin on the node which has a connection, or nullptr if no pins are connected. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API UMovieGraphPin* GetFirstConnectedOutputPin() const;

	/** Gets the GUID which uniquely identifies this node. */
	const FGuid& GetGuid() const { return Guid; }
	
	/** Determines which types of branches the node can be created in. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual EMovieGraphBranchRestriction GetBranchRestriction() const { return EMovieGraphBranchRestriction::Any; }

	/** Determines if this node can be disabled. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API virtual bool CanBeDisabled() const;

	/** Set whether this node is currently disabled. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API void SetDisabled(const bool bNewDisableState);

	/** Determines if this node is currently disabled. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API bool IsDisabled() const;

	/**
	 * Gets any validation errors that the node generated. Validation errors will halt the active render, if any. Called on the fully-evaluated
	 * node post-evaluation. Returns true if there were any errors (added to OutValidationErrors), else false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	MOVIERENDERPIPELINECORE_API virtual bool GetNodeValidationErrors(const FName& InBranchName, const UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<FText>& OutValidationErrors) const;

#if WITH_EDITOR
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	int32 GetNodePosX() const { return NodePosX; }
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	int32 GetNodePosY() const { return NodePosY; }

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	void SetNodePosX(const int32 InNodePosX) { NodePosX = InNodePosX; }
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	void SetNodePosY(const int32 InNodePosY) { NodePosY = InNodePosY; }

	/** Gets the node's title color, as visible in the graph. */
	MOVIERENDERPIPELINECORE_API virtual FLinearColor GetNodeTitleColor() const;

	/** Gets the node's icon and icon tint, as visible in the graph. */
	MOVIERENDERPIPELINECORE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const;

	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	FString GetNodeComment() const { return NodeComment; }
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	void SetNodeComment(const FString& InNodeComment) { NodeComment = InNodeComment; }

	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	bool IsCommentBubblePinned() const { return bIsCommentBubblePinned; }
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	void SetIsCommentBubblePinned(const uint8 bIsPinned) { bIsCommentBubblePinned = bIsPinned; }

	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	bool IsCommentBubbleVisible() const { return bIsCommentBubbleVisible; }
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	void SetIsCommentBubbleVisible(uint8 bIsVisible) { bIsCommentBubbleVisible = bIsVisible; }

	/**
	 * Gets the properties on this node that should be hidden by the base UMovieGraphNode customization. The FName in the pair is the property
	 * name, and the UClass* is the class that the property is defined on (may be a base class).
	 *
	 * Note that other customizations can easily override this behavior, but it's useful in cases where (for example) an inherited
	 * property should be hidden, and there aren't any other customizations needed.
	 */
	MOVIERENDERPIPELINECORE_API virtual TArray<TPair<FName, UClass*>> GetHiddenProperties() const;
#endif

	//~ Begin UObject Interface
	MOVIERENDERPIPELINECORE_API virtual void PostLoad() override;
	MOVIERENDERPIPELINECORE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	//~ End UObject Interface

public:
	FOnMovieGraphNodeChanged OnNodeChangedDelegate;

	/**
	 * Tags that can be used to identify this node within a pre/post render script. Tags can be unique in order to identify this specific node,
	 * or the same tag can be applied to multiple nodes in order to identify a grouping of nodes.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tags")
	TArray<FString> ScriptTags;

#if WITH_EDITORONLY_DATA
	/** Editor Node Graph representation. Not strongly typed to avoid circular dependency between editor/runtime modules. */
	UPROPERTY()
	TObjectPtr<UEdGraphNode>	GraphNode;

	MOVIERENDERPIPELINECORE_API class UEdGraphNode* GetGraphNode() const;
#endif

#if WITH_EDITOR
	/**
	 * Gets the node's title. Optionally gets a more descriptive, multi-line title for the node if bGetDescriptive is
	 * set to true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const PURE_VIRTUAL(UMovieGraphNode::GetNodeTitle, return FText(););

	/** Gets the category that the node belongs under. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual FText GetMenuCategory() const PURE_VIRTUAL(UMovieGraphNode::GetMenuCategory, return FText(); );

	/** Gets the keywords (space-separated) that will be searched in the node creation context menu. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	virtual FText GetKeywords() const { return FText(); }
#endif

protected:
	/** Gets the pin properties for all properties which have been exposed on the node. */
	MOVIERENDERPIPELINECORE_API virtual TArray<FMovieGraphPinProperties> GetExposedPinProperties() const;

	/** Register any delegates that need to be set up on the node. Called in PostLoad(). */
	virtual void RegisterDelegates() { }

protected:
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphPin>> InputPins;
	
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphPin>> OutputPins;

	/** Properties which can be dynamically declared on the node (vs. native properties which are always present). */
	UPROPERTY(EditAnywhere, meta=(FixedLayout, ShowOnlyInnerProperties), Category = "Properties")
	FInstancedPropertyBag DynamicProperties;

	/** A property view which points to the dynamic properties on this node. */
	UPROPERTY(Transient)
	mutable TObjectPtr<UMovieGraphMutableValueView> DynamicPropertiesView = nullptr;

	/** Tracks which properties have been exposed on the node as inputs. */
	UPROPERTY()
	TArray<FMovieGraphPropertyInfo> ExposedPropertyInfo;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	int32 NodePosX = 0;

	UPROPERTY()
	int32 NodePosY = 0;

	UPROPERTY()
	FString NodeComment;

	UPROPERTY()
	uint8 bIsCommentBubblePinned : 1;

	UPROPERTY()
	uint8 bIsCommentBubbleVisible : 1;
#endif

	/** Whether this node is currently disabled in the graph. */
	UPROPERTY()
	uint8 bIsDisabled : 1;

	/** A GUID which uniquely identifies this node. */
	UPROPERTY()
	FGuid Guid;
};

/**
* Nodes representing user settings should derive from this. This is the only node type copied into flattened eval.
*/
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UMovieGraphSettingNode : public UMovieGraphNode
{
	GENERATED_BODY()
	
public:
	// UMovieGraphNode Interface
	MOVIERENDERPIPELINECORE_API virtual TArray<FMovieGraphPinProperties> GetInputPinProperties() const override;
	MOVIERENDERPIPELINECORE_API virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const override;
	// ~UMovieGraphNode Interface

	/**
	 * An identifier that distinguishes this node from other nodes of the same type within a branch. During graph
	 * traversal, nodes with the same type and identifier are considered the same node. An empty string is a valid
	 * identifier.
	 */
	virtual FString GetNodeInstanceName() const { return FString(); }

	/**
	 * In some very rare cases, a node needs to be "primed" from the node that the flattening is starting from before it's actually put through the
	 * flattening process.
	 *
	 * The need for this should be exceedingly uncommon, so only use if absolutely necessary.
	 */
	virtual void PrepareForFlattening(const UMovieGraphSettingNode* InSourceNode) { }


	/**
	* When the graph asset was loaded, any deprecated properties were converted to their "real" representations
	* in PostLoad. However, scripting can edit these assets after load - either an editor script, or
	* an ExecuteScript node, and these custom scripts may still be (re) setting the deprecated property, at which
	* point the runtime code will ignore it.
	* 
	* To catch these scenarios, after the graph is flattened, we call PostFlatten on each node in the graph which
	* gives it one more chance to do the conversion from deprecated properties to real ones, and warn the user that
	* they need to update their scripts.
	* 
	* This is called on the flattened graph, so only one "merged" copy of the nodes exist at this point.
	*/
	virtual void PostFlatten() {}
	
	/*
	* This is called either on the CDO, or on a "flattened" instance of the node every frame when
	* generating filename/file metadata, allowing the node to add custom key-value pairs (FString, FString)
	* to be used as {format_tokens} in filenames, or to be included in File metadata. Nodes can read
	* their own settings (such as temporal sub-sample count) and add it to the available list of tokens.
	*
	* Because this is called either on the CDO or on a flattened instance, there is no need to worry about
	* resolving the settings of the graph, the node only needs to read its own values.
	*/
	virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const {}

	/**
	 * Resolves the values of properties that contain {tokens} to their final value. Graph evaluation performs this step after after the graph has
	 * been completely flattened, and it's the node's responsibility to implement this method if any of its properties have {tokens} that need to
	 * be resolved.
	 *
	 * All properties that need to be resolved should have ResolveFunc() called on them; this will update the node's property with the resolved
	 * value. Regular MRG tokens, as well as tokens from the Naming Tokens (CAT) system, will be resolved. Nodes can also implement custom resolver
	 * behavior here if needed, using the additional context provided. Select MRG tokens may not be resolved correctly here due to the point in the
	 * graph evaluation lifecycle that this resolve step occurs (eg, {camera_name}).
	 *
	 * Nodes that implement this method should also take care to call the super class in order to ensure inherited properties are covered.
	 *
	 * If the properties resolved here should use the token autocomplete widget in the details panel, include the MrgTokenAutocomplete
	 * UPROPERTY metadata on them (eg, `meta=(MrgTokenAutocomplete)`). Note that this metadata will only take effect for top-level properties
	 * (ie, properties that are directly declared on the node); properties that are embedded in structs, for example, will not be detected.
	 */
	MOVIERENDERPIPELINECORE_API virtual void ResolveTokenContainingProperties(TFunction<void(FString&)>& ResolveFunc, const FMovieGraphTokenResolveContext& InContext);
	
	/** Modify the Unreal URL and command line arguments when the node will be run in a new process. Only applies to nodes in the Globals branch. */
	virtual void BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const { }

	/** Updates telemetry data for this node. Should only be used by nodes that ship with Movie Render Graph. Called on the fully-evaluated node in the flattened graph. */
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const { }
};

UINTERFACE(MinimalAPI)
class UMovieGraphPostRenderNode : public UInterface
{
	GENERATED_BODY()
};

/**
 * A node which runs after renders have completed. Can run after a single shot renders and/or after a sequence renders all of its shots.
 */
class IMovieGraphPostRenderNode
{
	GENERATED_BODY()

public:
	/** Begins the export process for this node after all shots have completed. */
	virtual void BeginExport(UMovieGraphPipeline* InMoviePipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) = 0;

	/** Begins the export process for this node after an individual shot completes. */
	virtual void BeginShotExport(UMovieGraphPipeline* InMoviePipeline) = 0;

	/** Returns true if this node has finished its export process, else false. */
	virtual bool HasFinishedExporting() = 0;
};

UINTERFACE(MinimalAPI, NotBlueprintable)
class UMovieGraphEvaluationNodeInjector : public UInterface
{
	GENERATED_BODY()
};

/**
 * Provides the interface that nodes must implement if they need to inject additional nodes into the graph at various points in the
 * graph evaluation cycle.
 */
class IMovieGraphEvaluationNodeInjector : public IInterface
{
	GENERATED_BODY()

public:
	/**
	 * Injects nodes into the EVALUATED graph. Called on the fully evaluated node. All new nodes should be outered to InEvaluatedConfig. Because these
	 * nodes are injected post-evaluation, the movie pipeline will be immediately shut down if any of the returned nodes already exist in the branch
	 * (having two nodes of the same type in the same branch post-evaluation is not allowed). Note that modifying the Globals branch is not typically
	 * advised because at this stage, it has already been merged into all other branches.
	 */
	UFUNCTION(BlueprintCallable, Category = "Evaluation")
	MOVIERENDERPIPELINECORE_API virtual void InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes);
};
