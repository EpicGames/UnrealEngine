// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageWriteTask.h"
#include "MovieGraphDataTypes.h"

#if WITH_OCIO
#include "OpenColorIOWrapper.h"
#endif

/**
* Provides utilities for nodes that need to include OCIO output functionality.
*/
class FMovieGraphOCIOHelper
{
public:
	FMovieGraphOCIOHelper() = default;

#if WITH_OCIO
	/**
	 * Generates a new OCIO pixel PreProcessor and appends it to OutPixelPreProcessors. An OCIO context is generated
	 * based on the provided payload, pipeline, and evaluated graph.
	 */
	static MOVIERENDERPIPELINECORE_API bool GenerateOcioPixelPreProcessor(
		const UE::MovieGraph::FMovieGraphSampleState* InPayload, const UMovieGraphPipeline* InPipeline, const TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig,
		const FOpenColorIODisplayConfiguration& InConfiguration, const TMap<FString, FString>& InContext, TArray<FPixelPreProcessor>& OutPixelPreProcessors);

	/**
	 * Generates a new OCIO pixel PreProcessor and appends it to OutPixelPreProcessors. The context used by OCIO is
	 * provided via InResolvedContext.
	 */
	static MOVIERENDERPIPELINECORE_API bool GenerateOcioPixelPreProcessorWithContext(
		const UE::MovieGraph::FMovieGraphSampleState* InPayload, const FOpenColorIODisplayConfiguration& InConfiguration,
		const TMap<FString, FString>& InResolvedContext, TArray<FPixelPreProcessor>& OutPixelPreProcessors);
#endif	// WITH_OCIO

	/**
	 * Resolves an OpenColorIO context with supported tokens.
	 *
	 * @return The resolved key/value context.
	 */
	static MOVIERENDERPIPELINECORE_API TMap<FString, FString> ResolveOpenColorIOContext(
		const TMap<FString, FString>& InContext,
		const FMovieGraphRenderDataIdentifier& InRenderId,
		const UMovieGraphPipeline* InPipeline,
		TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig,
		const FMovieGraphTraversalContext& InTraversalContext);

private:
#if WITH_OCIO
	struct FOpenColorIOPixelPreProcessor
	{
		explicit FOpenColorIOPixelPreProcessor(FOpenColorIOWrapperProcessor&& InProcessor);

		void operator()(const FImagePixelData* PixelData) const;

		FOpenColorIOWrapperProcessor Processor;
	};
	
	/**
	 * Convenience function to create an OpenColorIO CPU processor based on the specified conversion settings.
	 * We use the OpenColorIO processor wrapper directly to avoid concurrency issues with the uobjects lifetime.
	 *
	 * @return The pixel preprocessor if successful, nullptr otherwise.
	*/
	static MOVIERENDERPIPELINECORE_API FPixelPreProcessor CreateOpenColorIOPixelPreProcessor(const FOpenColorIOColorConversionSettings& InConversionSettings, const TMap<FString, FString>& InContext);
#endif

	/* Utility function to warn the user in case they forgot to check "Disable Tone Curve", which in turn controls the render's scene capture source. */
	static MOVIERENDERPIPELINECORE_API void ValidateDisableTonecurve(const UE::MovieGraph::FMovieGraphSampleState& InPayload);
};
