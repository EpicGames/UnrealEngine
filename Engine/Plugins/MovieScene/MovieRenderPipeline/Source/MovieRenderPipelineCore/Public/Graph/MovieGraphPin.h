// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "MovieGraphCommon.h"
#include "MovieGraphPin.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
class UMovieGraphNode;
class UMovieGraphEdge;

struct FPinConnectionResponse;

USTRUCT(BlueprintType)
struct FMovieGraphPinProperties
{
	GENERATED_BODY()

	FMovieGraphPinProperties() = default;
	explicit FMovieGraphPinProperties(const FName& InLabel, const EMovieGraphValueType PinType, const TObjectPtr<const UObject>& TypeObject, bool bInAllowMultipleConnections)
		: Label(InLabel)
		, Type(PinType)
		, TypeObject(TypeObject)
		, bAllowMultipleConnections(bInAllowMultipleConnections)
	{}

	static FMovieGraphPinProperties MakeBranchProperties(const FName& InLabel = NAME_None)
	{
		FMovieGraphPinProperties Properties(InLabel, EMovieGraphValueType::None, nullptr, false);
		Properties.bIsBranch = true;
		return MoveTemp(Properties);
	}

	static FMovieGraphPinProperties MakeWildcardProperties(const FName& InLabel = NAME_None)
	{
		FMovieGraphPinProperties Properties(InLabel, EMovieGraphValueType::None, nullptr, false);
		Properties.bIsWildcard = true;
		return MoveTemp(Properties);
	}

	/** The name assigned to the pin. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	FName Label = NAME_None;

	/** The type of the pin. If the pin represents a branch, this type is ignored. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	EMovieGraphValueType Type = EMovieGraphValueType::Float;

	/** The value type of the pin, if the type is an enum, struct, class, or object. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	TObjectPtr<const UObject> TypeObject;

	/** Whether this pin can accept multiple connections. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	bool bAllowMultipleConnections = true;

	/** Whether this pin represents a branch. If it does not represent a branch, then it is a value-providing pin. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	bool bIsBranch = false;

	/** Whether this pin represents a wildcard, which means it can be any type (including being a branch). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	bool bIsWildcard = false;

	/**
	 * Whether this pin is built-in (ie, the pin ships with the node and cannot be removed). Option pins on the Select
	 * node would be an example of pins which are not built-in (they can be added and removed dynamically).
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	bool bIsBuiltIn = true;

	bool operator==(const FMovieGraphPinProperties& Other) const
	{
		return Label == Other.Label
			&& Type == Other.Type
			&& TypeObject == Other.TypeObject
			&& bAllowMultipleConnections == Other.bAllowMultipleConnections
			&& bIsBranch == Other.bIsBranch
			&& bIsBuiltIn == Other.bIsBuiltIn;
	}

	bool operator !=(const FMovieGraphPinProperties& Other) const
	{
		return !(*this == Other);
	}
};

/** Specifies a restriction on pin properties when searching for a pin on a node. */
UENUM(BlueprintType)
enum class EMovieGraphPinQueryRequirement : uint8
{
	/** The pin must be built-in, meaning that it is always present on the node. */
	BuiltIn,

	/** The pin must be dynamic, meaning that it may not always exist on the node. These are typically user-created pins. */
	Dynamic,

	/** The pin can be either built-in or dynamic. */
	BuiltInOrDynamic
};


UCLASS(MinimalAPI)
class UMovieGraphPin : public UObject
{
	GENERATED_BODY()

public:
	// UObject Interface
#if WITH_EDITOR
	UE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif
	// End UObject Interface
	
	UE_API bool AddEdgeTo(UMovieGraphPin* InOtherPin);
	UE_API bool BreakEdgeTo(UMovieGraphPin* InOtherPin);
	UE_API bool BreakAllEdges();
	UE_API FPinConnectionResponse CanCreateConnection_PinConnectionResponse(const UMovieGraphPin* InOtherPin) const;

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool CanCreateConnection(const UMovieGraphPin* InOtherPin) const;

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool IsConnected() const;

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool IsInputPin() const;
	
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool IsOutputPin() const;

	/** Gets the number of edges connected to this pin. May include invalid edges (which can happen if connected to an invalid node). */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API int32 EdgeCount() const;

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool AllowsMultipleConnections() const;

	/**
	 * Gets the first pin connected to this pin. Returns nullptr if no valid connection exists. If bFollowRerouteConnections is true,
	 * this method will continue traversing the graph past reroute nodes until a non-reroute pin is found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API UMovieGraphPin* GetFirstConnectedPin(const bool bFollowRerouteConnections = false) const;

	/** Gets all connected pins. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API TArray<UMovieGraphPin*> GetAllConnectedPins() const;

	/**
	* Utility function for scripting which gathers all of the nodes connected
	* to this particular pin. Equivalent to looping through all of the edges,
	* getting the connected pin, and then getting the node associated with that pin.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API TArray<UMovieGraphNode*> GetConnectedNodes() const;

	/**
	 * Determines if the connection between this pin and OtherPin follows branch restriction rules. OutError is populated
	 * with an error if the connection should be rejected and the function will return false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool IsConnectionToBranchAllowed(const UMovieGraphPin* OtherPin, FText& OutError) const;

	/** Determines if this pin's direction is compatible with another pin's direction. One needs to be an input, and the other an output. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool IsPinDirectionCompatibleWith(const UMovieGraphPin* OtherPin) const;

	/**
	 * Determines if this pin's type is compatible with another pin's type. Generally the types have to match exactly in order to be compatible, but
	 * there may be cases where different types can be used interchangeably.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool IsTypeCompatibleWith(const UMovieGraphPin* InOtherPin) const;

public:
	// The node that this pin belongs to.
	UPROPERTY(BlueprintReadOnly, Category = "Properties")
	TObjectPtr<UMovieGraphNode> Node = nullptr;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Properties", meta = (ShowOnlyInnerProperties))
	FMovieGraphPinProperties Properties;

	/** 
	* A list of edges between pins. This is marked as TextExportTransient so that when we copy/paste nodes,
	* we don't copy the edges, as they are rebuilt after paste based on the editor graph connections.
	*/
	UPROPERTY(BlueprintReadOnly, TextExportTransient, Category = "Properties")
	TArray<TObjectPtr<UMovieGraphEdge>> Edges;

private:
	/**
	 * Given a pair of input and output pins which a connection was made between, propagate the type either upstream or downstream depending on
	 * the nodes the connection was made between.
	 */
	UE_API void PropagatePinType(const UMovieGraphPin* InInputPin, const UMovieGraphPin* InOutputPin);

	/** Propagate the given properties either upstream or downstream from this pin. This pin is not changed. */
	UE_API void PropagatePinProperties(const FMovieGraphPinProperties PinProperties, const bool bPropagateUpstream) const;

	/**
	 * Potentially reverts InPin (and the reroute nodes connected to/from it) to a wildcard if InPin no longer has any upstream or downstream
	 * connections that provide a type. In other words, InPin is part of a reroute "island" where the reroute nodes are only connected to other
	 * reroute nodes.
	 */
	UE_API void MaybeRevertToWildcard(const UMovieGraphPin* InPin);
	
	// Recursion guard boolean to prevent PropagatePinProperties from hanging if there is a cycle of reroute nodes.
	mutable bool bPropagateTypeRecursionGuard = false;
};

#undef UE_API
