// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "HAL/ThreadSafeBool.h"
#include "MoviePipelineBase.h"
#include "MovieGraphDataTypes.h"
#include "MovieGraphConfig.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MovieGraphPipeline.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;
class UMovieGraphFileOutputNode;
class UMovieGraphScriptBase;
class IImageWriteQueue;
struct FMovieGraphRenderOutputData;

namespace UE::MovieGraph
{
	typedef TTuple<TFuture<bool>, UE::MovieGraph::FMovieGraphOutputFutureData> FMovieGraphOutputFuture;

	namespace Private
	{
		class FMovieGraphCVarManager;
	}
}

UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphPipeline : public UMoviePipelineBase
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphPipeline();

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API void Initialize(UMoviePipelineExecutorJob* InJob, const FMovieGraphInitConfig& InitConfig);

	/**
	* Returns the internal job being used by the Movie Graph Pipeline, which is a duplicate of
	* the job provided originaly by Initialize. This allows scripting to mutate the job/configuration
	* without leaking changes into assets or the original user-defined queue entry.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UMoviePipelineExecutorJob* GetCurrentJob() const { return CurrentJobDuplicate; }

	/**
	* Returns the time this movie pipeline was initialized at.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	FDateTime GetInitializationTime() const { return GraphInitializationTime; }

	/** 
	* The offset that should be applied to the GetInitializationTime() when generating
	* the {time} related filename tokens. GetInitializationTime() is in UTC so this is
	* either zero (if you called SetInitializationTime) or your offset from UTC.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	FTimespan GetInitializationTimeOffset() const { return InitializationTimeOffset; }


	/**
	* Override the time this movie pipeline was initialized at. This can be used for render farms
	* to ensure that jobs on all machines use the same date/time instead of each calculating it locally.
	* Clears the auto-calculated InitializationTimeOffset, meaning time tokens will be written in UTC.
	*
	* Needs to be called after ::Initialize(...)
	*
	* @param InDateTime - Expected to be in UTC timezone.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	void SetInitializationTime(const FDateTime& InDateTime) { GraphInitializationTime = InDateTime; InitializationTimeOffset = FTimespan(); }

	/** Gets the graph config for the shot if one was specified for the shot. Otherwise, gets the graph config for the associated primary job. */
	UE_API UMovieGraphConfig* GetRootGraphForShot(UMoviePipelineExecutorShot* InShot) const;

	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API FMovieGraphTraversalContext GetCurrentTraversalContext(const bool bForShot = true) const;

	/** Get the Active Shot list, which is the full shot list generated from the external data source, with disabled shots removed. */
	const TArray<TObjectPtr<UMoviePipelineExecutorShot>>& GetActiveShotList() const { return ActiveShotList; }
	
	/** Which index of the Active Shot List are we currently on */
	int32 GetCurrentShotIndex() const { return CurrentShotIndex; }

	/** Get the pointer to the Custom Engine Timestep we use to control engine ticking. Shared by all per-shot Time Steps.*/
	TObjectPtr<UMovieGraphEngineTimeStep> GetCustomEngineTimeStep() const { return CustomEngineTimeStep; }
	
	/** Called by the TimeStepInstance when it's time to set up for another shot. Don't call this unless you know what you're doing. */
	UE_API void SetupShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot);

	/** Prepares shot hierarchy and per-shot settings before the shot's first data source evaluation. */
	UE_API void BeginShotSetup(const TObjectPtr<UMoviePipelineExecutorShot>& InShot);

	/** Initializes render resources that require the shot's data source to have evaluated at least once. */
	UE_API void FinishShotSetup(const TObjectPtr<UMoviePipelineExecutorShot>& InShot);
	
	/** Called by the TimeStepInstance when it's time to tear down the current shot. Don't call this unless you know what you're doing. */
	UE_API void TeardownShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot);
	
	/** Used occasionally to cross-reference other components. Don't call this unless you know what you're doing. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API UMovieGraphTimeStepBase* GetTimeStepInstance() const;
	
	/** Used occasionally to cross-reference other components. Don't call this unless you know what you're doing. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UMovieGraphRendererBase* GetRendererInstance() const { return GraphRendererInstance; }
	
	/** Used occasionally to cross-reference other components. Don't call this unless you know what you're doing. */
	UMovieGraphDataSourceBase* GetDataSourceInstance() const { return GraphDataSourceInstance; }

	/** Used occasionally to cross-reference other components. Don't call this unless you know what you're doing. */
	UMovieGraphAudioRendererBase* GetAudioRendererInstance() const { return GraphAudioRendererInstance; }
	
	/** Gets the Output Merger for this Movie Pipeline which is responsible for gathering all of the data coming in for a given output frame, before making it available to the MovieGraphPipeline. */
	TSharedPtr<UE::MovieGraph::IMovieGraphOutputMerger> GetOutputMerger() const { return OutputMerger; }

	/** Gets cached data for cloth simulations, needed to save/restore cloth sub-stepping. */
	const TMap<TWeakObjectPtr<UObject>, TArray<MoviePipeline::FClothSimSettingsCache>>& GetClothSimCache() const { return ClothSimCache; }

	/** Gets cached data for cloth simulations, needed to save/restore cloth sub-stepping. */
	TMap<TWeakObjectPtr<UObject>, TArray<MoviePipeline::FClothSimSettingsCache>>& GetClothSimCache() { return ClothSimCache; }

	/** Writing images to disk is an async process. When you start writing, declare a future with the filename you will eventually write to, and complete the future once it is on disk. */
	UE_API void AddOutputFuture(TFuture<bool>&& InOutputFuture, const UE::MovieGraph::FMovieGraphOutputFutureData& InData);
	
	/** Used by the Renderer Instance to disable the preview widget before rendering so it isn't baked into the UI Renderer. */
	void SetPreviewWidgetVisible(bool bInIsVisible) { SetPreviewWidgetVisibleImpl(bInIsVisible); }

	/**
	 * Gets render data that was generated for each shot. This data is mutable, so processes that run after renders
	 * complete can add their additional outputs to the generated data. Care should be taken when removing output data.
	 */
	UE_API TArray<FMovieGraphRenderOutputData>& GetGeneratedOutputData();

	/**
	 * Resets the render layer subsystem and updates it to reflect the render layers that are present in the evaluated
	 * graph. Does not update the contents of the layers, however. See UpdateLayerContentsInRenderLayerSubsystem().
	 */
	UE_API void CreateLayersInRenderLayerSubsystem(const UMovieGraphEvaluatedConfig* EvaluatedConfig) const;

	/**
	 * Updates the contents of the layers that are currently present in the render layer subsystem by evaluating all
	 * available collections and modifiers. To update the layers that are present in the subsystem, see
	 * CreateLayersInRenderLayerSubsystem().
	 */
	UE_API void UpdateLayerContentsInRenderLayerSubsystem(const UMovieGraphEvaluatedConfig* EvaluatedConfig) const;

protected:
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API virtual void OnMoviePipelineFinishedImpl();

	UE_API virtual void OnEngineTickBeginFrame();
	UE_API virtual void OnEngineTickEndFrame();
	UE_API virtual void RenderFrame();
	UE_API virtual void BuildShotListFromDataSource();

	/**
	 * Sends the rendered pixel data to all file output nodes (they will use the "mask" provided to determine if they need to process the data).
	 * OnReceiveImageData() will be called on each of these nodes, and in turn they will typically call AddOutputFuture() on the pipeline to
	 * provide a future that communicates the status of their write task.
	 */
	UE_API virtual void ProcessOutstandingFinishedFrames();

	/**
	 * Processes any futures that have completed by updating GeneratedOutputData. Futures are processed sequentially in the order that they're
	 * generated. If the future next-in-line to be processed has not completed, this will early exit, and must be called again to continue
	 * processing futures. Optionally, all futures can be processed by setting "bStallAndProcessAllFutures" to true; this will still process
	 * all futures sequentially, but waits for the current one to finish before moving on to the next (and so on).
	 */
	UE_API virtual void ProcessOutstandingFutures(const bool bStallAndProcessAllFutures = false);

	UE_API virtual void TickProducingFrames();
	UE_API virtual void TickPostFinalizeExport(const bool bInForceFinish);
	UE_API virtual void TickFinalizeOutputContainers(const bool bInForceFinish);
	UE_API virtual void TransitionToState(const EMovieRenderPipelineState InNewState);
	UE_API virtual const TSet<TObjectPtr<UMovieGraphFileOutputNode>> GetOutputNodesUsed() const;
	UE_API virtual void BeginFinalize();

	/** Begins the export process for a primary job (not called for shot jobs). */
	UE_API virtual void BeginExport();
	
	/** Attempts to start an Unreal Insights capture to a file on disk adjacent to the movie output. */
	UE_API void StartUnrealInsightsCapture(UMovieGraphEvaluatedConfig* EvaluatedConfig);
	/** Attempts to stop an already started Unreal Insights capture. */
	UE_API void StopUnrealInsightsCapture();
	
	UE_API virtual void LoadPreviewWidget();
	UE_API virtual void SetPreviewWidgetVisibleImpl(bool bInIsVisible);

	UE_API virtual void DuplicateJobAndConfiguration();
	UE_API virtual void ExecutePreJobScripts();
	UE_API virtual void ExecutePostJobScripts(const FMoviePipelineOutputData& InData);
	UE_API virtual void ExecutePreShotScripts(const TObjectPtr<UMoviePipelineExecutorShot>& InShot);
	UE_API virtual void ExecutePostShotScripts(const FMoviePipelineOutputData& InData);

	/** Helps update variable assignments on the provided job to use duplicated graphs (reflected in the original-to-duplicate graph mapping). */
	template<typename JobType>
	void UpdateVariableAssignmentsHelper(JobType* InTargetJob, TMap<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>>& InOriginalToDuplicateGraphMap);

	// Update our data source to isolate the shot we're currently working on, so that expanded shots don't interfere with each other.
	UE_API virtual void SetSoloShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot);
	
	/*
	* Expand our data source (for a specific shot) for various rendering features (such as handle frames). Gets called twice, once where it 
	* expands the ranges, and again later to actually expand the tracks. This is required due to wanting "Handle Frames" to get counted in total frame counts.
	*/
	UE_API virtual void ExpandShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot, const int32 InNumHandleFrames, const bool bInHasMultipleTemporalSamples, const bool bIsPrePass,
		const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution, const int32 InWarmUpFrames);

	/** Resolve the version number that should be used for the specified shot (in {version} tokens). */
	UE_API int32 ResolveVersionForShot(const TObjectPtr<UMoviePipelineExecutorShot>& Shot, const TObjectPtr<UMovieGraphEvaluatedConfig>& EvaluatedConfig);

	// UMoviePipelineBase Interface
	UE_API virtual void RequestShutdownImpl(bool bIsError) override;
	UE_API virtual void ShutdownImpl(bool bIsError) override;
	virtual bool IsShutdownRequestedImpl() const override { return bShutdownRequested; }
	virtual EMovieRenderPipelineState GetPipelineStateImpl() const override { return PipelineState; }
	UE_API virtual bool IsPostShotCallbackNeeded() const override;
	// ~UMoviePipelineBase Interface
	
	// UObject Interface
	UE_API virtual void BeginDestroy() override;
	// ~UObject Interface

protected:
	/** Time step instances for each shot, where the index into the array corresponds to the shot index. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMovieGraphTimeStepBase>> GraphTimeStepInstances;

	/** 
	* If set, on the next TickProducingFrames, the GraphTimeStepInstance pointer will be swapped with this one.
	* Uses a deferred mechanism since SetupShot/TeardownShot are called by the current instance, so we don't want
	* to swap until the current instance isn't actively being used.
	*/
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphTimeStepBase> PendingTimeStepInstance;

	/**
	 * Sometimes the shot index can be incremented before the time step instance should be changed, so the current time
	 * step instance is tracked. This should generally be used to access the current time step instance, rather than
	 * indexing into GraphTimeStepInstances with the shot index.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphTimeStepBase> GraphTimeStepInstance;

	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphRendererBase> GraphRendererInstance;

	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphDataSourceBase> GraphDataSourceInstance;

	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphAudioRendererBase> GraphAudioRendererInstance;

	/**
	 * The evaluated graph that should be referenced after all shot rendering is complete. This graph will originate from a primary job (not shot) and
	 * can be used in post-rendering tasks.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphEvaluatedConfig> PostRenderEvaluatedGraph;

protected:
	/**
	* This is the job as was provided to the ::Initialize() call, which we duplicate
	* to allow modifications to scripting without leaking changes into assets.
	*/
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineExecutorJob> CurrentJob;

	/**
	* This is the duplicated job, parented to the pipeline. The shots inside
	* have been duplicated as well, and their graph configurations duplicated. Graph
	* configurations are assets and scripting may want to modify them, or it may want
	* to modify the variables in a job, so we have to duplicate both to allow a cleanly
	* mutable version for scripting.
	*/
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineExecutorJob> CurrentJobDuplicate;

	/**
	 * Contains all duplicated graphs (for scripting purposes). Maps the original graph (key) to the duplicated graph (value).
	 * Duplicated graphs are part of the transient package.
	 */
	UPROPERTY(Transient)
	TMap<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>> DuplicatedGraphs;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMoviePipelineExecutorShot>> ActiveShotList;

	UPROPERTY(Transient)
	TSubclassOf<UMovieGraphRenderPreviewWidget> PreviewWidgetClassToUse;

	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphRenderPreviewWidget> PreviewWidget;


	UPROPERTY(Transient)
	TArray<TObjectPtr<UMovieGraphScriptBase>> CurrentScriptInstances;

	/**
	* An array of Node CDOs that we sent data through to write data to disk.
	* This is the list of output nodes that will have OnAllFramesSubmitted/IsFinishedWritingToDisk
	* on them.
	*/
	UPROPERTY(Transient)
	TSet<TObjectPtr<UMovieGraphFileOutputNode>> OutputNodesDataSentTo;

	int32 CurrentShotIndex;

	/** Output frame number at the start of the current shot. */
	int32 CurrentShotStartOutputFrameNumber = 0;

	/** True if we're in a TransitionToState call. Used to prevent reentrancy. */
	bool bIsTransitioningState;

	/** True if we're in a TeardownShot call. Used to prevent reentrancy. */
	bool bIsTearingDownShot;

	/** When we originally initialize we store the offset from UTC (which is what GetInitializationTime() is in), but we clear this if you call SetInitializationTime. */
	FTimespan InitializationTimeOffset;

	/** True if RequestShutdown() was called. At the start of the next frame we will stop producing frames and start shutting down. */
	FThreadSafeBool bShutdownRequested;

	/** Set to true during Shutdown/RequestShutdown if we are shutting down due to an error. */
	FThreadSafeBool bShutdownSetErrorFlag;

	/** Which step of the rendering process is the graph currently in. */
	EMovieRenderPipelineState PipelineState;

	/** What time (in UTC) was Initialization called? Used internally for tracking total job duration. */
	FDateTime GraphInitializationTime;

	/** This gathers all of the produced data for an output frame (which may come in async, many frames later) before passing them onto the Output Containers. */
	TSharedPtr<UE::MovieGraph::IMovieGraphOutputMerger> OutputMerger;

	/** 
	* An array of Output Futures for files that have started writing to disk, but have not finished. 
	* Each frame we take each future that finishes writing to disk and push it into the GeneratedOutputData array.
	*/
	TArray<UE::MovieGraph::FMovieGraphOutputFuture> OutstandingOutputFutures;

	/**
	* An array of output data, one per shot. This only keeps track of the filenames generated during
	* rendering (and is only pushed to once the file is actually on disk), and does not store pixel data.
	*/
	TArray<FMovieGraphRenderOutputData> GeneratedOutputData;

	/** Responsible for managing cvars throughout the lifetime of the pipeline. */
	TSharedPtr<UE::MovieGraph::Private::FMovieGraphCVarManager> CVarManager;

	/**
	* A custom timestep created by the Movie Graph Pipeline to allow the time step managers to set what the delta times for the frame should be.
	*/
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphEngineTimeStep> CustomEngineTimeStep;

	/** The previous custom timestep the engine was using, if any. */
	UPROPERTY(Transient)
	TObjectPtr<UEngineCustomTimeStep> PrevCustomEngineTimeStep;

	/** Simulation settings cache per cloth interactor object. Needs one per LOD, hence the array. */
	TMap<TWeakObjectPtr<UObject>, TArray<MoviePipeline::FClothSimSettingsCache>> ClothSimCache;
	
	/** Whether an insights capture was started when the pipeline was initialized. */
	bool bDidStartInsightsCapture;

	/** Whether the original job used Basic config before it was converted to a graph. */
	bool bWasInitializedWithBasicConfig = false;
	
public:
	static UE_API FString DefaultPreviewWidgetAsset;
};

#undef UE_API
