// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/MovieGraphNamedResolution.h"

#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#include "MovieGraphProjectSettings.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Settings that apply to the Movie Graph.
 */
UCLASS(MinimalAPI, BlueprintType, config = MovieRenderPipeline, DefaultConfig, meta=(DisplayName = "Movie Graph Settings"))
class UMovieGraphProjectSettings : public UDeveloperSettings
{
public:
	GENERATED_BODY()

	UE_API UMovieGraphProjectSettings();

	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }
	
	/**
	 * Tries to find a match for the given option in DefaultNamedResolutions.
	 * @param InOption The name of the option.
	 * @return A raw pointer to an FMovieGraphNamedResolution if a match is found, nullptr otherwise.
	 */
	UE_API const FMovieGraphNamedResolution* FindNamedResolutionForOption(const FName& InOption) const;
	
	/**
	 * A list of default resolutions to render with, defined in Config/BaseMovieRenderPipeline.ini or Project Settings.
	 * These resolutions will appear in the MovieGraph's various output settings nodes.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Movie Graph")
	TArray<FMovieGraphNamedResolution> DefaultNamedResolutions;

private:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UE_API void EnsureUniqueNamesInDefaultNamedResolutions();
	
};

#undef UE_API
