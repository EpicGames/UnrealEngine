// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieRenderPipelineDataTypes.h"

#include "MovieGraphQuickRenderSettings.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMovieGraphConfig;
class UMovieJobVariableAssignmentContainer;

/** The type of render that the Quick Render toolbar button performs. */
UENUM(BlueprintType)
enum class EMovieGraphQuickRenderButtonMode : uint8
{
	/** Uses the active queue in the Movie Render Queue editor to do a render. */
	NormalMovieRenderQueue,

	/** Use the Quick Render configuration to do a render. */
	QuickRender
};

/** The available modes for Quick Render to be put into. */
UENUM(BlueprintType)
enum class EMovieGraphQuickRenderMode : uint8
{
	/** Renders the level sequence that is currently active in Sequencer. */
	CurrentSequence,

	/** Renders the level sequence active in Sequencer, but uses the viewport's camera instead (for the entire duration of the render). */
	UseViewportCameraInSequence,

	/** Renders a single frame using the current viewport's camera. */
	CurrentViewport UMETA(DisplayName="Current Viewport (Still)"),

	/** Renders one frame for each camera actor selected in the Outliner. */
	SelectedCameras UMETA(DisplayName="Selected Cameras (Still)"),

	/** Renders the shot that Sequencer's playhead is currently at. This mode is currently not implemented. */
	/* CurrentShotAtPlayhead, */
};

/** The frame range within the active level sequence that a Quick Render should use. */
UENUM(BlueprintType)
enum class EMovieGraphQuickRenderFrameRangeType : uint8
{
	/** Use the frame range specified by the sequence's start and end frame. */
	PlaybackRange,

	/** Use the sequence's selection as the start and end frame. */
	SelectionRange,

	/** Use a custom frame range. */
	Custom
};

UE_DEPRECATED(5.8, "Use EMoviePipelinePostRenderActionType instead")
typedef EMoviePipelinePostRenderActionType EMovieGraphQuickRenderPostRenderActionType;

/** The aspects of the viewport that should be applied to the quick render. */
UENUM(BlueprintType, Flags, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMovieGraphQuickRenderViewportLookFlags : uint8
{
	None		= 0	UMETA(Hidden),
	Ocio		= 1 << 0 UMETA(DisplayName = "OCIO"),
	ShowFlags	= 1 << 1,
	ViewMode	= 1 << 2,
	Visibility	= 1 << 3,
	EditorOnlyActors = 1 << 4 UMETA(DisplayName = "Show Editor-Only Actors")
};
ENUM_CLASS_FLAGS(EMovieGraphQuickRenderViewportLookFlags)

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE(FOnMovieGraphQuickRenderGraphChanged);
#endif

/** Data that specifies user-configurable aspects of a quick render. Settings are stored for each mode separately. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphQuickRenderSettings : public UObject
{
	GENERATED_BODY()

public:
	UMovieGraphQuickRenderSettings() = default;
	
	/** Gets the Quick Render settings (for a specific mode) that have been persisted to disk (Saved/MovieRenderPipeline/QuickRenderSettings.uasset). */
	UFUNCTION(BlueprintCallable, Category="Quick Render")
	static UE_API UMovieGraphQuickRenderModeSettings* GetSavedQuickRenderModeSettings(EMovieGraphQuickRenderMode SettingsMode);

private:
	inline static const FString QuickRenderSettingsPackagePath = TEXT("/Temp/MovieRenderPipeline/QuickRenderSettings");
	UE_API inline static FDelegateHandle OnEnginePreExitHandle;

	/**
	 * Individual settings for each mode. Some modes have shared settings, hence why this is keyed by FName (an internal identifier) instead of
	 * the mode enum.
	 */
	UPROPERTY()
	TMap<FName, TObjectPtr<UMovieGraphQuickRenderModeSettings>> ModeSettings;

private:
#if WITH_EDITOR
	/** Saves the settings to a uasset file (Saved/MovieRenderPipeline/QuickRenderSettings.uasset). The file will be created if it doesn't exist. */
	static UE_API void SaveSettings(const UMovieGraphQuickRenderSettings* InSettings);

	// Individual mode settings objects need to notify this parent settings container that it should save.
	friend class UMovieGraphQuickRenderModeSettings;
	UE_API void NotifyNeedsSave();
#endif	// WITH_EDITOR
};

/** Settings for a specific mode within Quick Render. Note that some modes share settings. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphQuickRenderModeSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphQuickRenderModeSettings();

	/**
	 * Within the settings object provided, refreshes the variable assignments in the settings to match the setting's GraphPreset. Note that
	 * changing the graph preset within the editor will do this automatically, but if changing the graph outside of the editor, this may need to be
	 * run manually.
	 */
	UFUNCTION(BlueprintCallable, Category="Quick Render")
	static UE_API void RefreshVariableAssignments(UMovieGraphQuickRenderModeSettings* InSettings);

	/**
	 * Gets the variable assignments for the specified graph asset. Note that there will normally only be one variable assignment container for these
	 * quick render settings (the assignments for the graph specified in `GraphPreset`). However, `GraphPreset` may have subgraphs, and subgraphs
	 * store their variable assignments in separate containers. If you need to update the variable assignments for those subgraph(s), this method
	 * provides a convenient way of getting their assignments (instead of iterating through `GraphVariableAssignments`).
	 */
	UFUNCTION(BlueprintCallable, Category="Quick Render")
	UE_API UMovieJobVariableAssignmentContainer* GetVariableAssignmentsForGraph(const TSoftObjectPtr<UMovieGraphConfig>& InGraphConfigPath) const;

#if WITH_EDITOR
	FOnMovieGraphQuickRenderGraphChanged OnGraphChangedDelegate;
#endif
	
	/** The graph preset that is used to configure the render. If not specified, the quick render default graph will be used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Configuration", DisplayName="Movie Render Graph Config")
	TSoftObjectPtr<UMovieGraphConfig> GraphPreset;
	
	/**
	 * If this override is set, this sequence will be used instead of the level sequence that's active in Sequencer. In UI-based Quick Render, this
	 * property is never set directly because Sequencer is always used. However, for scripting purposes, there may be cases where Sequencer is not
	 * open, and a specific level sequence needs to be used.
	 */
	UPROPERTY(BlueprintReadWrite, Category="Configuration")
	TSoftObjectPtr<class ULevelSequence> LevelSequenceOverride = nullptr;

	/** The action that quick render should perform after a render is finished. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Configuration", DisplayName="After Render")
	EMoviePipelinePostRenderActionType PostRenderBehavior = EMoviePipelinePostRenderActionType::PlayRenderOutput;

	/** Enables/disables the ViewportLookFlags property from taking effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Quick Render", meta=(InlineEditConditionToggle))
	bool bOverride_ViewportLookFlags = false;

	// Note: ViewportLookFlags is an int32 instead of the enum to satisfy the requirement of the Bitmask/BitmaskEnum metadata; this can be changed back to the enum once
	// a proper customization per the design is done.
	
	/** The properties of the viewport that should be applied to the quick render. These will override any equivalent properties specified in the graph. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Quick Render", DisplayName="Apply Viewport Look", meta=(EditCondition="bOverride_ViewportLookFlags", Bitmask, BitmaskEnum="/Script/MovieRenderPipelineCore.EMovieGraphQuickRenderViewportLookFlags"))
	int32 ViewportLookFlags = static_cast<int32>(EMovieGraphQuickRenderViewportLookFlags::None);

	/** The frame range that should be used to render from. Not applicable to modes that only render stills. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Quick Render", DisplayName="Sequencer Frame Range")
	EMovieGraphQuickRenderFrameRangeType FrameRangeType = EMovieGraphQuickRenderFrameRangeType::PlaybackRange;

	/** The frame in the level sequence that the render will begin on. This frame number is inclusive, meaning that this frame will be included in the rendered frames. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Quick Render", meta=(EditCondition="FrameRangeType == EMovieGraphQuickRenderFrameRangeType::Custom", EditConditionHides))
	int32 CustomStartFrame = 0;

	/** The frame in the level sequence that the render will end on. This frame number is exclusive, meaning that the rendered frames will include the frames up until (but excluding) this frame. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Quick Render", meta=(EditCondition="FrameRangeType == EMovieGraphQuickRenderFrameRangeType::Custom", EditConditionHides))
	int32 CustomEndFrame = 0;

	/** Values that are set on the variables contained within the graph preset. See note on RefreshVariableAssignments(). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Primary Graph Variables")
	TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>> GraphVariableAssignments;

private:
#if WITH_EDITOR
	//~ Begin UObject Interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	//~ End UObject Interface
#endif
};

#undef UE_API
