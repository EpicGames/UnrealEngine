// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Camera/CameraTypes.h"
#include "SceneView.h"

class FReferenceCollector;
class AActor;
class UMovieGraphPipeline;
struct FMovieGraphTimeStepData;

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Base class for a custom camera source created by a live UMovieGraphRenderPassNode instance
 * (not the class default object). Supplies a set of cameras that replaces the cameras from
 * the data source for the render pass node that created it.
 */
struct FMovieGraphRenderCameraSource
{
	virtual ~FMovieGraphRenderCameraSource() = default;

	/**
	 * Adds any UObject references held by this source to the collector for GC tracking.
	 * Subclasses that hold UObject pointers must override this to prevent those objects
	 * from being garbage collected while the source is alive.
	 *
	 * @param Collector (in) The reference collector provided by the GC system.
	 */
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) {}

	/**
	 * Called once per frame to prepare the source for a new frame. Must be called before querying camera data.
	 * 
	 * @param InMoviePipeline (in) The pipeline currently being evaluated.
	 * @param InTimeData      (in) Time step data for the current frame (frame time, delta time, etc.).
	 */
	UE_API virtual void OnBeginFrame(const UMovieGraphPipeline* InMoviePipeline, const FMovieGraphTimeStepData* InTimeData) { }

	/** Returns the number of cameras this source contributes for the current shot. */
	UE_API virtual int32 GetNumCameras() const { return 0; }

	/**
	 * Returns the unique camera name for the given camera index.
	 * Used by the renderer for the {camera_name} output token.
	 *
	 * @param InCameraIndex (in)  Index in range [0, NumCameras).
	 * @param OutCameraName (out) Receives the camera name for the requested index.
	 * @return              True if resolved; false to fall back to the shot's camera name.
	 */
	UE_API virtual bool GetCameraName(const int32 InCameraIndex, FString& OutCameraName) const { return false; }

	/**
	 * Returns the view actor for the specified camera index.
	 * Returning true with OutCameraActor = nullptr is valid and suppresses the default fallback.
	 *
	 * @param InCameraIndex  (in)  Index in range [0, NumCameras).
	 * @param OutCameraActor (out) Receives the actor used as the view origin, or nullptr.
	 * @return               True if the source handled this query; false to fall back to the player controller's view target.
	 */
	UE_API virtual bool GetCameraViewActor(const int32 InCameraIndex, AActor*& OutCameraActor) const { return false; }

	/**
	 * Gets the overscan value for the specified camera.
	 *
	 * @param InCameraIndex     (in)  Index in range [0, NumCameras).
	 * @param OutCameraOverscan (out) Receives the overscan value for the camera.
	 * @return                  True if resolved; false to fall back to the default sequence camera overscan.
	 */
	UE_API virtual bool GetCameraOverscan(const int32 InCameraIndex, float& OutCameraOverscan) const { return false; }

	/**
	 * Returns the view info for the specified camera index.
	 *
	 * @param InCameraIndex (in)  Index in range [0, NumCameras).
	 * @param OutViewInfo   (out) Receives the view transform, FOV, and other camera parameters.
	 * @return              True if resolved; false to fall back to the primary sequence camera.
	 */
	UE_API virtual bool GetCameraViewInfo(const int32 InCameraIndex, FMinimalViewInfo& OutViewInfo) const { return false; }

	/**
	 * Applies camera-specific adjustments to the projection data after it has been computed.
	 *
	 * @param InCameraIndex       (in)     Index in range [0, NumCameras).
	 * @param InOutProjectionData (in,out) Projection data to adjust; already populated by the time this is called.
	 */
	UE_API virtual void SetupCameraViewProjectionData(const int32 InCameraIndex, FSceneViewProjectionData& InOutProjectionData) const {}

	/**
	 * Returns the overscanned resolution and inner crop rectangle for the specified camera.
	 *
	 * @param InCameraIndex              (in)  Index in range [0, NumCameras).
	 * @param InOutOverscannedResolution (in,out) Receives the overscanned resolution.
	 * @param InOutOverscanCropRectangle (in,out) Receives the inner crop rectangle within the overscanned resolution.
	 * @param InOutOverscanFraction      (in,out) The final per-side overscan fraction [0..1] for this camera.
	 * @return                           True if resolved; false to fall back to the default resolution calculation.
	 */
	UE_API virtual bool GetCameraOverscannedResolution(
		const int32 InCameraIndex,
		FIntPoint&  InOutOverscannedResolution,
		FIntRect&   InOutOverscanCropRectangle,
		float&      InOutOverscanFraction) const { return false; }
};

#undef UE_API