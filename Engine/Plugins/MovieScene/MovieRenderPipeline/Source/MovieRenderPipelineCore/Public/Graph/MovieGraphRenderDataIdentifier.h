// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "MovieGraphRenderDataIdentifier.generated.h"

/**
* This data structure can be used to identify what render data a set of pixels represents
* by knowing what the render layer name is, what renderer produced it, if it's a sub-resource,
* and what camera it is for. Can be used as the key in a TMap.
*/
USTRUCT(BlueprintType)
struct FMovieGraphRenderDataIdentifier
{
	GENERATED_BODY()

	FMovieGraphRenderDataIdentifier()
	{}

	FMovieGraphRenderDataIdentifier(const FName& InRootBranchName, const FString& InLayerName, const FString& InRendererName,
		const FString& InSubRenderResourceName, const FString& InCameraName)
		: RootBranchName(InRootBranchName)
		, LayerName(InLayerName)
		, RendererName(InRendererName)
		, SubResourceName(InSubRenderResourceName)
		, CameraName(InCameraName)
	{
	}

	bool operator == (const FMovieGraphRenderDataIdentifier& InRHS) const
	{
		return RootBranchName == InRHS.RootBranchName && 
			LayerName == InRHS.LayerName &&
			RendererName == InRHS.RendererName &&
			SubResourceName == InRHS.SubResourceName &&
			CameraName == InRHS.CameraName;
	}

	bool operator != (const FMovieGraphRenderDataIdentifier& InRHS) const
	{
		return !(*this == InRHS);
	}

	friend uint32 GetTypeHash(FMovieGraphRenderDataIdentifier InIdentifier)
	{
		return HashCombineFast(GetTypeHash(InIdentifier.RootBranchName),
			HashCombineFast(GetTypeHash(InIdentifier.LayerName),
				HashCombineFast(GetTypeHash(InIdentifier.RendererName),
					HashCombineFast(GetTypeHash(InIdentifier.SubResourceName),
						GetTypeHash(InIdentifier.CameraName)))));
	}

	friend FString LexToString(const FMovieGraphRenderDataIdentifier InIdentifier)
	{
		return FString::Printf(TEXT("LayerName: %s RootBranch: %s Renderer:%s SubResource: %s Camera: %s"), *InIdentifier.LayerName, *InIdentifier.RootBranchName.ToString(), *InIdentifier.RendererName, *InIdentifier.SubResourceName, *InIdentifier.CameraName);
	}

	bool IsBranchAndCameraEqual(const FMovieGraphRenderDataIdentifier& InIdentifier) const
	{
		return CameraName == InIdentifier.CameraName &&
			RootBranchName == InIdentifier.RootBranchName;
	}

public:
	/** 
	* The root branch name from the Outputs node that this identifier is for. This is useful to 
	* know which branch it came from, as RenderLayer is user-defined and can be redefined multiple
	* times within one graph. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Graph")
	FName RootBranchName;

	/**
	* The Render Layer name (as defined by the "Render Layer" node.). This is effectively a 
	* "display" name for identifiers. If there is no Render Layer node then this will be the 
	* RootBranchName (so that the {render_layer} token works in the case of data from things 
	* on the Globals branch).
	*/ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Graph")
	FString LayerName;

	/** Which renderer was used to produce this image ("panoramic" "deferred" "path tracer", etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Graph")
	FString RendererName;

	/** A sub-resource name for the renderer (ie: "beauty", "object id", "depth", etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Graph")
	FString SubResourceName;

	/** The name of the camera being used for this render. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Graph")
	FString CameraName;
};

/** A simple struct wrapper around a string array so it can be included in a TMap. */
USTRUCT(BlueprintType)
struct FMovieGraphStringArray
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TArray<FString> Array;
};

USTRUCT(BlueprintType)
struct FMovieGraphRenderLayerOutputData
{
	GENERATED_BODY()
public:
	/** A list of file paths on disk (in order) that were generated for this particular render pass. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TArray<FString> FilePaths;
	
	/**
	 * The "frame templated" file paths. This groups together a set of output frames by frame number, eg "Seq.Shot.{frame_placeholder}.exr".
	 * If the render layer has more than one output type in it (eg, EXR and JPEG), there may be more than one path here, but there should only
	 * be one per output type.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TArray<FString> FrameTemplatedFilePaths;
	
	/** Maps a node type to all of the files that the node type generated. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TMap<FSoftClassPath, FMovieGraphStringArray> NodeTypeToFilePaths;

	/** The index of the render layer these files belong to. Index 0 is the first render layer rendered, and so on. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	int32 RenderLayerIndex = 0;
};

USTRUCT(BlueprintType)
struct FMovieGraphRenderOutputData
{
	GENERATED_BODY()
public:
	/** Which shot is this output data for. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Graph")
	TWeakObjectPtr<class UMoviePipelineExecutorShot> Shot;

	/**
	* A mapping between render layers (such as "beauty") and an array containing the files written for that shot.
	* Will be multiple files if using image sequences.
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Graph")
	TMap<FMovieGraphRenderDataIdentifier, FMovieGraphRenderLayerOutputData> RenderLayerData;
};
