// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Graph/MovieGraphSharedWidgets.h"
#include "Subsystems/WorldSubsystem.h"
#include "Textures/SlateIcon.h"

#if WITH_EDITOR
#include "ContentBrowserDelegates.h"
#include "ISceneOutlinerColumn.h"
#endif	// WITH_EDITOR

#include "MovieGraphRenderLayerSubsystem.generated.h"

class SWidget;
class UDataLayerAsset;

/** Operation types available on condition groups. */
UENUM(BlueprintType)
enum class EMovieGraphConditionGroupOpType : uint8
{
	/** Adds the contents of the condition group to the results from the previous condition group (if any). */
	Add,

	/** Removes the contents of the condition group from the result of the previous condition group (if any). Any items in this condition group that aren't also found in the previous condition group will be ignored. */
	Subtract,

	/** Replaces the results of the previous condition group(s) with only the elements that exist in both that group, and this group. Intersecting with an empty condition group will result in an empty condition group. */
	And
};

/** Operation types available on condition group queries. */
UENUM(BlueprintType)
enum class EMovieGraphConditionGroupQueryOpType : uint8
{
	/** Adds the results of the query to the results from the previous query (if any). */
	Add,

	/** Removes the results of the query from the results of the previous query (if any). Any items in this query result that aren't also found in the previous query result will be ignored. */
	Subtract,
	
	/** Replaces the results of the previous queries with only the items that exist in both those queries, and this query result. Intersecting with a query which returns nothing will create an empty query result. */
	And
};

/** Base class that all condition group queries must inherit from. */
UCLASS(MinimalAPI, Abstract)
class UMovieGraphConditionGroupQueryBase : public UObject
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API UMovieGraphConditionGroupQueryBase();

	/** Delegate which is called when the contents of a query has changed. */
	DECLARE_DELEGATE(FMovieGraphConditionGroupQueryContentsChanged)

	/**
	 * Sets how the condition group query interacts with the condition group. This call is ignored for the first query
	 * in the condition group (the first is always Union).
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void SetOperationType(const EMovieGraphConditionGroupQueryOpType OperationType);

	/** Gets the condition group query operation type. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API EMovieGraphConditionGroupQueryOpType GetOperationType() const;

	/**
	 * Determines which of the provided actors (in the given world) match the query. Matches are added to OutMatchingActors.
	 * Note that this method will not be called if ShouldEvaluateComponents() returns true; EvaluateActorsAndComponents() will
	 * be called instead.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const;

	/**
	 * Similar to Evaluate(), but returns both actors and components if the query can match both.
	 * Note that this method will only be called if ShouldEvaluateComponents() returns true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API virtual void EvaluateActorsAndComponents(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const;

	/**
	 * Determines if the public properties on the query class will have their names hidden in the details panel. Returns
	 * false by default. Most query subclasses will only have one property and do not need to clutter the UI with the
	 * property name (eg, the "Actor Name" query only shows one text box with entries for the actor names, no need to
	 * show the property name).
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API virtual bool ShouldHidePropertyNames() const;

	/**
	 * Determines if this query should additionally match components, rather than just matching actors. Off by default. If this returns true, then
	 * EvaluateActorsAndComponents() will be called during evaluation instead of Evaluate().
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API virtual bool ShouldEvaluateComponents() const;

	/** Gets the icon that represents this query class in the UI. */
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const;

	/** Gets the display name for this query class as shown in the UI. */
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const;

#if WITH_EDITOR
	/**
	 * Gets the widgets that should be displayed for this query. If no custom widgets are specified (returning an empty array), the default
	 * name/value widgets will be shown for all query properties tagged with EditAnywhere.
	 */
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets();

	/**
	 * Returns true if this query should expose an Add menu, or false if no Add menu is visible.
	 *
	 * @see GetAddMenuContents()
	 */
	MOVIERENDERPIPELINECORE_API virtual bool HasAddMenu() const;

	/**
	 * Gets the contents of the "Add" menu in the UI, if any. When the Add menu updates properties within the query, OnAddFinished should be called
	 * in order to give the UI a chance to update itself. Note that HasAddMenu() must return true in order for the contents returned from this method
	 * to be displayed in the UI.
	 *
	 * @see HasAddMenu()
	 */
	MOVIERENDERPIPELINECORE_API virtual TSharedRef<SWidget> GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished);
#endif

	/** Determines if this query is only respected when run within the editor. Used for providing a UI hint. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API virtual bool IsEditorOnlyQuery() const;

	/** Sets whether this query is enabled. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void SetEnabled(const bool bEnabled);

	/** Determines if this query is enabled. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API bool IsEnabled() const;

	/** Determines if this is the first condition group query under the parent condition group. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API bool IsFirstConditionGroupQuery() const;

protected:
	/**
	 * Utility that returns the given actor in the current world. If currently in PIE, converts editor actors to PIE actors, and vice-versa. If no
	 * conversion is needed, returns the provided actor as-is.
	 */
	UE_DEPRECATED(5.8, "Use GetActorForCurrentWorld(AActor*, const UWorld*) when an evaluation/render world is available.")
	static MOVIERENDERPIPELINECORE_API AActor* GetActorForCurrentWorld(AActor* InActorToConvert);

	/**
	 * Utility that returns the given actor in the specified target world. This avoids relying on editor globals, which may be unavailable in remote renders.
	 */
	static MOVIERENDERPIPELINECORE_API AActor* GetActorForCurrentWorld(AActor* InActorToConvert, const UWorld* InTargetWorld);

private:
	/** The operation type that the query is using. */
	UPROPERTY()
	EMovieGraphConditionGroupQueryOpType OpType;

	/** Whether this query is currently enabled within the condition group. */
	UPROPERTY()
	bool bIsEnabled;
};

/** Contains the actors and components to match within the Actor condition group query. */
USTRUCT(BlueprintType)
struct FMovieGraphActorQueryEntry
{
	GENERATED_BODY()
	
	MOVIERENDERPIPELINECORE_API bool operator==(const FMovieGraphActorQueryEntry& InOther) const;
	
	/**
	 * The query must match this actor in order to be a match. If these are editor actors, they will be converted to PIE actors automatically.
	 * If there are any ComponentsToMatch, they must be part of this actor.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General")
	TSoftObjectPtr<AActor> ActorToMatch;
	
	/** The query must match these components in order to be a match. These must be contained within ActorToMatch. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General")
    TArray<TSoftObjectPtr<UActorComponent>> ComponentsToMatch;
};

/** Query type which filters actors via an explicit actor list. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_Actor final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual void EvaluateActorsAndComponents(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const override;
	MOVIERENDERPIPELINECORE_API virtual bool ShouldEvaluateComponents() const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
	MOVIERENDERPIPELINECORE_API virtual bool HasAddMenu() const override;
	MOVIERENDERPIPELINECORE_API virtual TSharedRef<SWidget> GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished) override;
#endif

	MOVIERENDERPIPELINECORE_API virtual void PostLoad() override;

private:
#if WITH_EDITOR
	/** Adds the provided actors to the query, updating the UI as needed. Calls InOnAddFinished when done. Can optionally close the Add menu. */
	MOVIERENDERPIPELINECORE_API void AddActors(const TArray<AActor*>& InActors, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished, const bool bCloseAddMenu = true);

	/** Removes the provided entries from the query, updating the UI as needed. */
	MOVIERENDERPIPELINECORE_API void RemoveEntries(const TArray<FMovieGraphActorQueryEntry>& InEntries);

	/** Gets the context menu for an entry in the list widget. */
	MOVIERENDERPIPELINECORE_API void GetListContextMenu(FMenuBuilder& InMenuBuilder, TArray<TSharedPtr<FMovieGraphActorQueryEntry>> SelectedEntries);

	/** Refreshes the list's data source to reflect the data model. */
	MOVIERENDERPIPELINECORE_API void RefreshListDataSource();
#endif

public:
	/** The query must match one of the actors in order to be a match. If these are editor actors, they will be converted to PIE actors automatically. */
	UE_DEPRECATED(5.6, "This property is deprecated. Please use ActorsAndComponentsToMatch instead.")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="This property is deprecated. Please use ActorsAndComponentsToMatch instead."))
	TArray<TSoftObjectPtr<AActor>> ActorsToMatch;

	/** The query must match one of the actors (or components on an actor) to be a match. If these are editor actors, they will be converted to PIE actors automatically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="General")
	TArray<FMovieGraphActorQueryEntry> ActorsAndComponentsToMatch;

private:
#if WITH_EDITOR
	/** Custom outliner column that allows adding/removing an actor from an Actor condition group query (via checkbox). */
	class FActorSelectionColumn final : public ISceneOutlinerColumn
	{
	public:
		explicit FActorSelectionColumn(const TWeakObjectPtr<UMovieGraphConditionGroupQuery_Actor> InWeakActorQuery)
			: WeakActorQuery(InWeakActorQuery)
		{}
		
		static FName GetID();
		virtual FName GetColumnID() override;
		virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
		virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;

	private:
		/** Determines if the given tree item (corresponding to one actor) is checked. */
		ECheckBoxState IsRowChecked(const FActorTreeItem* InActorTreeItem) const;

		/** Updates the associated actor query when a row is checked or unchecked. */
		void OnCheckStateChanged(const ECheckBoxState NewState, const FActorTreeItem* InActorTreeItem) const;
	
	private:
		/** The Actor condition group query that populates the data for this column. */
		TWeakObjectPtr<UMovieGraphConditionGroupQuery_Actor> WeakActorQuery;
	};

	static const inline FName ColumnID_ActorName = FName(TEXT("ActorName"));
	static const inline FName ColumnID_ActorType = FName(TEXT("ActorType"));
	static const inline FName ColumnID_Components = FName(TEXT("Components"));

	/** Custom row widget for the actor list so multiple columns can be populated. */
	class SActorListRow final : public SMultiColumnTableRow<TSharedPtr<FMovieGraphActorQueryEntry>>
	{
	public:
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FMovieGraphActorQueryEntry> InEntry);
		
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	private:
		TWeakPtr<FMovieGraphActorQueryEntry> WeakEntry;
	};
	
	TSharedPtr<class ISceneOutliner> ActorPickerWidget;

	/** Displays the actors which have been chosen. */
	TSharedPtr<SMovieGraphSimpleList<TSharedPtr<FMovieGraphActorQueryEntry>>> ActorsList;

	// Not ideal to store a duplicate of ActorsToMatch, but SListView requires TSharedPtr<...> as the data source, and UPROPERTY does not
	// support TSharedPtr<...>
	TArray<TSharedPtr<FMovieGraphActorQueryEntry>> ListDataSource;
#endif
};

/** Query type which filters actors via tags on actors. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_ActorTagName final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
#endif

public:
	/**
	 * Tags on the actor must match one or more of the specified tags to be a match. Not case sensitive. One tag per line. Wildcards ("?" and "*") are supported but not required.
	 * The "*" wildcard matches zero or more characters, and "?" matches exactly one character (and that character must be present).
	 * 
	 * Wildcard examples:
	 * Foo* would match Foo, FooBar, and FooBaz, but not BarFoo.
	 * *Foo* would match the above in addition to BarFoo.
	 * Foo?Bar would match Foo.Bar and Foo_Bar, but not FooBar.
	 * Foo? would match Food, but not FooBar or BarFoo.
	 * Foo??? would match FooBar and FooBaz, but not Foo or Food.
	 * ?oo? would match Food, but not Foo.
	 * ?Foo* would match AFooBar, but not FooBar 
	 */
	UPROPERTY(EditAnywhere, Category="General")
	FString TagsToMatch;
};

/** Query type which filters actors via their name (label). */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_ActorName final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
#endif

	MOVIERENDERPIPELINECORE_API virtual bool IsEditorOnly() const override;

public:
	/**
	 * The name that the actor needs to have in order to be a match. Not case sensitive. One name per line. Wildcards ("?" and "*") are supported but not required.
	 * The "*" wildcard matches zero or more characters, and "?" matches exactly one character (and that character must be present).
	 * 
	 * Wildcard examples:
	 * Foo* would match Foo, FooBar, and FooBaz, but not BarFoo.
	 * *Foo* would match the above in addition to BarFoo.
	 * Foo?Bar would match Foo.Bar and Foo_Bar, but not FooBar.
	 * Foo? would match Food, but not FooBar or BarFoo.
	 * Foo??? would match FooBar and FooBaz, but not Foo or Food.
	 * ?oo? would match Food, but not Foo.
	 * ?Foo* would match AFooBar, but not FooBar 
	 */
	UPROPERTY(EditAnywhere, Category="General")
	FString WildcardSearch;
};

/** Query type which filters actors by type. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_ActorType final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
	MOVIERENDERPIPELINECORE_API virtual bool HasAddMenu() const override;
	MOVIERENDERPIPELINECORE_API virtual TSharedRef<SWidget> GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished) override;
#endif

public:
	/** The type (class) that the actor needs to have in order to be a match. */
	UPROPERTY(EditAnywhere, Category="General")
	TArray<TObjectPtr<UClass>> ActorTypes;

private:
#if WITH_EDITOR
	static MOVIERENDERPIPELINECORE_API const FSlateBrush* GetRowIcon(TObjectPtr<UClass> InActorType);
	static MOVIERENDERPIPELINECORE_API FText GetRowText(TObjectPtr<UClass> InActorType);
	
	/** Adds the provided actor types to the query, updating the UI as needed. Calls InOnAddFinished when done. */
	MOVIERENDERPIPELINECORE_API void AddActorTypes(const TArray<UClass*>& InActorTypes, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished);

	/** Displays the actor types which have been chosen. */
	TSharedPtr<SMovieGraphSimpleList<TObjectPtr<UClass>>> ActorTypesList;
#endif
	
	/** The class viewer widget to show in the Add menu. */
	TSharedPtr<class SClassViewer> ClassViewerWidget;
};

/** Query type which filters actors by light type. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_LightActorType final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API UMovieGraphConditionGroupQuery_LightActorType();

	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
#endif

	/**
	 * Sets the enable state for the given light actor type. All enabled light types will be matched by the query. If the given actor type isn't
	 * one of the supported light types, this does nothing.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lights")
	void SetEnableStateForLightType(UClass* InLightActorType, const bool bNewEnableState);

	/** Gets the enable state for the given light actor type. If the provided type isn't a supported light type, returns false. */
	UFUNCTION(BlueprintCallable, Category = "Lights")
	bool GetEnableStateForLightType(UClass* InLightActorType);

private:
	/** The light types that are enabled (and will be matched by the query). */
	UPROPERTY()
	TArray<TObjectPtr<UClass>> EnabledLightActorTypes;

private:
	/** Gets the supported light actor types. */
	static TArray<UClass*>* GetSupportedLightActorTypes();

#if WITH_EDITOR
	static MOVIERENDERPIPELINECORE_API const FSlateBrush* GetRowIcon(UClass* InLightActorType);
	static MOVIERENDERPIPELINECORE_API FText GetRowText(UClass* InLightActorType);

	/** The list widget which displays the light actor types and their enable/disable state. */
	TSharedPtr<SMovieGraphSimpleList<UClass*>> LightActorTypesList;
#endif
};

/** Query type which filters actors by tags on their components. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_ComponentTagName final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API UMovieGraphConditionGroupQuery_ComponentTagName();
	
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual void EvaluateActorsAndComponents(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const override;
	MOVIERENDERPIPELINECORE_API virtual bool ShouldEvaluateComponents() const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
#endif

	//~ Begin UObject Interface
	MOVIERENDERPIPELINECORE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

public:
	/**
	 * A component on the actor must have one or more of the specified tags to be a match. Not case sensitive. One tag per line. Wildcards ("?" and "*") are supported but not required.
	 * The "*" wildcard matches zero or more characters, and "?" matches exactly one character (and that character must be present).
	 * 
	 * Wildcard examples:
	 * Foo* would match Foo, FooBar, and FooBaz, but not BarFoo.
	 * *Foo* would match the above in addition to BarFoo.
	 * Foo?Bar would match Foo.Bar and Foo_Bar, but not FooBar.
	 * Foo? would match Food, but not FooBar or BarFoo.
	 * Foo??? would match FooBar and FooBaz, but not Foo or Food.
	 * ?oo? would match Food, but not Foo.
	 * ?Foo* would match AFooBar, but not FooBar 
	 */
	UPROPERTY(EditAnywhere, Category="General")
	FString TagsToMatch;

	/** Whether this should match components or actors. If false, any components that match will instead match their parent actor. */
	UPROPERTY(EditAnywhere, Category="General")
	bool bOnlyMatchComponents;
};

/** Query type which filters actors via the component types contained in them. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_ComponentType final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API UMovieGraphConditionGroupQuery_ComponentType();
	
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual void EvaluateActorsAndComponents(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const override;
	MOVIERENDERPIPELINECORE_API virtual bool ShouldEvaluateComponents() const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
	MOVIERENDERPIPELINECORE_API virtual bool HasAddMenu() const override;
	MOVIERENDERPIPELINECORE_API virtual TSharedRef<SWidget> GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished) override;
#endif

	//~ Begin UObject Interface
	MOVIERENDERPIPELINECORE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

private:
#if WITH_EDITOR
	static MOVIERENDERPIPELINECORE_API const FSlateBrush* GetRowIcon(TObjectPtr<UClass> InComponentType);
	static MOVIERENDERPIPELINECORE_API FText GetRowText(TObjectPtr<UClass> InComponentType);

	/** Displays the component types which have been chosen. */
	TSharedPtr<SMovieGraphSimpleList<TObjectPtr<UClass>>> ComponentTypesList;
#endif
	
	/** The class viewer widget to show in the Add menu. */
	TSharedPtr<class SClassViewer> ClassViewerWidget;

public:
	/** The actor must have one or more of the component type(s) in order to be a match. */
	UPROPERTY(EditAnywhere, Category="General")
	TArray<TObjectPtr<UClass>> ComponentTypes;

	/** Whether this should match components or actors. If false, any components that match will instead match their parent actor. */
	UPROPERTY(EditAnywhere, Category="General")
	bool bOnlyMatchComponents;
};

/** Query type which filters actors via the components contained in them (by name). */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_ComponentName final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API UMovieGraphConditionGroupQuery_ComponentName();

	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual void EvaluateActorsAndComponents(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors, TSet<UActorComponent*>& OutMatchingComponents) const override;
	MOVIERENDERPIPELINECORE_API virtual bool ShouldEvaluateComponents() const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
#endif

public:
	/** The actor must have one or more of the component name(s) in order to be a match. */
	UPROPERTY(EditAnywhere, Category="General")
	FString ComponentNames;

	/** Whether this should match components or actors. If false, any components that match will instead match their parent actor. */
	UPROPERTY(EditAnywhere, Category="General")
	bool bOnlyMatchComponents;
};

/** Query type which filters actors via the editor folder that they're contained in. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_EditorFolder final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;
	MOVIERENDERPIPELINECORE_API virtual bool IsEditorOnlyQuery() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
	MOVIERENDERPIPELINECORE_API virtual bool HasAddMenu() const override;
	MOVIERENDERPIPELINECORE_API virtual TSharedRef<SWidget> GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished) override;
#endif

private:
#if WITH_EDITOR
	static MOVIERENDERPIPELINECORE_API const FSlateBrush* GetRowIcon(FName InFolderPath);
	static MOVIERENDERPIPELINECORE_API FText GetRowText(FName InFolderPath);

	/** Adds the provided folders to the query, updating the UI as needed. Calls InOnAddFinished when done. */
	MOVIERENDERPIPELINECORE_API void AddFolders(const TArray<FName>& InFolderPaths, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished);

	/** Displays the paths of folders which have been chosen. */
	TSharedPtr<SMovieGraphSimpleList<FName>> FolderPathsList;

	/** The folder browser widget to show in the Add menu. */
	TSharedPtr<class ISceneOutliner> FolderPickerWidget;
#endif

public:
	/** The actor must be in one of the chosen folders in order to be a match. */
	UPROPERTY(EditAnywhere, Category="General")
	TArray<FName> FolderPaths;
};

/** Query type which filters actors via the sublevel that they're contained in. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_Sublevel final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
	MOVIERENDERPIPELINECORE_API virtual bool HasAddMenu() const override;
	MOVIERENDERPIPELINECORE_API virtual TSharedRef<SWidget> GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished) override;
#endif

private:
#if WITH_EDITOR
	static MOVIERENDERPIPELINECORE_API const FSlateBrush* GetRowIcon(TSharedPtr<TSoftObjectPtr<UWorld>> InSublevel);
	static MOVIERENDERPIPELINECORE_API FText GetRowText(TSharedPtr<TSoftObjectPtr<UWorld>> InSublevel);
	static MOVIERENDERPIPELINECORE_API bool IsSublevelInActivePersistentLevel(FName InSublevelPackageName);

	/** Adds the provided levels to the query, updating the UI as needed. Calls InOnAddFinished when done. */
	MOVIERENDERPIPELINECORE_API void AddLevels(const TArray<UWorld*>& InLevels, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished);

	/** Refreshes the list's data source to reflect the data model. */
	MOVIERENDERPIPELINECORE_API void RefreshListDataSource();

	/** Displays the names of sublevels which have been chosen. */
	TSharedPtr<SMovieGraphSimpleList<TSharedPtr<TSoftObjectPtr<UWorld>>>> SublevelsList;

	/** Refreshes the contents of the level picker widget when called. */
	FRefreshAssetViewDelegate RefreshLevelPicker;

	// Not ideal to store a duplicate of Sublevels, but SListView requires TSharedPtr<...> as the data source, and UPROPERTY does not
	// support TSharedPtr<...>
	TArray<TSharedPtr<TSoftObjectPtr<UWorld>>> ListDataSource;
#endif

#if WITH_EDITORONLY_DATA
	/**
	 * Whether the add menu should only show sublevels from the active persistent level. The value is persisted with the query, but
	 * only impacts how the UI is filtered.
	 */
	UPROPERTY()
	bool bOnlyShowSublevelsFromActivePersistentLevel = true;
#endif

public:
	/** The actor must be in one of the chosen sublevels in order to be a match. */
	UPROPERTY(EditAnywhere, Category="General")
	TArray<TSoftObjectPtr<UWorld>> Sublevels;
};

/** Query type which filters actors via Actor Layers. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_ActorLayer final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;
	MOVIERENDERPIPELINECORE_API virtual bool IsEditorOnly() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
	MOVIERENDERPIPELINECORE_API virtual bool HasAddMenu() const override;
	MOVIERENDERPIPELINECORE_API virtual TSharedRef<SWidget> GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished) override;
#endif

private:
#if WITH_EDITOR
	static MOVIERENDERPIPELINECORE_API const FSlateBrush* GetRowIcon(FName InLayerName);
	static MOVIERENDERPIPELINECORE_API FText GetRowText(FName InLayerName);

	/** Adds the provided actor layers to the query, updating the UI as needed. Calls InOnAddFinished when done. */
	MOVIERENDERPIPELINECORE_API void AddActorLayers(const TArray<FName>& InActorLayers, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished);

	/** Displays the layers which have been chosen. */
	TSharedPtr<SMovieGraphSimpleList<FName>> LayerNamesList;

	/** The data source for the layer picker widget (contains all layers which are available and not yet picked). */
	TArray<FName> LayerPickerDataSource;
#endif

public:
	/** The actor must be in one of the actor layers with these names in order to be a match. */
	UPROPERTY(EditAnywhere, Category="General")
	TArray<FName> LayerNames;
};

/** Query type which filters actors via World Partition Data Layers. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_DataLayer final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual TArray<TSharedRef<SWidget>> GetWidgets() override;
	MOVIERENDERPIPELINECORE_API virtual bool HasAddMenu() const override;
	MOVIERENDERPIPELINECORE_API virtual TSharedRef<SWidget> GetAddMenuContents(const FMovieGraphConditionGroupQueryContentsChanged& OnAddFinished) override;
#endif

private:
#if WITH_EDITOR
	static MOVIERENDERPIPELINECORE_API const FSlateBrush* GetRowIcon(TSharedPtr<TSoftObjectPtr<UDataLayerAsset>> InDataLayer);
	static MOVIERENDERPIPELINECORE_API FText GetRowText(TSharedPtr<TSoftObjectPtr<UDataLayerAsset>> InDataLayer);
	
	/** Adds the provided data layers to the query, updating the UI as needed. Calls InOnAddFinished when done. */
	MOVIERENDERPIPELINECORE_API void AddDataLayers(const TArray<const UDataLayerAsset*>& InDataLayers, const FMovieGraphConditionGroupQueryContentsChanged& InOnAddFinished);

	/** Refreshes the list's data source to reflect the data model. */
	MOVIERENDERPIPELINECORE_API void RefreshListDataSource();

	/** Displays the layers which have been chosen. */
	TSharedPtr<SMovieGraphSimpleList<TSharedPtr<TSoftObjectPtr<UDataLayerAsset>>>> DataLayersList;
	
	// Not ideal to store a duplicate of DataLayersList, but SListView requires TSharedPtr<...> as the data source, and UPROPERTY does not
	// support TSharedPtr<...>
	TArray<TSharedPtr<TSoftObjectPtr<UDataLayerAsset>>> ListDataSource;

	/** Refreshes the contents of the data layer picker widget when called. */
	FRefreshAssetViewDelegate RefreshDataLayerPicker;
#endif

public:
	/** The actor must be in one of the these data layer assets in order to be a match. */
	UPROPERTY(EditAnywhere, Category="General")
	TArray<TSoftObjectPtr<UDataLayerAsset>> DataLayers;
};

/** Query type which filters actors by their spawnable status. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroupQuery_IsSpawnable final : public UMovieGraphConditionGroupQueryBase
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API virtual void Evaluate(const TArray<AActor*>& InActorsToQuery, const UWorld* InWorld, TSet<AActor*>& OutMatchingActors) const override;
	MOVIERENDERPIPELINECORE_API virtual const FSlateIcon& GetIcon() const override;
	MOVIERENDERPIPELINECORE_API virtual const FText& GetDisplayName() const override;

public:
	/** Whether the actor is a spawnable or not. */
	UPROPERTY(EditAnywhere, Category="General")
	bool bIsSpawnable = false;
};

/** Provides the actors/components returned by a collection or condition group evaluation. */
USTRUCT(BlueprintType)
struct FMovieGraphEvaluationResult
{
	GENERATED_BODY()

	FMovieGraphEvaluationResult() = default;
	MOVIERENDERPIPELINECORE_API FMovieGraphEvaluationResult(TSet<TObjectPtr<AActor>>&& InActors, TSet<TObjectPtr<UActorComponent>>&& InComponents);

	/** Empties this evaluation result and resets it to defaults. */
	MOVIERENDERPIPELINECORE_API void Reset();

	/** Appends the actors/components in InOther to the actors/components in this evaluation result. */
	MOVIERENDERPIPELINECORE_API void Append(const FMovieGraphEvaluationResult& InOther);

	/** Unions this evaluation result with InOther. */
	MOVIERENDERPIPELINECORE_API FMovieGraphEvaluationResult Union(const FMovieGraphEvaluationResult& InOther) const;

	/** Intersects this evaluation result with InOther. */
	MOVIERENDERPIPELINECORE_API FMovieGraphEvaluationResult Intersect(const FMovieGraphEvaluationResult& InOther) const;

	/** Differences this evaluation result with InOther. */
	MOVIERENDERPIPELINECORE_API FMovieGraphEvaluationResult Difference(const FMovieGraphEvaluationResult& InOther) const;

	/** Gets all components of the specified type across MatchingActors and MatchingComponents. */
	template<typename T>
	TArray<T*> GetAllComponentsOfType() const;

	/** The actors that were matched during evaluation. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	TSet<TObjectPtr<AActor>> MatchingActors;

	/** The components that were matched during evaluation. */
	UPROPERTY(BlueprintReadOnly, Category = "Movie Graph")
	TSet<TObjectPtr<UActorComponent>> MatchingComponents;
};

/** A group of queries which can be added to a collection. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConditionGroup : public UObject
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API UMovieGraphConditionGroup();

	/**
	 * Sets how the condition group interacts with the collection. This call is ignored for the first condition group
	 * in the collection (the first is always Union).
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void SetOperationType(const EMovieGraphConditionGroupOpType OperationType);

	/** Gets the condition group operation type. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API EMovieGraphConditionGroupOpType GetOperationType() const;

	/** Determines the actors that match the condition group by running the queries contained in it. */
	UE_DEPRECATED(5.6, "Please use EvaluateActorsAndComponents() instead.")
	UFUNCTION(BlueprintCallable, Category = "Settings", meta = (DeprecatedFunction, DeprecationMessage = "Please use EvaluateActorsAndComponents() instead."))
	MOVIERENDERPIPELINECORE_API TSet<AActor*> Evaluate(const UWorld* InWorld) const;

	/** Determines the actors and components that match the condition group by running the queries contained in it. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API FMovieGraphEvaluationResult EvaluateActorsAndComponents(const UWorld* InWorld) const;

	/**
	 * Adds a new condition group query to the condition group and returns a ptr to it. The condition group owns the
	 * created query. By default the query is added to the end, but an optional index can be provided if the query
	 * should be placed in a specific location among the existing queries.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API UMovieGraphConditionGroupQueryBase* AddQuery(const TSubclassOf<UMovieGraphConditionGroupQueryBase>& InQueryType, const int32 InsertIndex = -1);

	/** Gets all queries currently contained in the condition group. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API const TArray<UMovieGraphConditionGroupQueryBase*>& GetQueries() const;

	/** Removes the specified query from the condition group if it exists. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API bool RemoveQuery(UMovieGraphConditionGroupQueryBase* InQuery);

	/**
	 * Duplicates the condition group query at the specified index. The duplicate is placed at the end of the query list. Returns the duplicate
	 * query on success, else nullptr.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API UMovieGraphConditionGroupQueryBase* DuplicateQuery(const int32 QueryIndex);

	/** Determines if this is the first condition group under the parent collection. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API bool IsFirstConditionGroup() const;

	/**
	 * Move the specified query to a new index within the condition group. Returns false if the query was not found or the index
	 * specified is invalid, else true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API bool MoveQueryToIndex(UMovieGraphConditionGroupQueryBase* InQuery, const int32 NewIndex);

	/** Gets a persistent unique identifier for this condition group. */
	MOVIERENDERPIPELINECORE_API const FGuid& GetId() const;

private:
	/** A unique identifier for this condition group. Needed in some cases because condition groups do not have names. */
	UPROPERTY()
	FGuid Id;
	
	/** The operation type that the condition group is using. */
	UPROPERTY(EditAnywhere, Category="General")
	EMovieGraphConditionGroupOpType OpType;

	/** The queries that are contained within the condition group. */
	UPROPERTY(EditAnywhere, Category="General", Instanced)
	TArray<TObjectPtr<UMovieGraphConditionGroupQueryBase>> Queries;	// Note: Marked as Instanced so conditions get duplicated during copy/paste (not referenced)

	/** Persisted actor set which can be re-used for query evaluations across frames to prevent constantly re-allocating it. */
	UPROPERTY(Transient, DuplicateTransient)
	mutable FMovieGraphEvaluationResult QueryResult;

	/** Persisted actor set which can be re-used for condition group evaluations across frames to prevent constantly re-allocating it. */
	UPROPERTY(Transient, DuplicateTransient)
	mutable FMovieGraphEvaluationResult EvaluationResult;
};

/** A group of actors generated by actor queries. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphCollection : public UObject
{
	GENERATED_BODY()

public:
	/** Delegate which is called when the collection name changes. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FMovieGraphCollectionNameChanged, UMovieGraphCollection*)
	
	UMovieGraphCollection() = default;
	
#if WITH_EDITOR
	//~ Begin UObject interface
	MOVIERENDERPIPELINECORE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

	/** Sets the name of the collection as seen in the UI. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void SetCollectionName(const FString& InName);

	/** Gets the name of the collection as seen in the UI. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API const FString& GetCollectionName() const;

	/**
	 * Gets matching actors by having condition groups evaluate themselves, and performing set operations on the
	 * condition group results (eg, union'ing condition group A and B).
	 */
	UE_DEPRECATED(5.6, "Please use EvaluateActorsAndComponents() instead.")
	UFUNCTION(BlueprintCallable, Category = "Settings", meta = (DeprecatedFunction, DeprecationMessage = "Please use EvaluateActorsAndComponents() instead."))
	MOVIERENDERPIPELINECORE_API TSet<AActor*> Evaluate(const UWorld* InWorld) const;

	/**
	 * Gets matching actors and components by having condition groups evaluate themselves, and performing set operations on the
	 * condition group results (eg, union'ing condition group A and B).
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API FMovieGraphEvaluationResult EvaluateActorsAndComponents(const UWorld* InWorld) const;

	/**
	 * Adds a new condition group to the collection and returns a ptr to it. The collection owns the created
	 * condition group.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API UMovieGraphConditionGroup* AddConditionGroup();

	/** Gets all condition groups currently contained in the collection. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API const TArray<UMovieGraphConditionGroup*>& GetConditionGroups() const;

	/**
	 * Removes the specified condition group from the collection if it exists. Returns true on success, else false.
	 * Removes all child queries that belong to this group at the same time.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API bool RemoveConditionGroup(UMovieGraphConditionGroup* InConditionGroup);

	/**
	 * Move the specified condition group to a new index within the collection. Returns false if the condition group was not found or the index
	 * specified is invalid, else true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API bool MoveConditionGroupToIndex(UMovieGraphConditionGroup* InConditionGroup, const int32 NewIndex);

public:
#if WITH_EDITOR
	/** Called when the collection name changes. */
	FMovieGraphCollectionNameChanged OnCollectionNameChangedDelegate;
#endif

private:
	/** The display name of the collection, shown in the UI. Does not need to be unique across collections. */
	UPROPERTY(EditAnywhere, Category="Collection")
	FString CollectionName;
	
	/** The condition groups that are contained within the collection. */
	UPROPERTY(EditAnywhere, Category="Collection")
	TArray<TObjectPtr<UMovieGraphConditionGroup>> ConditionGroups;
};

/**
 * The base class that all modifiers should inherit from. Does not support collections (see UMovieGraphCollectionModifier).
 */
UCLASS(MinimalAPI, Abstract)
class UMovieGraphModifierBase : public UObject
{
	GENERATED_BODY()

public:
	/** Applies this modifier in the provided world. Called once per layer. */
	UFUNCTION(BlueprintCallable, Category = "Modifier")
	virtual void ApplyModifier(const UWorld* World) PURE_VIRTUAL(UMovieGraphModifierBase::ApplyModifier, );

	/** Undoes the effects of this modifier. The modifier is responsible for tracking changes that it makes. Called once per layer. */
	UFUNCTION(BlueprintCallable, Category = "Modifier")
	virtual void UndoModifier() PURE_VIRTUAL(UMovieGraphModifierBase::UndoModifier, );

	/** Gets the name of this modifier. Typically used for UI display purposes. */
	UFUNCTION(BlueprintCallable, Category = "Modifier")
	virtual FText GetModifierName() PURE_VIRTUAL(UMovieGraphModifierBase::GetModifierName, return FText::GetEmpty(););
};

/**
 * Base class for providing actor modification functionality via collections. For modifiers without collections, see UMovieGraphModifierBase.
 */
UCLASS(MinimalAPI, Abstract)
class UMovieGraphCollectionModifier : public UMovieGraphModifierBase
{
	GENERATED_BODY()

public:
	/** Adds a collection to the existing set of collections in this modifier. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void AddCollection(UMovieGraphCollection* Collection);

	/** Overwrites the existing collections with the provided array of collections. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void SetCollections(const TArray<UMovieGraphCollection*> InCollections);
	
	/** Gets all collections that this modifier is using. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API TArray<UMovieGraphCollection*> GetCollections() const;

	//~ Begin UMovieGraphModifierBase Interface
	MOVIERENDERPIPELINECORE_API virtual FText GetModifierName() override;
	//~ End UMovieGraphModifierBase Interface

protected:
	/** The collections which this modifier will operate on. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphCollection>> Collections;
};

/**
 * Modifies actor materials.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphMaterialModifier : public UMovieGraphCollectionModifier
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Settings")
	void SetMaterial(TSoftObjectPtr<UMaterialInterface> InMaterial) { Material = InMaterial; }

	//~ Begin UMovieGraphModifierBase Interface
	MOVIERENDERPIPELINECORE_API virtual void ApplyModifier(const UWorld* World) override;
	MOVIERENDERPIPELINECORE_API virtual void UndoModifier() override;
	MOVIERENDERPIPELINECORE_API virtual FText GetModifierName() override;
	//~ End UMovieGraphModifierBase Interface

private:
	typedef TTuple<int32, TSoftObjectPtr<UMaterialInterface>> FMaterialSlotAssignment;
	typedef TMap<TSoftObjectPtr<UPrimitiveComponent>, TArray<FMaterialSlotAssignment>> FComponentToMaterialMap;
	
	/** Maps a component to its original material assignments (per index). */
	FComponentToMaterialMap ModifiedComponents;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Material : 1;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_Material"))
	TSoftObjectPtr<UMaterialInterface> Material;
};

/**
 * Modifies actor visibility.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphRenderPropertyModifier : public UMovieGraphCollectionModifier
{
	GENERATED_BODY()

public:
	MOVIERENDERPIPELINECORE_API UMovieGraphRenderPropertyModifier();

	// ~UObject interface
	MOVIERENDERPIPELINECORE_API virtual void PostLoad() override;
#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// ~UObject Interface
	
	UFUNCTION(BlueprintCallable, Category = "Settings")
	void SetHidden(const bool bInIsHidden) { bIsHidden = bInIsHidden; }

	UFUNCTION(BlueprintCallable, Category = "Settings")
	bool IsHidden() const { return bIsHidden; }

	//~ Begin UMovieGraphModifierBase Interface
	MOVIERENDERPIPELINECORE_API virtual void ApplyModifier(const UWorld* World) override;
	MOVIERENDERPIPELINECORE_API virtual void UndoModifier() override;
	MOVIERENDERPIPELINECORE_API virtual FText GetModifierName() override;
	//~ End UMovieGraphModifierBase Interface

private:
	/** Various visibility properties for an actor. */
	struct FActorState
	{
		TSoftObjectPtr<AActor> Actor = nullptr;
		uint8 bIsHidden : 1 = false;
	};

	/** Various visibility properties for a component. */
	struct FComponentState
	{
		TSoftObjectPtr<USceneComponent> Component = nullptr;
		
		// Note: The default values specified here reflect the defaults on the scene component. If a modifier property is marked as overridden, the
		// override will initially be a no-op due to the defaults being the same.
		uint8 bCastsShadows : 1 = true;
		uint8 bCastShadowWhileHidden : 1 = false;
		uint8 bAffectIndirectLightingWhileHidden : 1 = false;
		uint8 bHoldout : 1 = false;
		uint8 bVisibleInReflections : 1 = true;
		uint8 bIsHidden : 1 = false;
	};

	/**
	 * Updates actor/component state to the state contained in InActorState/InComponentState. If bUseStateFromNode is true, InActorState and InComponent
	 * state will be ignored; the node's state will be applied to all actors and components that are currently cached. This is typically used to set
	 * the state of actors and components before a render after their state has been cached out.
	 */
	MOVIERENDERPIPELINECORE_API void SetActorAndComponentState(const TArray<FActorState>& InActorState, const TArray<FComponentState>& InComponentState, const bool bUseStateFromNode);

	/** Convenience function to ensure that output alpha and primitive alpha holdout settings are enabled if required. */
	MOVIERENDERPIPELINECORE_API void ValidateProjectSettings() const;

private:
	/** Tracks relevant actor state prior to having the modifier applied. Only actors that are affected are included in the cache. */
	TArray<FActorState> CachedActorState;

	/** Tracks relevant component state prior to having the modifier applied. Only components that are affected are included in the cache. */
	TArray<FComponentState> CachedComponentState;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bIsHidden : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCastsShadows : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCastShadowWhileHidden : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bAffectIndirectLightingWhileHidden : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bHoldout : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bVisibleInReflections : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bProcessEditorOnlyActors : 1;
	
	/**
	 * If true, the actor will not be visible and will not contribute to any secondary effects (shadows, indirect
	 * lighting) unless their respective flags are set below.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_bIsHidden"))
	uint8 bIsHidden : 1;
	
	/** If true, the primitive will cast shadows. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_bCastsShadows"))
	uint8 bCastsShadows : 1;

	/** If true, the primitive will cast shadows even if it is hidden. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_bCastShadowWhileHidden"))
	uint8 bCastShadowWhileHidden : 1;

	/** Controls whether the primitive should affect indirect lighting when hidden. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_bAffectIndirectLightingWhileHidden"))
	uint8 bAffectIndirectLightingWhileHidden : 1;

	/**
	 * If true, the primitive will render black with an alpha of 0, but all secondary effects (shadows, reflections,
	 * indirect lighting) remain. This feature requires activating the project setting(s) "Alpha Output", and "Support Primitive Alpha Holdout" if using the deferred renderer.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_bHoldout"))
	uint8 bHoldout : 1;

	/** If true, this component will be visible in ray traced reflections. Screen space traces will ignore this flag, as they directly sample SceneColor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_bVisibleInReflections"))
	uint8 bVisibleInReflections : 1;

	/**
	 * If true, this modifier will process editor-only actors (eg, the billboard icons that represent lights). Defaults to false. Generally, processing
	 * editor-only actors is a waste of cycles because they will never be shown in a render, and there's no point in messing with them. However there
	 * are some niche cases where they need to be modified (eg, Quick Render). This is not a property exposed to the UI or saved because it's only
	 * meant to be used temporarily (like by scripting; note that bOverride_bProcessEditorOnlyActors still needs to be set).
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Settings")
	uint8 bProcessEditorOnlyActors : 1;
};

/**
 * Provides a means of assembling modifiers together to generate a desired view of a scene. 
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphRenderLayer : public UObject
{
	GENERATED_BODY()

public:
	UMovieGraphRenderLayer() = default;

	UFUNCTION(BlueprintCallable, Category = "Settings")
	FName GetRenderLayerName() const { return RenderLayerName; };

	UFUNCTION(BlueprintCallable, Category = "Settings")
	void SetRenderLayerName(const FName& NewName) { RenderLayerName = NewName; }

	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API UMovieGraphCollection* GetCollectionByName(const FString& Name) const;

	UE_DEPRECATED(5.7, "Use AddLayerModifier() instead.")
	UFUNCTION(BlueprintCallable, Category = "Settings", meta = (DeprecatedFunction, DeprecationMessage = "Use AddLayerModifier() instead."))
	MOVIERENDERPIPELINECORE_API void AddModifier(UMovieGraphCollectionModifier* Modifier);

	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void AddLayerModifier(UMovieGraphModifierBase* InModifier);

	UE_DEPRECATED(5.7, "Use GetLayerModifiers() instead.")
	UFUNCTION(BlueprintCallable, Category = "Settings", meta = (DeprecatedFunction, DeprecationMessage = "Use GetLayerModifiers() instead."))
	TArray<UMovieGraphCollectionModifier*> GetModifiers() const;

	UFUNCTION(BlueprintCallable, Category = "Settings")
	TArray<UMovieGraphModifierBase*> GetLayerModifiers() const;

	UE_DEPRECATED(5.7, "Use RemoveLayerModifier() instead.")
	UFUNCTION(BlueprintCallable, Category = "Settings", meta = (DeprecatedFunction, DeprecationMessage = "Use RemoveLayerModifier() instead."))
	MOVIERENDERPIPELINECORE_API void RemoveModifier(UMovieGraphCollectionModifier* Modifier);

	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void RemoveLayerModifier(UMovieGraphModifierBase* Modifier);

	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void Apply(const UWorld* World);

	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void Revert();

private:
	/** The name of this render layer. */
	UPROPERTY()
	FName RenderLayerName;

	/** The modifiers that are active when this render layer is active. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphModifierBase>> Modifiers;
};

/**
 * The primary means of controlling render layers in MRQ. Render layers can be added/registered with the subsystem, then
 * made active in order to view them. Collections and modifiers can also be viewed, but they do not need to be added to
 * the subsystem ahead of time.
 */
UCLASS(MinimalAPI)
class UMovieGraphRenderLayerSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	UMovieGraphRenderLayerSubsystem() = default;

	/* Get this subsystem for a specific world. Handy for use from Python. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	static MOVIERENDERPIPELINECORE_API UMovieGraphRenderLayerSubsystem* GetFromWorld(const UWorld* World);

	//~ Begin USubsystem interface
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override { return true; }
	MOVIERENDERPIPELINECORE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	MOVIERENDERPIPELINECORE_API virtual void Deinitialize() override;
	//~ End USubsystem interface

	/** Clear out all tracked render layers and collections. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void Reset();

	/**
	 * Adds a render layer to the system, which can later be made active by SetActiveRenderLayer*(). Returns true
	 * if the layer was added successfully, else false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API bool AddRenderLayer(UMovieGraphRenderLayer* RenderLayer);

	/** Gets all render layers which are currently tracked by the system. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	const TArray<UMovieGraphRenderLayer*>& GetRenderLayers() { return RenderLayers; }

	/** Removes the render layer with the given name. After removal it can no longer be made active with SetActiveRenderLayerBy*(). */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void RemoveRenderLayer(const FString& RenderLayerName);

	/** Gets the currently active render layer (the layer with its modifiers applied). */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	UMovieGraphRenderLayer* GetActiveRenderLayer() const { return ActiveRenderLayer; }

	/** Applies the layer with the given name. The layer needs to have been registered with AddRenderLayer(). */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void SetActiveRenderLayerByName(const FName& RenderLayerName);

	/** Applies the given layer. The layer does not need to have been registered with AddRenderLayer(). */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void SetActiveRenderLayerByObj(UMovieGraphRenderLayer* RenderLayer);

	/** Clears the currently active render layer and reverts its modifiers. */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINECORE_API void ClearActiveRenderLayer();

private:
	/** Clears the currently active render layer and reverts its modifiers. */
	MOVIERENDERPIPELINECORE_API void RevertAndClearActiveRenderLayer();

	/** Sets the active render layer and applies its modifiers. */
	MOVIERENDERPIPELINECORE_API void SetAndApplyRenderLayer(UMovieGraphRenderLayer* RenderLayer);

private:
	/** Render layers which have been added/registered with the subsystem. These can be found by name. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMovieGraphRenderLayer>> RenderLayers;

	/** The render layer that currently has its modifiers applied. */
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphRenderLayer> ActiveRenderLayer = nullptr;
};
