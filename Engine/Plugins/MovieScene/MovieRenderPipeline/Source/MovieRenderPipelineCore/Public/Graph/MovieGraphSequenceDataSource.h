// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/MovieGraphDataTypes.h"
#include "Misc/FrameTime.h"
#include "MovieSceneTimeController.h"

#include "MovieGraphSequenceDataSource.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
class ULevelSequence;
class ALevelSequenceActor;
class UMovieSceneSequencePlayer;

namespace UE::MovieGraph
{
	struct FMovieGraphSequenceTimeController : public FMovieSceneTimeController
	{
		virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override;
		void SetCachedFrameTiming(const FQualifiedFrameTime& InTimeCache) { TimeCache = InTimeCache; }

	private:
		/** Simply store the number calculated and return it when requested. */
		FQualifiedFrameTime TimeCache;
	};
}

/**
* The UMovieGraphSequenceDataSource allows using a ULevelSequence as the external datasource for the Movie Graph.
* It will build the range of shots based on the contents of the level sequence (one shot per camera cut found inside
* the sequence hierarchy, not allowing overlapping Camera Cut sections), and then it will evaluate the level sequence
* for the given time when rendering.
*/
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphSequenceDataSource : public UMovieGraphDataSourceBase
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphSequenceDataSource();
	UE_API virtual void CacheDataPreJob(const FMovieGraphInitConfig& InInitConfig) override;
	UE_API virtual void RestoreCachedDataPostJob() override;
	UE_API virtual void UpdateShotList() override;
	UE_API virtual FFrameRate GetTickResolution() const override;
	UE_API virtual FFrameRate GetDisplayRate() const override;
	UE_API virtual void SyncDataSourceTime(const FFrameTime& InTime) override;
	UE_API virtual void PlayDataSource() override;
	UE_API virtual void PauseDataSource() override;
	UE_API virtual void StopDataSource() override;
	UE_API virtual void JumpDataSource(const FFrameTime& InTimeToJumpTo) override;
	UE_API virtual void CacheHierarchyForShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot) override;
	UE_API virtual void RestoreHierarchyForShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot) override;
	UE_API virtual void MuteShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot) override;
	UE_API virtual void UnmuteShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot) override;
	UE_API virtual void ExpandShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot, const int32 InLeftDeltaFrames, const int32 InLeftDeltaFramesUserPoV,
		const int32 InRightDeltaFrames, const bool bInPrepass) override;
	UE_API virtual TArray<UE::MovieGraph::FMinimalCameraInfo> GetCameraInformation(UMoviePipelineExecutorShot* InShot, bool bIncludeSidecar) const override;
	UE_API virtual void InitializeShot(const TObjectPtr<UMoviePipelineExecutorShot>& InShot, const FFrameTime& InEvalTime) override;

public:
	/** Returns the sequence player used to evaluate the level sequence during rendering. */
	UE_API UMovieSceneSequencePlayer* GetSequencePlayer() const;

protected:
	UE_API void OverrideSequencePlaybackRangeFromGlobalOutputSettings(ULevelSequence* InSequence);
	UE_API void CacheLevelSequenceData(ULevelSequence* InSequence);
	UE_API void OnSequenceEvaluated(const UMovieSceneSequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime);

protected:
	UPROPERTY(Transient)
	TObjectPtr<ALevelSequenceActor> LevelSequenceActor;

	/** Custom Time Controller for the Sequence Player, used to match Custom TimeStep without any floating point accumulation errors. */
	TSharedPtr<UE::MovieGraph::FMovieGraphSequenceTimeController> CustomSequenceTimeController;

	TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> CachedSequenceHierarchyRoot;
};

#undef UE_API
