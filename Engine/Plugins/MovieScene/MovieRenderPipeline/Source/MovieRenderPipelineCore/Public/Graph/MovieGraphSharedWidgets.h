// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Don't even declare these widgets unless this is an editor build. This allows us to inherit from FSelfRegisteringEditorUndoClient.
#if WITH_EDITOR

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "MovieGraphSharedWidgets"

// NOTE: There should not be widgetry defined in core. To fix this, the condition group queries in the render layer subsystem need to be refactored to
// expose their custom widgetry in MovieRenderPipelineEditor.

/**
 * A widget which lists items in rows w/ an alternating row color, where each item has an icon and text. A summary row appears at the end of the
 * list indicating how many items are in the list.
 */
template<typename ListType>
class SMovieGraphSimpleList final : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(const FSlateBrush*, FGetRowIcon, ListType);
	DECLARE_DELEGATE_RetVal_OneParam(FText, FGetRowText, ListType);
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<ITableRow>, FGetCustomRow, const TSharedRef<STableViewBase>&, ListType);
	DECLARE_DELEGATE_OneParam(FOnDelete, TArray<ListType>);
	DECLARE_DELEGATE_TwoParams(FOnGetContextMenuContent, FMenuBuilder&, TArray<ListType>);
	DECLARE_DELEGATE(FOnRefreshDataSourceRequested);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FGetRowEnableState, ListType);
	DECLARE_DELEGATE_TwoParams(FSetRowEnableState, ListType, bool);
	
	SLATE_BEGIN_ARGS(SMovieGraphSimpleList<ListType>)
		: _SelectionMode(ESelectionMode::Multi)
		, _ShowEnableDisable(false)
		{}
		
		/** The source of data that the list will display. */
		SLATE_ATTRIBUTE(TArray<ListType>*, DataSource)

		/**
		 * If the data source provided to the list is not the data model being modified by the transaction system, then this event
		 * should be provided a delegate that refreshes the data source. This delegate will be called when the list needs to be refreshed.
		 */
		SLATE_EVENT(FOnRefreshDataSourceRequested, OnRefreshDataSourceRequested)

		/** The name of the data type that will be shown in the summary row. */
		SLATE_ATTRIBUTE(FText, DataType)

		/** The plural of the DataType attribute. */
		SLATE_ATTRIBUTE(FText, DataTypePlural)

		/**
		 * Gets the widget that should be shown in the summary row. By default, "DataType" and "DataTypePlural" will be used to auto-generate the summary
		 * row widget. If this attribute is bound, those will be ignored, and this will be used instead. To hide the summary row, use SNullWidget::NullWidget
		 * as the widget.
		 */
		SLATE_ATTRIBUTE(TSharedPtr<SWidget>, CustomSummaryWidget);

		/** The selection mode the list should be put into. Defaults to Multi. */
		SLATE_ATTRIBUTE(ESelectionMode::Type, SelectionMode)

		/** Gets the icon for a row in the list. */
		SLATE_EVENT(FGetRowIcon, OnGetRowIcon);

		/** Gets the text for a row in the list. Use OnGetCustomRow if a custom HeaderRow is in use. */
		SLATE_EVENT(FGetRowText, OnGetRowText);

		/** If needed, custom rows can be generated. This is needed if using a custom header row. */
		SLATE_EVENT(FGetCustomRow, OnGetCustomRow);

		/** Invoked when a delete operation is performed. */
		SLATE_EVENT(FOnDelete, OnDelete)

		/** Gets the menu items that should show up in the context menu when it is shown. Note that the delete operation is handled separately via OnDelete. */
		SLATE_EVENT(FOnGetContextMenuContent, OnGetContextMenuContent)

		/** Whether the enable/disable checkbox should be displayed. */
		SLATE_ATTRIBUTE(bool, ShowEnableDisable)

		/** Gets the enable state of the row. Called if ShowEnableDisable is turned on. */
		SLATE_EVENT(FGetRowEnableState, OnGetRowEnableState)

		/** Sets the enable state of the row. */
		SLATE_EVENT(FSetRowEnableState, OnSetRowEnableState)

		/** Optional header row that will be displayed in the list. Custom rows should also be implemented if this is provided. */
		SLATE_ARGUMENT_DEFAULT(TSharedPtr<SHeaderRow>, HeaderRow) = nullptr;
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs)
	{
		DataSource = InArgs._DataSource.Get();
		OnRefreshDataSourceRequested = InArgs._OnRefreshDataSourceRequested;
		DataType = InArgs._DataType.Get();
		DataTypePlural = InArgs._DataTypePlural.Get();
		CustomSummaryWidget = InArgs._CustomSummaryWidget;
		SelectionMode = InArgs._SelectionMode.Get();
		OnGetRowIcon = InArgs._OnGetRowIcon;
		OnGetRowText = InArgs._OnGetRowText;
		OnGetCustomRow = InArgs._OnGetCustomRow;
		OnDelete = InArgs._OnDelete;
		OnGetContextMenuContent = InArgs._OnGetContextMenuContent;
		bShowEnableDisable = InArgs._ShowEnableDisable.Get();
		OnGetRowEnableState = InArgs._OnGetRowEnableState;
		OnSetRowEnableState = InArgs._OnSetRowEnableState;
		HeaderRow = InArgs._HeaderRow;

		CommandList = MakeShared<FUICommandList>();

		CommandList->MapAction(
			FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &SMovieGraphSimpleList<ListType>::HandleDelete),
			FCanExecuteAction::CreateLambda([this]()
			{
				return OnDelete.IsBound();
			}));

		// The contents of the summary row is usually auto-generated to show the number of items in the data source, but it can optionally be
		// completely custom if needed
		const TSharedRef<SWidget> SummaryRowContents = CustomSummaryWidget.IsSet()
			? CustomSummaryWidget.Get().ToSharedRef()
			: SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text_Lambda([this]()
				{
					return FText::FromString(FString::Printf(TEXT("%i %s"), DataSource->Num(), DataSource->Num() == 1 ? *DataType.ToString() : *DataTypePlural.ToString()));
				});

		ChildSlot
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(ListView, SListView<ListType>)
				.ListItemsSource(DataSource)
				.SelectionMode(SelectionMode)
				.HeaderRow(HeaderRow)
				.OnKeyDownHandler_Lambda([this](const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
				{
					return CommandList->ProcessCommandBindings(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();
				})
				.OnGenerateRow(this, &SMovieGraphSimpleList<ListType>::GenerateRow)
				.OnContextMenuOpening(this, &SMovieGraphSimpleList<ListType>::OnContextMenuOpening)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Visibility_Lambda([this]()
				{
					// Hide the summary widget if it's a null widget
					return CustomSummaryWidget.IsSet() && (CustomSummaryWidget.Get() == SNullWidget::NullWidget.ToSharedPtr())
						? EVisibility::Collapsed
						: EVisibility::Visible;
				})
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.Padding(FMargin(14, 4, 7, 4))
				[
					SummaryRowContents
				]
			]
		];
	}

	/** Refreshes the contents of the list. */
	void Refresh()
	{
		if (ListView)
		{
			OnRefreshDataSourceRequested.ExecuteIfBound();
			ListView->RequestListRefresh();
		}
	}

private:
	/** Gets the row icon associated with the given list data. */
	const FSlateBrush* GetRowIcon(const ListType& InListData) const
	{
		if (OnGetRowIcon.IsBound())
		{
			return OnGetRowIcon.Execute(InListData);
		}
	
		return nullptr;
	}

	/** Gets the row text associated with the given list data. */
	FText GetRowText(const ListType& InListData) const
	{
		if (OnGetRowText.IsBound())
		{
			return OnGetRowText.Execute(InListData);
		}
	
		return FText();
	}

	/** Determines if the row that displays the given data should be enabled. */
	bool IsRowEnabled(const ListType& InListData) const
	{
		if (OnGetRowEnableState.IsBound())
		{
			return OnGetRowEnableState.Execute(InListData);
		}

		return true;
	}

	/** Deletes selected items in the list. */
	void HandleDelete() const
	{
		TArray<ListType> SelectedItems;
		ListView->GetSelectedItems(SelectedItems);

		if (!SelectedItems.IsEmpty())
		{
			if (OnDelete.IsBound())
			{
				OnDelete.Execute(SelectedItems);
			}
		}
	}

	TSharedPtr<SWidget> OnContextMenuOpening()
	{
		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

		MenuBuilder.SetSearchable(true);
		MenuBuilder.AddSearchWidget();

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("ContextMenuSection_Edit", "Edit"));
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		}
		MenuBuilder.EndSection();

		// Other custom context menu entries can be added by the delegate if needed
		if (OnGetContextMenuContent.IsBound())
		{
			TArray<ListType> SelectedItems;
			ListView->GetSelectedItems(SelectedItems);
			
			OnGetContextMenuContent.Execute(MenuBuilder, SelectedItems);
		}

		return MenuBuilder.MakeWidget();
	}

	/** Generates a row in the list for the specified data. */
	TSharedRef<ITableRow> GenerateRow(ListType InListData, const TSharedRef<STableViewBase>& InOwnerTable) const
	{
		// There may be completely-custom rows supplied, rather than the typical auto-generated ones that are displayed
		if (OnGetCustomRow.IsBound())
		{
			return OnGetCustomRow.Execute(InOwnerTable, InListData);
		}
		
		const FSlateBrush* RowIcon = GetRowIcon(InListData);
		
		return
			SNew(STableRow<ListType>, InOwnerTable)
			.Style(FAppStyle::Get(), "TableView.AlternatingRow")
			.ShowWires(false)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(7.f, 5.f, 7.f, 5.f)
				[
					SNew(SCheckBox)
					.Visibility_Lambda([this]()
					{
						return bShowEnableDisable ? EVisibility::Visible : EVisibility::Collapsed;
					})
					.IsChecked_Lambda([this, InListData]()
					{
						return IsRowEnabled(InListData) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this, InListData](ECheckBoxState NewState)
					{
						if (OnSetRowEnableState.IsBound())
						{
							OnSetRowEnableState.Execute(InListData, NewState == ECheckBoxState::Checked);
						}
					})
				]
				
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(3.f, 5.f, 7.f, 5.f)
				.AutoWidth()
				[
					SNew(SImage)
					.IsEnabled_Lambda([this, InListData]() { return IsRowEnabled(InListData); })
					.Visibility_Lambda([RowIcon]() { return RowIcon ? EVisibility::Visible : EVisibility::Collapsed; })
					.Image(RowIcon)
				]
				
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
					.IsEnabled_Lambda([this, InListData]() { return IsRowEnabled(InListData); })
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text(GetRowText(InListData))
				]
			];
	}

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override
	{
		Refresh();
	}
	
	virtual void PostRedo(bool bSuccess) override
	{
		Refresh();
	}
	//~ End FEditorUndoClient Interface

private:
	TSharedPtr<SListView<ListType>> ListView;
	TSharedPtr<FUICommandList> CommandList;
	FGetRowIcon OnGetRowIcon;
	FGetRowText OnGetRowText;
	FGetCustomRow OnGetCustomRow;
	FOnDelete OnDelete;
	FOnGetContextMenuContent OnGetContextMenuContent;
	FOnRefreshDataSourceRequested OnRefreshDataSourceRequested;
	bool bShowEnableDisable = false;
	FGetRowEnableState OnGetRowEnableState;
	FSetRowEnableState OnSetRowEnableState;
	TSharedPtr<SHeaderRow> HeaderRow;
	FText DataType;
	FText DataTypePlural;
	TAttribute<TSharedPtr<SWidget>> CustomSummaryWidget;
	TArray<ListType>* DataSource = nullptr;
	ESelectionMode::Type SelectionMode = ESelectionMode::Multi;
};

/**
 * A simple widget that displays a list that can be picked from. Meant for use within pop-ups. Shows a title above the list, and an optional message
 * to be displayed (instead of the list) if the data source is empty.
 */
template<typename DataType>
class SMovieGraphSimplePicker : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnItemPicked, DataType);
	DECLARE_DELEGATE_RetVal_OneParam(const FSlateBrush*, FGetRowIcon, DataType);
	DECLARE_DELEGATE_RetVal_OneParam(FText, FGetRowText, DataType);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnFilter, DataType);

	SLATE_BEGIN_ARGS(SMovieGraphSimplePicker)
		{}

		/** Called when an item is picked in the list. */
		SLATE_EVENT(FOnItemPicked, OnItemPicked)

		/** Gets the icon for a row in the list. */
		SLATE_EVENT(FGetRowIcon, OnGetRowIcon);

		/** Gets the text for a row in the list. */
		SLATE_EVENT(FGetRowText, OnGetRowText);

		/** The title text shown within the picker widget. */
		SLATE_ARGUMENT(FText, Title)

		/** The message displayed if the data source is empty. */
		SLATE_ARGUMENT(FText, DataSourceEmptyMessage)

		/** The data that is displayed within the picker. */
		SLATE_ARGUMENT(TArray<DataType>, DataSource);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnItemPicked = InArgs._OnItemPicked;
		OnGetRowIcon = InArgs._OnGetRowIcon;
		OnGetRowText = InArgs._OnGetRowText;
		Title = InArgs._Title;
		DataSourceEmptyMessage = InArgs._DataSourceEmptyMessage;
		DataSource = InArgs._DataSource;
		
		ChildSlot
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SBox)
			.WidthOverride(200.f)
			.HeightOverride(200.f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(5.f)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(Title)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				]
				
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Fill)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex_Lambda([this] { return DataSource.IsEmpty() ? 0 : 1; })

					+ SWidgetSwitcher::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(DataSourceEmptyMessage)
					]

					+ SWidgetSwitcher::Slot()
					[
						SNew(SListView<DataType>)
						.ListItemsSource(&DataSource)
						.SelectionMode(ESelectionMode::Single)
						.OnSelectionChanged(this, &SMovieGraphSimplePicker::OnItemSelected)
						.OnGenerateRow(this, &SMovieGraphSimplePicker::GenerateRow)
					]
				]
			]
		];
	}

private:
	/** Handles an item selected event. */
	void OnItemSelected(const DataType Item, ESelectInfo::Type Type) const
	{
		if (OnItemPicked.IsBound())
		{
			OnItemPicked.Execute(Item);
		}

		FSlateApplication::Get().DismissAllMenus();
	}
	
	/** Gets the row icon associated with the given item. */
	const FSlateBrush* GetRowIcon(const DataType InItem) const
	{
		if (OnGetRowIcon.IsBound())
		{
			return OnGetRowIcon.Execute(InItem);
		}

		return FAppStyle::GetBrush("Icons.FilledCircle");
	}

	/** Gets the row text associated with the given list data. */
	FText GetRowText(const DataType InListData) const
	{
		if (OnGetRowText.IsBound())
		{
			return OnGetRowText.Execute(InListData);
		}
		
		return FText();
	}

	/** Generates a row which displays a single item. */
	TSharedRef<ITableRow> GenerateRow(const DataType InItem, const TSharedRef<STableViewBase>& InOwnerTable) const
	{
		return
			SNew(STableRow<FName>, InOwnerTable)
			.Style(FAppStyle::Get(), "TableView.AlternatingRow")
			.ShowWires(false)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(7.f, 5.f, 7.f, 5.f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(GetRowIcon(InItem))
				]
				
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text(GetRowText(InItem))
				]
			];
	}

protected:
	/**
	 * The data source for the list view widget. Names of the items that can be picked. Widgets that inherit from this class can optionally control
	 * this data member directly rather than providing it as an argument.
	 */
	TArray<DataType> DataSource;

private:
	FOnItemPicked OnItemPicked;
	FGetRowIcon OnGetRowIcon;
	FGetRowText OnGetRowText;
	FText Title;
	FText DataSourceEmptyMessage;
};

#undef LOCTEXT_NAMESPACE

#endif	// WITH_EDITOR