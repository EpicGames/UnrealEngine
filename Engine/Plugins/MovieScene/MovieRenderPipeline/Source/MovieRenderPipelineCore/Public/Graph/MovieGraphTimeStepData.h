// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Timecode.h"
#include "MovieGraphTimeStepData.generated.h"

class UMovieGraphEvaluatedConfig;

/** 
* This data structure needs to be filled out each frame by the UMovieGraphTimeStepBase,
* which will eventually be passed to the renderer. It controls per-sample behavior such
* as the delta time, if this is the first/last sample for an output frame, etc.
*/
USTRUCT(BlueprintType)
struct FMovieGraphTimeStepData
{
	GENERATED_BODY()

public:
	FMovieGraphTimeStepData()
		: OutputFrameNumber(0)
		, ShotOutputFrameNumber(0)
		, RenderedFrameNumber(0)
		, FrameDeltaTime(0.f)
		, WorldTimeDilation(0.f)
		, bHasRelativeTimeBeenUsed(false)
		, WorldSeconds(0.f)
		, MotionBlurFraction(0.f)
		, FrameRate(0, 0)
		, TemporalSampleIndex(0)
		, TemporalSampleCount(0)
		, SpatialSampleIndex(0)
		, SpatialSampleCount(0)
		, bIsFirstTemporalSampleForFrame(false)
		, bIsLastTemporalSampleForFrame(false)
		, bDiscardOutput(false)
		, bIsCameraCut(false)
		, bRequiresAccumulator(false)
	{}

	/*
	 * Relative to zero for the entire render.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 OutputFrameNumber;

	/*
	 * Relative to zero for the current shot.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 ShotOutputFrameNumber;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 RenderedFrameNumber;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	float FrameDeltaTime;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	float WorldTimeDilation;

	/** True if time dilation was used on this frame, or any in the past, even if WorldTimeDilation has returned to 1.0 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	bool bHasRelativeTimeBeenUsed;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	float WorldSeconds;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	float MotionBlurFraction;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	FFrameRate FrameRate;

	/*
	* Index out of TemporalSampleCount we're on. No guarantee that we'll ever reach Index == Count-1,
	* if bIsLastTemporalSampleForFrame has priority over that (to allow early outs)
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 TemporalSampleIndex;

	/*
	* What is the maximum number of Temporal Samples this frame is expected to be able to process.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 TemporalSampleCount;

	/** Index out of SpatialSampleCount that we're on. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 SpatialSampleIndex;

	/** The maximum number of Spatial Samples this frame is expected to be able to process. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 SpatialSampleCount;

	/** 
	* Should be set to true for the first sample of each output frame. Used to determine
	* if various systems need to reset or gather data for a new frame. Can be true at
	* the same time as bIsLastTemporalSampleForFrame (ie: 1TS)
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	bool bIsFirstTemporalSampleForFrame;

	/**
	* Should be set to true for the last sample of each output frame. Can be true at
	* the same time as bIsFirstTemporalSampleForFrame (ie: 1TS)
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	bool bIsLastTemporalSampleForFrame;

	/**
	* Should the rendered result be discarded after a render? This will skip any
	* accumulators or readback and is used for frames that are only produced to
	* warm up the renderer.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	bool bDiscardOutput;

	/**
	 * When true, signals to the renderer that this sample is a camera cut. The velocity rendering pass
	 * skips most primitives (suppressing motion blur and TSR/TAA ghosting from stale previous-frame
	 * transform data), and TAA/TSR history buffers are cleared to force a fresh temporal accumulation.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	bool bIsCameraCut;

	/**
	* Should be set to true for every sample if there is more than one temporal sample
	* making up this render. This will cause the renderer to allocate accumulators
	* to store the multi-frame data into.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	bool bRequiresAccumulator;

	/**
	* The evaluated config holds the configuration used for this given frame. This pointer
	* can potentially change each frame (if the graph for that frame is different) but
	* users can rely on the EvaluatedConfig being correct for a given frame, thus all
	* resolves (such as filenames) should use the config for that frame, not the latest
	* one available.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig;

	/** The current timecode at the root (sequence) level. Note that this is adjusted according to the timecode settings on the Global Output Settings node. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	FTimecode RootTimeCode;

	/*
	 * The current frame number at the root (sequence) level.
	 * This is the same number as what is shown in sequencer when viewing the sequence.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	FFrameNumber RootFrameNumber;

	/** The current timecode at the shot level. Note that this is adjusted according to the timecode settings on the Global Output Settings node. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	FTimecode ShotTimeCode;

	/*
	 * The current frame number at the shot level.
	 * This is the same number as what is shown in sequencer when viewing the shot.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	FFrameNumber ShotFrameNumber;
};
