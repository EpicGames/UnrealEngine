// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Graph/MovieGraphRenderDataIdentifier.h"
#include "Graph/MovieGraphTimeStepData.h"
#include "MovieGraphTraversalContext.generated.h"

// Forward Declares
class UMovieGraphConfig;
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;


USTRUCT(BlueprintType)
struct FMovieGraphBranch
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MovieGraph")
	FName BranchName;
};


USTRUCT(BlueprintType)
struct FMovieGraphTraversalContext
{
	GENERATED_BODY();

public:
	FMovieGraphTraversalContext()
		: ShotIndex(0)
		, ShotCount(0)
	{}

	/** Which shot (out of ShotCount) is this time step for? */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 ShotIndex;

	/** The total number of shots being rendered for this job. This is from the active shot list, not total in the Level Sequence. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	int32 ShotCount;

	/** The job in the queue this traversal context is for. Needed to fetch variable values from the job. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TObjectPtr<UMoviePipelineExecutorJob> Job;

	/** The shot in the queue this traversal context is for (if any). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TObjectPtr<UMoviePipelineExecutorShot> Shot;

	/** The root graph to start our traversal from. This could be a shared config for the whole job, or a shot-specific override. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Graph")
	TObjectPtr<UMovieGraphConfig> RootGraph;

	/** The name of the render resource this state was captured for. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MovieGraph")
	FMovieGraphRenderDataIdentifier RenderDataIdentifier;

	/** The time data(output frame, delta times, etc.) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MovieGraph")
	FMovieGraphTimeStepData Time;
};