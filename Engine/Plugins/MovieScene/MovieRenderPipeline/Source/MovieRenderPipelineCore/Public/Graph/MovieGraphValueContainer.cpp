// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphValueContainer.h"

#include "MovieGraphCommon.h"

// The property bag has one "default" property in it

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphValueContainer)
const FName UMovieGraphValueContainer::PropertyBagDefaultPropertyName("Value");

UMovieGraphValueContainer::UMovieGraphValueContainer()
{
	PropertyName = PropertyBagDefaultPropertyName;

	MultiValueContainer = CreateDefaultSubobject<UMovieGraphMultiValueContainer>(TEXT("MultiValueContainer"));

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Add a default double property if one does not exist already
		if (MultiValueContainer->Values.GetNumPropertiesInBag() == 0)
		{
			MultiValueContainer->Values.AddProperty(PropertyName, EPropertyBagPropertyType::Double);
		}
	}
}

void UMovieGraphValueContainer::SetPropertyName(const FName& InName)
{
	if (const FPropertyBagPropertyDesc* Desc = MultiValueContainer->Values.FindPropertyDescByName(PropertyName))
	{
		if (Desc->Name == InName)
		{
			return;
		}

		Modify();
		PropertyName = InName;

		// Changing the property name requires a new desc and a migration
		FPropertyBagPropertyDesc NewDesc(*Desc);
		NewDesc.Name = InName;

		const UPropertyBag* NewPropBag = UPropertyBag::GetOrCreateFromDescs({NewDesc});
		MultiValueContainer->Values.MigrateToNewBagStruct(NewPropBag);
	}
}

FName UMovieGraphValueContainer::GetPropertyName() const
{
	return PropertyName;
}

void UMovieGraphValueContainer::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Migrate the old property bag over to the new multi-value container
		if (Value.GetNumPropertiesInBag() > 0)
		{
			const TConstArrayView<FPropertyBagPropertyDesc> ExistingDescs = Value.GetPropertyBagStruct()->GetPropertyDescs();
			MultiValueContainer->Values.AddProperties(ExistingDescs);
			MultiValueContainer->Values.CopyMatchingValuesByName(Value);
			Value.Reset();
		}
	}
}

bool UMovieGraphValueContainer::GetValueBool(bool& bOutValue) const
{
	return MultiValueContainer->GetValueBool(PropertyName, bOutValue);
}

bool UMovieGraphValueContainer::GetValueByte(uint8& OutValue) const
{
	return MultiValueContainer->GetValueByte(PropertyName, OutValue);
}

bool UMovieGraphValueContainer::GetValueInt32(int32& OutValue) const
{
	return MultiValueContainer->GetValueInt32(PropertyName, OutValue);
}

bool UMovieGraphValueContainer::GetValueInt64(int64& OutValue) const
{
	return MultiValueContainer->GetValueInt64(PropertyName, OutValue);
}

bool UMovieGraphValueContainer::GetValueFloat(float& OutValue) const
{
	return MultiValueContainer->GetValueFloat(PropertyName, OutValue);
}

bool UMovieGraphValueContainer::GetValueDouble(double& OutValue) const
{
	return MultiValueContainer->GetValueDouble(PropertyName, OutValue);
}

bool UMovieGraphValueContainer::GetValueName(FName& OutValue) const
{
	return MultiValueContainer->GetValueName(PropertyName, OutValue);
}

bool UMovieGraphValueContainer::GetValueString(FString& OutValue) const
{
	return MultiValueContainer->GetValueString(PropertyName, OutValue);
}

bool UMovieGraphValueContainer::GetValueText(FText& OutValue) const
{
	return MultiValueContainer->GetValueText(PropertyName, OutValue);
}

bool UMovieGraphValueContainer::GetValueEnum(uint8& OutValue, const UEnum* RequestedEnum) const
{
	return MultiValueContainer->GetValueEnum(PropertyName, OutValue, RequestedEnum);
}

bool UMovieGraphValueContainer::GetValueStruct(FStructView& OutValue, const UScriptStruct* RequestedStruct) const
{
	return MultiValueContainer->GetValueStruct(PropertyName, OutValue, RequestedStruct);
}

bool UMovieGraphValueContainer::GetValueObject(UObject*& OutValue, const UClass* RequestedClass) const
{
	return MultiValueContainer->GetValueObject(PropertyName, OutValue, RequestedClass);
}

bool UMovieGraphValueContainer::GetValueClass(UClass*& OutValue) const
{
	return MultiValueContainer->GetValueClass(PropertyName, OutValue);
}

FString UMovieGraphValueContainer::GetValueSerializedString()
{
	return MultiValueContainer->GetValueSerializedString(PropertyName);
}

bool UMovieGraphValueContainer::SetValueBool(const bool bInValue)
{
	return MultiValueContainer->SetValueBool(PropertyName, bInValue);
}

bool UMovieGraphValueContainer::SetValueByte(const uint8 InValue)
{
	return MultiValueContainer->SetValueByte(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueInt32(const int32 InValue)
{
	return MultiValueContainer->SetValueInt32(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueInt64(const int64 InValue)
{
	return MultiValueContainer->SetValueInt64(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueFloat(const float InValue)
{
	return MultiValueContainer->SetValueFloat(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueDouble(const double InValue)
{
	return MultiValueContainer->SetValueDouble(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueName(const FName InValue)
{
	return MultiValueContainer->SetValueName(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueString(const FString& InValue)
{
	return MultiValueContainer->SetValueString(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueText(const FText& InValue)
{
	return MultiValueContainer->SetValueText(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueEnum(const uint8 InValue, const UEnum* Enum)
{
	return MultiValueContainer->SetValueEnum(PropertyName, InValue, Enum);
}

bool UMovieGraphValueContainer::SetValueStruct(FConstStructView InValue)
{
	return MultiValueContainer->SetValueStruct(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueObject(UObject* InValue)
{
	return MultiValueContainer->SetValueObject(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueClass(UClass* InValue)
{
	return MultiValueContainer->SetValueClass(PropertyName, InValue);
}

bool UMovieGraphValueContainer::SetValueSerializedString(const FString& NewValue)
{
	return MultiValueContainer->SetValueSerializedString(PropertyName, NewValue);
}

EMovieGraphValueType UMovieGraphValueContainer::GetValueType() const
{
	return MultiValueContainer->GetValueType(PropertyName);
}

void UMovieGraphValueContainer::SetValueType(EMovieGraphValueType ValueType, UObject* InValueTypeObject)
{
	return MultiValueContainer->SetValueType(PropertyName, ValueType, InValueTypeObject);
}

const UObject* UMovieGraphValueContainer::GetValueTypeObject() const
{
	return MultiValueContainer->GetValueTypeObject(PropertyName);
}

void UMovieGraphValueContainer::SetValueTypeObject(const UObject* ValueTypeObject)
{
	return MultiValueContainer->SetValueTypeObject(PropertyName, ValueTypeObject);
}

EMovieGraphContainerType UMovieGraphValueContainer::GetValueContainerType() const
{
	return MultiValueContainer->GetValueContainerType(PropertyName);
}

void UMovieGraphValueContainer::SetValueContainerType(EMovieGraphContainerType ContainerType)
{
	return MultiValueContainer->SetValueContainerType(PropertyName, ContainerType);
}

TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> UMovieGraphValueContainer::GetArrayRef()
{
	return MultiValueContainer->GetArrayRef(PropertyName);
}

void UMovieGraphValueContainer::SetFromDesc(const FPropertyBagPropertyDesc* InDesc, const FString& InString)
{
	if (InDesc)
	{
		Modify();

		FPropertyBagPropertyDesc NewDesc;
		NewDesc.Name = InDesc->Name;
		NewDesc.ValueType = InDesc->ValueType;
		NewDesc.ContainerTypes = InDesc->ContainerTypes;
		NewDesc.ValueTypeObject = InDesc->ValueTypeObject;

		PropertyName = InDesc->Name;

		MultiValueContainer->Values.Reset();
		MultiValueContainer->Values.AddProperties({NewDesc});

		SetValueSerializedString(InString);
	}
}
