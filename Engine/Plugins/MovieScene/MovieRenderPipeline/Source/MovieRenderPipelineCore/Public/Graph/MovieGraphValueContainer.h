// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieGraphCommon.h"
#include "MovieGraphMultiValueContainer.h"
#include "StructUtils/PropertyBag.h"
#include "Templates/ValueOrError.h"

#include "MovieGraphValueContainer.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Holds a generic value, with an API for getting/setting the value, as well as getting/setting its type
 * and container type (eg, array).
 *
 * For a container that holds multiple values, see UMovieGraphMultiValueContainer.
 */
UCLASS(MinimalAPI)
class UMovieGraphValueContainer : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphValueContainer();

	/** Gets the bool value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueBool(bool& bOutValue) const;

	/** Gets the byte value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueByte(uint8& OutValue) const;

	/** Gets the int32 value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueInt32(int32& OutValue) const;

	/** Gets the int64 value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueInt64(int64& OutValue) const;

	/** Gets the float value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueFloat(float& OutValue) const;

	/** Gets the double value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueDouble(double& OutValue) const;

	/** Gets the FName value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueName(FName& OutValue) const;

	/** Gets the FString value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueString(FString& OutValue) const;

	/** Gets the FText value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueText(FText& OutValue) const;

	/** Gets the enum value (for a specific enum) of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueEnum(uint8& OutValue, const UEnum* RequestedEnum = nullptr) const;

	/** Gets the struct value (for a specific struct) of the held data. Returns true on success, else false. */
	UE_API bool GetValueStruct(FStructView& OutValue, const UScriptStruct* RequestedStruct = nullptr) const;

	/** Gets the object value (for a specific class) of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueObject(UObject*& OutValue, const UClass* RequestedClass = nullptr) const;

	/** Gets the UClass value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool GetValueClass(UClass*& OutValue) const;

	/** Gets the serialized string value of the held data. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API FString GetValueSerializedString();

	/** Gets the enum value of the held data. Returns true on success, else false. */
	template <typename T>
	bool GetValueEnum(T& OutValue) const
	{
		return GetValueEnum(PropertyName, OutValue);
	}

	/** Gets the struct value of the held data. Returns true on success, else false. */
	template <typename T>
	bool GetValueStruct(T*& OutValue) const
	{
		return MultiValueContainer->GetValueStruct(PropertyName, OutValue);
	}

	/** Gets the object value of the held data. Returns true on success, else false. */
	template <typename T>
	bool GetValueObject(T*& OutValue) const
	{
		return MultiValueContainer->GetValueObject(PropertyName, OutValue);
	}

	/** Sets the bool value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueBool(const bool bInValue);

	/** Sets the byte value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueByte(const uint8 InValue);

	/** Sets the int32 value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueInt32(const int32 InValue);

	/** Sets the int64 value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueInt64(const int64 InValue);

	/** Sets the float value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueFloat(const float InValue);

	/** Sets the double value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueDouble(const double InValue);

	/** Sets the FName value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueName(const FName InValue);

	/** Sets the FString value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueString(const FString& InValue);

	/** Sets the FText value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueText(const FText& InValue);

	/** Sets the enum value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueEnum(const uint8 InValue, const UEnum* Enum);

	/** Sets the struct value of the held data. Returns true on success, else false. */
	UE_API bool SetValueStruct(FConstStructView InValue);

	/** Sets the object value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueObject(UObject* InValue);

	/** Sets the class value of the held data. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueClass(UClass* InValue);

	/** Sets the serialized value of the held data. The string should be the serialized representation of the value. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API bool SetValueSerializedString(const FString& NewValue);

	/** Sets the enum value of the held data. Returns true on success, else false. */
	template <typename T>
	bool SetValueEnum(const T InValue)
	{
		return MultiValueContainer->SetValueEnum<T>(PropertyName, InValue);
	}

	/** Sets the struct value of the held data. Returns true on success, else false. */
	template <typename T>
	bool SetValueStruct(const T& InValue)
	{
		return MultiValueContainer->SetValueStruct<T>(PropertyName, InValue);
	}

	/** Sets the object value of the held data. Returns true on success, else false. */
	template <typename T>
	bool SetValueObject(T* InValue)
	{
		return MultiValueContainer->SetValueObject<T>(PropertyName, InValue);
	}

	/** Gets the type of the stored data. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API EMovieGraphValueType GetValueType() const;

	/** Sets the type of the stored data. Enums, structs, and classes must specify a value type object. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API void SetValueType(EMovieGraphValueType ValueType, UObject* InValueTypeObject = nullptr);

	/** Gets the object that defines the enum, struct, or class. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API const UObject* GetValueTypeObject() const;

	/** Sets the object that defines the enum, struct, or class. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API void SetValueTypeObject(const UObject* ValueTypeObject);

	/** Gets the container type of the stored value. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API EMovieGraphContainerType GetValueContainerType() const;

	/** Sets the container type of the stored value. */
	UFUNCTION(BlueprintCallable, Category="Config")
	UE_API void SetValueContainerType(EMovieGraphContainerType ContainerType);

	/**
	 * Gets a reference to the array backing the value, if any. GetValueContainerType() will return
	 * EMovieGraphContainerType::Array if the value is holding an array.
	 */
	UE_API TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> GetArrayRef();

	/**
	 * Sets the name of the property that the value container holds. This is mostly unneeded unless the display name
	 * of the property is important (eg, in the details panel).
	 */
	UE_API void SetPropertyName(const FName& InName);

	/** Gets the name of the property that the value container holds. */
	UE_API FName GetPropertyName() const;

private:
	virtual void PostLoad() override;
	
	/**
	 * Sets the configuration of this container from a property desc. This is less safe than the strongly-typed methods,
	 * and is reserved only for UMovieJobVariableAssignmentContainer to call. The serialized representation of the value
	 * needs to be supplied via InString.
	 */
	friend class UMovieJobVariableAssignmentContainer;
	UE_API void SetFromDesc(const FPropertyBagPropertyDesc* InDesc, const FString& InString);

private:
	/** The default name of the single property stored in the property bag. */
	static UE_API const FName PropertyBagDefaultPropertyName;

	/** The name of the single property stored in the property bag. */
	UPROPERTY()
	FName PropertyName;
	
	/** UNUSED (kept for backwards compatibility purposes in PostLoad()). The value held by this object. */
	UPROPERTY()
	FInstancedPropertyBag Value;

	/**
	 * The container that holds the single value.
	 *
	 * It's strange for a single-value class to internally hold a multi-value container. However, this was done in order to centralize the core
	 * logic of how MRG interacts with property bags in primarily one class (the multi-value container).
	 */
	UPROPERTY(EditAnywhere, Instanced, Category = "Value")
	TObjectPtr<UMovieGraphMultiValueContainer> MultiValueContainer;
};

#undef UE_API
