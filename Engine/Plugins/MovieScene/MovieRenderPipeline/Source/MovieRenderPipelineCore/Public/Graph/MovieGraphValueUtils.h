// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieGraphCommon.h"

namespace UE::MovieGraph::ValueUtils
{
	/** Gets the value type and value type object for a given FProperty. Returns true if the types could be determined, else false. */
	bool GetTypesForFProperty(const FProperty* InProperty, EMovieGraphValueType& OutValueType, TObjectPtr<UObject>& OutValueTypeObject);
}
