// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphValueView.h"

#include "MovieGraphCommon.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphValueView)

FMovieGraphValueViewProperty::FMovieGraphValueViewProperty(
	const FName& InPropertyName, const EMovieGraphValueType ValueType, const TObjectPtr<UObject>& InValueTypeObject, const TMap<FName, FString>& InMetadata)
	: PropertyName(InPropertyName)
	, ValueType(ValueType)
	, ValueTypeObject(InValueTypeObject)
	, Metadata(InMetadata)
{
	
}

UMovieGraphValueView::UMovieGraphValueView(FInstancedPropertyBag* InPropertyBag)
{
	PropertyBag = InPropertyBag;
}

void UMovieGraphValueView::ViewProperties(FInstancedPropertyBag* InPropertyBag)
{
	PropertyBag = InPropertyBag;
}

bool UMovieGraphValueView::IsValid() const
{
	return PropertyBag && PropertyBag->IsValid();
}

bool UMovieGraphValueView::AddProperty(const FName& InPropertyName, const EMovieGraphValueType ValueType, UObject* InValueTypeObject, const TMap<FName, FString>& InMetadata)
{
	if (!bAllowPropertyAddRemove)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Adding properties via this value view is not allowed; it was marked as non-mutable.");
		return false;
	}

	Modify();
	
	FMovieGraphValueViewProperty NewProperty;
	NewProperty.PropertyName = InPropertyName;
	NewProperty.ValueType = ValueType;
	NewProperty.ValueTypeObject = InValueTypeObject;
	NewProperty.Metadata = InMetadata;

	TArray<FMovieGraphValueViewProperty> NewProperties;
	NewProperties.Emplace(MoveTemp(NewProperty));

	return AddProperties(NewProperties);
}

bool UMovieGraphValueView::AddProperties(const TArray<FMovieGraphValueViewProperty>& InProperties)
{
	if (!bAllowPropertyAddRemove)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Adding properties via this value view is not allowed; it was marked as non-mutable.");
		return false;
	}

	Modify();

	TArray<FPropertyBagPropertyDesc> NewPropertyDescs;

	for (const FMovieGraphValueViewProperty& NewProperty : InProperties)
	{
		FPropertyBagPropertyDesc NewDesc(
			NewProperty.PropertyName, static_cast<EPropertyBagPropertyType>(NewProperty.ValueType), NewProperty.ValueTypeObject);

		NewDesc.ID = FGuid::NewGuid();

#if WITH_EDITOR
		for (const TPair<FName, FString>& MetadataPair : NewProperty.Metadata)
		{
			NewDesc.MetaData.Add(FPropertyBagPropertyDescMetaData(MetadataPair.Key, MetadataPair.Value));
		}
#endif

		NewPropertyDescs.Add(NewDesc);
	}

	return PropertyBag->AddProperties(NewPropertyDescs) == EPropertyBagAlterationResult::Success;
}

bool UMovieGraphValueView::RemoveProperty(const FName& InPropertyName)
{
	if (!bAllowPropertyAddRemove)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Removing properties via this value view is not allowed; it was marked as non-mutable.");
		return false;
	}

	Modify();

	const EPropertyBagAlterationResult Result = PropertyBag->RemovePropertyByName(InPropertyName);
	return Result == EPropertyBagAlterationResult::Success;
}

void UMovieGraphValueView::RemoveAllProperties()
{
	if (!bAllowPropertyAddRemove)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Removing properties via this value view is not allowed; it was marked as non-mutable.");
		return;
	}

	Modify();

	PropertyBag->Reset();
}

bool UMovieGraphValueView::HasProperty(const FName& InPropertyName) const
{
	return PropertyBag->FindPropertyDescByName(InPropertyName) != nullptr;
}

int32 UMovieGraphValueView::GetNumProperties() const
{
	return PropertyBag->GetNumPropertiesInBag();
}

TArray<FName> UMovieGraphValueView::GetPropertyNames() const
{
	TArray<FName> PropertyNames;

	if (const UPropertyBag* Bag = PropertyBag->GetPropertyBagStruct())
	{
		TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs = Bag->GetPropertyDescs();
		PropertyNames.Reserve(PropertyDescs.Num());

		for (const FPropertyBagPropertyDesc& Desc : PropertyDescs)
		{
			PropertyNames.Add(Desc.Name);
		}
	}

	return PropertyNames;	
}

#if WITH_EDITOR
TMap<FName, FString> UMovieGraphValueView::GetPropertyMetadata(const FName& InPropertyName) const
{
	TMap<FName, FString> MetadataMap;

	if (const FPropertyBagPropertyDesc* Desc = PropertyBag->FindPropertyDescByName(InPropertyName))
	{
		for (const FPropertyBagPropertyDescMetaData& MetadataEntry : Desc->MetaData)
		{
			MetadataMap.Add(MetadataEntry.Key, MetadataEntry.Value);
		}
	}

	return MetadataMap;
}
#endif	// WITH_EDITOR

bool UMovieGraphValueView::GetValueBool(const FName& InPropertyName, bool& bOutValue) const
{
	TValueOrError<bool, EPropertyBagResult> Result = PropertyBag->GetValueBool(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<bool>(Result, bOutValue);
}

bool UMovieGraphValueView::GetValueByte(const FName& InPropertyName, uint8& OutValue) const
{
	TValueOrError<uint8, EPropertyBagResult> Result = PropertyBag->GetValueByte(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<uint8>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueInt32(const FName& InPropertyName, int32& OutValue) const
{
	TValueOrError<int32, EPropertyBagResult> Result = PropertyBag->GetValueInt32(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<int32>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueInt64(const FName& InPropertyName, int64& OutValue) const
{
	TValueOrError<int64, EPropertyBagResult> Result = PropertyBag->GetValueInt64(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<int64>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueFloat(const FName& InPropertyName, float& OutValue) const
{
	TValueOrError<float, EPropertyBagResult> Result = PropertyBag->GetValueFloat(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<float>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueDouble(const FName& InPropertyName, double& OutValue) const
{
	TValueOrError<double, EPropertyBagResult> Result = PropertyBag->GetValueDouble(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<double>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueName(const FName& InPropertyName, FName& OutValue) const
{
	TValueOrError<FName, EPropertyBagResult> Result = PropertyBag->GetValueName(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<FName>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueString(const FName& InPropertyName, FString& OutValue) const
{
	TValueOrError<FString, EPropertyBagResult> Result = PropertyBag->GetValueString(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<FString>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueText(const FName& InPropertyName, FText& OutValue) const
{
	TValueOrError<FText, EPropertyBagResult> Result = PropertyBag->GetValueText(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<FText>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueEnum(const FName& InPropertyName, uint8& OutValue, const UEnum* RequestedEnum) const
{
	TValueOrError<uint8, EPropertyBagResult> Result = PropertyBag->GetValueEnum(InPropertyName, RequestedEnum);
	return UE::MovieGraph::Private::GetOptionalValue<uint8>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueStruct(const FName& InPropertyName, FStructView& OutValue, const UScriptStruct* RequestedStruct) const
{
	TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag->GetValueStruct(InPropertyName, RequestedStruct);
	return UE::MovieGraph::Private::GetOptionalValue<FStructView>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueObject(const FName& InPropertyName, UObject*& OutValue, const UClass* RequestedClass) const
{
	TValueOrError<UObject*, EPropertyBagResult> Result = PropertyBag->GetValueObject(InPropertyName, RequestedClass);
	return UE::MovieGraph::Private::GetOptionalValue<UObject*>(Result, OutValue);
}

bool UMovieGraphValueView::GetValueClass(const FName& InPropertyName, UClass*& OutValue) const
{
	TValueOrError<UClass*, EPropertyBagResult> Result = PropertyBag->GetValueClass(InPropertyName);
	return UE::MovieGraph::Private::GetOptionalValue<UClass*>(Result, OutValue);
}

FString UMovieGraphValueView::GetValueSerializedString(const FName& InPropertyName) const
{
	TValueOrError<FString, EPropertyBagResult> Result = PropertyBag->GetValueSerializedString(InPropertyName);
	FString ResultString;
	UE::MovieGraph::Private::GetOptionalValue<FString>(Result, ResultString);
	return ResultString;
}

bool UMovieGraphValueView::SetValueBool(const FName& InPropertyName, const bool bInValue)
{
	Modify();
	return PropertyBag->SetValueBool(InPropertyName, bInValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueByte(const FName& InPropertyName, const uint8 InValue)
{
	Modify();
	return PropertyBag->SetValueByte(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueInt32(const FName& InPropertyName, const int32 InValue)
{
	Modify();
	return PropertyBag->SetValueInt32(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueInt64(const FName& InPropertyName, const int64 InValue)
{
	Modify();
	return PropertyBag->SetValueInt64(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueFloat(const FName& InPropertyName, const float InValue)
{
	Modify();
	return PropertyBag->SetValueFloat(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueDouble(const FName& InPropertyName, const double InValue)
{
	Modify();
	return PropertyBag->SetValueDouble(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueName(const FName& InPropertyName, const FName InValue)
{
	Modify();
	return PropertyBag->SetValueName(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueString(const FName& InPropertyName, const FString& InValue)
{
	Modify();
	return PropertyBag->SetValueString(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueText(const FName& InPropertyName, const FText& InValue)
{
	Modify();
	return PropertyBag->SetValueText(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueEnum(const FName& InPropertyName, const uint8 InValue, const UEnum* Enum)
{
	Modify();
	return PropertyBag->SetValueEnum(InPropertyName, InValue, Enum) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueStruct(const FName& InPropertyName, FConstStructView InValue)
{
	Modify();
	return PropertyBag->SetValueStruct(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueObject(const FName& InPropertyName, UObject* InValue)
{
	Modify();
	return PropertyBag->SetValueObject(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueClass(const FName& InPropertyName, UClass* InValue)
{
	Modify();
	return PropertyBag->SetValueClass(InPropertyName, InValue) == EPropertyBagResult::Success;
}

bool UMovieGraphValueView::SetValueSerializedString(const FName& InPropertyName, const FString& NewValue)
{
	Modify();
	return PropertyBag->SetValueSerializedString(InPropertyName, NewValue) == EPropertyBagResult::Success;
}

EMovieGraphValueType UMovieGraphValueView::GetValueType(const FName& InPropertyName) const
{
	if (const FPropertyBagPropertyDesc* Desc = PropertyBag->FindPropertyDescByName(InPropertyName))
	{
		return static_cast<EMovieGraphValueType>(Desc->ValueType);
	}

	return EMovieGraphValueType::None;
}

void UMovieGraphValueView::SetValueType(const FName& InPropertyName, EMovieGraphValueType ValueType, UObject* InValueTypeObject)
{
	if (const FPropertyBagPropertyDesc* Desc = PropertyBag->FindPropertyDescByName(InPropertyName))
	{
		Modify();

		constexpr bool bOverwrite = true;

		// If the property is already a container, set the value type of the container. Otherwise, just set the value type of the property.
		if (!Desc->ContainerTypes.IsEmpty())
		{
			PropertyBag->AddContainerProperty(
				InPropertyName, Desc->ContainerTypes.GetFirstContainerType(), static_cast<EPropertyBagPropertyType>(ValueType),
				InValueTypeObject, bOverwrite, Desc->KeyType, Desc->KeyTypeObject);
		}
		else
		{
			PropertyBag->AddProperty(InPropertyName, static_cast<EPropertyBagPropertyType>(ValueType), InValueTypeObject, bOverwrite);
		}
	}
}

const UObject* UMovieGraphValueView::GetValueTypeObject(const FName& InPropertyName) const
{
	if (const FPropertyBagPropertyDesc* Desc = PropertyBag->FindPropertyDescByName(InPropertyName))
	{
		return Desc->ValueTypeObject;
	}
	
	return nullptr;
}

void UMovieGraphValueView::SetValueTypeObject(const FName& InPropertyName, const UObject* ValueTypeObject)
{
	if (const FPropertyBagPropertyDesc* Desc = PropertyBag->FindPropertyDescByName(InPropertyName))
	{
		Modify();

		constexpr bool bOverwrite = true;

		if (Desc->ContainerTypes.IsEmpty())
		{
			PropertyBag->AddProperty(InPropertyName, Desc->ValueType, ValueTypeObject, bOverwrite);
		}
		else
		{
			PropertyBag->AddContainerProperty(
				InPropertyName, Desc->ContainerTypes, Desc->ValueType, ValueTypeObject, bOverwrite, Desc->KeyType, Desc->KeyTypeObject);
		}
	}
}

EMovieGraphContainerType UMovieGraphValueView::GetValueContainerType(const FName& InPropertyName) const
{
	if (const FPropertyBagPropertyDesc* Desc = PropertyBag->FindPropertyDescByName(InPropertyName))
	{
		return static_cast<EMovieGraphContainerType>(Desc->ContainerTypes.GetFirstContainerType());
	}

	return EMovieGraphContainerType::None;
}

void UMovieGraphValueView::SetValueContainerType(const FName& InPropertyName, EMovieGraphContainerType ContainerType)
{
	if (const FPropertyBagPropertyDesc* Desc = PropertyBag->FindPropertyDescByName(InPropertyName))
	{
		Modify();

		constexpr bool bOverwrite = true;
		PropertyBag->AddContainerProperty(InPropertyName, static_cast<EPropertyBagContainerType>(ContainerType), Desc->ValueType, Desc->ValueTypeObject, bOverwrite);
	}
}

TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> UMovieGraphValueView::GetArrayRef(const FName& InPropertyName)
{
	return PropertyBag->GetMutableArrayRef(InPropertyName);
}

// ---------

UMovieGraphFixedValueView::UMovieGraphFixedValueView()
{
	bAllowPropertyAddRemove = false;
}

UMovieGraphFixedValueView::UMovieGraphFixedValueView(FInstancedPropertyBag* InPropertyBag)
	: Super(InPropertyBag)
{
	bAllowPropertyAddRemove = false;
}

UMovieGraphMutableValueView::UMovieGraphMutableValueView()
{
	bAllowPropertyAddRemove = true;
}

UMovieGraphMutableValueView::UMovieGraphMutableValueView(FInstancedPropertyBag* InPropertyBag)
	: Super(InPropertyBag)
{
	bAllowPropertyAddRemove = true;
}