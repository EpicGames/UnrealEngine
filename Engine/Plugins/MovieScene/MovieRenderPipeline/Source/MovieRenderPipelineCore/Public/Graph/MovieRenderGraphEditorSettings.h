// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "UObject/SoftObjectPath.h"

#include "MovieGraphDataTypes.h"
#include "MovieGraphQuickRenderSettings.h"
#include "MoviePipelinePostRenderSettings.h"

#include "MovieRenderGraphEditorSettings.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** The default configuration type assigned to newly created jobs in the queue. */
UENUM()
enum class EMoviePipelineDefaultConfigType : uint8
{
	Basic,
	MovieRenderGraph,
	LegacyPreset
};

UCLASS(MinimalAPI, BlueprintType, Config=EditorPerProjectUserSettings, meta=(DisplayName="Movie Render Graph"))
class UMovieRenderGraphEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UMovieRenderGraphEditorSettings();

	UE_API virtual FName GetCategoryName() const override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	/** If playing back media post-render, these settings dictate how that will be done. */
	UPROPERTY(Config, EditAnywhere, Category="Play Render Output", meta=(ShowOnlyInnerProperties))
	FMovieGraphPostRenderSettings PostRenderSettings;

	/** The configuration type assigned to newly created jobs within the editor. */
	UPROPERTY(Config, EditAnywhere, Category="Queue Defaults")
	EMoviePipelineDefaultConfigType DefaultConfigType = EMoviePipelineDefaultConfigType::Basic;

	/**
	 * Output types visible as toggle buttons in Basic Config mode.
	 * Each entry is the soft class path of a UMovieGraphFileOutputNode subclass.
	 */
	UPROPERTY(Config)
	TArray<FSoftClassPath> VisibleBasicConfigOutputTypes;

	/** Whether the Burn In Class property is shown in Basic Config mode. */
	UPROPERTY(Config)
	bool bShowBurnInForBasicConfig = false;
};

#undef UE_API
