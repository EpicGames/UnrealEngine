// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"

#include "MovieGraphApplyCVarPresetNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class IMovieSceneConsoleVariableTrackInterface;

/** A node which can apply a console variable preset. */
UCLASS(MinimalAPI)
class UMovieGraphApplyCVarPresetNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()
	
public:
	UMovieGraphApplyCVarPresetNode() = default;

	UE_API virtual FString GetNodeInstanceName() const override;
	UE_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	UE_API virtual TArray<FMovieGraphPropertyInfo> GetOverrideablePropertyInfo() const override;
	UE_API virtual TArray<FPropertyBagPropertyDesc> GetDynamicPropertyDescriptions() const override;
	UE_API virtual bool GetDynamicPropertyValue(const FName PropertyName, FString& OutValue) override;
	UE_API virtual void TogglePromotePropertyToPin(const FName& PropertyName) override;
	UE_API virtual void PrepareForFlattening(const UMovieGraphSettingNode* InSourceNode) override;

	/**
	 * Gets the name and resolved value of any cvars that were overridden via promoted pins. This method only makes sense to call on a
	 * flattened (resolved) node.
	 */
	UE_API TArray<TPair<FString, float>> GetConsoleVariableOverrides() const;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ConsoleVariablePreset : 1;

	/** The console variable preset that should be applied. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_ConsoleVariablePreset"))
	TScriptInterface<IMovieSceneConsoleVariableTrackInterface> ConsoleVariablePreset;
};

#undef UE_API
