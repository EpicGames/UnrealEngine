// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "ShowFlags.h"

#include "MovieGraphApplyViewportLookNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class FLevelEditorViewportClient;
class FSceneView;

// Note: This node is marked as HideDropdown so it does not appear in the node creation menu within the graph. It is meant to
// be created by Quick Render only. This node will not properly provide all of its functionality outside of Quick Render.

/** A node which applies the look of the viewport to the render (show flags, view mode, OCIO, etc). */
UCLASS(MinimalAPI, HideDropDown, NotBlueprintType)
class UMovieGraphApplyViewportLookNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UMovieGraphApplyViewportLookNode() = default;

	//~ Begin UMovieGraphNode interface (not editor-only)
	UE_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;
	//~ End UMovieGraphNode interface

#if WITH_EDITOR
	/** Gets the viewport client for the currently active viewport. */
	static UE_API FLevelEditorViewportClient* GetViewportClient();
	
	/** Gets the show flags and view mode index for the currently active viewport. */
	UE_API bool GetViewportInfo(FEngineShowFlags& OutShowFlags, EViewModeIndex& OutViewModeIndex) const;

	/** Updates the given scene view to be like the current viewport's scene view. */
	UE_API void UpdateSceneView(FSceneView* InSceneView) const;

	//~ Begin UMovieGraphNode interface (editor-only)
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UMovieGraphNode interface
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bOcio : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bShowFlags : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bViewMode : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bVisibility : 1;
	
	/** Set to true to apply the viewport's OCIO settings to the render. Only render nodes that have "Allow OCIO" turned on will be impacted. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Viewport Look", DisplayName="OCIO", meta=(EditCondition="bOverride_bOcio"))
	bool bOcio;

	/** Set to true to apply the viewport's show flags to the render. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Viewport Look", meta=(EditCondition="bOverride_bShowFlags"))
	bool bShowFlags;

	/** Set to true to apply the viewport's view mode to the render. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Viewport Look", meta=(EditCondition="bOverride_bViewMode"))
	bool bViewMode;

	/** Set to true to apply editor visibility to actors in the render. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Viewport Look", meta=(EditCondition="bOverride_bVisibility"))
	bool bVisibility;
};

#undef UE_API
