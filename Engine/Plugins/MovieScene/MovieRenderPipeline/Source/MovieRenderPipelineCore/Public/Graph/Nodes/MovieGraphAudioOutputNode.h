// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Nodes/MovieGraphFileOutputNode.h"
#include "Sound/SampleBufferIO.h"

#include "MovieGraphAudioOutputNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * A node which outputs .wav audio after all renders have completed.
 */
UCLASS(MinimalAPI)
class UMovieGraphAudioOutputNode : public UMovieGraphFileOutputNode
{
	GENERATED_BODY()
	
public:
	UE_API UMovieGraphAudioOutputNode();

	// UMovieGraphSettingNode Interface
	UE_API virtual void BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	// ~UMovieGraphSettingNode Interface

	// UMovieGraphNode Interface
	UE_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive) const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	// ~UMovieGraphNode Interface

	// UMovieGraphFileOutputNode interface
	UE_API virtual FText GetBasicConfigShortDisplayName() const override;
#endif

	/**
	 * Although this node does not represent a proper "renderer", it will sometimes be identified in data structures
	 * as a renderer. If that's the case, this is the name of the audio "renderer".
	 */
	inline static const FString RendererName = TEXT("Audio");

	/** The extension of the audio files generated by this node. */
	inline static const FString OutputExtension = TEXT("wav");


protected:
	// UMovieGraphFileOutputNode Interface
	UE_API virtual void OnAllFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) override;
	UE_API virtual void OnAllShotFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, const UMoviePipelineExecutorShot* InShot, TObjectPtr<UMovieGraphEvaluatedConfig>& InShotEvaluatedGraph, const bool bFlushToDisk) override;
	UE_API virtual bool IsFinishedWritingToDiskImpl() const override;
	// ~UMovieGraphFileOutputNode Interface

private:
	/** Represents an audio file, with its associated audio data. */
	struct FFinalAudioData
	{
		FString FilePath;
		Audio::TSampleBuffer<int16> SampleBuffer;
		int32 ShotIndex;
		FMovieGraphRenderDataIdentifier RenderIdentifier;
	};

	/** Generate the final output path for the audio file. */
	UE_API FString GenerateOutputPath(const FMovieGraphRenderDataIdentifier& InRenderIdentifier, const TObjectPtr<UMoviePipelineExecutorShot>& InShot) const;

	/** Generates final audio data from all of the audio segments that were collected during render time. */
	UE_API void GenerateFinalAudioData(TArray<FFinalAudioData>& OutFinalAudioData);

	/** Begins the audio export process. */
	UE_API void StartAudioExport();
	
	/** Returns true if there should be one export per shot, else false. */
	UE_API bool NeedsPerShotFlushing() const;

// The pipeline generates many instances of the same node throughout its execution; however, some nodes need to have persistent data throughout the
// pipeline's lifetime. This static data enables the node to have shared data across instances.
private:
	/** Kept alive during finalization because the writer writes async to disk but doesn't expect to fall out of scope. */
	UE_API inline static TArray<TUniquePtr<Audio::FSoundWavePCMWriter>> ActiveWriters;

	/** Keep track of segments that have already been written to disk to avoid re-writing them (and generating new output futures). */
	UE_API inline static TSet<FGuid> AlreadyWrittenSegments;

private:
	/** The pipeline that is running this node. */
	TWeakObjectPtr<UMovieGraphPipeline> CachedPipeline;

	/**
	 * The graph that should be accessed during execution of the node. Do not access the graph from the pipeline as it may be invalid depending on
	 * if a shot-level or sequence-level export is being performed.
	 */
	TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedGraph;
};

#undef UE_API
