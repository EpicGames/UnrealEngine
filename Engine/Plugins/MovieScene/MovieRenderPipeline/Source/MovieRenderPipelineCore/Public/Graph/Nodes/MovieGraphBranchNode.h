// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"

#include "MovieGraphBranchNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** 
* A node which creates a True/False branching condition. A user Graph Variable can be plugged
* into the conditional pin and this will be evaluated when flattening the graph, choosing which
* branch path to follow.
*/
UCLASS(MinimalAPI)
class UMovieGraphBranchNode : public UMovieGraphNode
{
	GENERATED_BODY()

public:
	UMovieGraphBranchNode() = default;

	UE_API virtual TArray<FMovieGraphPinProperties> GetInputPinProperties() const override;
	UE_API virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const override;
	UE_API virtual TArray<UMovieGraphPin*> EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const override;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif
};

#undef UE_API
