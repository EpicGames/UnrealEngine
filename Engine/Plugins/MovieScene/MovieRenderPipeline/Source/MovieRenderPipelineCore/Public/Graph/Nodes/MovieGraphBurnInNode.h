// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Nodes/MovieGraphWidgetRendererBaseNode.h"

#include "MovieGraphBurnInNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
class SVirtualWindow;
class UMovieGraphBurnInWidget;
class UMovieGraphDefaultRenderer;
struct FMovieGraphRenderPassLayerData;

/**
 * A node which generates a widget burn-in, rendered to a standalone image or composited on top of a render layer.
 *
 * DEPRECATED. Use the burn-in settings on output nodes instead to generate burn-ins.
 */
UCLASS(MinimalAPI, Hidden, HideDropdown, HideCategories = (Naming))
class UMovieGraphBurnInNode : public UMovieGraphWidgetRendererBaseNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphBurnInNode();

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

	/**
	 * Gets an existing widget instance of type WidgetClass if one has been created. Otherwise, returns a new instance
	 * of WidgetClass with owner InOwner.
	 */
	UE_API TObjectPtr<UMovieGraphBurnInWidget> GetOrCreateBurnInWidget(UClass* WidgetClass, UWorld* InOwner);

public:
	UE_DEPRECATED(5.7, "RendererName usage is deprecated, please use GetRendererName() instead.")
	static UE_API const FString RendererName;

	/** The path to the default widget class that will be used for the burn-in. */
	static UE_API const FString DefaultBurnInWidgetAsset;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInClass : 1;

	/** The widget that the burn-in should use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(MetaClass="/Script/MovieRenderPipelineCore.MovieGraphBurnInWidget", EditCondition="bOverride_BurnInClass"), Category="Widget Settings")
	FSoftClassPath BurnInClass;

protected:
	/** A burn-in pass for a specific render layer. Instances are stored on the UMovieGraphWidgetRendererBaseNode CDO. */
	struct FMovieGraphBurnInPass final : public FMovieGraphWidgetPass
	{
		virtual void Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, const FMovieGraphRenderPassLayerData& InLayer) override;
		virtual TSharedPtr<SWidget> GetWidget(UMovieGraphWidgetRendererBaseNode* InNodeThisFrame) override;
		virtual void Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) override;
		virtual int32 GetCompositingSortOrder() const override;
		virtual UMovieGraphWidgetRendererBaseNode* GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const override;

		/** Gets the burn-in class that this pass will be using. */
		UClass* GetBurnInClass() const;

	private:
		TObjectPtr<UMovieGraphBurnInWidget> GetBurnInWidget(UMovieGraphWidgetRendererBaseNode* InNodeThisFrame) const;
		TSubclassOf<UMovieGraphBurnInWidget> CachedBurnInWidgetClass;

		/**
		 * If this is an output-specific burn-in (eg, just for JPG), the burn-in renderer node will have a specific instance name. That will
		 * identify it amongst all of the other output-specific burn-in nodes on the branch (if there are any others). If this is empty, then this
		 * pass doesn't represent an output-specific burn-in, and is for the legacy burn-in node.
		 */
		FString RendererNodeInstanceName;
	};

private:
	/** Burn-in widget instances shared with all FMovieGraphBurnInPass instances, keyed by burn-in class. */
	UPROPERTY(Transient)
	TMap<TObjectPtr<const UClass>, TObjectPtr<UMovieGraphBurnInWidget>> BurnInWidgetInstances;

protected:
	// UMovieGraphWidgetRendererBaseNode Interface
	UE_API virtual TUniquePtr<FMovieGraphWidgetPass> GeneratePass() override;
	// ~UMovieGraphWidgetRendererBaseNode Interface
	
	// UMovieGraphRenderPassNode Interface
	UE_API virtual void GatherOutputPassesImpl(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const override;
	UE_API virtual void TeardownImpl() override;
	// ~UMovieGraphRenderPassNode Interface
};

/**
 * A node which generates a widget burn-in, rendered to a standalone image or composited on top of a render layer.
 *
 * This node is not meant to be added by the user directly. Instead, it's injected into the graph by a specific output node (eg, JPG). It can control
 * which output type(s) write it out (see the OutputRestriction property).
 * 
 */
UCLASS(MinimalAPI, Hidden, HideDropdown)
class UMovieGraphOutputBurnInNode : public UMovieGraphBurnInNode
{
	GENERATED_BODY()

public:
	UMovieGraphOutputBurnInNode() = default;

	// UMovieGraphSettingNode Interface
	virtual FString GetNodeInstanceName() const override;
	// ~UMovieGraphSettingNode Interface

	// UMovieGraphNode Interface
#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive) const override;
#endif
	virtual bool GetNodeValidationErrors(const FName& InBranchName, const UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<FText>& OutValidationErrors) const override;
	// ~UMovieGraphNode Interface

protected:
	virtual TArray<FSoftClassPath> GetOutputTypeRestrictionsImpl() const override;
	virtual FString GetRendererNameImpl() const override;

	// UMovieGraphWidgetRendererBaseNode Interface
	UE_API virtual FString GetFileNameFormatOverride() const override;
	UE_API virtual FString GetLayerNameOverride() const override;
	// ~UMovieGraphWidgetRendererBaseNode Interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputName : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_FileNameFormat : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputRestriction : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_LayerNameFormat : 1;
	
	/**
	 * The name of the output that is associated with this burn-in (eg, "JPG"). Will be used as the instance name for the node, so it should be
	 * unique if multiple burn-in nodes need to co-exist in the same branch.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta=(EditCondition="bOverride_OutputName"))
	FString OutputName;

	/** The type(s) of nodes that can write out this burn-in. If no output types are specified, all active output types will write out the burn-in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta=(EditCondition="bOverride_OutputRestriction"))
	FSoftClassPath OutputRestriction;

	/** If not composited, this is the format string used for the file name that the burn-in is written to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta=(EditCondition="bOverride_FileNameFormat"))
	FString FileNameFormat = "{sequence_name}.{layer_name}.{frame_number}";

	/** For multi-layer output formats, this is the format string used to generate the layer name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta=(EditCondition="bOverride_LayerNameFormat"))
	FString LayerNameFormat;
};

#undef UE_API
