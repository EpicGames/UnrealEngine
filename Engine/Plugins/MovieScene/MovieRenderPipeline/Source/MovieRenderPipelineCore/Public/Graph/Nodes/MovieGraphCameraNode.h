// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "MovieRenderPipelineDataTypes.h" // For EMoviePipelineShutterTiming
#include "MovieGraphCameraNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A node which configures global camera settings that are shared among all renders. */
UCLASS(MinimalAPI)
class UMovieGraphCameraSettingNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UMovieGraphCameraSettingNode()
		: ShutterTiming(EMoviePipelineShutterTiming::FrameCenter)
		, OverscanPercentage(0.f)
		, bRenderAllCameras(false)
	{}

	virtual EMovieGraphBranchRestriction GetBranchRestriction() const override { return EMovieGraphBranchRestriction::Globals; }
	UE_API virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const override;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ShutterTiming : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OverscanPercentage : 1;


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bRenderAllCameras : 1;

	/**
	* Shutter Timing allows you to bias the timing of your shutter angle to either be before, during, or after
	* a frame. When set to FrameClose, it means that the motion gathered up to produce frame N comes from 
	* before and right up to frame N. When set to FrameCenter, the motion represents half the time before the
	* frame and half the time after the frame. When set to FrameOpen, the motion represents the time from 
	* Frame N onwards.
	* 
	* As an example, here are 4 temporal subsamples with 180 degree shutter angle (50% motion blur) between frames A & B:
	*
	* FrameClose
	* F-------F-------F
	* ----1234---------
	* ----AAAA---------
	*
	* FrameCenter
	* F-------F-------F
	* ------1234-------
	* ------AABB-------
	*
	* FrameOpen
	* F-------F-------F
	* --------1234-----
	* --------BBBB-----
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bOverride_ShutterTiming"))
	EMoviePipelineShutterTiming ShutterTiming;

	/**
	* Overscan percent enables rendering additional pixels beyond the current target resolution, and can be used in conjunction 
	* with the EXR output format to add post-processing effects such as lens distortion. Please note that using this feature may cause
	* the render to appear different in some scenarios (for example, auto-exposure may be impacted due to the increased FoV picking
	* up additional lighting).
	* 
	* With EXR outputs, the non-overscanned pixel data is contained in the EXR "display window", and the overscanned pixel data is
	* contained in the "data window". For example, a 1920x1080 render with 10% overscan will result in a 2112x1188 render; the display
	* window will be 1920x1080, and the data window will be 2112x1188. For all other formats, the pixels are cropped to the
	* original target resolution (ie, no extra pixel data is included).
	*
	* Note: This uses values in the [0-100] range, and not [0-1] like the preset system, to bring it in-line with other usages
	* of overscan in the engine (like nDisplay).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Overscan Percentage Override", UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "100"), Category = "Settings", meta = (EditCondition = "bOverride_OverscanPercentage"))
	float OverscanPercentage;

	/*
	* If enabled Movie Render Queue will examine your Level Sequence for additional cameras and create an additional render for each renderer for that camera.
	* The Camera Cut Track/Section is still used to determine the range of time to render, and then all Camera Actors that are in the level sequence adjacent
	* to the Camera Cut Track will be considered for rendering. They are expected to exist the entire time and do not support rendering sub-ranges.
	* 
	* This increases render duration (100% per camera) and has increased VRAM/RAM requirements. However all cameras are rendered on the same engine tick so they
	* should all see a consistent view of the world which can be useful for things like particle effects, etc.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bOverride_bRenderAllCameras"))
	bool bRenderAllCameras;
};

#undef UE_API
