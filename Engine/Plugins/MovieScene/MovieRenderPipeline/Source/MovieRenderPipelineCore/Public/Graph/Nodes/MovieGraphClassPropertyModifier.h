// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "Graph/MovieGraphMultiValueContainer.h"
#include "StructUtils/PropertyBag.h"

#include "MovieGraphClassPropertyModifier.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A scripting-friendly representation of a property on an AActor or UActorComponent. */
USTRUCT(BlueprintType)
struct FMovieGraphPropertyReference
{
	GENERATED_BODY()

public:
	UE_API FMovieGraphPropertyReference() = default;

	/**
	 * Providing the FProperty is optional. Providing it just bypasses the lookup of the FProperty that will occur if only the property name and
	 * class/component are provided.
	 */
	UE_API FMovieGraphPropertyReference(
		const TObjectPtr<UClass>& InPropertyClass,
		const TSubclassOf<UActorComponent>& InComponentClass,
		const FName& InPropertyName,
		FProperty* InProperty = nullptr);

	/** Gets the value type of the referenced property. */
	UE_API EMovieGraphValueType GetValueType() const;

	/** Gets the value type object of the referenced property (for objects, structs, and enums). */
	UE_API UObject* GetValueTypeObject() const;

	/** Gets the FProperty that backs this property reference. */
	UE_API FProperty* GetProperty() const;

	/** Gets the name of the property that this reference refers to. */
	UE_API FName GetPropertyName() const;

	/** Gets the actor class that this property exists in. If the property exists in a component, this is the class that the component belongs to. */
	UE_API TSubclassOf<AActor> GetPropertyClass() const;

	/** Gets the class of the component that the property exists in. Can be nullptr if the property does not exist in a component. */
	UE_API TSubclassOf<UActorComponent> GetComponentClass() const;

	/** Determines if this property reference points to a valid property. */
	UE_API bool IsValidProperty() const;

	// Called by TStructOpsTypeTraits
	void PostSerialize(const FArchive& Ar);

public:
	/**
	 * An additional property name which may be used for other purposes. Does not affect any internal FMovieGraphPropertyReference functionality and
	 * can be left empty if not needed.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Property")
	FName AuxPropertyName;

private:
	/** The actor class that the property exists in. If the property exists within a component, this is the class that the component exists in. */
	UPROPERTY(BlueprintReadOnly, Category = "Property", meta = (AllowPrivateAccess = "true"))
	TSubclassOf<AActor> PropertyClass = nullptr;

	/** The UActorComponent class that the property exists in, or nullptr if the property does not exist in a component. */
	UPROPERTY(BlueprintReadOnly, Category = "Property", meta = (AllowPrivateAccess = "true"))
	TSubclassOf<UActorComponent> ComponentClass = nullptr;

	/** The name of the property within either the class or the component class. */
	UPROPERTY(BlueprintReadOnly, Category = "Property", meta = (AllowPrivateAccess = "true"))
	FName PropertyName;

	/** Calculated (or optionally provided): The resolved FProperty that lives on the PropertyClass or ComponentClass. */
	FProperty* Property = nullptr;

	/** Calculated: The value type of the referenced property. */
	UPROPERTY(BlueprintReadOnly, Category = "Property", meta = (AllowPrivateAccess = "true"))
	EMovieGraphValueType ValueType = EMovieGraphValueType::None;

	/** Calculated: The value type object of the referenced property (for objects, structs, and enums). */
	UPROPERTY(BlueprintReadOnly, Category = "Property", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UObject> ValueTypeObject = nullptr;

private:
	/** Calculates and caches property-related information so it doesn't need to be constantly re-derived. */
	void PopulateDerivedData();
};

template<>
struct TStructOpsTypeTraits<FMovieGraphPropertyReference> : public TStructOpsTypeTraitsBase2<FMovieGraphPropertyReference>
{
	enum
	{
		WithPostSerialize = true,
	};
};

/**
 * Modifies properties on specific actor or component type(s).
 *
 * The properties that should be modified are added via MovieGraphPropertyReference objects via AddProperties(). Note that the
 * properties are matched to their EXACT AActor and/or UActorComponent class within the property reference (ie, subclass checks
 * are not made).
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphPropertyModifier final : public UMovieGraphCollectionModifier
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphPropertyModifier();

	//~ Begin UMovieGraphModifierBase Interface
	UE_API virtual void ApplyModifier(const UWorld* World) override;
	UE_API virtual void UndoModifier() override;
	UE_API virtual FText GetModifierName() override;
	//~ End UMovieGraphModifierBase Interface

	/**
	 * Scripting helper that makes unreal.MovieGraphPropertyReference instances for use with this node. Little use outside of scripting. See
	 * FMovieGraphPropertyReference documentation for further details about the property reference.
	 */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	static UE_API FMovieGraphPropertyReference MakePropertyReference(
		UClass* InPropertyClass,
		const TSubclassOf<UActorComponent>& InComponentClass,
		const FName& InPropertyName);

	/**
	 * Adds the properties which should be modified.
	 *
	 * The property reference can:
	 * 1) Specify an actor without its component to modify an actor's property.
	 * 2) Specify a component with an actor to modify that component's property only on specific actor types.
	 * 3) Specify a component without an actor type to modify that component property on all actors it exists on.
	 */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	UE_API void AddProperties(const TArray<FMovieGraphPropertyReference>& InProperties);

	/** Gets the property name that needs to be used with GetPropertiesForSettingValues() to set a "Custom" property value. */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	static UE_API FName GetPropertyNameForSettingCustomValue(const FMovieGraphPropertyReference& InPropertyReference);

	/**
	 * Gets a value view which allows property values to be set. Properties cannot be added/removed with this view.
	 *
	 * Note that in order to modify a property value, you must use the property name provided by GetPropertyNameForSettingCustomValue().
	 */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	UE_API UMovieGraphFixedValueView* GetPropertiesForSettingValues();

private:
	/**
	 * Gets the unique internal property name for the given property reference. This property name is used to track the property within the
	 * internal property bags.
	 */
	static FName GetInternalPropertyName(const FMovieGraphPropertyReference& InPropertyReference);

private:
	/** All of the properties that will be modified by this modifier. */
	TMap<TSubclassOf<AActor>, TArray<FMovieGraphPropertyReference>> PropertiesToModifyByActorClass;

	/** Holds all of the properties and their associated values which will be set when the modifier is applied. */
	UPROPERTY(Transient)
	FInstancedPropertyBag UpdatedPropertyValues;

	/** Holds all of the values of the properties prior to modification. */
	UPROPERTY(Transient)
	FInstancedPropertyBag PreviousPropertyValues;

	/** View for the UpdatedPropertyValues property bag. */
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphMutableValueView> UpdatedPropertyValuesView = nullptr;

	/** View for the PreviousPropertyValues property bag. */
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphMutableValueView> PreviousPropertyValuesView = nullptr;

	/**
	 * The property view that's user-facing and does not allow adding/removing properties (so this node can enforce how properties
	 * are added/removed internally). Created lazily via GetPropertiesForSettingValues().
	 */
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphFixedValueView> UserFacingPropertiesView = nullptr;

	/** The actors and their properties that had values modified. Cached so the actor does not have to be re-resolved when undoing the modifier. */
	TArray<TPair<UObject*, FMovieGraphPropertyReference>> ModifiedProperties;
};

#undef UE_API