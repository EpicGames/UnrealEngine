// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"

#include "MovieGraphCollectionNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declare
class UMovieGraphCollection;

/** 
* A collection node specifies an interface for doing dynamic scene queries for actors in the world. Collections work in tandem with
* UMovieGraphModifiers to select which actors the modifiers should be run on.
*/
UCLASS(MinimalAPI)
class UMovieGraphCollectionNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphCollectionNode();

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

	UE_API virtual FString GetNodeInstanceName() const override;

protected:
	UE_API virtual void RegisterDelegates() override;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Collection : 1 = 1;	// The collection is customized in the details panel, so the override should always be enabled

	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = "General", meta=(EditCondition="bOverride_Collection"))
	TObjectPtr<UMovieGraphCollection> Collection;
};

#undef UE_API
