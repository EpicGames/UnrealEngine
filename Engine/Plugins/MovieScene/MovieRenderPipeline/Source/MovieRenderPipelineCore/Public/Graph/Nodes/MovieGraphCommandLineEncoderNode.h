// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"

#include "MovieGraphCommandLineEncoderNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMovieGraphPipeline;
struct FMovieGraphRenderOutputData;

/**
 * A node which kicks off an encode process after all renders have completed.
 */
UCLASS(MinimalAPI)
class UMovieGraphCommandLineEncoderNode : public UMovieGraphSettingNode, public IMovieGraphPostRenderNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphCommandLineEncoderNode();

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

	// UMovieGraphNode interface
	UE_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;
	// ~UMovieGraphNode interface

	// UMovieGraphSettingNode interface
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	// ~UMovieGraphSettingNode interface

	/** Begins the encode process (as long as there are no validation errors). */
	UE_API void StartEncodingProcess(TArray<FMovieGraphRenderOutputData>& InGeneratedData, const bool bInIsShotEncode);
	
	// IMovieGraphPostRenderNode interface
	UE_API virtual void BeginExport(UMovieGraphPipeline* InMoviePipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) override;
	UE_API virtual void BeginShotExport(UMovieGraphPipeline* InMoviePipeline) override;
	UE_API virtual bool HasFinishedExporting() override;
	// ~IMovieGraphPostRenderNode interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_FileNameFormat : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bDeleteSourceFiles : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bSkipEncodeOnRenderCanceled : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_VideoCodec : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AudioCodec : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputFileExtension : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_CommandLineFormat : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_VideoInputStringFormat : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AudioInputStringFormat : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_EncodeSettings : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bRetainInputTextFiles : 1;

	/** 
	* File name format string override. If specified it will override the FileNameFormat from the Output setting.
	* If {shot_name} or {camera_name} is used, encoding will begin after each shot finishes rendering.
	* Can be different from the main one in the Output setting so you can render out frames to individual
	* shot folders but encode to one file.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General", meta=(EditCondition="bOverride_FileNameFormat", MrgTokenAutocomplete))
	FString FileNameFormat;

	/** Whether the source files should be deleted on disk after encoding. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General", meta=(EditCondition="bOverride_bDeleteSourceFiles"))
	bool bDeleteSourceFiles;

	/** Whether encoding should be skipped on frames produced if rendering was canceled before finishing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General", meta=(EditCondition="bOverride_bSkipEncodeOnRenderCanceled"))
	bool bSkipEncodeOnRenderCanceled;

	/** Which video codec should we use? Run 'MovieRenderPipeline.DumpCLIEncoderCodecs' for options. */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General", meta=(EditCondition="bOverride_VideoCodec"))
	FString VideoCodec;

	/** Which audio codec should we use? Run 'MovieRenderPipeline.DumpCLIEncoderCodecs' for options. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General", meta=(EditCondition="bOverride_AudioCodec"))
	FString AudioCodec;

	/** Extension for the output files. Many encoders use this to determine the container type they are placed in. Should be without dot, ie: "webm". */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General", meta=(EditCondition="bOverride_OutputFileExtension"))
	FString OutputFileExtension;

	/** The format string used when building the final command line argument to launch. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Arguments", meta=(EditCondition="bOverride_CommandLineFormat", MultiLine=true))
	FString CommandLineFormat;
	
	/** Format string used for each video input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Arguments", meta=(EditCondition="bOverride_VideoInputStringFormat", MultiLine=true))
	FString VideoInputStringFormat;
	
	/** Format string used for each audio input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Arguments", meta=(EditCondition="bOverride_AudioInputStringFormat", MultiLine=true))
	FString AudioInputStringFormat;
	
	/** Additional flags used for specifying encode quality. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Arguments", meta=(EditCondition="bOverride_EncodeSettings", MultiLine=true))
	FString EncodeSettings;

	/** Retain the intermediate audio and video input text files that are passed to the encoder. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug", meta=(EditCondition="bOverride_bRetainInputTextFiles"))
	bool bRetainInputTextFiles;

private:
	struct FActiveJob
	{
		FActiveJob()
			: ReadPipe(nullptr)
			, WritePipe(nullptr)
		{}

		FProcHandle ProcessHandle;
		void* ReadPipe;
		void* WritePipe;

		/** Files that should be deleted after the process has finished. */
		TArray<FString> FilesToDelete;
	};

	struct FEncoderParams
	{
		FStringFormatNamedArguments NamedArguments;
		TMap<FString, TArray<FString>> FilesByExtensionType;
		TWeakObjectPtr<UMoviePipelineExecutorShot> Shot;
		FMovieGraphRenderDataIdentifier RenderDataIdentifier;
	};

	/**
	 * Returns true if all project and node settings are valid. Returns false if there are validation errors (and
	 * provides the errors in OutErrors).
	 */
	UE_API bool AreSettingsValid(TArray<FText>& OutErrors) const;

	/** Manages the actively running encode jobs, potentially aborting them if pipeline needs to shut down. */
	UE_API void OnTick();

	/** Returns true if there should be one export per shot, else false. */
	UE_API bool NeedsPerShotFlushing() const;

	/**
	 * Generates encoder parameters for each shot (using only the first render layer that's found within the shot). Will only generate one set of
	 * params if bInIsShotEncode is set to false (ie, this is a sequence encode).
	 */
	UE_API TMap<FMovieGraphRenderDataIdentifier, FEncoderParams> GenerateRenderLayerEncoderParams(TArray<FMovieGraphRenderOutputData>& InGeneratedData, const bool bInIsShotEncode) const;

	/** Gets the path to the file that the encode process should write to. */
	UE_API FString GetResolvedOutputFilename(const FMovieGraphRenderDataIdentifier& RenderIdentifier, const TWeakObjectPtr<UMoviePipelineExecutorShot>& Shot, const FString& FileNameFormatString) const;

	/**
	 * Generates the temporary video and audio input files that will be passed to the encoder process. These inputs
	 * typically contain a list of the files (eg, image files) that will be used in the encode process.
	 */
	UE_API void GenerateTemporaryEncoderInputFiles(const FEncoderParams& InParams, TArray<FString>& OutVideoInputFilePaths, TArray<FString>& OutAudioInputFilePaths) const;

	/** Generates the encoder command that should be executed. */
	UE_API FString BuildEncoderCommand(const FEncoderParams& InParams, TArray<FString>& InVideoInputFilePaths, TArray<FString>& InAudioInputFilePaths) const;

	/** Starts the actual encoder process. After the process is started, it will be monitored in OnTick(). */
	UE_API void LaunchEncoder(const FEncoderParams& InParams);

	/** Gets a setting on the branch that this node exists on. */
	template<typename T>
	T* GetSettingOnBranch(const bool bIncludeCDOs = true, const bool bExactMatch = true) const;

	/** Determines the appropriate evaluated config to use for the current context (primary job encode or shot based encode). */
	UE_API TObjectPtr<UMovieGraphEvaluatedConfig> GetEvaluatedConfig() const;

// The pipeline generates many instances of the same node throughout its execution; however, some nodes need to have persistent data throughout the
// pipeline's lifetime. This static data enables the node to have shared data across instances.
private:
	/** Encode jobs which are currently running. */
	static UE_API inline TArray<FActiveJob> ActiveEncodeJobs;

	/** The pipeline that started the export. */
	static UE_API inline TWeakObjectPtr<UMovieGraphPipeline> CachedPipeline;

	/** The job-level graph that should be accessed during execution of the node. Do not access the graph from the pipeline as it will be invalid. */
	static UE_API inline TObjectPtr<UMovieGraphEvaluatedConfig> PrimaryJobEvaluatedGraph;
};

#undef UE_API
