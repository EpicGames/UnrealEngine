// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "MovieGraphDebugNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A node which configures various debug settings that may be useful when debugging an issue. */
UCLASS(MinimalAPI)
class UMovieGraphDebugSettingNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UMovieGraphDebugSettingNode() = default;

	virtual EMovieGraphBranchRestriction GetBranchRestriction() const override { return EMovieGraphBranchRestriction::Globals; }

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCaptureFramesWithRenderDoc : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_RenderDocCaptureFrame : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCaptureUnrealInsightsTrace : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_UnrealInsightsTraceFileNameFormat : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCaptureFramesWithDumpGpu : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DumpGpuCaptureFrameStart : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DumpGpuCaptureFrameStop : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DumpGpuCaptureFilePathFormat : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DumpGpuCaptureUnifiedPassResourceFilters : 1;

	/** 
	* If true, automatically trigger RenderDoc to capture rendering information. RenderDoc plugin must be enabled, 
	* and the editor must have been launched with -AttachRenderDoc. Resulting capture will be in /Saved/RenderDocCaptures.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RenderDoc", meta = (EditCondition = "bOverride_bCaptureFramesWithRenderDoc"))
	bool bCaptureFramesWithRenderDoc;

	/**
	* If bCaptureFramesWithRenderDoc is true, which frame (on the root Sequencer time line) should we capture data for?
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RenderDoc", meta = (EditCondition = "bOverride_RenderDocCaptureFrame"))
	int32 RenderDocCaptureFrame;
	
	/** 
	* If true, automatically capture an Unreal Insights trace file for the duration of the render.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Unreal Insights", meta = (EditCondition = "bOverride_bCaptureUnrealInsightsTrace"))
	bool bCaptureUnrealInsightsTrace;

	/** 
	* If bCaptureUnrealInsightsTrace is true, name of the UnrealInsights trace.
	* Resulting capture will be in the global Output Directory for the job.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Unreal Insights", meta = (EditCondition = "bOverride_UnrealInsightsTraceFileNameFormat", MrgTokenAutocomplete))
	FString UnrealInsightsTraceFileNameFormat = TEXT("{sequence_name}_UnrealInsights");

	/** 
	* If true, automatically trigger DumpGPU to capture rendering information.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DumpGPU", meta = (EditCondition = "bOverride_bCaptureFramesWithDumpGpu"))
	bool bCaptureFramesWithDumpGpu;

	/**
	* If bCaptureFramesWithDumpGpu is true, from which included frame (on the root Sequencer time line) start index should we capture? No start bound if negative.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DumpGPU", meta = (EditCondition = "bOverride_DumpGpuCaptureFrameStart"))
	int32 DumpGpuCaptureFrameStart = -1;

	/**
	* If bCaptureFramesWithDumpGpu is true, up to which excluded frame (on the root Sequencer time line) stop index should we capture? No stop bound if negative.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DumpGPU", meta = (EditCondition = "bOverride_DumpGpuCaptureFrameStop"))
	int32 DumpGpuCaptureFrameStop = -1;

	/**
	* If bCaptureFramesWithDumpGpu is true, name of the output directory for the capture (related to frame output directory).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DumpGPU", meta = (EditCondition = "bOverride_DumpGpuCaptureFilePathFormat", MrgTokenAutocomplete))
	FString DumpGpuCaptureFilePathFormat = TEXT("{sequence_name}.{frame_number}_DumpGPU");

	/**
	* If bCaptureFramesWithDumpGpu is true, wildcard enabled filters to apply to the path of the passes to dump.
	* Syntax: Separate pass names to resources with "@". Filter multiple passes:resources by separating them with "|" 
	* Example: "*TileClassificationMark*@StochasticLighting.EncodedReprojectionVector|*TemporalAccumulation*@MegaLights.ResolvedDiffuseLighting"
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DumpGPU", meta = (EditCondition = "bOverride_DumpGpuCaptureUnifiedPassResourceFilters"))
	FString DumpGpuCaptureUnifiedPassResourceFilters = TEXT("*");
};

#undef UE_API
