// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MovieGraphExecuteScriptNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
* This is the base class used for writing scripts that can be executed as part
* of a Movie Graph Pipeline. After creating a UMovieGraphExecuteScriptNode 
* (named 'Execute Script' in the UI) in your Movie Graph Configuration asset,
* and choosing the type of your script, an instance will be created when
* the job starts. This instance will persist until the end of the job. The instance
* will receive callbacks during the render which contain data that you can 
* manipulate with your script, or data that was generated by the pipeline.
*
* This can be implemented in C++, Blueprints, or Python. For C++ you can use standard
* inheritance. Blueprints should use the MovieGraphScriptBase class. For Python,
* you need to write a UClass implemented in Python. This has slightly different
* syntax than standard Python (see below).
* 
* In this example, create a "Python" folder in your Content folder. Then create
* a file named "MovieGraphPipelineScriptExample.py", and a file named "init_unreal.py"
* 
* In "init_unreal.py" add "import MovieGraphPipelineScriptExample" as the contents.
* In "MovieGraphPipelineScriptExample.py" use the following:
*
*import unreal
*@unreal.uclass()
*class MovieGraphPipelineScriptExample(unreal.MovieGraphScriptBase):
*    @unreal.ufunction(override=True)
*    def on_job_start(self, in_job_copy):
*        super().on_job_start(in_job_copy)
*        unreal.log("OnJobStart")
*
* Save and restart Unreal Editor, then create a Movie Graph Configuration
* with an Execute Script Node, and set it to run MovieGraphPipelineScriptExample. 
* When the job is run, the output console should print "OnJobStart" as a warning.
* 
* Use "help(unreal.MovieGraphScriptBase)" in the Python Console to see all available 
* overridable functions.
* 
* See /Engine/Plugins/MovieScene/MovieRenderPipeline/Content/Python/MovieGraphScriptNodeExample.py 
* for more python examples.
*/
UCLASS(MinimalAPI, BlueprintType, Blueprintable, Abstract)
class UMovieGraphScriptBase : public UObject
{
	GENERATED_BODY()

public:
	/**
	* This callback is called very early on in the Movie Graph Pipeline, before
	* data is read from the job. The provided job is a duplicate of the job defined
	* in the UMoviePipelineQueue, allowing you to make changes to the job without
	* worrying about restoring them at the end of a render. Additionally, the 
	* Graph Configuration assets (for both the Primary configuration, and shot
	* overrides) have been duplicated to allow mutation of the graph configuration
	* without worrying about leaking these changes to the assets on disk. Other
	* pointers to external packages (such as the Level Sequence on the job) have
	* not been duplicated though, so be aware that changes to these assets will
	* leak, unless you restore the changes in OnJobFinished. OnJobFinished will
	* be called even if the job is canceled.
	* 
	* For C++ implementations override `virtual void OnJobStart_Implementation(UMoviePipelineExecutorJob* InJobCopy) override`
	* For Python implementations override
	*	@unreal.ufunction(override=True)
	*	def on_job_start(self, inJobCopy):
	*		super().on_job_start(inJobCopy)
	* 
	* 
	* @param InJobCopy - A copy of the job being rendered with this graph config,
	*					 with its Graph Preset configurations also copied to avoid
	*					 accidental mutation of assests on disk.
	*/
	UFUNCTION(BlueprintNativeEvent, Category = "Movie Graph")
	UE_API void OnJobStart(UMoviePipelineExecutorJob* InJobCopy);
	virtual void OnJobStart_Implementation(UMoviePipelineExecutorJob* InJobCopy) {}

	/**
	* This callback is called at the very end of the Movie Graph Pipeline, right before
	* shutting down. The files specified by the OutputData should have been written
	* to disk by this point. This function will get called both in the case of a successful
	* job, or one canceled by the user. You can use OutputData.bSuccess to determine if
	* the render was successfuly completed.
	* 
	* For C++ implementations override `virtual void OnJobFinished_Implementation(UMoviePipelineExecutorJob* InJobCopy, const FMovieGraphPipelineOutputData& OutputData) override`
	* For Python implementations override
	*	@unreal.ufunction(override=True)
	*	def on_job_finished(self, inJobCopy, inOutputData):
	*		super().on_job_finished(inJobCopy, inOutputData)
	* 
	* 
	* @param InJobCopy - A copy of the job being rendered with this graph config,
	*					 with its Graph Preset configurations also copied to avoid
	*					 accidental mutation of assests on disk.
	* @param InOutputData - Information about the files generated during this render.
	* 						Contains a flag (bSuccess) for checking if the job was 
	*						successful or not, and contains a list of render layers,
	* 						and the files written to disk for that render layer. Contains
	*						data for all shots rendered up to this point.
	*/
	UFUNCTION(BlueprintNativeEvent, Category = "Movie Graph")
	UE_API void OnJobFinished(UMoviePipelineExecutorJob* InJobCopy, const FMoviePipelineOutputData& InOutputData);
	virtual void OnJobFinished_Implementation(UMoviePipelineExecutorJob* InJobCopy, const FMoviePipelineOutputData& InOutputData ) {}

	/**
	* This is similar to OnJobStart but called before a shot is set up. This is only called
	* if IsPerShotCallbackNeeded() returns true.
	* 
	* For C++ implementations override `virtual void OnShotStart_Implementation(UMoviePipelineExecutorJob* InJobCopy,  UMoviePipelineExecutorShot* InShotCopy) override`
	* For Python implementations override
	*	@unreal.ufunction(override=True)
	*	def on_shot_start(self, inJobCopy, inShotCopy):
	*		super().on_shot_start(inJobCopy, inShotCopy)
	* 
	* 
	* @param InJobCopy - A copy of the job being rendered with this graph config,
	*					 with its Graph Preset configurations also copied to avoid
	*					 accidental mutation of assests on disk.
	* @param InShotCopy - A copy of the shot that is about to be started.
	*/
	UFUNCTION(BlueprintNativeEvent, Category = "Movie Graph")
	UE_API void OnShotStart(UMoviePipelineExecutorJob* InJobCopy, UMoviePipelineExecutorShot* InShotCopy);
	virtual void OnShotStart_Implementation(UMoviePipelineExecutorJob* InJobCopy, UMoviePipelineExecutorShot* InShotCopy) {}

	/**
	* This is similar to OnJobFinished but called after a shot is completed. This is only called
	* if IsPerShotCallbackNeeded() returns true.
	* 
	* For C++ implementations override `virtual void OnShotFinished_Implementation(UMoviePipelineExecutorJob* InJobCopy,  UMoviePipelineExecutorShot* InShotCopy, const FMovieGraphPipelineOutputData& InOutputData) override`
	* For Python implementations override
	*	@unreal.ufunction(override=True)
	*	def on_shot_finished(self, inJobCopy, inShotCopy, inOutputData):
	*		super().on_shot_finished(inJobCopy, inShotCopy, inOutputData)
	* 
	* 
	* @param InJobCopy - A copy of the job being rendered with this graph config,
	*					 with its Graph Preset configurations also copied to avoid
	*					 accidental mutation of assests on disk.
	* @param InShotCopy - A copy of the shot that was just finished.
	* @param InOutputData - The data generated by this shot. Only contains the files
	* 						rendered for this particular shot, not all files generated
	* 						by the movie render (use OnJobFinished for that).
	*/
	UFUNCTION(BlueprintNativeEvent, Category = "Movie Graph")
	UE_API void OnShotFinished(UMoviePipelineExecutorJob* InJobCopy, UMoviePipelineExecutorShot* InShotCopy, const FMoviePipelineOutputData& OutputData);
	virtual void OnShotFinished_Implementation(UMoviePipelineExecutorJob* InJobCopy, UMoviePipelineExecutorShot* InShotCopy, const FMoviePipelineOutputData& OutputData) {}


	/**
	* If you return true here the system will stall at the end of every shot and wait until
	* all files have been written to disk before invoking this callback. This can increase 
	* render times but is important if you need to operate on the files on disk as part of 
	* the callback.
	*
	* For C++ implementations override `virtual bool IsPerShotCallbackNeeded_Implementation() const override`
	* For Python implementations override
	*	@unreal.ufunction(override=True)
	*	def is_per_shot_callback_needed():
	*		return False
	*/
	UFUNCTION(BlueprintNativeEvent, Category = "Movie Graph")
	UE_API bool IsPerShotCallbackNeeded() const;
	virtual bool IsPerShotCallbackNeeded_Implementation() const
	{
		return false;
	}
};

/** Designates the types of builds a script (specified in an Execute Script node) is allowed to execute in. */
UENUM(BlueprintType)
enum class EMovieGraphExecuteScriptMode : uint8
{
	/** The script should only run in an editor context. */
	EditorOnly = 0,

	/** The script can run both in the editor and in a runtime build. */
	EditorAndRuntime = 1,

	/** Invalid value, keep as the last entry in the enum. */
	Count UMETA(Hidden)
};

/** 
 This node allows users to run code before and after a movie graph is being executed. Example use cases
 would be to run custom logic that overrides the output directory, or to trigger an event after a shot finishes.
 These bits of script can be implemented in either C++, or in Python (as a UClass) because we don't serialize the
 object and only create it at runtime during the render. See UMovieGraphScriptBase for details.
*/
UCLASS(MinimalAPI)
class UMovieGraphExecuteScriptNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UMovieGraphExecuteScriptNode() = default;

	virtual EMovieGraphBranchRestriction GetBranchRestriction() const override { return EMovieGraphBranchRestriction::Globals; }
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

	/** Allocates an instance of the script type specified on the node. Will return nullptr if the script type is invalid. */
	UE_API UMovieGraphScriptBase* AllocateScriptInstance() const;

protected:
	//~ Begin UObject Interface
	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

public:
	UE_DEPRECATED(5.8, "Use the bOverride_EditorOnlyScript property or bOverride_EditorAndRuntimeScript property instead.")
	UPROPERTY(BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle, DeprecatedProperty, DeprecationMessage = "Use the bOverride_EditorOnlyScript property or bOverride_EditorAndRuntimeScript property instead."))
	uint8 bOverride_Script: 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_EditorOnlyScript : 1;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_EditorAndRuntimeScript : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Mode : 1;

	/**
	* Which script should this node instantiate? The class will be instantiated when the job starts,
	* and then different functions will be called on that instance throughout the life-time of the 
	* movie render. Scripts can be implemented in Python or C++, see MovieGraphExecuteScriptNode.h
	* for more details.
	*/
	UE_DEPRECATED(5.8, "Use the EditorOnlyScript property or EditorAndRuntimeScript property instead.")
	UPROPERTY(BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bOverride_Script", MetaClass = "/Script/MovieRenderPipelineCore.MovieGraphScriptBase", DeprecatedProperty, DeprecationMessage = "Use the EditorOnlyScript property or EditorAndRuntimeScript property instead."))
	FSoftClassPath Script;

	/** The context that this script is allowed to execute in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bOverride_Mode"))
	EMovieGraphExecuteScriptMode Mode = EMovieGraphExecuteScriptMode::EditorOnly;

#if WITH_EDITORONLY_DATA
	/**
	 * The script that this node should run. The script will be created when the job starts, and different functions will be called on the
	 * script throughout the lifetime of the render. Scripts can be implemented in Python, Blueprints, or C++; see MovieGraphExecuteScriptNode.h
	 * for more details.
	 *
	 * Editor-only scripts will run in editor-based renders and not runtime builds (if attempting to use it in a runtime build, the script will not
	 * execute, and a warning will be logged). This is mainly intended for typical MRG renders using a Python-based script (but the script can be
	 * C++ or Blueprint-based as well). Using this script type allows the Asset Referencing Restrictions plugin to properly detect that this script
	 * will not execute in a runtime build.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", DisplayName = "Editor-Only Script", meta = (EditCondition = "bOverride_EditorOnlyScript", MetaClass = "/Script/MovieRenderPipelineCore.MovieGraphScriptBase"))
	FSoftClassPath EditorOnlyScript;
#endif

	/**
	 * The script that this node should run. The script will be created when the job starts, and different functions will be called on the
	 * script throughout the lifetime of the render. Scripts can be implemented in Python, Blueprints, or C++; see MovieGraphExecuteScriptNode.h
	 * for more details.
	 *
	 * Editor and Runtime scripts will run both in the editor and in runtime builds. Note that if a Python-based script type is being used in a
	 * runtime build, the Asset Referencing Restrictions plugin will flag an issue. This script type is mostly intended for use with Blueprints and
	 * C++-based scripts.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", DisplayName = "Editor + Runtime Script", meta = (EditCondition = "bOverride_EditorAndRuntimeScript", MetaClass = "/Script/MovieRenderPipelineCore.MovieGraphScriptBase"))
	FSoftClassPath EditorAndRuntimeScript;
};

#undef UE_API
