// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphDataTypes.h"
#include "MovieGraphFileOutputNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
* The UMovieGraphFileOutputNode node defines a output file format that MRQ may place produced render data into.
* At runtime, the individual pieces of render data will be generated by the Movie Graph Renderer, and then 
* all data will be collected into a single datastructure by the FMovieGraphOutputMerger before being passed back
* into the main Movie Graph Pipeline. Once this happens, the CDOs deriving from UMovieGraphFileOutputNode will be
* given a pointer to the information for the whole frame, so they can decide what to do with it. There are potentially
* multiple output formats at once (such as exr and jpeg).
*/
UCLASS(MinimalAPI, Abstract)
class UMovieGraphFileOutputNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()
public:
	UMovieGraphFileOutputNode() = default;
	
	/** 
	* This is called by the Game Thread containing the final output data for a given output frame. InRawFrameData contains
	* a list of all of the data generated for this output frame, and InMask specifies which render layers within InRawFrameData
	* should actually be written to disk by this output node.
	*/
	void OnReceiveImageData(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask) { OnReceiveImageDataImpl(InPipeline, InRawFrameData, InMask); }

	/**
	 * This is called when the pipeline has finished rendering all shots, and the sequence export process has not begun yet.
	 * See OnAllShotFramesSubmitted() for a method that is called when an individual shot has finished rendering.
	 */
	void OnAllFramesSubmitted(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) { OnAllFramesSubmittedImpl(InPipeline, InPrimaryJobEvaluatedGraph); }
	
	/**
	 * This is called when the pipeline has finished rendering all shots, and the sequence export process is finishing up.
	 */
	void OnAllFramesFinalized(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) { OnAllFramesFinalizedImpl(InPipeline, InPrimaryJobEvaluatedGraph); }

	/**
	 * This is called when a shot has finished rendering its last frame, and the shot export process has not begun yet. If bFlushToDisk is true,
	 * all files this node is generating for the current shot should be written to disk before returning.
	 * See OnAllFramesSubmitted() for a method that is called after all shots have finished rendering.
	 */
	void OnAllShotFramesSubmitted(UMovieGraphPipeline* InPipeline, const UMoviePipelineExecutorShot* InShot, TObjectPtr<UMovieGraphEvaluatedConfig>& InShotEvaluatedGraph, const bool bFlushToDisk = false) { OnAllShotFramesSubmittedImpl(InPipeline, InShot, InShotEvaluatedGraph, bFlushToDisk); }

	/** Returns whether this node has finished writing all of its files to disk yet. */
	bool IsFinishedWritingToDisk() const { return IsFinishedWritingToDiskImpl(); }

	//~ Begin UMovieGraphSettingNode interface
	virtual TArray<FMovieGraphPinProperties> GetInputPinProperties() const override
	{
		TArray<FMovieGraphPinProperties> Properties;
		Properties.Add(FMovieGraphPinProperties::MakeBranchProperties());
		return Properties;
	}

	virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const override
	{
		TArray<FMovieGraphPinProperties> Properties;
		Properties.Add(FMovieGraphPinProperties::MakeBranchProperties());
		return Properties;
	}
	//~ End UMovieGraphSettingNode interface

#if WITH_EDITOR
	virtual FText GetMenuCategory() const override
	{
		return NSLOCTEXT("MovieGraphNodes", "FileOutputGraphNode_Category", "Output Type");
	}

	/**
	 * Short display name for use in Basic config toggle buttons (e.g., "EXR", "PNG", etc).
	 * Return empty FText to exclude this output type from the Basic config UI.
	 */
	UE_API virtual FText GetBasicConfigShortDisplayName() const;
#endif
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_FileNameFormat : 1;

	/** What format string should the final files use? Can include folder prefixes, and format string ({shot_name}, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output", meta=(EditCondition="bOverride_FileNameFormat", MrgTokenAutocomplete))
	FString FileNameFormat = "{sequence_name}.{layer_name}.{frame_number}";

protected:
	virtual void OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask) {}
	virtual void OnAllFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) {}
	virtual void OnAllFramesFinalizedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) {}
	virtual void OnAllShotFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, const UMoviePipelineExecutorShot* InShot, TObjectPtr<UMovieGraphEvaluatedConfig>& InShotEvaluatedGraph, const bool bFlushToDisk) {}

	UE_DEPRECATED(5.6, "Please use the version of OnAllShotFramesSubmittedImpl() that has the new bFlushToDisk argument at the end.")
	virtual void OnAllShotFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, const UMoviePipelineExecutorShot* InShot, TObjectPtr<UMovieGraphEvaluatedConfig>& InShotEvaluatedGraph) final {}
	
	virtual bool IsFinishedWritingToDiskImpl() const { return true; }

	/** Returns whether any overscan should be automatically cropped from the image before compositing and output */
	virtual bool ShouldCropOverscanImpl() const { return true; }
	
	/** Returns the number of evaluated (active) file nodes on the specified branch. */
	static UE_API int32 GetNumFileOutputNodes(const UMovieGraphEvaluatedConfig& InEvaluatedConfig, const FName& InBranchName);

	/** Updates InOutFilenameFormatString to include tokens (like {layer_name}) that will disambiguate the output if multiple outputs are attempting to write to the same file. */
	UE_DEPRECATED(5.7, "Please use the version of DisambiguateFilename() that takes a UMovieGraphFileOutputNode instance. The new version supports enhanced disambiguation rules.")
	static UE_API void DisambiguateFilename(FString& InOutFilenameFormatString, const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const FName& InNodeName, const FMovieGraphPassData& InRenderData);

	/** Updates InOutFilenameFormatString to include tokens (like {layer_name}) that will disambiguate the output if multiple outputs are attempting to write to the same file. */
	static UE_API void DisambiguateFilename(FString& InOutFilenameFormatString, const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const UMovieGraphFileOutputNode* InParentNode, const FMovieGraphPassData& InRenderData);

	/** Convenience function to get the list of active composite passes from render data. */
	static UE_API TArray<FMovieGraphPassData> GetCompositedPasses(UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData);
};

#undef UE_API
