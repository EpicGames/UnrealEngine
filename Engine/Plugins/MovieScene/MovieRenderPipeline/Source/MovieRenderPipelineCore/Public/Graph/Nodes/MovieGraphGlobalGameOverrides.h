// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "MoviePipelineGameOverrideSetting.h"

#include "MovieGraphGlobalGameOverrides.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** Scalability quality levels available for use in renders generated from the graph. */
UENUM(BlueprintType)
enum class EMovieGraphScalabilityQualityLevel : uint8
{
	Low = 0,
	Medium = 1,
	High = 2,
	Epic = 3,
	Cinematic = 4
};

/** A node which configures the global game overrides. */
UCLASS(MinimalAPI)
class UMovieGraphGlobalGameOverridesNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphGlobalGameOverridesNode();

	// UMovieGraphNode interface
	virtual EMovieGraphBranchRestriction GetBranchRestriction() const override { return EMovieGraphBranchRestriction::Globals; }
	// ~UMovieGraphNode interface

	// UMovieGraphSettingNode interface
	UE_API virtual void BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	// ~UMovieGraphSettingNode interface

	// UObject interface
	UE_API virtual void PostLoad() override;
	// ~UObject interface

	/**
	 * Applies any cvars, scalability settings, etc. to reflect the properties set on the node. Remembers what the
	 * settings are before they are set, so bOverrideValues can be set to false to revert them to their original
	 * values. The world needs to be provided as context for any commands that are executed.
	 */
	UE_API void ApplySettings(const bool bOverrideValues, UWorld* InWorld);

	/**
	 * Gets the game mode override if the job specified is using a graph configuration, otherwise gets the game mode
	 * override from the legacy non-graph config system.
	 */
	static UE_API TSubclassOf<AGameModeBase> GetGameModeOverride(const UMoviePipelineExecutorJob* InJob);

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

public:
	UE_DEPRECATED(5.5, "Please use the bOverride_SoftGameModeOverride property instead.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle, DeprecatedProperty, DeprecationMessage = "Please use the bOverride_SoftGameModeOverride property instead."))
	uint8 bOverride_GameModeOverride : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SoftGameModeOverride : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ScalabilityQualityLevel : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (ScriptName="Override_DisableTextureStreaming;Override_bDisableTextureStreaming", InlineEditConditionToggle))
	uint8 bOverride_bDisableTextureStreaming : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (ScriptName="Override_DisableLods;Override_bDisableLODs", InlineEditConditionToggle))
	uint8 bOverride_bDisableLODs : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (ScriptName="Override_DisableHlods;Override_bDisableHLODs", InlineEditConditionToggle))
	uint8 bOverride_bDisableHLODs : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (ScriptName="Override_FlushLevelStreaming;Override_bFlushLevelStreaming", InlineEditConditionToggle))
	uint8 bOverride_bFlushLevelStreaming : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (ScriptName="Override_FlushAssetCompiler;Override_bFlushAssetCompiler", InlineEditConditionToggle))
	uint8 bOverride_bFlushAssetCompiler: 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (ScriptName="Override_FlushShaderCompiler;Override_bFlushShaderCompiler", InlineEditConditionToggle))
	uint8 bOverride_bFlushShaderCompiler : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (ScriptName="Override_FlushGrassStreaming;Override_bFlushGrassStreaming", InlineEditConditionToggle))
	uint8 bOverride_bFlushGrassStreaming : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (ScriptName="Override_FlushStreamingManagers;Override_bFlushStreamingManagers", InlineEditConditionToggle))
	uint8 bOverride_bFlushStreamingManagers : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_VirtualTextureFeedbackFactor : 1;

	UE_DEPRECATED(5.8, "Lumen scene rebuild is now automatically applied during layer warm-ups in the deferred renderer. Use Layer Warm Up Frames on the Warm Up Settings or Render Layer nodes instead.")
	UPROPERTY(BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle, DeprecatedProperty, DeprecationMessage = "Lumen scene rebuild is now automatically applied during layer warm-ups in the deferred renderer. Use Layer Warm Up Frames on the Warm Up Settings or Render Layer nodes instead."))
	uint8 bOverride_bRebuildLumenSceneBetweenRenderLayers : 1;

	/**
	 * Optional game mode to override the map's default game mode with. This can be useful if the game's normal mode
	 * displays UI elements or loading screens that you don't want captured.
	 */
	UE_DEPRECATED(5.5, "Please use the SoftGameModeOverride property instead.")
	UPROPERTY(BlueprintReadWrite, Category = "Game", meta = (EditCondition = "bOverride_GameModeOverride", DeprecatedProperty, DeprecationMessage = "Please use the SoftGameModeOverride property instead."))
	TSubclassOf<AGameModeBase> GameModeOverride;

	/**
	 * Optional game mode to override the map's default game mode with. This can be useful if the game's normal mode
	 * displays UI elements or loading screens that you don't want captured.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Game", DisplayName = "Game Mode Override", meta = (EditCondition = "bOverride_SoftGameModeOverride"))
	TSoftClassPtr<AGameModeBase> SoftGameModeOverride;

	/**
	 * The scalability quality level that should be used in renders. See the Scalability Reference documentation for
	 * information on how to edit cvars to add/change default quality values.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Game", meta = (EditCondition = "bOverride_ScalabilityQualityLevel"))
	EMovieGraphScalabilityQualityLevel ScalabilityQualityLevel;

	/**
	 * Toggles whether texture streaming is disabled. Can solve objects being blurry after camera cuts.
	 *
	 * Configures the following cvars:
	 * - r.TextureStreaming
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = "bOverride_bDisableTextureStreaming"))
	bool bDisableTextureStreaming;

	/**
	 * Disabling LODs will use the highest quality LOD for meshes and particle systems, regardless of distance. Note
	 * that this does not affect Nanite.
	 *
	 * Configures the following cvars:
	 * - r.ForceLOD
	 * - r.SkeletalMeshLODBias
	 * - r.ParticleLODBias
	 * - foliage.DitheredLOD
	 * - foliage.ForceLOD
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (ScriptName="bDisableLods;DisableLODs", EditCondition = "bOverride_bDisableLODs"))
	bool bDisableLODs;

	/**
	 * Determines if hierarchical LODs should be disabled and their real meshes used instead, regardless of distance.
	 * Note that this does not affect World Partition HLODs.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (ScriptName="bDisableHlods;bDisableHLODs", EditCondition = "bOverride_bDisableHLODs"))
	bool bDisableHLODs;

	/**
	 * Flushing level streaming ensures that any pending changes to sub-levels or world partition are fully processed before we render
	 * the frame. This feature generally only adds to render times on the frames that have level visibility state changes, so generally
	 * safe to leave turned on all the time.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = "bOverride_bFlushLevelStreaming"))
	bool bFlushLevelStreaming;

	/**
	 * This ensures that any asynchronously compiled assets (static meshes, distance fields, etc.) required for rendering the frame
	 * are completed before rendering the frame. This feature generally only adds to render times on the frames where a new asset
	 * is introduced (ie, spawned) that may not be fully compiled. Results are stored in the DDC on subsequent uses.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = "bOverride_bFlushAssetCompiler"))
	bool bFlushAssetCompiler;

	/**
	 * This ensures that any asynchronously compiled shader permutations are completed before rendering the frame. When using
	 * On Demand Shader Compilation the editor will skip compiling currently unneeded permutations for the material graph to
	 * decrease artist iteration time, but these permutations need to be compiled when rendering. Results are stored in the DDC on
	 * subsequent uses.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = "bOverride_bFlushShaderCompiler"))
	bool bFlushShaderCompiler;

	/**
	 * Flushing grass streaming prevents visible pop-in/culling of grass instances, but may come at a high GPU memory
	 * cost (depending on rendering feature set), and grass density. Try turning this off if you are low on GPU memory
	 * and have dense grass.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = "bOverride_bFlushGrassStreaming"))
	bool bFlushGrassStreaming;

	/** 
	 * This ensures that we wait on any streaming managers that may have outstanding work to finish their work before we render
	 * the frame. Many GPU-based features rely on a feedback loop where a frame is rendered, then results are read back from the GPU
	 * to the CPU. These results are analyzed, and additional data is loaded to be used in a subsequent frame. This feature ensures
	 * that the data is fully loaded before we render the next frame, but does NOT solve issues related to generating the those
	 * feedback requests in the first place. Virtual Textures and Nanite must render the frame to generate the request -- 
	 * this option cannot solve that; it only helps ensure that the requests that are made are fully processed before
	 * the frame is rendered.
	 *
	 * Configures the following cvars:
	 * - r.Streaming.SyncStatesWhenBlocking
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = "bOverride_bFlushStreamingManagers"))
	bool bFlushStreamingManagers;

	/** The virtual texture feedback resolution factor. A lower factor will increase virtual texture feedback resolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = "bOverride_VirtualTextureFeedbackFactor"))
	int32 VirtualTextureFeedbackFactor;

	/**
	 * Deprecated: Lumen scene rebuild is now automatically applied during layer warm-ups in the deferred renderer.
	 * Use Layer Warm Up Frames on the Warm Up Settings or Render Layer nodes instead.
	 */
	UE_DEPRECATED(5.8, "Lumen scene rebuild is now automatically applied during layer warm-ups in the deferred renderer. Use Layer Warm Up Frames on the Warm Up Settings or Render Layer nodes instead.")
	UPROPERTY(BlueprintReadWrite, Category = "Lumen Render Layers (Experimental)", meta = (EditCondition = "bOverride_bRebuildLumenSceneBetweenRenderLayers", DeprecatedProperty, DeprecationMessage = "Lumen scene rebuild is now automatically applied during layer warm-ups in the deferred renderer. Use Layer Warm Up Frames on the Warm Up Settings or Render Layer nodes instead."))
	bool bRebuildLumenSceneBetweenRenderLayers;

private:
	Scalability::FQualityLevels PreviousQualityLevels;
	int32 PreviousTextureStreaming;
	int32 PreviousForceLOD;
	int32 PreviousSkeletalMeshBias;
	int32 PreviousParticleLODBias;
	int32 PreviousFoliageDitheredLOD;
	int32 PreviousFoliageForceLOD;
	int32 PreviousStreamingManagerSyncState;
	int32 PreviousLumenRadiosityUpdateFactor;
	int32 PreviousLumenSurfaceCacheCardCaptureFactor;
	int32 PreviousLumenSurfaceCacheFeedback;
	int32 PreviousLumenSurfaceCacheRecaptureEveryFrame;
	int32 PreviousAnimationUROEnabled;
	int32 PreviousSkyLightRealTimeReflectionCaptureTimeSlice;
	int32 PreviousVolumetricRenderTarget;
	int32 PreviousVolumetricRenderTargetMode;
	int32 PreviousIgnoreStreamingPerformance;
	float PreviousChaosImmPhysicsMinStepTime;
	int32 PreviousSkipRedundantTransformUpdate;
	int32 PreviousChaosClothUseTimeStepSmoothing;
	int32 PreviousSkipWaterInfoTextureRenderWhenWorldRenderingDisabled;
	int32 PreviousNaniteVSMInvalidateOnLODDelta;
	int32 PreviousAllowSlateThrottling;
	bool PreviousAlphaOutput;
	int32 PreviousLUTSize;

#if WITH_EDITOR
	int32 PreviousGeoCacheStreamerShowNotification;
	int32 PreviousGeoCacheStreamerBlockTillFinish;
#endif
};

#undef UE_API
