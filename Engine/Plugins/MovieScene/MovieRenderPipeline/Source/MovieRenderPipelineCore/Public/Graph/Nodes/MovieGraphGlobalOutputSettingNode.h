// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNamedResolution.h"
#include "Graph/MovieGraphNode.h"

#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"

#include "MovieGraphGlobalOutputSettingNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

USTRUCT(BlueprintType)
struct FMovieGraphVersioningSettings
{
	GENERATED_BODY()

	/**
	 * If true, {version} tokens specified in the Output Directory and File Name Format properties will automatically
	 * be incremented with each local render. If false, the version specified in Version Number will be used instead.
	 *
	 * Auto-versioning will search across all render branches and use the highest version found as the basis for the
	 * next version used.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Versioning")
	bool bAutoVersioning = true;
	
	/** The value to use for the version token if versions are not automatically incremented (Auto Version is off). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Versioning", meta = (UIMin = 1, UIMax = 50, ClampMin = 1))
	int32 VersionNumber = 1;
};

UENUM(BlueprintType)
enum class EMovieGraphSequenceRangeType : uint8
{
	/** Use the Playback Range value from the Level Sequence (without overriding it.) */
	SequenceDefault,
	/** Override the Playback Range value from the Level Sequence and instead override it to use a custom Frame Number. */
	Custom
};

USTRUCT(BlueprintType)
struct FMovieGraphSequencePlaybackRangeBound
{
	GENERATED_BODY()

public:
	FMovieGraphSequencePlaybackRangeBound()
		: Type(EMovieGraphSequenceRangeType::Custom)
		, Value(0)
	{}
			
	/** 
	* By default the render will use the Playback Range Start/End frame to determine what to render. Set this to Custom to
	* override the Playback Range bound value, and instead use the Value below as the Start or End frame. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Value")
	EMovieGraphSequenceRangeType Type;

	/**
	* If Type is set to Custom, then this value is used for the Playback Range Start or End frame. Value is considered frames at the original Sequence
	* frame rate (it is applied before Frame Rate Override.)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Value", meta = (EditCondition = "Type == EMovieGraphSequenceRangeType::Custom"))
	int32 Value;
};

UENUM(BlueprintType)
enum class EMovieGraphAspectRatioAdaptBehavior : uint8
{
	/** 
	* Output resolution will not be modified. If the camera has Constrain Aspect Ratio enabled, then 
	* letterboxing may be shown to preserve that aspect ratio within the specified Output Resolution.
	*/
	Disabled,

	/**
	* If the camera has Constrain Aspect Ratio enabled, then this should match Disabled, except the letterboxing
	* will be cropped off. No behavior change if Constrain Aspect Ratio is not enabled.
	*
	* The output resolution will be resized to respect the camera aspect ratio, matching either the specified Width,
	* or Height. Which dimension is picked will depend on the aspect ratio (ie: a tall camera aspect ratio will 
	* preserve the height of the output, and crop the width, while a wide camera aspect ratio will preserve 
	* the width of the output and crop the height.)
	*/
	Automatic,

	/**
	* If the camera has Constrain Aspect Ratio enabled, then the width of the output resolution will be preserved,
	* and the height will automatically be adjusted based on the camera aspect ratio.
	* 
	* ie: If you have a 1.77 (16:9) Camera Aspect Ratio and target a 1.0 (1:1) output image, then the output image
	* will have its height adjusted to preserve the 1.77 aspect ratio of the camera, without showing letterboxing.
	* For example, a 1.77 Camera Aspect Ratio and a 1024x1024 output resolution will produce a render that is
	* 1024x576, which preserves the existing aspect ratio and the given width. No behavior change if Constrain Aspect Ratio is not enabled.
	*/
	ScaleToWidth,

	/**
	* If the camera has Constrain Aspect Ratio enabled, then the height of the output resolution will be preserved,
	* and the width will automatically be adjusted based on the camera aspect ratio.
	*
	* ie: If you have a 1.77 (16:9) Camera Aspect Ratio and target a 1.0 (1:1) output image, then the output image
	* will have its width adjusted to preserve the 1.77 aspect ratio of the camera, without showing letterboxing.
	* For example, a 1.77 Camera Aspect Ratio and a 1024x1024 output resolution will produce a render that is
	* 1820x1024, which preserves the existing aspect ratio and the given height. No behavior change if Constrain Aspect Ratio is not enabled.
	*/
	ScaleToHeight
};

UCLASS(MinimalAPI)
class UMovieGraphGlobalOutputSettingNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()
public:
	UE_API UMovieGraphGlobalOutputSettingNode();
	
	// UObject Interface
	UE_API virtual void PostLoad() override;
	// ~UObject Interface

	// UMovieGraphSettingNode Interface
	UE_API virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	UE_API virtual void PostFlatten() override;
#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;
#endif
	UE_API virtual bool GetCustomPromotionDefaultValue(const FMovieGraphPropertyInfo& InPropertyInfo, FString& OutSerializedValue) const override;
	// ~UMovieGraphSettingNode Interface
	
protected:
	/*
	* This is called from PostLoad and when the UMovieGraphPipeline is initialized to convert any legacy properties. We do it this way to preserve
	* existing code that may be configuring assets after loading them but before rendering them.
	*
	* @param bEmitWarning - If true, a warning will be printed that the conversion took place and the users need to update their scripts.
	*/	
	UE_API virtual void ApplyPostLoadPropertyConversions(bool bEmitWarning);

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputDirectory : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputResolution : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AdaptResolution : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputFrameRate : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bOverwriteExistingOutput : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ZeroPadFrameNumbers : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_FrameNumberOffset : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_HandleFrameCount : 1;

	UE_DEPRECATED(5.6, "Use bOverride_CustomPlaybackRangeStart instead.")
	UPROPERTY(BlueprintReadWrite, Category = Overrides, meta = (DeprecatedProperty, DeprecationMessage = "Use bOverride_CustomPlaybackRangeStart instead."))
	uint8 bOverride_CustomPlaybackRangeStartFrame : 1;

	UE_DEPRECATED(5.6, "Use bOverride_CustomPlaybackRangeEnd instead.")
	UPROPERTY(BlueprintReadWrite, Category = Overrides, meta = (DeprecatedProperty, DeprecationMessage = "Use bOverride_CustomPlaybackRangeEnd instead."))
	uint8 bOverride_CustomPlaybackRangeEndFrame : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_CustomPlaybackRangeStart : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_CustomPlaybackRangeEnd : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_CustomTimecodeStart : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bDropFrameTimecode : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_VersioningSettings : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bFlushDiskWritesPerShot : 1;

	/** What directory should all of our output files be relative to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output", meta=(EditCondition="bOverride_OutputDirectory", MrgTokenAutocomplete))
	FDirectoryPath OutputDirectory;

	/** What resolution should our output files be exported at? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output", meta = (EditCondition = "bOverride_OutputResolution"))
	FMovieGraphNamedResolution OutputResolution;

	/** Should the output resolution be automatically adjusted to match the aspect ratio on cameras with Constrain Aspect Ratio? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output", meta = (EditCondition = "bOverride_AdaptResolution"))
	EMovieGraphAspectRatioAdaptBehavior AdaptResolution;
	
	/** What frame rate should the output files be exported at? This overrides the Display Rate of the target sequence. If not overwritten, uses the default Sequence Display Rate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output", meta=(EditCondition="bOverride_OutputFrameRate"))
	FFrameRate OutputFrameRate;
	
	/** If true, output containers should attempt to override any existing files with the same name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output", meta = (EditCondition = "bOverride_bOverwriteExistingOutput"))
	bool bOverwriteExistingOutput;

	/** How many digits should all output frame numbers be padded to? MySequence_1.png -> MySequence_0001.png. Useful for software that struggles to recognize frame ranges when non-padded. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output", meta = (EditCondition = "bOverride_ZeroPadFrameNumbers", UIMin = "1", UIMax = "5", ClampMin = "1"))
	int32 ZeroPadFrameNumbers;

	/**
	* How many frames should we offset the output frame number by? This is useful when using handle frames on Sequences that start at frame 0,
	* as the output would start in negative numbers. This can be used to offset by a fixed amount to ensure there's no negative numbers.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite,  Category = "File Output", meta = (EditCondition = "bOverride_FrameNumberOffset"))
	int32 FrameNumberOffset;

	/**
	* Top level shot track sections will automatically be expanded by this many frames in both directions, and the resulting
	* additional time will be rendered as part of that shot. The inner sequence should have sections long enough to cover
	* this expanded range, otherwise these tracks will not evaluate during handle frames and may produce unexpected results.
	* This can be used to generate handle frames for traditional non linear editing tools.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Frames", meta = (UIMin = 0, ClampMin = 0, EditCondition = "bOverride_HandleFrameCount"))
	int32 HandleFrameCount;

	/**
	* If overwritten, and the Type is set to "Custom", then the Value field will override the Sequence's Playback Range Start when rendering.
	* Values are expected to be in the Sequence's original Frame Rate (the custom range is applied before Frame Rate Override).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Frames", meta = (EditCondition = "bOverride_CustomPlaybackRangeStart"))
	FMovieGraphSequencePlaybackRangeBound CustomPlaybackRangeStart;
	

	/**
	* If overwritten, and the Type is set to "Custom", then the Value field will override the Sequence's Playback Range End when rendering.
	* Values are expected to be in the Sequence's original Frame Rate (the custom range is applied before Frame Rate Override).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Frames", meta = (EditCondition = "bOverride_CustomPlaybackRangeEnd"))
	FMovieGraphSequencePlaybackRangeBound CustomPlaybackRangeEnd;

	/** Start the timecode at a specific value, rather than the value coming from the Level Sequence. Only applicable to output formats that support timecode. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Timecode", meta = (EditCondition = "bOverride_CustomTimecodeStart"))
	FTimecode CustomTimecodeStart;

	/** Whether the embedded timecode track should be written using drop-frame format. Only applicable to output formats that support timecode, and when the sequence framerate is 29.97. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Timecode", DisplayName = "Use DF Timecode if 29.97 FPS", meta = (EditCondition = "bOverride_bDropFrameTimecode"))
	bool bDropFrameTimecode;

	/**
	 * Determines how versioning should be handled (Auto Version, Version Number, etc.).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Versioning", meta = (EditCondition = "bOverride_VersioningSettings"))
	FMovieGraphVersioningSettings VersioningSettings;

	/**
	* If true, the game thread will stall at the end of each shot to flush the rendering queue, and then flush any outstanding writes to disk, finalizing any
	* outstanding videos and generally completing the work. This is only relevant for scripting where scripts may do post-shot callback work.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scripting", meta = (EditCondition = "bOverride_bFlushDiskWritesPerShot"))
	bool bFlushDiskWritesPerShot;

// These are deprecated but not moved to WITH_EDITORONLY_DATA because that would break existing plugins when they compiled for shipping. 	
public:

	UE_DEPRECATED(5.6, "Use CustomPlaybackRangeStart with Type set to Custom and Value set to the desired value instead.")
	UPROPERTY(BlueprintReadWrite, Category = "Frames", meta = (DeprecatedFunction, DeprecationMessage = "Use CustomPlaybackRangeStart with Type set to Custom and Value set to the desired value instead."))
	int32 CustomPlaybackRangeStartFrame;
	
	UE_DEPRECATED(5.6, "Use CustomPlaybackRangeEnd with Type set to Custom and Value set to the desired value instead.")
	UPROPERTY(BlueprintReadWrite, Category = "Frames", meta = (DeprecatedFunction, DeprecationMessage = "Use CustomPlaybackRangeEnd with Type set to Custom and Value set to the desired value instead."))
	int32 CustomPlaybackRangeEndFrame;

};

#undef UE_API
