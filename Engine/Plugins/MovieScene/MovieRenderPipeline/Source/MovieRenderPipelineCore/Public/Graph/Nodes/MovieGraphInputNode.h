// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"

#include "MovieGraphInputNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A graph node which displays all input members available in the graph. */
UCLASS(MinimalAPI)
class UMovieGraphInputNode : public UMovieGraphNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphInputNode();

	//~ Begin UMovieGraphNode interface
	UE_API virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const override;
	UE_API virtual TArray<UMovieGraphPin*> EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const override;
	UE_API virtual FString GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext) const override;
	UE_API virtual bool CanBeDisabled() const override;
	
	virtual bool CanBeAddedByUser() const override { return false; }

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif
	//~ End UMovieGraphNode interface

private:
	UE_API virtual void RegisterDelegates() override;

	/** Register delegates for the provided input member. */
	UE_API void RegisterInputDelegates(UMovieGraphInput* Input);

	/** Update data (name, etc) on all existing output pins on this node to reflect the input members on the graph. */
	UE_API void UpdateExistingPins(UMovieGraphMember* ChangedVariable) const;
};

#undef UE_API
