// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Scene.h"
#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "MovieGraphClassPropertyModifier.h"
#include "MovieGraphModifierNode.h"

#include "MovieGraphLightModifierNode.generated.h"

class UMovieGraphPropertyModifier;

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Controls which intensity properties are used to control lighting intensity in the Light Modifier node.
 */
UENUM(BlueprintType)
enum class EMovieGraphLightModifierIntensityMethod : uint8
{
	/** Apply the "Intensity" property to Point, Rect, and Spot lights. */
	PointRectSpot	UMETA(DisplayName = "Point, Rect, and Spot Lights"),

	/** The main "Intensity" property is ignored, and the type-specific "Intensity" properties are used instead. */
	PerLightActor	UMETA(DisplayName = "Per Light Actor Type")
};

/** 
 * A node which modifies properties on light components within the world.
 *
 * There are several common light properties (eg, Light Color) that can be modified directly. For less common light properties, they
 * can be modified by adding them as "Custom" properties. The common properties apply to all standard light types; custom properties apply only
 * to a specific light type.
 */
UCLASS(MinimalAPI)
class UMovieGraphLightModifierNode final : public UMovieGraphSettingNode, public IMovieGraphModifierNodeInterface
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphLightModifierNode();

#if WITH_EDITOR
	//~ Begin UMovieGraphNode interface
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UMovieGraphNode interface

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

	//~ Begin UMovieGraphNode interface
	UE_API virtual TArray<FPropertyBagPropertyDesc> GetDynamicPropertyDescriptions() const override;
	UE_API virtual TArray<FMovieGraphPropertyInfo> GetOverrideablePropertyInfo() const override;
	//~ End UMovieGraphNode interface

	//~ Begin IMovieGraphModifierNodeInterface interface
	UE_API virtual TArray<UMovieGraphModifierBase*> GetAllModifiers() const override;
	UE_API virtual bool SupportsCollections() const override;
	UE_API virtual TArray<FName> GetAllCollections() const override;
	UE_API virtual void AddCollection(const FName& InCollectionName) override;
	UE_API virtual bool RemoveCollection(const FName& InCollectionName) override;
	//~ End IMovieGraphModifierNodeInterface interface

	//~ Begin UMovieGraphSettingNode interface
	UE_API virtual FString GetNodeInstanceName() const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	UE_API virtual void PrepareForFlattening(const UMovieGraphSettingNode* InSourceNode) override;
	//~ End UMovieGraphSettingNode interface

	/**
	 * Adds the property from the specified class to the "Custom" lighting properties. Does nothing if the Custom properties already contains it.
	 * Returns true on success, else false.
	 *
	 * Adding a custom property does not immediately make it take effect. Its override state needs to be set to "true" via
	 * UpdateCustomLightPropertyOverrideState().
	 */
	UE_API bool AddCustomLightProperty(UClass* InLightComponentClass, const FProperty* InLightProperty);

	/**
	 * Adds the property from the specified class to the "Custom" lighting properties. Does nothing if the Custom properties already contains it.
	 * Returns true on success, else false.
	 * 
	 * Adding a custom property does not immediately make it take effect. Its override state needs to be set to "true" via
	 * UpdateCustomLightPropertyOverrideState().
	 */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	UE_API bool AddCustomLightProperty(UClass* InLightComponentClass, const FName& InPropertyName);

	/**
	 * Removes the property in the specified class from the "Custom" lighting properties. Does nothing if the Custom properties don't contain the
	 * property. Returns true on success, else false.
	 */
	UE_API bool RemoveCustomLightProperty(const UClass* InLightComponentClass, const FProperty* InLightProperty);

	/**
	 * Removes the property in the specified class from the "Custom" lighting properties. Does nothing if the Custom properties don't contain the
	 * property. Returns true on success, else false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	UE_API bool RemoveCustomLightProperty(const UClass* InLightComponentClass, const FName& InPropertyName);

	/** Determines if the given property exists within the "Custom" lighting properties. */
	UE_API bool HasCustomLightProperty(const UClass* InLightComponentClass, const FProperty* InLightProperty) const;
	
	/** Determines if the given property exists within the "Custom" lighting properties. */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	UE_API bool HasCustomLightProperty(const UClass* InLightComponentClass, const FName& InPropertyName) const;

	/**
	 * Determines if the given "Custom" light property has been marked as overridden. A custom property MUST be marked as overridden in
	 * order to take effect. The override state is visualized as the checkbox in front of the property name the UI.
	 */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	UE_API bool IsCustomLightPropertyOverridden(const UClass* InLightComponentClass, const FName& InPropertyName) const;
	
	/**
	 * Updates the override state for a "Custom" light property. A custom property MUST be marked as overridden in order to take effect. Returns true
	 * on success, else false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	UE_API bool UpdateCustomLightPropertyOverrideState(const UClass* InLightComponentClass, const FName& InPropertyName, const bool bIsOverridden);

	/** Gets how many "Custom" lighting properties have been added. */
	UFUNCTION(BlueprintCallable, Category = "Properties")
	UE_API int32 GetNumCustomLightProperties() const;

public:
	// Always merge the modifier name, no need for the user to do this explicitly
	UPROPERTY()
	uint8 bOverride_ModifierName : 1 = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DirectionalLightIntensity : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SkyLightIntensityScale : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_IntensityMethod : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Intensity : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_PointLightIntensity : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_PointLightIntensityUnits : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_RectLightIntensity : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_RectLightIntensityUnits : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SpotLightIntensity : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SpotLightIntensityUnits : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_IntensityUnits : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_LightColor : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bAffectsWorld : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_CastShadows : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_IndirectLightingIntensity : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_VolumetricScatteringIntensity : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_LightingChannels : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SamplesPerPixel : 1;

	/** The name of this modifier. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modifier")
	FString ModifierName;

	/** 
	 * Maximum illumination from the light (in lux).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity", meta = (EditCondition = "bOverride_DirectionalLightIntensity", UIMin = "0.0", UIMax = "150.0", SliderExponent = "2.0", Units = "lux"))
	float DirectionalLightIntensity = 10.f;

	/** 
	 * Total energy that the sky light emits.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity", meta = (EditCondition = "bOverride_SkyLightIntensityScale", UIMin = "0.0", UIMax = "50000.0", SliderExponent = "10.0"))
	float SkyLightIntensityScale = 1.f;

	/** 
	 * Whether point, rect, and spot lights share the same intensity, or the intensity (and intensity units) are unique for each type.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity", meta = (EditCondition = "bOverride_IntensityMethod"))
	EMovieGraphLightModifierIntensityMethod IntensityMethod = EMovieGraphLightModifierIntensityMethod::PointRectSpot;

	/** 
	 * Total energy that the light emits.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity", meta = (EditCondition = "bOverride_Intensity", UIMin = "0.0", UIMax = "20.0", Units = "cd"))
	float Intensity = 8.f;

	/** 
	 * Total energy that the point light emits.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity|PointLight", meta = (EditCondition = "bOverride_PointLightIntensity", UIMin = "0.0", UIMax = "20.0"))
	float PointLightIntensity = 8.f;

	/** 
	 * Units used for the point light intensity. 
	 * The peak luminous intensity is measured in candelas, while the luminous flux is measured in lumens.
	 * When the units are set in Nits, the light's power is also determined by the size of the light source (larger sources will emit more light).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity|PointLight", meta = (EditCondition = "bOverride_PointLightIntensityUnits"))
	ELightUnits PointLightIntensityUnits = ELightUnits::Candelas;

	/** 
	 * Total energy that the rect light emits.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity|RectLight", meta = (EditCondition = "bOverride_RectLightIntensity", UIMin = "0.0", UIMax = "20.0"))
	float RectLightIntensity = 8.f;

	/** 
	 * Units used for the rect light intensity. 
	 * The peak luminous intensity is measured in candelas, while the luminous flux is measured in lumens.
	 * When the units are set in Nits, the light's power is also determined by the size of the light source (larger sources will emit more light).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity|RectLight", meta = (EditCondition = "bOverride_RectLightIntensityUnits"))
	ELightUnits RectLightIntensityUnits = ELightUnits::Candelas;

	/** 
	 * Total energy that the spot light emits.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity|SpotLight", meta = (EditCondition = "bOverride_SpotLightIntensity", UIMin = "0.0", UIMax = "20.0"))
	float SpotLightIntensity = 8.f;

	/** 
	 * Units used for the spot light intensity. 
	 * The peak luminous intensity is measured in candelas, while the luminous flux is measured in lumens.
	 * When the units are set in Nits, the light's power is also determined by the size of the light source (larger sources will emit more light).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity|SpotLight", meta = (EditCondition = "bOverride_SpotLightIntensityUnits"))
	ELightUnits SpotLightIntensityUnits = ELightUnits::Candelas;

	/** 
	 * Units used for the intensity. 
	 * The peak luminous intensity is measured in candelas, while the luminous flux is measured in lumens.
	 * When the units are set in Nits, the light's power is also determined by the size of the light source (larger sources will emit more light).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light|Intensity", meta = (EditCondition = "bOverride_IntensityUnits"))
	ELightUnits IntensityUnits = ELightUnits::Candelas;

	/** 
	 * Filter color of the light.
	 * Note that this can change the light's effective intensity.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta = (EditCondition = "bOverride_LightColor", HideAlphaChannel))
	FColor LightColor = FColor::White;

	/** 
	 * Whether the light can affect the world, or whether it is disabled.
	 * A disabled light will not contribute to the scene in any way.  This setting cannot be changed at runtime and unbuilds lighting when changed.
	 * Setting this to false has the same effect as deleting the light, so it is useful for non-destructive experiments.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta = (EditCondition = "bOverride_bAffectsWorld"))
	uint32 bAffectsWorld : 1 = 1;

	/**
	 * Whether the light should cast any shadows.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta = (EditCondition = "bOverride_CastShadows"))
	uint32 CastShadows : 1 = 1;

	/** 
	 * Scales the indirect lighting contribution from this light. 
	 * A value of 0 disables any GI from this light. Default is 1.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta = (UIMin = "0.0", UIMax = "6.0", EditCondition = "bOverride_IndirectLightingIntensity"))
	float IndirectLightingIntensity = 1.f;

	/** Intensity of the volumetric scattering from this light.  This scales Intensity and LightColor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta = (UIMin = "0.25", UIMax = "4.0", EditCondition = "bOverride_VolumetricScatteringIntensity"))
	float VolumetricScatteringIntensity = 1.f;

	/** 
	 * Channels that this light should affect.  
	 * These channels only apply to opaque materials, direct lighting, and dynamic lighting and shadowing.
	 * Lighting channels are only supported on translucent materials using forward shading (i.e. when not using the translucency lighting volume).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta = (EditCondition = "bOverride_LightingChannels"))
	FLightingChannels LightingChannels;

	/** Samples per pixel for ray tracing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RayTracing", meta = (EditCondition = "bOverride_SamplesPerPixel"))
	int SamplesPerPixel = 1;

private:
	/** Gets the property name used for the given property within the DynamicProperties property bag. */
	UE_API static FName GetLightingPropertyName(const UClass* InLightComponentClass, const FProperty* InLightProperty);

	/**
	 * Gets the bOverride_* property name used for the given property within the DynamicProperties property bag. The "main" property name
	 * is the property name returned by GetLightingPropertyName().
	 */
	UE_API static FName GetLightingOverridePropertyName(const FName& InMainPropertyName);

private:
	// Always merge collection names, no need for the user to do this explicitly
	UPROPERTY()
	uint8 bOverride_Collections : 1 = 1;

	// Same with the custom property info; always merge it
	UPROPERTY()
	uint8 bOverride_CustomPropertyInfo : 1 = 1;

	/** The collections modified by this node. Only lights will be modified. */
	UPROPERTY(meta = (HideInActiveRenderSettings))
	TArray<FName> Collections;

	/** The light modifier associated with this node. */
	UPROPERTY(meta = (HideInActiveRenderSettings))
	TObjectPtr<UMovieGraphPropertyModifier> Modifier;

	/**
	 * Stores supplemental information for each "Custom" lighting property. Key is the name of the corresponding non-override property
	 * within DynamicProperties.
	 */
	UPROPERTY(meta = (HideInActiveRenderSettings))
	TMap<FName, FMovieGraphPropertyReference> CustomPropertyInfo;
};

#undef UE_API
