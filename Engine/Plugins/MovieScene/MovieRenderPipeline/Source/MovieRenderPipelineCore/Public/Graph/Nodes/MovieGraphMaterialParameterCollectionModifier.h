// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphRenderLayerSubsystem.h"

#include "MovieGraphMaterialParameterCollectionModifier.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMaterialParameterCollection;

/**
 * A modifier that allows for changing scalar and vector parameters within a Material Parameter Collection.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphMaterialParameterCollectionModifier : public UMovieGraphModifierBase
{
	GENERATED_BODY()

public:
	//~ Begin UMovieGraphModifierBase Interface
	UE_API virtual void ApplyModifier(const UWorld* World) override;
	UE_API virtual void UndoModifier() override;
	UE_API virtual FText GetModifierName() override;
	//~ End UMovieGraphModifierBase Interface

	/** Clear the scalar and vector parameter values that have been set on this modifier. */
	UFUNCTION(BlueprintCallable, Category = "Modifier")
	UE_API void ClearParameterValues();

public:
	/** The Material Parameter Collection that this modifier should update. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	TSoftObjectPtr<UMaterialParameterCollection> MaterialParameterCollection;

	/** The names and values of scalar parameters that this modifier will apply. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	TMap<FName, float> ScalarParameterUpdates;

	/** The names and values of vector parameters that this modifier will apply. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	TMap<FName, FLinearColor> VectorParameterUpdates;

private:
	/**
	 * Applies the given scalar and vector values to the Material Parameter Collection associated with this modifier.
	 * Optionally, the current values can be cached for later restoration.
	 */
	void ApplyScalarAndVectorValues(const TMap<FName, float>& InScalarValues, const TMap<FName, FLinearColor>& InVectorValues, const bool bCacheCurrentValues = false);

private:
	/** The cached names/values of scalar parameters before the modifier was applied. */
	TMap<FName, float> PriorScalarValues;
	
	/** The cached names/values of vector parameters before the modifier was applied. */
	TMap<FName, FLinearColor> PriorVectorValues;

	/** The world that the modifier is being applied in. */
	UPROPERTY(Transient)
	TWeakObjectPtr<const UWorld> CachedWorld = nullptr;
};

#undef UE_API