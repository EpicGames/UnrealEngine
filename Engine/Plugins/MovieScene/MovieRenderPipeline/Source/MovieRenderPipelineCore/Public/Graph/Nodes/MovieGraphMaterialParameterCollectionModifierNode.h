// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "MovieGraphModifierNode.h"

#include "MovieGraphMaterialParameterCollectionModifierNode.generated.h"

class UMaterialParameterCollection;
class UMovieGraphMaterialParameterCollectionModifier;

#define UE_API MOVIERENDERPIPELINECORE_API

/** 
 * A node which modifies the scalar and vector parameters of a Material Parameter Collection. 
 */
UCLASS(MinimalAPI)
class UMovieGraphMaterialParameterCollectionModifierNode : public UMovieGraphSettingNode, public IMovieGraphModifierNodeInterface
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphMaterialParameterCollectionModifierNode();

#if WITH_EDITOR
	//~ Begin UMovieGraphNode interface
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UMovieGraphNode interface

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostLoad() override;
	//~ End UObject interface
#endif

	//~ Begin UMovieGraphNode interface
	UE_API virtual TArray<FPropertyBagPropertyDesc> GetDynamicPropertyDescriptions() const override;
	UE_API virtual bool GetDynamicPropertyValue(const FName PropertyName, FString& OutValue) override;
	UE_API virtual TArray<FMovieGraphPropertyInfo> GetOverrideablePropertyInfo() const override;
	//~ End UMovieGraphNode interface

	//~ Begin IMovieGraphModifierNodeInterface interface
	UE_API virtual TArray<UMovieGraphModifierBase*> GetAllModifiers() const override;
	UE_API virtual bool SupportsCollections() const override;
	//~ End IMovieGraphModifierNodeInterface interface

	//~ Begin UMovieGraphSettingNode interface
	UE_API virtual FString GetNodeInstanceName() const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	UE_API virtual void PrepareForFlattening(const UMovieGraphSettingNode* InSourceNode) override;
	//~ End UMovieGraphSettingNode interface

public:	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_MaterialParameterCollection : 1;

	/** The Material Parameter Collection that should be modified. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Material Parameter Collection", meta = (EditCondition = "bOverride_MaterialParameterCollection"))
	TSoftObjectPtr<UMaterialParameterCollection> MaterialParameterCollection;

private:
	void OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);
	
	// These should really be defined somewhere outside of MRG, but they don't appear to be.
	static inline const FName EditConditionMetadataKey = TEXT("EditCondition");
	static inline const FName CategoryMetadataKey = TEXT("Category");
	static inline const FName DisplayNameMetadataKey = TEXT("DisplayName");
	static inline const FName InlineEditConditionToggleMetadataKey = TEXT("InlineEditConditionToggle");
	
	/** The Material Parameter Collection modifier associated with this node. */
	UPROPERTY()
	TObjectPtr<UMovieGraphMaterialParameterCollectionModifier> Modifier;
};

#undef UE_API
