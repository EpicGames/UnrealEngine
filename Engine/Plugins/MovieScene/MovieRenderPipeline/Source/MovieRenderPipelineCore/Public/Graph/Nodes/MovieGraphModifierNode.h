// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphRenderLayerSubsystem.h"

#include "MovieGraphModifierNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A container which allows an array of modifiers to be merged correctly. */
UCLASS()
class UMovieGraphMergeableModifierContainer final : public UObject, public IMovieGraphTraversableObject
{
	GENERATED_BODY()

public:
	// IMovieGraphTraversableObject interface
	virtual void Merge(const IMovieGraphTraversableObject* InSourceObject) override;
	virtual TArray<TPair<FString, FString>> GetMergedProperties() const override;
	// ~IMovieGraphTraversableObject interface

private:
	void MergeProperties(const TObjectPtr<UMovieGraphCollectionModifier>& InDestModifier, const TObjectPtr<UMovieGraphCollectionModifier>& InSourceModifier);

public:
	UPROPERTY()
	TArray<TObjectPtr<UMovieGraphCollectionModifier>> Modifiers;
};

UINTERFACE(MinimalAPI, NotBlueprintable)
class UMovieGraphModifierNodeInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Provides the interface that nodes must implement if they want to provide modifier behavior.
 *
 * If a modifier node does not support collections, it must specify this explicitly via SupportsCollections(). If it does not support collections,
 * all collection-related methods can remain unimplemented.
 *
 * Note that some of these methods may not be called by MRG yet, but they may be called in the future (eg, in future UIs). To ensure modifier
 * functionality in the future, implement all methods that make sense for the modifier.
 */
class IMovieGraphModifierNodeInterface : public IInterface
{
	GENERATED_BODY()

public:
	/** Gets all modifiers that will be applied with this modifier node. */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	virtual TArray<UMovieGraphModifierBase*> GetAllModifiers() const
	{
		return {};
	}

	/** Gets whether this modifier node supports/uses collections. */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	virtual bool SupportsCollections() const
	{
		return true;
	}

	/** Gets the enable state (within this modifier) of the collection with the given name. */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	virtual bool IsCollectionEnabled(const FName& InCollectionName) const
	{
		return true;
	}

	/**
	 * Sets the enable state (within this modifier) of the collection with the given name. Disabled collections will not be modified by this modifier
	 * node. Collections that are added to the modifier are enabled by default.
	 */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	virtual void SetCollectionEnabled(const FName& InCollectionName, const bool bIsCollectionEnabled)
	{
		
	}

	/** Gets all collections that will be affected by the modifiers used by this node. */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	virtual TArray<FName> GetAllCollections() const
	{
		return {};
	}

	/** Adds a collection identified by the given name which will be affected by the modifiers on this node. */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	virtual void AddCollection(const FName& InCollectionName)
	{
		
	}

	/** Removes a collection identified by the given name. Returns true if the collection was found and removed successfully, else false. */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	virtual bool RemoveCollection(const FName& InCollectionName)
	{
		return false;
	}
};

/** 
 * A modifier node which allows render properties and materials to be changed.
 */
UCLASS(MinimalAPI)
class UMovieGraphModifierNode : public UMovieGraphSettingNode, public IMovieGraphModifierNodeInterface
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphModifierNode();

#if WITH_EDITOR
	//~ Begin UMovieGraphNode interface
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UMovieGraphNode interface

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

	//~ Begin UMovieGraphSettingNode interface
	UE_API virtual FString GetNodeInstanceName() const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	//~ End UMovieGraphSettingNode interface

	//~ Begin IMovieGraphModifierNodeInterface
	UE_API virtual TArray<UMovieGraphModifierBase*> GetAllModifiers() const override;
	UE_API virtual void AddCollection(const FName& InCollectionName) override;
	UE_API virtual bool RemoveCollection(const FName& InCollectionName) override;
	UE_API virtual TArray<FName> GetAllCollections() const override;
	UE_API virtual bool SupportsCollections() const override;
	UE_API virtual void SetCollectionEnabled(const FName& InCollectionName, const bool bIsCollectionEnabled) override;
	UE_API virtual bool IsCollectionEnabled(const FName& InCollectionName) const override;
	//~ End IMovieGraphModifierNodeInterface

	/** Gets the modifier of the specified type, or nullptr if one does not exist on this node. */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	UE_API UMovieGraphCollectionModifier* GetModifier(TSubclassOf<UMovieGraphCollectionModifier> ModifierType) const;

	/** Gets all modifiers currently added to the node. */
	UE_DEPRECATED(5.7, "Use GetAllModifiers() instead.")
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	UE_API const TArray<UMovieGraphCollectionModifier*>& GetModifiers() const;

	/**
	 * Adds a new modifier of the specified type. Returns a pointer to the new modifier, or nullptr if a modifier of the specified type already
	 * exists on this node (only one modifier of each type can be added to the node).
	 */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	UE_API UMovieGraphCollectionModifier* AddModifier(TSubclassOf<UMovieGraphCollectionModifier> ModifierType);

	/** Removes the modifier of the specified type. Returns true on success, or false if a modifier of the specified type does not exist on the node. */
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	UE_API bool RemoveModifier(TSubclassOf<UMovieGraphCollectionModifier> ModifierType);

	/** Gets all collections that will be affected by the modifiers on this node. */
	UE_DEPRECATED(5.7, "Use GetAllCollections() instead.")
	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	UE_API const TArray<FName>& GetCollections() const;

public:
	UPROPERTY()
	uint8 bOverride_ModifierName : 1 = 1;	// Always merge the modifier name, no need for the user to do this explicitly

	/** The name of this modifier. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modifier")
	FString ModifierName;

private:
	// These private override properties exist so that the associated non-override properties are merged properly
	// by UMovieGraphMergeableModifierContainer
	UPROPERTY()
	uint8 bOverride_Collections : 1 = 1;

	UPROPERTY()
	uint8 bOverride_ModifiersContainer : 1 = 1;

	UPROPERTY()
	uint8 bOverride_DisabledCollections : 1 = 1;
	
	/** The names of collections being modified. */
	UPROPERTY()
	TArray<FName> Collections;
	
	/** The modifiers this node should run. */
	UPROPERTY(meta=(DisplayName="Modifiers"))
	TObjectPtr<UMovieGraphMergeableModifierContainer> ModifiersContainer;

	/** The collections on this node that have been disabled. */
	UPROPERTY()
	TSet<FName> DisabledCollections;
};

#undef UE_API
