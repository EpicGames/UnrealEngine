// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"

#include "MovieGraphOutputNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A graph node which displays all output members available in the graph. */
UCLASS(MinimalAPI)
class UMovieGraphOutputNode : public UMovieGraphNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphOutputNode();
	
	UE_API virtual TArray<FMovieGraphPinProperties> GetInputPinProperties() const override;
	UE_API virtual bool CanBeDisabled() const override;
	
	virtual bool CanBeAddedByUser() const override { return false; }

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

private:
	UE_API virtual void RegisterDelegates() override;

	/** Register delegates for the provided output member. */
	UE_API void RegisterOutputDelegates(UMovieGraphOutput* Output);

	/** Update data (name, etc) on all existing input pins on this node to reflect the output members on the graph. */
	UE_API void UpdateExistingPins(UMovieGraphMember* ChangedVariable) const;
};

#undef UE_API
