// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "Templates/SubclassOf.h"

#include "MovieGraphRemoveRenderSettingNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A node which can remove other nodes in the graph. */
UCLASS(MinimalAPI)
class UMovieGraphRemoveRenderSettingNode : public UMovieGraphNode
{
	GENERATED_BODY()

public:
	UMovieGraphRemoveRenderSettingNode() = default;

	UE_API virtual TArray<FMovieGraphPinProperties> GetInputPinProperties() const override;
	UE_API virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const override;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

public:
	/** The type of node (exact match) that should be removed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	TSubclassOf<UMovieGraphSettingNode> NodeType;
};

#undef UE_API
