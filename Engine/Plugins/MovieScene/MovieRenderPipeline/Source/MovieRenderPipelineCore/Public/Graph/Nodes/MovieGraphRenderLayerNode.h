// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "MovieGraphRenderLayerNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

UCLASS(MinimalAPI)
class UMovieGraphRenderLayerNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()
public:
	UE_API UMovieGraphRenderLayerNode();

	FString GetRenderLayerName() const { return LayerName; }

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_LayerName : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_LayerWarmUpFrames : 1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_LayerName"))
	FString LayerName;

	/**
	 * If overridden, the number of warm-up frames to use for this render layer instead of the global value from the Warm Up Settings node.
	 * 
	 * Generally this value should remain untouched unless this layer has specific needs that differ from the other layers. For
	 * example, if Lumen needs extra time to converge on this particular layer, then increase the value here.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bOverride_LayerWarmUpFrames", UIMin=0, ClampMin=0))
	int32 LayerWarmUpFrames;
};

#undef UE_API
