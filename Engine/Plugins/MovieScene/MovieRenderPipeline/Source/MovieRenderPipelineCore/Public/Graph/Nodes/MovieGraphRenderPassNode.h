// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphRenderCameraSource.h"
#include "Styling/AppStyle.h"

#include "MovieGraphRenderPassNode.generated.h"

// Forward Declare
struct FMovieGraphRenderPassSetupData;
struct FMovieGraphTimeStepData;
struct FMoviePipelineEndShotRenderTelemetryContext;

#define UE_API MOVIERENDERPIPELINECORE_API

/**
* The UMovieGraphRenderPassNode node defines a render pass that MRQ may produce. This node can be implemented
* in the graph multiple times, and the exact settings it should use can be created out of a mixture of nodes. Because
* of this, when rendering, MRQ will figure out how many layers there are that actually use this CDO and will call
* the function on the CDO once, providing the information about all instances. This will allow the node to create any
* number of instances (decoupled from the number of times the node is used in the graph).
*/
UCLASS(MinimalAPI, Abstract)
class UMovieGraphRenderPassNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UMovieGraphRenderPassNode()
	{
		RendererName = TEXT("UnnamedRenderPass");
		RendererSubName = TEXT("beauty");
	}

	/** Get the name of this renderer. Deferred, Path Tracer, Panoramic, etc. */
	FString GetRendererName() const { return GetRendererNameImpl(); }

	/**
	 * Get the sub-name of this renderer, which identifies a specific pass that this renderer generates. Each type of pass that this
	 * renderer generates should have a unique sub-name.
	 */
	FString GetRendererSubName() const { return GetRendererSubNameImpl(); }
	
	/**
	 * Create a new camera source for this render node instance, or nullptr if this node uses
	 * default sequence cameras. Called on non-CDO instances.
	 */
	TSharedPtr<FMovieGraphRenderCameraSource> CreateRenderCameraSource(const UMovieGraphPipeline* InPipeline, const UMovieGraphEvaluatedConfig* EvaluatedConfig) { return CreateRenderCameraSourceImpl(InPipeline, EvaluatedConfig); }

	/**
	 * Gets an existing camera source for this render node instance, or nullptr if this node uses
	 * default sequence cameras. Called on non-CDO instances.
	 */
	TSharedPtr<FMovieGraphRenderCameraSource> GetRenderCameraSource() const { return GetRenderCameraSourceImpl(); }

	/**
	 * Collects custom camera sources from all active pass instances into InOutSources.
	 * Called on the CDO.
	 * @param InOutSources (in, out) May be non-empty on input; unique sources are appended.
	 */
	void GatherRenderCameraSources(TArray<TSharedRef<FMovieGraphRenderCameraSource>>& InOutSources) const { GatherRenderCameraSourcesImpl(InOutSources); }

	/** Called when this should set up for rendering a new shot. Called on the CDO. */
	void Setup(const FMovieGraphRenderPassSetupData& InSetupData) { SetupImpl(InSetupData); }
	
	/** Called when this should do teardown of resources. FlushRenderingCommands() will have already been called by this point. Called on the CDO. */
	void Teardown() { TeardownImpl(); }
	
	/** Called each tick (once per temporal sample). Called on the CDO. */
	void Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) { RenderImpl(InFrameTraversalContext, InTimeData); }

	/** 
	* Called each output frame. Should add a series of FMovieGraphRenderDataIdentifiers to the array, and then when producing frames
	* in Render, the resulting image data should have the matching FMovieGraphRenderDataIdentifiers associated with it. Used by the
	* Output Merger to ensure all of the render data for a given frame has been generated before passing it on to write to disk.
	* Called on the CDO.
	*/
	void GatherOutputPasses(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const { GatherOutputPassesImpl(InConfig, OutExpectedPasses); }

	/** Called by the renderer to gather runtime telemetry from live render pass instances. Called on the CDO. */
	void UpdateRenderPassTelemetry(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const { UpdateRenderPassTelemetryImpl(InOutTelemetry); }

	/** Gets the number of Scene Views (that is, renders of the 3d scene) that this pass will produce. Can be zero for things like UI Renderers,
	* or more than one for things like panoramic or tiling. */
	int32 GetNumSceneViewsRendered() const { return GetNumSceneViewsRenderedImpl(); }

	/**
	* Called by the renderer when gathering preview data. Implementations should *append* any banner messages
	* contributed by pass instances that produced the preview tile identified by InIdentifier. Default implementation
	* contributes nothing.
	*/
	void GetPreviewBannerMessages(const FMovieGraphRenderDataIdentifier& InIdentifier, TArray<FMovieGraphPreviewBannerMessage>& OutMessages) const { GetPreviewBannerMessagesImpl(InIdentifier, OutMessages); }

	/**
	* Get the cooling down frame count when denoising based on temporal frames are used.
	**/
	virtual int32 GetCoolingDownFrameCount() const { return 0; }

	/**
	 * Gets the type(s) of output nodes that are allowed to process/output the pixel data originating from this node. Typically the return value is
	 * an empty array, meaning that all output nodes process the data.
	 */
	TArray<FSoftClassPath> GetOutputTypeRestrictions() const { return GetOutputTypeRestrictionsImpl(); }

	virtual void ResolveTokenContainingProperties(TFunction<void(FString&)>& ResolveFunc, const FMovieGraphTokenResolveContext& InContext) override
	{
		Super::ResolveTokenContainingProperties(ResolveFunc, InContext);

		ResolveFunc(RendererName);
		ResolveFunc(RendererSubName);
	}

#if WITH_EDITOR
	virtual FText GetMenuCategory() const override
	{
		return NSLOCTEXT("MovieGraphNodes", "RenderPassGraphNode_Category", "Rendering");
	}

	virtual FLinearColor GetNodeTitleColor() const override
	{
		return FLinearColor(0.572f, 0.274f, 1.f);
	}
	
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override
	{
		static const FSlateIcon DeferredRendererIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "SequenceRecorder.TabIcon");
		OutColor = FLinearColor::White;
		return DeferredRendererIcon;
	}
#endif	// WITH_EDITOR

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_RendererName : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_RendererSubName : 1;

	/** The value that will be used in the {renderer_name} token. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Naming", meta = (EditCondition="bOverride_RendererName", MrgTokenAutocomplete))
	FString RendererName;

	/** The value that will be used in the {renderer_sub_name} token. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Naming", meta = (EditCondition="bOverride_RendererSubName", MrgTokenAutocomplete))
	FString RendererSubName;

protected:
	virtual FString GetRendererNameImpl() const { return RendererName; }
	virtual FString GetRendererSubNameImpl() const { return RendererSubName; }
	virtual void SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData) {}
	virtual void TeardownImpl() {}
	virtual void RenderImpl(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) {}
	virtual void GatherOutputPassesImpl(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const {}
	virtual void UpdateRenderPassTelemetryImpl(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const {}
	virtual int32 GetNumSceneViewsRenderedImpl() const { return 0; }
	virtual void GetPreviewBannerMessagesImpl(const FMovieGraphRenderDataIdentifier& InIdentifier, TArray<FMovieGraphPreviewBannerMessage>& OutMessages) const {}
	virtual TArray<FSoftClassPath> GetOutputTypeRestrictionsImpl() const { return {}; } 
	virtual TSharedPtr<FMovieGraphRenderCameraSource> CreateRenderCameraSourceImpl(const UMovieGraphPipeline* InMovieGraphPipeline, const UMovieGraphEvaluatedConfig* InEvaluatedConfig) { return nullptr; }
	virtual TSharedPtr<FMovieGraphRenderCameraSource> GetRenderCameraSourceImpl() const { return nullptr; }
	virtual void GatherRenderCameraSourcesImpl(TArray<TSharedRef<FMovieGraphRenderCameraSource>>& InOutSources) const {};
};

#undef UE_API
