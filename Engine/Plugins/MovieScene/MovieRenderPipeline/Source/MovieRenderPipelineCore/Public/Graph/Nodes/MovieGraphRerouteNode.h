// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "MovieGraphRerouteNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A node which is effectively a no-op/passthrough. Allows a connection to be routed untouched through this node to organize the graph. */
UCLASS(MinimalAPI)
class UMovieGraphRerouteNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphRerouteNode();

	//~ UMovieGraphNode Interface
	UE_API virtual TArray<FMovieGraphPinProperties> GetInputPinProperties() const override;
	UE_API virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const override;
	UE_API virtual bool CanBeDisabled() const override;
#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
#endif
	//~ End UMovieGraphNode Interface

	/** Gets the pin opposite to the specified InFromPin. */
	UE_API virtual UMovieGraphPin* GetPassThroughPin(const UMovieGraphPin* InFromPin) const;

	/** Sets the pin properties for this reroute node. Note that this applies to both the input and output pin. */
	UFUNCTION(BlueprintPure, Category = "Movie Graph")
	UE_API FMovieGraphPinProperties GetPinProperties() const;

	/**
	 * Sets the pin properties for this reroute node (both the input and output pin have the same properties). This generally should not be called
	 * unless you know what you're doing; normal connection/disconnection should handle setting the properties correctly.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API void SetPinProperties(const FMovieGraphPinProperties& InPinProperties);

private:
	/** Pin properties that are shared with both the input and output pins. */
	UPROPERTY()
	FMovieGraphPinProperties InputOutputProperties;
};

#undef UE_API
