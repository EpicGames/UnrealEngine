// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"

#include "MovieGraphSamplingMethodNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A node which configures sampling properties for renderers. */
UCLASS(MinimalAPI)
class UMovieGraphSamplingMethodNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphSamplingMethodNode();

	UE_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;
	UE_API virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const override;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (InlineEditConditionToggle))
	uint8 bOverride_SamplingMethodClass : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (InlineEditConditionToggle))
	uint8 bOverride_TemporalSampleCount : 1;

	/** The type of sampling the render should use. Not currently marked as BlueprintReadWrite/EditAnywhere as there is only one class anyways.*/
	UPROPERTY(NoClear, meta = (EditCondition = "bOverride_SamplingMethodClass", DisplayName = "Sampling Method", MetaClass = "/Script/MovieRenderPipelineCore.MovieGraphTimeStepBase", ShowDisplayNames))
	FSoftClassPath SamplingMethodClass;

	/** The number of temporal samples which should be taken on one frame. Applies only to linear sampling. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (UIMin = 1, ClampMin = 1, EditCondition = "bOverride_TemporalSampleCount"))
	int32 TemporalSampleCount;
};

#undef UE_API
