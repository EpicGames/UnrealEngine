// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"

#include "MovieGraphSelectNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A node which creates a condition that selects from a set of input branches. */
UCLASS(MinimalAPI)
class UMovieGraphSelectNode : public UMovieGraphNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphSelectNode();

	UE_API virtual TArray<FMovieGraphPinProperties> GetInputPinProperties() const override;
	UE_API virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const override;
	UE_API virtual TArray<UMovieGraphPin*> EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const override;

	/**
	 * Sets the data type for the select options and selected option (note that their data will be reset when this
	 * is called). For structs, enums, and objects, the value type object needs to be provided.
	 */
	UE_API void SetDataType(const EMovieGraphValueType ValueType, UObject* InValueTypeObject = nullptr);

	/** Gets the data type that the select node is currently using. */
	UE_API EMovieGraphValueType GetValueType() const;

	/** Gets the value type object associated with the value type currently set (if any). */
	UE_API const UObject* GetValueTypeObject() const;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

private:
	/** The options that are available on the node. */
	UPROPERTY(EditAnywhere, Instanced, Category = "Options", meta=(ShowInnerProperties, FullyExpand="true"))
	TObjectPtr<UMovieGraphValueContainer> SelectOptions;

	/** The currently selected option. */
	UPROPERTY(EditAnywhere, Instanced, Category = "Options", meta=(ShowInnerProperties, FullyExpand="true"))
	TObjectPtr<UMovieGraphValueContainer> SelectedOption;
};

#undef UE_API
