// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"

#include "MovieGraphSetCVarValueNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** A node which can set a specific console variable's value. */
UCLASS(MinimalAPI)
class UMovieGraphSetCVarValueNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()
public:
	UMovieGraphSetCVarValueNode() = default;

	UE_API virtual FString GetNodeInstanceName() const override;
	UE_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	UE_API virtual TArray<FMovieGraphPropertyInfo> GetOverrideablePropertyInfo() const override;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Name : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Value : 1;

	/** The name of the CVar having its value set. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_Name"))
	FString Name;

	/** The new value of the CVar. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "bOverride_Value"))
	float Value;
};

#undef UE_API
