// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphConfig.h"

#include "MovieGraphSetMetadataAttributesNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Represents a metadata attribute that can be included in a file.
 */
USTRUCT(BlueprintType)
struct FMovieGraphMetadataAttribute
{
	GENERATED_BODY()
	
	FMovieGraphMetadataAttribute(const FString& InName, const FString& InValue, const bool bInIsEnabled = true)
		: Name(InName)
		, Value(InValue)
		, bIsEnabled(bInIsEnabled)
	{
	}

	FMovieGraphMetadataAttribute()
		: bIsEnabled(true)
	{
	}

	/* The name of the metadata attribute. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(DisplayName="Metadata Name"))
	FString Name;

	/* The value of the metadata attribute. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(DisplayName="Metadata Value"))
	FString Value;

	/* Enable state. If disabled, this metadata attribute won't be added to the file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(DisplayName="Metadata Enable State"))
	bool bIsEnabled;
};

UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphMetadataAttributeCollection : public UObject, public IMovieGraphTraversableObject
{
	GENERATED_BODY()

public:
	/**
	* An array of metadata attributes to be added to output files.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Metadata", meta=(DisplayName="Metadata Attributes"))
	TArray<FMovieGraphMetadataAttribute> MetadataAttributes;

	// IMovieGraphTraversableObject interface
	UE_API virtual void Merge(const IMovieGraphTraversableObject* InSourceObject) override;
	UE_API virtual TArray<TPair<FString, FString>> GetMergedProperties() const override;
	// ~IMovieGraphTraversableObject interface
};

/** A node which can set a specific metadata attributes. */
UCLASS(MinimalAPI)
class UMovieGraphSetMetadataAttributesNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphSetMetadataAttributesNode();

	UE_API virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const override;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	
	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_MetadataAttributeCollection : 1 = 1;
	
	/**
	* A container of metadata attributes to be added to the file.
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category="Metadata", meta=(DisplayName="Metadata Attribute Collection", EditCondition="bOverride_MetadataAttributeCollection"))
	TObjectPtr<UMovieGraphMetadataAttributeCollection> MetadataAttributeCollection;
};

#undef UE_API
