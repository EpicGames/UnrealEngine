// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphNode.h"

#include "MovieGraphSetStartEndConsoleCommandsNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** Console commands that can execute within the UMovieGraphSetStartEndConsoleCommandsNode. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphStartEndConsoleCommands : public UObject, public IMovieGraphTraversableObject
{
	GENERATED_BODY()

public:
	/**
	 * Console commands to execute when this shot starts rendering. If the command(s) need to be undone after the shot finishes rendering, add a
	 * matching entry to Add End Commands.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Add Commands")
	TArray<FString> AddStartCommands;

	/** Console commands to execute when this shot finishes rendering. Used to restore changes made by Add Console Commands. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Add Commands")
	TArray<FString> AddEndCommands;

	/** Start commands that should be removed from upstream nodes. Commands entered here must be an exact match to upstream command(s) in order to be removed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Remove Commands")
	TArray<FString> RemoveStartCommands;

	/** End commands that should be removed from upstream nodes. Commands entered here must be an exact match to upstream command(s) in order to be removed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Remove Commands")
	TArray<FString> RemoveEndCommands;

	// IMovieGraphTraversableObject interface
	UE_API virtual void Merge(const IMovieGraphTraversableObject* InSourceObject) override;
	UE_API virtual TArray<TPair<FString, FString>> GetMergedProperties() const override;
	// ~IMovieGraphTraversableObject interface
};

/** A node which can set console commands to run at the start and end of a render. */
UCLASS(MinimalAPI)
class UMovieGraphSetStartEndConsoleCommandsNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()
public:
	UE_API UMovieGraphSetStartEndConsoleCommandsNode();

	//~ Begin UMovieGraphNode interface
	UE_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;
	//~ End UMovieGraphNode interface

#if WITH_EDITOR
	//~ Begin UMovieGraphNode interface
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UMovieGraphNode interface

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ConsoleCommands : 1 = 1;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category="Settings", meta=(EditCondition="bOverride_ConsoleCommands"))
	TObjectPtr<UMovieGraphStartEndConsoleCommands> ConsoleCommands;
};

#undef UE_API
