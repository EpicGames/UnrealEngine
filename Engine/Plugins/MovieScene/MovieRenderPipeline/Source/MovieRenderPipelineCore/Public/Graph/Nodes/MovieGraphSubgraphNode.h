// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphPin.h"

#include "MovieGraphSubgraphNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * A node which represents another graph asset. Inputs/outputs on this subgraph will update if the underlying graph
 * asset's inputs/outputs change.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphSubgraphNode : public UMovieGraphNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphSubgraphNode();

	//~ Begin UMovieGraphNode interface
	UE_API virtual TArray<FMovieGraphPinProperties> GetInputPinProperties() const override;
	UE_API virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const override;
	UE_API virtual TArray<UMovieGraphPin*> EvaluatePinsToFollow(FMovieGraphEvaluationContext& InContext) const override;
	UE_API virtual FString GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext) const override;

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
#endif
	//~ End UMovieGraphNode interface

	//~ Begin UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

	/** Sets the graph asset this subgraph points to. */
	UFUNCTION(BlueprintCallable, Category="Graph")
	UE_API void SetSubGraphAsset(const TSoftObjectPtr<UMovieGraphConfig>& InSubgraphAsset);

	/** Gets the graph asset this subgraph points to. */
	UFUNCTION(BlueprintCallable, Category="Graph")
	UE_API UMovieGraphConfig* GetSubgraphAsset() const;

private:
	/** Update the subgraph to reflect the subgraph asset when the subgraph asset is saved. */
	UE_API void RefreshOnSubgraphAssetSaved();

private:
	UPROPERTY(EditAnywhere, Category="Graph")
	TSoftObjectPtr<UMovieGraphConfig> SubgraphAsset;
};

#undef UE_API
