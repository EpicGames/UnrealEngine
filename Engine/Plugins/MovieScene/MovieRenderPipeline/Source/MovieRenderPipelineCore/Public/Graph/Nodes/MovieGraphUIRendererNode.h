// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Nodes/MovieGraphWidgetRendererBaseNode.h"
#include "Styling/AppStyle.h"

#include "MovieGraphUIRendererNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
class UMovieGraphDefaultRenderer;
class UWidget;
struct FMovieGraphRenderPassLayerData;

/** A node which renders the viewport's UMG widget to a standalone image, or composited on top of a render layer. */
UCLASS(MinimalAPI)
class UMovieGraphUIRendererNode : public UMovieGraphWidgetRendererBaseNode
{
	GENERATED_BODY()

public:
	UMovieGraphUIRendererNode();

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:
	UE_DEPRECATED(5.7, "RendererName usage is deprecated, please use GetRendererName() instead.")
	static UE_API const FString RendererName;

protected:
	/** A UI pass for a specific render layer. Instances are stored on the UMovieGraphWidgetRendererBaseNode CDO. */
	struct FMovieGraphUIPass final : public FMovieGraphWidgetPass
	{
		virtual void Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, const FMovieGraphRenderPassLayerData& InLayer) override;
		virtual TSharedPtr<SWidget> GetWidget(UMovieGraphWidgetRendererBaseNode* InNodeThisFrame) override;
		virtual int32 GetCompositingSortOrder() const override;
	};
	
	// UMovieGraphWidgetRendererBaseNode Interface
	UE_API virtual TUniquePtr<FMovieGraphWidgetPass> GeneratePass() override;
	// ~UMovieGraphWidgetRendererBaseNode Interface
};

#undef UE_API
