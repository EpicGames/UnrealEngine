// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphNode.h"

#include "MovieGraphVariableNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMovieJobVariableAssignmentContainer;

/** A node which gets the value of a variable which has been defined on the graph. */
UCLASS(MinimalAPI)
class UMovieGraphVariableNode : public UMovieGraphNode
{
	GENERATED_BODY()

public:
	UMovieGraphVariableNode() = default;

	//~ Begin UObject interface
	UE_API virtual void PostEditImport() override;
	//~ End UObject interface

	UE_API virtual TArray<FMovieGraphPinProperties> GetOutputPinProperties() const override;
	UE_API virtual FString GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext) const override;
	UE_API virtual bool GetResolvedValueForOutputPin(const FName& InPinName, const FMovieGraphTraversalContext* InContext, TObjectPtr<UMovieGraphValueContainer>& OutValueContainer) const override;

	/** Gets the variable that this node represents. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API UMovieGraphVariable* GetVariable() const;

	/** Sets the variable that this node represents. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API void SetVariable(UMovieGraphVariable* InVariable);

	/** Returns true if this node represents a global variable, else false. */
	UFUNCTION(BlueprintCallable, Category = "Movie Graph")
	UE_API bool IsGlobalVariable() const;

	/**
	 * Resolves the provided variable, given a specific traversal context (needed in order to account for job-level overrides). If the variable value
	 * could be resolved, returns true and provides the value in OutValueContainer, otherwise returns false.
	 */
	UE_API static bool GetResolvedVariableValue(const TObjectPtr<UMovieGraphVariable> InGraphVariable, const FMovieGraphTraversalContext* InContext, TObjectPtr<UMovieGraphValueContainer>& OutValueContainer);

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

private:
	UE_API virtual void RegisterDelegates() override;
	
	/** Updates the output pin on the node to match the provided variable. */
	UE_API void UpdateOutputPin(UMovieGraphMember* ChangedVariable) const;

	/** Determines if the job in the given context has a valid, enabled variable assignment for the provided variable. Sets OutVariableAssignment if it does. */
	UE_API static bool ContextHasEnabledAssignmentForVariable(const TObjectPtr<UMovieGraphVariable> InGraphVariable, const FMovieGraphTraversalContext* InContext, TObjectPtr<UMovieJobVariableAssignmentContainer>& OutVariableAssignment);

private:
	/** The underlying graph variable this node represents. */
	UPROPERTY()
	TObjectPtr<UMovieGraphVariable> GraphVariable = nullptr;

	/** The properties for the output pin on this node. */
	UPROPERTY(Transient)
	FMovieGraphPinProperties OutputPin;
};

#undef UE_API
