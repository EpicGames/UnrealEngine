// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/AsyncWork.h"
#include "Async/TaskGraphInterfaces.h"
#include "ImagePixelData.h"
#include "MovieGraphFileOutputNode.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"

#include "MovieGraphVideoOutputNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

namespace MoviePipeline { struct FAudioState; }

namespace MovieRenderGraph
{
	struct IVideoCodecWriter
	{
		/**
		 * The filename without de-duplication numbers, used to match up multiple incoming frames back to the same writer.
		 * We use this when looking for existing writers so that we can avoid the de-duplication numbers perpetually increasing
		 * due to the file existing on disk after the first frame comes in, and then the next one de-duplicating to one more than that.
		 */
		FString StableFileName;

		// This array contains the indexes of each shot that contributed to this writer. This is needed so that after a shot/render, when
		// all we have is the completed blocks of audio samples, we can figure out which blocks go to which video clips. Because of how you can
		// split videos up by either shot, or by the whole sequence, there isn't an obvious mapping, so this array will tell the audio writing
		// system which audio blocks should be sent to the shot.
		struct FLightweightSourceData
		{
			int32 SubmittedFrameCount;
		};
		TMap<int32, FLightweightSourceData> LightweightSourceData;
	};

	namespace Audio
	{
		/**
		 * Resolves per-shot audio recordings from the audio renderer into shot-aligned int16 buffers and invokes
		 * InWriteSampleCallback once per shot, in increasing shot-index order.
		 *
		 * The main submix has ~24 ms of recording latency, so each shot's recording is offset: shot 0 is short by L,
		 * and shot N>0's first L samples are actually shot (N-1)'s tail. This helper redistributes samples from
		 * shot[N+1]'s head onto shot[N]'s tail until each shot matches its expected length, then truncates each shot
		 * to its expected length before passing it to InWriteSampleCallback. The deficit accumulates onto the last
		 * shot, which ends ~24 ms early.
		 *
		 * @param InAudioData              Audio state captured by the audio renderer (provides per-shot FinishedSegments).
		 * @param InLightweightSourceData  Per-shot writer source data (provides SubmittedFrameCount).
		 * @param InOutputFrameRate        Video frame rate used to size each shot's expected sample count.
		 * @param InWriteSampleCallback    Invoked once per shot with the channel-interleaved int16 buffer.
		 */
		UE_API void WriteShotAudioSamples(
			const MoviePipeline::FAudioState& InAudioData,
			const TMap<int32, IVideoCodecWriter::FLightweightSourceData>& InLightweightSourceData,
			const FFrameRate& InOutputFrameRate,
			TFunctionRef<void(TArrayView<int16>)> InWriteSampleCallback);
	}
}

/**
 * A base node for nodes that generate video in the Movie Render Graph.
 */
UCLASS(MinimalAPI, BlueprintType, Abstract)
class UMovieGraphVideoOutputNode : public UMovieGraphFileOutputNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphVideoOutputNode();

protected:
	// The parameters to Initialize_GameThread() have changed a lot -- using a struct as a the only parameter will make future changes easier.
	struct FMovieGraphVideoNodeInitializationContext
	{
		UMovieGraphPipeline* Pipeline;
		TObjectPtr<UMovieGraphEvaluatedConfig> EvaluatedConfig;
		const FMovieGraphTraversalContext* TraversalContext;
		const FMovieGraphPassData* PassData;
		FIntPoint Resolution;
		FString FileName;
		bool bAllowOCIO;
	};
	
	// UMovieGraphFileOutputNode Interface
	UE_API virtual void OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask) override;
	UE_API virtual void OnAllFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) override;
	UE_API virtual void OnAllFramesFinalizedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) override;
	UE_API virtual void OnAllShotFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, const UMoviePipelineExecutorShot* InShot, TObjectPtr<UMovieGraphEvaluatedConfig>& InShotEvaluatedGraph, const bool bFlushToDisk) override;
	UE_API virtual bool IsFinishedWritingToDiskImpl() const override;
	// ~UMovieGraphFileOutputNode Interface

	virtual TUniquePtr<MovieRenderGraph::IVideoCodecWriter> Initialize_GameThread(const FMovieGraphVideoNodeInitializationContext& InInitializationContext)  PURE_VIRTUAL(UMovieGraphVideoOutputNode::Initialize_GameThread, return nullptr; );
	virtual bool Initialize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) PURE_VIRTUAL(UMovieGraphVideoOutputNode::Initialize_EncodeThread, return true;);
	virtual void WriteFrame_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<FMovieGraphPassData>&& InCompositePasses, TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FString& InBranchName) PURE_VIRTUAL(UMovieGraphVideoOutputNode::WriteFrame_EncodeThread);
	virtual void BeginFinalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) PURE_VIRTUAL(UMovieGraphVideoOutputNode::BeginFinalize_EncodeThread);
	virtual void Finalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) PURE_VIRTUAL(UMovieGraphVideoOutputNode::Finalize_EncodeThread);
	virtual const TCHAR* GetFilenameExtension() const PURE_VIRTUAL(UMovieGraphVideoOutputNode::GetFilenameExtension, return TEXT(""););
	virtual bool IsAudioSupported() const PURE_VIRTUAL(UMovieGraphVideoOutputNode::IsAudioSupported, return false;);

private:
	struct FMovieGraphCodecWriterWithPromise
	{
		FMovieGraphCodecWriterWithPromise(TUniquePtr<MovieRenderGraph::IVideoCodecWriter>&& InWriter, TPromise<bool>&& InPromise, UClass* InNodeType);
		
		/** The codec writer. */
		TUniquePtr<MovieRenderGraph::IVideoCodecWriter> CodecWriter;

		/** The promise that is provided to the pipeline that specifies whether or not the writer has finished. */
		TPromise<bool> Promise;
		
		/** The type of node associated with this writer. */
		UClass* NodeType;
	};

	/**
	 * Generates a "stable" and "final" filename for a writer. The stable filename has not been put through a de-duplication procedure (ie, it
	 * might reference an existing file on disk). The final filename is what will be written to disk and will not reference an existing filename
	 * on disk (unless the user has specified that overwriting existing files is ok).
	 */
	UE_API void GetOutputFilePaths(const UMovieGraphPipeline* InPipeline, const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, FMovieGraphPassData& InRenderPassData, const TArray<FMovieGraphPassData>& InCompositedPasses, FString& OutFinalFilePath, FString& OutStableFilePath);

	/**
	 * Generates the writer that is responsible for doing the encoding work. The writer will be added to AllWriters. If there was a problem creating
	 * the writer, nullptr will be returned and an error will be sent to the log.
	 */
	UE_API FMovieGraphCodecWriterWithPromise* GetOrCreateOutputWriter(UMovieGraphPipeline* InPipeline, const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, FMovieGraphPassData& InRenderPassData, const TArray<FMovieGraphPassData>& InCompositedPasses);

private:
	// The pipeline generates many instances of the same node throughout its execution; however, some nodes need to have persistent data throughout the
	// pipeline's lifetime. This static data enables the node to have shared data across instances.
	/** All writers that are currently being run. There is one writer per filename. There might be multiple writers due to multiple passes being written out. */
	UE_API inline static TArray<FMovieGraphCodecWriterWithPromise> AllWriters;

	/** Whether the output encountered any error, like failing to initialize properly. */
	bool bHasError;
};

#undef UE_API
