// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Graph/MovieGraphNode.h"
#include "Misc/FrameRate.h"
#include "MovieGraphWarmUpSettingNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API


UCLASS(MinimalAPI)
class UMovieGraphWarmUpSettingNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()
public:
	UE_API UMovieGraphWarmUpSettingNode();

	// UMovieGraphSettingNode Interface
	UE_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif
	// ~UMovieGraphSettingNode Interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_NumWarmUpFrames : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_LayerWarmUpFrames : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEmulateMotionBlur : 1;

	/** At the start of each shot, how many frames should we run the engine (without writing renders to disk) to warm up various systems. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(EditCondition="bOverride_NumWarmUpFrames"))
	int32 NumWarmUpFrames;

	/**
	 * The number of warm-up frames to run before each render layer's final render, with that layer's
	 * modifiers active. Layer warm-up frames let Lumen's lighting converge to the layer's contents before
	 * rendering the frame that's actually written to disk. Contrast with NumWarmUpFrames, which runs at the
	 * start of the shot rather than per-layer.
	 *
	 * Layer warm-ups are helpful when layer contents change substantially in ways that affect lighting.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(EditCondition="bOverride_LayerWarmUpFrames", UIMin=0, ClampMin=0))
	int32 LayerWarmUpFrames;

	/** 
	If true, we will evaluate frame 0, and then wait NumWarmUpFrames frames, rendering each one as we go. Then we will evaluate frame 1, and then frame 0 again. This emulates motion blur on the first frame (which normally needs data before frame 0). If false, we will "walk" towards the first frame of the shot, starting NumWarmUpFrames before the shot normally starts. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bOverride_bEmulateMotionBlur"))
	bool bEmulateMotionBlur;
};

#undef UE_API
