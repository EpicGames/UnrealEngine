// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Nodes/MovieGraphRenderPassNode.h"

#include "Styling/AppStyle.h"
#include "MovieGraphWidgetRendererBaseNode.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
class FWidgetRenderer;
class SVirtualWindow;
class SWidget;
class UMovieGraphDefaultRenderer;
class UTextureRenderTarget2D;
struct FMovieGraphRenderPassLayerData;
struct FMovieGraphRenderPassSetupData;

/** Base node containing common logic for nodes that render widgets. */
UCLASS(MinimalAPI, Abstract)
class UMovieGraphWidgetRendererBaseNode : public UMovieGraphRenderPassNode
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCompositeOntoFinalImage : 1;

	/** If true, the pass will be composited onto each render. Does not apply to multi-layer EXR files. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(EditCondition="bOverride_bCompositeOntoFinalImage"), Category="Widget Settings")
	bool bCompositeOntoFinalImage = true;

protected:
	/** A render pass for a specific render layer, generated by the UMovieGraphWidgetRendererBaseNode CDO. */
	struct FMovieGraphWidgetPass
	{
		virtual ~FMovieGraphWidgetPass() = default;

		virtual void Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, const FMovieGraphRenderPassLayerData& InLayer);
		virtual void Teardown();
		virtual TSharedPtr<SWidget> GetWidget(UMovieGraphWidgetRendererBaseNode* InNodeThisFrame) = 0;
		virtual void Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData);
		virtual void GatherOutputPasses(TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const;
		virtual int32 GetCompositingSortOrder() const = 0;
		virtual UMovieGraphWidgetRendererBaseNode* GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const;

	protected:
		FMovieGraphRenderPassLayerData LayerData;

		/** Unique identifier passed in GatherOutputPasses and with each render that identifies the data produced by this renderer. */
		FMovieGraphRenderDataIdentifier RenderDataIdentifier;

		/** The renderer associated with this pass. */
		TWeakObjectPtr<UMovieGraphDefaultRenderer> Renderer;
	};

	/** Widget renderer shared with all FMovieGraphWidgetPass instances. */
	TSharedPtr<FWidgetRenderer> WidgetRenderer;

	/** Render passes generated by GeneratePass() calls on the CDO. */
	TArray<TUniquePtr<FMovieGraphWidgetPass>> CurrentInstances;

protected:
	/** Generate a new pass that the widget renderer will use. */
	virtual TUniquePtr<FMovieGraphWidgetPass> GeneratePass() PURE_VIRTUAL(UMovieGraphWidgetRendererBaseNode::GeneratePass, return nullptr;);

	/** Gets the file name that the render will write to if not performing a composite. */
	UE_API virtual FString GetFileNameFormatOverride() const;

	/** Gets the layer name this render will use when writing to multi-layer formats. An empty string means no override, and the default will be used . */
	UE_API virtual FString GetLayerNameOverride() const;
	
	// UMovieGraphRenderPassNode Interface
	UE_API virtual void SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData) override;
	UE_API virtual void TeardownImpl() override;
	UE_API virtual void RenderImpl(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) override;
	UE_API virtual void GatherOutputPassesImpl(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const override;
	// ~UMovieGraphRenderPassNode Interface

private:
	/** Virtual window instances shared with all FMovieGraphWidgetPass instances, keyed by resolution. */
	TMap<const FIntPoint, TSharedPtr<SVirtualWindow>> SharedVirtualWindows;

private:
	/** Gets an existing virtual window with the specified resolution, or creates a new one if one does not exist yet. */
	UE_API TSharedPtr<SVirtualWindow> GetOrCreateVirtualWindow(const FIntPoint& InResolution);
};

#undef UE_API
