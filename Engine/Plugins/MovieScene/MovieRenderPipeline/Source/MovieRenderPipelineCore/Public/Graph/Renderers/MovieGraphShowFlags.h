// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "ShowFlags.h"
#include "Graph/MovieGraphConfig.h"
#include "MovieGraphShowFlags.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
 * Stores show flag enable/disable state, as well as per-flag override state.
 *
 * Here, enable/disable state refers to the actual state of the show flag itself (ie, whether it is turned on or off).
 * This is the value that will be applied to renders. "Override" state is whether the flag has been marked as overridden
 * in the UI (ie, whether it is a value that graph traversal should respect). Flags must be marked as overridden in order
 * for their values to deviate from the default.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphShowFlags : public UObject, public IMovieGraphTraversableObject
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphShowFlags();

	/** Gets a copy of the show flags. If they need to be modified, use one of the Set*() methods. */
	UE_API FEngineShowFlags GetShowFlags() const;

	/** Gets the enable/disable state for a specific show flag. */
	UE_API bool IsShowFlagEnabled(const uint32 ShowFlagIndex) const;

	/** Sets the enable/disable state for a specific show flag. The flag must be marked as overridden for this to have an effect. */
	UE_API void SetShowFlagEnabled(const uint32 ShowFlagIndex, const bool bIsShowFlagEnabled);

	/** Gets the override state for a specific show flag. */
	UE_API bool IsShowFlagOverridden(const uint32 ShowFlagIndex) const;

	/** Sets the override state for a specific show flag. */
	UE_API void SetShowFlagOverridden(const uint32 ShowFlagIndex, const bool bIsOverridden);

	/** Gets the indices of all show flags that have been overridden. */
	UE_API const TSet<uint32>& GetOverriddenShowFlags() const;

	/** Determines if the provided show flag is set to a non-default value. The override state does not impact the value returned. */
	UE_API bool IsShowFlagSetToDefaultValue(const uint32 ShowFlagIndex) const;

	/** Reverts the provided show flag to its default value. Does not impact the override state for the flag. */
	UE_API void RevertShowFlagToDefaultValue(const uint32 ShowFlagIndex);

	/**
	 * Applies a default value to the show flags. However, the default will not be applied if a user has already
	 * specified an override for the flag. This should generally only be called from renderers in order for them to
	 * specify defaults that are relevant.
	 */
	UE_API void ApplyDefaultShowFlagValue(const uint32 ShowFlagIndex, const bool bShowFlagState);

	// IMovieGraphTraversableObject interface
	UE_API virtual void Merge(const IMovieGraphTraversableObject* InSourceObject) override;
	UE_API virtual TArray<TPair<FString, FString>> GetMergedProperties() const override;
	// ~IMovieGraphTraversableObject interface

private:
	/** Copies the overrides from ShowFlagEnableState to InEngineShowFlags. */
	UE_API void CopyOverridesToShowFlags(FEngineShowFlags& InEngineShowFlags) const;

private:
	/**
	 * The show flags which have been marked as overridden in the UI (note this is NOT the enabled/disabled state of the
	 * flag itself).
	 */
	UPROPERTY()
	TSet<uint32> OverriddenShowFlags;

	/**
	 * If the flag has been marked as overridden, this stores the enable/disable state of the flag. Key = show flag
	 * index, value = enable/disable state of the flag.
	 */
	UPROPERTY()
	TMap<uint32, bool> ShowFlagEnableState;

	/**
	 * The default show flag state. Since FEngineShowFlags itself cannot be serialized (it is not a USTRUCT), we need to
	 * keep track of sparse overrides (ShowFlagEnableState) on top of the base state that these flags are initialized
	 * with (ESFIM_Game, plus any additional flags that a specialized renderer specifies).
	 */
	FEngineShowFlags ShowFlags;
};

#undef UE_API
