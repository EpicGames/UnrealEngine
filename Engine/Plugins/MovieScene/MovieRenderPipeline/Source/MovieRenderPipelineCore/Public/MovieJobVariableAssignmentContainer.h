// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Graph/MovieGraphCommon.h"
#include "Graph/MovieGraphValueContainer.h"	// For GetOptionalValue()
#include "StructUtils/PropertyBag.h"

#include "MovieJobVariableAssignmentContainer.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMovieGraphConfig;
class UMovieGraphVariable;

/**
 * Holds a group of properties which override variable values on the job's associated graph (if any). Overrides are not
 * added manually. Instead, UpdateGraphVariableOverrides() should be called which will update, add, or remove overrides
 * as appropriate. After the update, overrides can have their values retrieved and set.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieJobVariableAssignmentContainer : public UObject
{
	GENERATED_BODY()

public:
	UMovieJobVariableAssignmentContainer() = default;

	/**
	 * Sets the graph config associated with the variable assignments.
	 */
	UFUNCTION(BlueprintCallable, Category="Graph")
	UE_API void SetGraphConfig(const TSoftObjectPtr<UMovieGraphConfig>& InGraphConfig);

	/** Gets the graph that is associated with this container. */
	UFUNCTION(BlueprintCallable, Category="Graph")
	UE_API TSoftObjectPtr<UMovieGraphConfig> GetGraphConfig() const;

	/** Gets the number of variable assignments present in this container. Assignments that are disabled are counted. */
	UE_API uint32 GetNumAssignments() const;

	/** Gets the bool value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueBool(const UMovieGraphVariable* InGraphVariable, bool& bOutValue) const;

	/** Gets the byte value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueByte(const UMovieGraphVariable* InGraphVariable, uint8& OutValue) const;

	/** Gets the int32 value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueInt32(const UMovieGraphVariable* InGraphVariable, int32& OutValue) const;

	/** Gets the int64 value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueInt64(const UMovieGraphVariable* InGraphVariable, int64& OutValue) const;

	/** Gets the float value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueFloat(const UMovieGraphVariable* InGraphVariable, float& OutValue) const;

	/** Gets the double value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueDouble(const UMovieGraphVariable* InGraphVariable, double& OutValue) const;

	/** Gets the FName value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueName(const UMovieGraphVariable* InGraphVariable, FName& OutValue) const;

	/** Gets the FString value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueString(const UMovieGraphVariable* InGraphVariable, FString& OutValue) const;

	/** Gets the FText value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueText(const UMovieGraphVariable* InGraphVariable, FText& OutValue) const;

	/** Gets the enum value (for a specific enum) of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueEnum(const UMovieGraphVariable* InGraphVariable, uint8& OutValue, const UEnum* RequestedEnum = nullptr) const;

	/** Gets the struct value (for a specific struct) of the specified property. Returns true on success, else false. */
	UE_API bool GetValueStruct(const UMovieGraphVariable* InGraphVariable, FStructView& OutValue, const UScriptStruct* RequestedStruct = nullptr) const;

	/** Gets the object value (for a specific class) of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueObject(const UMovieGraphVariable* InGraphVariable, UObject*& OutValue, const UClass* RequestedClass = nullptr) const;

	/** Gets the UClass value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool GetValueClass(const UMovieGraphVariable* InGraphVariable, UClass*& OutValue) const;

	/** Gets the serialized string value of the specified property. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API FString GetValueSerializedString(const UMovieGraphVariable* InGraphVariable);

	/** Gets the enum value of the specified property. Returns true on success, else false. */
	template <typename T>
	bool GetValueEnum(const UMovieGraphVariable* InGraphVariable, T& OutValue) const
	{
		TValueOrError<T, EPropertyBagResult> Result = Value.GetValueEnum<T>(ConvertVariableToInternalName(InGraphVariable));
		return UE::MovieGraph::Private::GetOptionalValue<T>(Result, OutValue);
	}

	/** Gets the struct value of the specified property. Returns true on success, else false. */
	template <typename T>
	bool GetValueStruct(const UMovieGraphVariable* InGraphVariable, T*& OutValue) const
	{
		TValueOrError<T*, EPropertyBagResult> Result = Value.GetValueStruct<T*>(ConvertVariableToInternalName(InGraphVariable));
		return UE::MovieGraph::Private::GetOptionalValue<T*>(Result, OutValue);
	}

	/** Gets the object value of the specified property. Returns true on success, else false. */
	template <typename T>
	bool GetValueObject(const UMovieGraphVariable* InGraphVariable, T*& OutValue) const
	{
		TValueOrError<T*, EPropertyBagResult> Result = Value.GetValueObject<T*>(ConvertVariableToInternalName(InGraphVariable));
		return UE::MovieGraph::Private::GetOptionalValue<T*>(Result, OutValue);
	}

	/** Sets the bool value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueBool(const UMovieGraphVariable* InGraphVariable, const bool bInValue);

	/** Sets the byte value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueByte(const UMovieGraphVariable* InGraphVariable, const uint8 InValue);

	/** Sets the int32 value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueInt32(const UMovieGraphVariable* InGraphVariable, const int32 InValue);

	/** Sets the int64 value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueInt64(const UMovieGraphVariable* InGraphVariable, const int64 InValue);

	/** Sets the float value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueFloat(const UMovieGraphVariable* InGraphVariable, const float InValue);

	/** Sets the double value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueDouble(const UMovieGraphVariable* InGraphVariable, const double InValue);

	/** Sets the FName value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueName(const UMovieGraphVariable* InGraphVariable, const FName InValue);

	/** Sets the FString value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueString(const UMovieGraphVariable* InGraphVariable, const FString& InValue);

	/** Sets the FText value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueText(const UMovieGraphVariable* InGraphVariable, const FText& InValue);

	/** Sets the enum value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueEnum(const UMovieGraphVariable* InGraphVariable, const uint8 InValue, const UEnum* Enum);

	/** Sets the struct value of the specified property. Returns true on success, else false. */
	UE_API bool SetValueStruct(const UMovieGraphVariable* InGraphVariable, FConstStructView InValue);

	/** Sets the object value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueObject(const UMovieGraphVariable* InGraphVariable, UObject* InValue);

	/** Sets the class value of the specified property. Returns true on success, else false. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueClass(const UMovieGraphVariable* InGraphVariable, UClass* InValue);

	/** Sets the serialized value of this member. The string should be the serialized representation of the value. Returns true on success, else false.*/
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API bool SetValueSerializedString(const UMovieGraphVariable* InGraphVariable, const FString& NewValue);

	/** Sets the enum value of the specified property. Returns true on success, else false. */
	template <typename T>
	bool SetValueEnum(const UMovieGraphVariable* InGraphVariable, const T InValue)
	{
		return Value.SetValueEnum<T>(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
	}

	/** Sets the struct value of the specified property. Returns true on success, else false. */
	template <typename T>
	bool SetValueStruct(const UMovieGraphVariable* InGraphVariable, const T& InValue)
	{
		return Value.SetValueStruct<T>(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
	}

	/** Sets the object value of the specified property. Returns true on success, else false. */
	template <typename T>
	bool SetValueObject(const UMovieGraphVariable* InGraphVariable, T* InValue)
	{
		return Value.SetValueObject<T>(ConvertVariableToInternalName(InGraphVariable), InValue) == EPropertyBagResult::Success;
	}

	/** Gets the type of the value stored in the specified property. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API EMovieGraphValueType GetValueType(const UMovieGraphVariable* InGraphVariable) const;

	/** Gets the object that defines the enum, struct, or class stored in the specified property. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API const UObject* GetValueTypeObject(const UMovieGraphVariable* InGraphVariable) const;

	/** Gets the container type of the stored value in the specified property. */
	UFUNCTION(BlueprintCallable, Category="Value")
	UE_API EMovieGraphContainerType GetValueContainerType(const UMovieGraphVariable* InGraphVariable) const;

	/**
	 * Gets a value container object rather than a strongly-typed value. Useful if the type of the value is not known
	 * ahead of time. If no property for the specified variable exists, OutValueContainer will not be modified.
	 */
	UE_API bool GetValueContainer(const UMovieGraphVariable* InGraphVariable, TObjectPtr<UMovieGraphValueContainer>& OutValueContainer);

	/**
	 * Updates an existing variable assignment for the provided graph variable to a new enable state, or adds a new
	 * assignment and updates its enable state. Returns true on success, else false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Graph")
	UE_API bool SetVariableAssignmentEnableState(const UMovieGraphVariable* InGraphVariable, bool bIsEnabled);

	/**
	 * Gets the enable state of the variable assignment for the provided graph variable. The enable state is provided
	 * via bOutIsEnabled. Returns true if an enable state was set on the variable and bOutIsEnabled was changed, else false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Graph")
	UE_API bool GetVariableAssignmentEnableState(const UMovieGraphVariable* InGraphVariable, bool& bOutIsEnabled);

#if WITH_EDITOR
	/**
	 * Updates the stored variable overrides to reflect the graph preset. Existing overrides will be updated to match
	 * the graph variable name, value type, object type, and container type. Additionally, stale overrides that have no
	 * corresponding graph variable will be removed, and overrides will be created for graph variables which do not have
	 * existing overrides.
	 */
	UFUNCTION(BlueprintCallable, Category = "Graph")
	UE_API void UpdateGraphVariableOverrides();
#endif

protected:
	/**
	 * Add a variable assignment for the graph. Returns true if the variable was added or it already exists,
	 * else false.
	 */
	UE_API bool AddVariableAssignment(const UMovieGraphVariable* InGraphVariable);
	
	/**
	 * Finds a variable assignment for the provided variable, or adds one if one does not already exist (and
	 * bAddIfNotExists is set to true). Returns nullptr if the operation failed. 
	 */
	UE_API bool FindOrGenerateVariableOverride(const UMovieGraphVariable* InGraphVariable, FPropertyBagPropertyDesc* OutPropDesc = nullptr,
		FPropertyBagPropertyDesc* OutEditConditionPropDesc = nullptr, bool bGenerateIfNotExists = true);

	/** Generates a variable override for the provided variable, as well as the associated EditCondition for it. */
	UE_API bool GenerateVariableOverride(const UMovieGraphVariable* InGraphVariable, FPropertyBagPropertyDesc* OutPropDesc = nullptr,
		FPropertyBagPropertyDesc* OutEditConditionPropDesc = nullptr);

	/** Given a graph variable, retrieve the hidden internal name used by our actual property bag. */
	UE_API FName ConvertVariableToInternalName(const UMovieGraphVariable* InGraphVariable) const;

private:
	/* The metadata key on properties that is associated with a value which identifies a graph variable (by GUID). */
	static inline FName VariableGuidMetaDataKey = FName(TEXT("VariableGUID"));
	
	/* The metadata key on properties that is associated with a tooltip value. */
	static inline FName ToolTipMetaDataKey = FName(TEXT("ToolTip"));

	/* The metadata key on properties that is associated with the variable's category. */
	static inline FName CategoryMetaDataKey = FName(TEXT("Category"));

	/* The metadata key on properties that is associated with the variable's display name. */
	static inline FName DisplayNameMetaDataKey = FName(TEXT("DisplayName"));
	
	/* The metadata key on property bag properties that allows them to show up grouped in the details panel. */
	static inline FName EnableCategoriesMetaDataKey = FName(TEXT("EnableCategories"));
	
	/** The properties managed by this object. */
	UPROPERTY(EditAnywhere, meta=(ShowOnlyInnerProperties, FixedLayout), Category = "Value")
	FInstancedPropertyBag Value;

	/** The graph preset associated with the variable overrides. */
	UPROPERTY()
	TSoftObjectPtr<UMovieGraphConfig> GraphPreset;
};

#undef UE_API
