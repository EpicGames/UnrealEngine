// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "HAL/ThreadSafeBool.h"
#include "Engine/EngineCustomTimeStep.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineBlueprintLibrary.h"
#if WITH_EDITOR
#include "MovieSceneExportMetadata.h"
#endif
#include "MovieSceneTimeController.h"
#include "Async/Future.h"
#include "MoviePipelineBase.h"
#include "MoviePipeline.generated.h"

// Forward Declares
class UMoviePipelinePrimaryConfig;
class ULevelSequence;
class UMovieSceneSequencePlayer;
class UMoviePipelineCustomTimeStep;
class UEngineCustomTimeStep;
class ALevelSequenceActor;
class UMovieRenderDebugWidget;
class FMovieRenderViewport;
class FMovieRenderViewportClient;
struct FImagePixelPipe;
struct FMoviePipelineTimeController;
class FMoviePipelineOutputMerger;
class IImageWriteQueue;
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;
class UMoviePipelineSetting;
class UTexture;

typedef TTuple<TFuture<bool>, MoviePipeline::FMoviePipelineOutputFutureData> FMoviePipelineOutputFuture;

DECLARE_MULTICAST_DELEGATE_TwoParams(FMoviePipelineFinishedNative, UMoviePipeline*, bool);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMoviePipelineFinished, UMoviePipeline*, MoviePipeline, bool, bFatalError);

UCLASS(MinimalAPI, Blueprintable)
class UMoviePipeline : public UMoviePipelineBase
{
	GENERATED_BODY()
	
public:
	MOVIERENDERPIPELINECORE_API UMoviePipeline();

	/** 
	* Initialize the movie pipeline with the specified settings. This kicks off the rendering process. 
	* @param InJob	- This contains settings and sequence to render this Movie Pipeline with.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	MOVIERENDERPIPELINECORE_API void Initialize(UMoviePipelineExecutorJob* InJob);

	// UMoviePipelineBase Interface
	MOVIERENDERPIPELINECORE_API virtual void RequestShutdownImpl(bool bIsError) override;
	MOVIERENDERPIPELINECORE_API virtual void ShutdownImpl(bool bError ) override;
	virtual bool IsShutdownRequestedImpl() const override { return bShutdownRequested; }
	virtual EMovieRenderPipelineState GetPipelineStateImpl() const override { return PipelineState; }
	virtual bool IsPostShotCallbackNeeded() const override { return IsFlushDiskWritesPerShot(); }
	// ~UMoviePipelineBase Interface

	/**
	* Returns the time this movie pipeline was initialized at.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	FDateTime GetInitializationTime() const { return InitializationTime; }

	/** 
	* The offset that should be applied to the GetInitializationTime() when generating
	* the {time} related filename tokens. GetInitializationTime() is in UTC so this is
	* either zero (if you called SetInitializationTime) or your offset from UTC.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	FTimespan GetInitializationTimeOffset() const { return InitializationTimeOffset; }

	/**
	* Override the time this movie pipeline was initialized at. This can be used for render farms
	* to ensure that jobs on all machines use the same date/time instead of each calculating it locally.
	* Clears the auto-calculated InitializationTimeOffset, meaning time tokens will be written in UTC.
	*
	* Needs to be called after ::Initialize(...)
	*
	* @param InDateTime - Expected to be in UTC timezone.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	void SetInitializationTime(const FDateTime& InDateTime) { InitializationTime = InDateTime; InitializationTimeOffset = FTimespan(); }

	/**
	* Get the Primary Configuration used to render this shot. This contains the global settings for the shot, as well as per-shot
	* configurations which can contain their own settings.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	MOVIERENDERPIPELINECORE_API UMoviePipelinePrimaryConfig* GetPipelinePrimaryConfig() const;
	
public:
	ULevelSequence* GetTargetSequence() const { return TargetSequence; }

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UTexture* GetPreviewTexture() const { return PreviewTexture; }

	void SetPreviewTexture(UTexture* InTexture) { PreviewTexture = InTexture; }

	const TArray<UMoviePipelineExecutorShot*>& GetActiveShotList() const { return ActiveShotList; }

	int32 GetCurrentShotIndex() const { return CurrentShotIndex; }
	const FMoviePipelineFrameOutputState& GetOutputState() const { return CachedOutputState; }

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelineExecutorJob* GetCurrentJob() const { return CurrentJob; }

	MOVIERENDERPIPELINECORE_API FMoviePipelineOutputData GetOutputDataParams();

	MOVIERENDERPIPELINECORE_API void GetSidecarCameraData(UMoviePipelineExecutorShot* InShot, int32 InCameraIndex, FMinimalViewInfo& OutViewInfo, class UCameraComponent** OutCameraComponent) const;
	MOVIERENDERPIPELINECORE_API bool GetSidecarCameraViewPoints(UMoviePipelineExecutorShot* InShot, TArray<FVector>& OutSidecarViewLocations, TArray<FRotator>& OutSidecarViewRotations) const;

	/** Gets any cached overscan for the specified camera, or 0 if no cached overscan was found */
	MOVIERENDERPIPELINECORE_API float GetCachedCameraOverscan(int32 InCameraIndex) const;

	/** Gets whether there is a cached overscan value for the specified camera */
	MOVIERENDERPIPELINECORE_API bool HasCachedCameraOverscan(int32 InCameraIndex) const;

	/** Caches the provided overscan value for the specified camera */
	MOVIERENDERPIPELINECORE_API void CacheCameraOverscan(int32 InCameraIndex, float InCameraOverscan);

	/** Outputs a warning message regarding animated overscan to the MRQ log if one has not already been output  */
	UE_DEPRECATED(5.6, "WarnAboutAnimatedOverscan is no longer used; animated overscan is supported in 5.6")
	MOVIERENDERPIPELINECORE_API void WarnAboutAnimatedOverscan(float InInitialOverscan);
	
#if WITH_EDITOR
	const FMovieSceneExportMetadata& GetOutputMetadata() const { return OutputMetadata; }
#endif

public:
#if WITH_EDITOR
	MOVIERENDERPIPELINECORE_API void AddFrameToOutputMetadata(const FString& ClipName, const FString& ImageSequenceFileName, const FMoviePipelineFrameOutputState& FrameOutputState, const FString& Extension, const bool bHasAlpha);
#endif
	MOVIERENDERPIPELINECORE_API void AddOutputFuture(TFuture<bool>&& OutputFuture, const MoviePipeline::FMoviePipelineOutputFutureData& InData);

	MOVIERENDERPIPELINECORE_API void ProcessOutstandingFinishedFrames();
	MOVIERENDERPIPELINECORE_API void ProcessOutstandingFutures();
	MOVIERENDERPIPELINECORE_API void OnSampleRendered(TUniquePtr<FImagePixelData>&& OutputSample);
	const MoviePipeline::FAudioState& GetAudioState() const { return AudioState; }
	void SetFlushDiskWritesPerShot(const bool bFlushWrites) { bFlushDiskWritesPerShot = bFlushWrites; }
	bool IsFlushDiskWritesPerShot() const { return bFlushDiskWritesPerShot; }
public:
	template<typename SettingType>
	SettingType* FindOrAddSettingForShot(const UMoviePipelineExecutorShot* InShot) const
	{
		return (SettingType*)UMoviePipelineBlueprintLibrary::FindOrGetDefaultSettingForShot(SettingType::StaticClass(), GetPipelinePrimaryConfig(), InShot);
	}

	template<typename SettingType>
	TArray<SettingType*> FindSettingsForShot(const UMoviePipelineExecutorShot* InShot) const
	{
		TArray<SettingType*> Result;
		TArray<UMoviePipelineSetting*> FoundSettings = FindSettingsForShot(SettingType::StaticClass(), InShot);
		Result.Reserve(FoundSettings.Num());
		for (UMoviePipelineSetting* Setting : FoundSettings)
		{
			Result.Add(CastChecked<SettingType>(Setting));
		}
		return Result;
	}

	MOVIERENDERPIPELINECORE_API TArray<UMoviePipelineSetting*> FindSettingsForShot(TSubclassOf<UMoviePipelineSetting> InSetting, const UMoviePipelineExecutorShot* InShot) const;

	/**
	* Resolves the provided InFormatString by converting {format_strings} into settings provided by the primary config.
	* @param	InFormatString		A format string (in the form of "{format_key1}_{format_key2}") to resolve.
	* @param	InFormatOverrides	A series of Key/Value pairs to override particular format keys. Useful for things that
	*								change based on the caller such as filename extensions.
	* @return	OutFinalPath			The final filepath based on a combination of the format string, the format overrides, and the current output state.
	* @return	OutFinalFormatArgs	The format arguments that were actually used to fill the format string (including file metadata)
	*
	* @param	InOutputState		(optional) The output state for frame information.
	* @param	InFrameNumberOffset	(optional) Frame offset of the frame we want the filename for, if not the current frame
	*								as specified in InOutputState
	*/
	MOVIERENDERPIPELINECORE_API void ResolveFilenameFormatArguments(const FString& InFormatString, const TMap<FString, FString>& InFormatOverrides, FString& OutFinalPath, FMoviePipelineFormatArgs& OutFinalFormatArgs, const FMoviePipelineFrameOutputState* InOutputState=nullptr, const int32 InFrameNumberOffset=0) const;

	/**
	* Allows initialization of some viewport-related arguments that aren't related to the job. Needs to
	* be called before the Initialize function. Optional.
	*/
	void SetViewportInitArgs(const UE::MoviePipeline::FViewportArgs& InArgs) { ViewportInitArgs = InArgs; }

protected:
	/**
	* This function should be called by the Executor when execution has finished (this should still be called in the event of an error)
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	MOVIERENDERPIPELINECORE_API virtual void OnMoviePipelineFinishedImpl();
private:

	/** Instantiate our Debug UI Widget and initialize it to ourself. */
	MOVIERENDERPIPELINECORE_API void LoadDebugWidget();
	
	/** Called before the Engine ticks for the given frame. We use this to calculate delta times that the frame should use. */
	MOVIERENDERPIPELINECORE_API void OnEngineTickBeginFrame();
	
	/** Called after the Engine has ticked for a given frame. Everything in the world has been updated by now so we can submit things to render. */
	MOVIERENDERPIPELINECORE_API void OnEngineTickEndFrame();

	MOVIERENDERPIPELINECORE_API void ValidateSequenceAndSettings() const;


	/** Runs the per-tick logic when doing the ProducingFrames state. */
	MOVIERENDERPIPELINECORE_API void TickProducingFrames();

	MOVIERENDERPIPELINECORE_API void ProcessEndOfCameraCut(UMoviePipelineExecutorShot* InCameraCut);


	/** Called once when first moving to the Finalize state. */
	MOVIERENDERPIPELINECORE_API void BeginFinalize();
	/** Called once when first moving to the Export state. */
	MOVIERENDERPIPELINECORE_API void BeginExport();

	/** Attempts to start an Unreal Insights capture to a file on disk adjacent to the movie output. */
	MOVIERENDERPIPELINECORE_API void StartUnrealInsightsCapture();
	/** Attempts to stop an already started Unreal Insights capture. */
	MOVIERENDERPIPELINECORE_API void StopUnrealInsightsCapture();

	/** 
	* Runs the per-tick logic when doing the Finalize state.
	*
	* @param bInForceFinish		If true, this function will not return until all Output Containers say they have finalized.
	*/
	MOVIERENDERPIPELINECORE_API void TickFinalizeOutputContainers(const bool bInForceFinish);
	/** 
	* Runs the per-tick logic when doing the Export state. This is spread over multiple ticks to allow non-blocking background 
	* processes (such as extra encoding).
	*
	* @param bInForceFinish		If true, this function will not return until all exports say they have finished.
	*/
	MOVIERENDERPIPELINECORE_API void TickPostFinalizeExport(const bool bInForceFinish);

	/** Return true if we should early out of the TickProducingFrames function. Decrements the remaining number of steps when false. */
	MOVIERENDERPIPELINECORE_API bool DebugFrameStepPreTick();
	/** Returns true if we are idling because of debug frame stepping. */
	MOVIERENDERPIPELINECORE_API bool IsDebugFrameStepIdling() const;

	/** Debugging/Information. Don't hinge any logic on this as it will get called multiple times per frame in some cases. */
	MOVIERENDERPIPELINECORE_API void OnSequenceEvaluated(const UMovieSceneSequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime);

	/** Set up per-shot state for the specific shot, tearing down old state (if it exists) */
	MOVIERENDERPIPELINECORE_API void InitializeShot(UMoviePipelineExecutorShot* InShot);
	MOVIERENDERPIPELINECORE_API void TeardownShot(UMoviePipelineExecutorShot* InShot);

	/** Initialize the rendering pipeline for the given shot. This should not get called if rendering work is still in progress for a previous shot. */
	MOVIERENDERPIPELINECORE_API void SetupRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot);
	/** Deinitialize the rendering pipeline for the given shot. */
	MOVIERENDERPIPELINECORE_API void TeardownRenderingPipelineForShot(UMoviePipelineExecutorShot* InShot);

	/** Flush any async resources in the engine that need to be finalized before submitting anything to the GPU, ie: Streaming Levels and Shaders */
	MOVIERENDERPIPELINECORE_API void FlushAsyncEngineSystems();

	/** If the log verbosity is high enough, prints out the files specified in the shot output data. */
	MOVIERENDERPIPELINECORE_API void PrintVerboseLogForFiles(const TArray<FMoviePipelineShotOutputData>& InOutputData) const;

	/** Tell our submixes to start capturing the data they are generating. Should only be called once output frames are being produced. */
	MOVIERENDERPIPELINECORE_API void StartAudioRecording();
	/** Tell our submixes to stop capturing the data, and then store a copy of it. */
	MOVIERENDERPIPELINECORE_API void StopAudioRecording();
	/** Attempt to process the audio thread work. This is complicated by our non-linear time steps */
	MOVIERENDERPIPELINECORE_API void ProcessAudioTick();
	MOVIERENDERPIPELINECORE_API void SetupAudioRendering();
	MOVIERENDERPIPELINECORE_API void TeardownAudioRendering();
	
	/** 
	* Renders the next frame in the Pipeline. This updates/ticks all scene view render states
	* and produces data. This may not result in an output frame due to multiple renders 
	* accumulating together to produce an output. frame.
	* Should not be called if we're idling (debug), not initialized yet, or finalizing/exporting. 
	*/
	MOVIERENDERPIPELINECORE_API void RenderFrame();

	/** Allow any Settings to modify the (already duplicated) sequence. This allows inserting automatic pre-roll, etc. */
	MOVIERENDERPIPELINECORE_API void ModifySequenceViaExtensions(ULevelSequence* InSequence);

	/**
	* Should the Progress UI be visible on the player's screen?
	*/
	MOVIERENDERPIPELINECORE_API void SetProgressWidgetVisible(bool bVisible);

	/** Returns list of render passes for a given shot */
	MOVIERENDERPIPELINECORE_API TArray<UMoviePipelineRenderPass*> GetAllRenderPasses(const UMoviePipelineExecutorShot* InShot);


private:
	/** Iterates through the changes we've made to a shot and applies the original settings. */
	MOVIERENDERPIPELINECORE_API void RestoreTargetSequenceToOriginalState();

	/** Initialize a new Level Sequence Actor to evaluate our target sequence. Disables any existing Level Sequences pointed at our original sequence. */
	MOVIERENDERPIPELINECORE_API void InitializeLevelSequenceActor();

	/** This builds the shot list from the target sequence, and expands Playback Bounds to cover any future evaluation we may need. */
	MOVIERENDERPIPELINECORE_API void BuildShotListFromSequence();

	/** 
	* Modifies the TargetSequence to ensure that only the specified Shot has it's associated Cinematic Shot Section enabled.
	* This way when Handle Frames are enabled and the sections are expanded, we don't end up evaluating the previous shot. 
	*/
	MOVIERENDERPIPELINECORE_API void SetSoloShot(UMoviePipelineExecutorShot* InShot);

	/* Expands the specified shot (and contained camera cuts)'s ranges for the given settings. */
	MOVIERENDERPIPELINECORE_API void ExpandShot(UMoviePipelineExecutorShot* InShot, const int32 InNumHandleFrames, const bool bIsPrePass);

	/** Calculates lots of useful numbers used in timing based off of the current shot. These are constant for a given shot. */
	MOVIERENDERPIPELINECORE_API MoviePipeline::FFrameConstantMetrics CalculateShotFrameMetrics(const UMoviePipelineExecutorShot* InShot) const;

	/** It can be useful to know where the data we're generating was relative to the original Timeline, so this calculates that. */
	MOVIERENDERPIPELINECORE_API void CalculateFrameNumbersForOutputState(const MoviePipeline::FFrameConstantMetrics& InFrameMetrics, const UMoviePipelineExecutorShot* InCameraCut, FMoviePipelineFrameOutputState& InOutOutputState) const;

	/** Handles transitioning between states, preventing reentrancy. Normal state flow should be respected, does not handle arbitrary x to y transitions. */
	MOVIERENDERPIPELINECORE_API void TransitionToState(const EMovieRenderPipelineState InNewState);
private:
	/** Custom TimeStep used to drive the engine while rendering. */
	UPROPERTY(Transient, Instanced)
	TObjectPtr<UMoviePipelineCustomTimeStep> CustomTimeStep;

	/** Custom Time Controller for the Sequence Player, used to match Custom TimeStep without any floating point accumulation errors. */
	TSharedPtr<FMoviePipelineTimeController> CustomSequenceTimeController;

	/** Hold a reference to the existing custom time step (if any) so we can restore it after we're done using our custom one. */
	UPROPERTY(Transient)
	TObjectPtr<UEngineCustomTimeStep> CachedPrevCustomTimeStep;

	/** This is our duplicated sequence that we're rendering. This will get modified throughout the rendering process. */
	UPROPERTY(Transient)
	TObjectPtr<ULevelSequence> TargetSequence;

	/** The Level Sequence Actor we spawned to play our TargetSequence. */
	UPROPERTY(Transient)
	TObjectPtr<ALevelSequenceActor> LevelSequenceActor;

	/** The Debug UI Widget that is spawned and placed on the player UI */
	UPROPERTY(Transient)
	TObjectPtr<UMovieRenderDebugWidget> DebugWidget;

	UPROPERTY(Transient)
	TObjectPtr<UTexture> PreviewTexture;

	/** A list of all of the shots we are going to render out from this sequence. */
	TArray<UMoviePipelineExecutorShot*> ActiveShotList;

	/** What state of the overall flow are we in? See enum for specifics. */
	EMovieRenderPipelineState PipelineState;

	/** What is the index of the shot we are working on. -1 if not initialized, may be greater than ShotList.Num() if we've reached the end. */
	int32 CurrentShotIndex;

	/** The time (in UTC) that Initialize was called. Used to track elapsed time. */
	FDateTime InitializationTime;

	FMoviePipelineFrameOutputState CachedOutputState;

	/** Number of final output frames rendered for the current shot. */
	int32 CurrentShotRenderedFrameCount = 0;

	MoviePipeline::FAudioState AudioState;

	/** Cached state of GAreScreenMessagesEnabled. We disable them since some messages are written to the FSceneView directly otherwise. */
	bool bPrevGScreenMessagesEnabled;

	/** 
	* Have we hit the callback for the BeginFrame at least once? This solves an issue where the delegates
	* get registered mid-frame so you end up calling EndFrame before BeginFrame which is undesirable.
	*/
	bool bHasRunBeginFrameOnce;
	/** Should we pause the game at the end of the frame? Used to implement frame step debugger. */
	bool bPauseAtEndOfFrame;

	/** Should we flush outstanding work between each shot, ensuring all files are written to disk before we move on? */
	bool bFlushDiskWritesPerShot;

	/** True if RequestShutdown() was called. At the start of the next frame we will stop producing frames (if needed) and start shutting down. */
	FThreadSafeBool bShutdownRequested;

	/** True if an error or other event occured which halted frame production prematurely. */
	FThreadSafeBool bFatalError;

	/** True if we're in a TransitionToState call. Used to prevent reentrancy. */
	bool bIsTransitioningState;

	/** When using temporal sub-frame stepping common counts (such as 3) don't result in whole ticks. We keep track of how many ticks we lose so we can add them the next time there's a chance. */
	float AccumulatedTickSubFrameDeltas;

	/** When we originally initialize we store the offset from UTC (which is what GetInitializationTime() is in), but we clear this if you call SetInitializationTime. */
	FTimespan InitializationTimeOffset;

	/**
	 * We have to apply camera motion vectors manually. So we keep the current and previous frame's camera view and rotation.
	 * Then we render a sequence of the same movement, and update after running the game sim.
	 **/
	MoviePipeline::FMoviePipelineFrameInfo FrameInfo;

public:

	/** This gathers all of the produced data for an output frame (which may come in async many frames later) before passing them onto the Output Containers. */
	TSharedPtr<FMoviePipelineOutputMerger, ESPMode::ThreadSafe> OutputBuilder;

	/** A debug image sequence writer in the event they want to dump every sample generated on its own. */
	IImageWriteQueue* ImageWriteQueue;

	/** Used to track first-render submissions (for 3d renders) to set the correct flags on the renderer module. */
	bool bHasRenderedFirstViewThisFrame;

private:
	/** Keep track of which job we're working on. This holds our Configuration + which shots we're supposed to render from it. */
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineExecutorJob> CurrentJob;

#if WITH_EDITOR
	/** Keep track of clips we've exported, for building FCPXML and other project files */
	FMovieSceneExportMetadata OutputMetadata;
#endif
	/** Keeps track of files written for each shot so it can be retrieved later via scripting for post-processing. */
	TArray<FMoviePipelineShotOutputData> GeneratedShotOutputData;

	/** Files that we've requested be written to disk but have not yet finished writing. */
	TArray<FMoviePipelineOutputFuture> OutputFutures;

	UE::MoviePipeline::FViewportArgs ViewportInitArgs;

	TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> CachedSequenceHierarchyRoot;


	/** Simulation settings cache per cloth interactor object. Needs one per LOD, hence the array. */
	TMap<TWeakObjectPtr<UObject>, TArray<MoviePipeline::FClothSimSettingsCache>> ClothSimCache;

	struct FRenderTimeStatistics
	{
		FDateTime StartTime;
		FDateTime EndTime;
	};

	TMap<int32, FRenderTimeStatistics> RenderTimeFrameStatistics;

	/** Caches the camera overscan used during setup to ensure that overscan-scaled resolution stays constant for every frame during a render */
	TMap<int32, float> CameraOverscanCache;

	/** Indicates if the user has already been warned about animate overscan if it is detected so that logs aren't flooded with warning messages */
	bool bHasWarnedAboutAnimatedOverscan = false;
	
public:
	static MOVIERENDERPIPELINECORE_API FString DefaultDebugWidgetAsset;
};


UCLASS()
class UMoviePipelineCustomTimeStep : public UEngineCustomTimeStep
{
	GENERATED_BODY()

public:
	// UEngineCustomTimeStep Interface
	virtual bool Initialize(UEngine* InEngine) override { return true; }
	virtual void Shutdown(UEngine* InEngine) override {}
	virtual bool UpdateTimeStep(UEngine* InEngine) override;
	virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const override { return ECustomTimeStepSynchronizationState::Synchronized; }
	// ~UEngineCustomTimeStep Interface

	void SetCachedFrameTiming(const MoviePipeline::FFrameTimeStepCache& InTimeCache);

	// Cache and Restore some of the World Settings settings, as rendering with MRQ in a runtime world needs to restore any changes made to World Settings.
	void CacheWorldSettings();
	void RestoreCachedWorldSettings();
private:
	/** We don't do any thinking on our own, instead we just spit out the numbers stored in our time cache. */
	MoviePipeline::FFrameTimeStepCache TimeCache;

	// Not cached in TimeCache as TimeCache is reset every frame.
	float PrevMinUndilatedFrameTime;
	float PrevMaxUndilatedFrameTime;
};

struct FMoviePipelineTimeController : public FMovieSceneTimeController
{
	virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override;
	void SetCachedFrameTiming(const FQualifiedFrameTime& InTimeCache) { TimeCache = InTimeCache; }

private:
	/** Simply store the number calculated and return it when requested. */
	FQualifiedFrameTime TimeCache;
};
