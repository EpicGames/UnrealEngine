// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineSetting.h"
#include "MovieRenderPipelineDataTypes.h"
#include "Engine/Scene.h"
#include "MoviePipelineUtils.h"
#include "MoviePipelineAntiAliasingSetting.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineAntiAliasingSetting : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:
	UMoviePipelineAntiAliasingSetting()
		: SpatialSampleCount(1)
		, TemporalSampleCount(1)
		, bOverrideAntiAliasing(false)
		, AntiAliasingMethod(EAntiAliasingMethod::AAM_None)
		, RenderWarmUpCount(32)
		, bUseCameraCutForWarmUp(false)
		, EngineWarmUpCount(0)
		, bRenderWarmUpFrames(false)
		, AccumulationGamma(1.f)
	{
	}

#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "AntiAliasingSettingDisplayName", "Anti-aliasing"); }
#endif

protected:
	virtual bool IsValidOnShots() const override { return true; }
	virtual bool IsValidOnPrimary() const override { return true; }
	UE_API virtual void ValidateStateImpl() override;
	UE_API virtual void GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:

	/**
	* How many frames should we accumulate together before contributing to one overall sample. This lets you
	* increase the anti-aliasing quality of an sample, or have high quality anti-aliasing if you don't want
	* any motion blur due to accumulation over time in SampleCount.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 1, ClampMin = 1), Category = "Render Settings")
	int32 SpatialSampleCount;

	/**
	* The number of frames we should combine together to produce each output frame. This blends the
	* results of this many sub-steps together to produce one output frame. See CameraShutterAngle to
	* control how much time passes between each sub-frame. See SpatialSampleCount to see how many
	* samples we average together to produce a sub-step. (This means rendering complexity is
	* SampleCount * TileCount^2 * SpatialSampleCount * NumPasses).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 1, ClampMin = 1), Category = "Render Settings")
	int32 TemporalSampleCount;

	/**
	* Should we override the Project's anti-aliasing setting during a movie render? This can be useful to have
	* TAA on during normal work in the editor but force it off for high quality renders /w many spatial samples.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Render Settings")
	bool bOverrideAntiAliasing;

	/**
	* If we are overriding the AA method, what do we use? None will turn off anti-aliasing.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition="bOverrideAntiAliasing"), Category = "Render Settings")
	TEnumAsByte<EAntiAliasingMethod> AntiAliasingMethod;

	/**
	* The number of frames at the start of each shot that the engine will render and then discard. This is useful for
	* ensuring there is history for temporal effects (such as anti-aliasing). It can be set to a lower number if not
	* using temporal effects. 
	*
	* This is more expensive than EngineWarmUpCount (which should be used for particle warm-ups, etc.)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0), AdvancedDisplay, Category = "Render Settings")
	int32 RenderWarmUpCount;

	/**
	* Should we use the excess in the camera cut track to determine engine warmup? When disabled, the sequence is evaluated
	* once at the first frame and then waits there for EngineWarmUpCount many frames. When this is enabled, the number of 
	* warmup frames is based on how much excess there is in the camera cut track outside of the playback range AND
	* the sequence is evaluated for each frame which can allow time for skeletal meshes to animate from a bind pose, etc.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Render Settings")
	bool bUseCameraCutForWarmUp;

	/**
	* The number of frames at the start of each shot that the engine will run without rendering. This allows pre-warming
	* systems (such as particle systems, or level loading) which need time to run before you want to start capturing frames. 
	* This ticks the game thread but does not submit anything to the GPU to be rendered.
	*
	* This is more cheaper than RenderWarmUpCount and is the preferred way to have time pass at the start of a shot.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, EditCondition = "!bUseCameraCutForWarmUp"), AdvancedDisplay, Category = "Render Settings")
	int32 EngineWarmUpCount;

	/**
	* Should we submit the warm-up frames to the GPU? Generally you want this disabled (as it is more performant), but
	* some systems (such as gpu particles) need to be rendered to actually perform their warm-up. Enabling this will
	* cause any warm up frames to also be submitted to the GPU which resolves this issue.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Render Settings")
	bool bRenderWarmUpFrames;

	/**
	* For advanced users, the gamma space to apply accumulation in. During accumulation, pow(x,AccumulationGamma) 
	* is applied and pow(x,1/AccumulationGamma) is applied after accumulation is finished
	*/
	float AccumulationGamma;
};

#undef UE_API
