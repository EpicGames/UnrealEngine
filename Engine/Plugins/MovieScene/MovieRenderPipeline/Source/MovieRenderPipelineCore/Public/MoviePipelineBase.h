// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineBase.generated.h"

DECLARE_MULTICAST_DELEGATE_OneParam(FMoviePipelineWorkFinishedNative, FMoviePipelineOutputData);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMoviePipelineWorkFinished, FMoviePipelineOutputData, Results);

UCLASS(MinimalAPI, BlueprintType, Abstract)
class UMoviePipelineBase : public UObject
{
	GENERATED_BODY()
public:
	/**
	* Request the movie pipeline to shut down at the next available time. The pipeline will attempt to abandon
	* the current frame (such as if there are more temporal samples pending) but may be forced into finishing if
	* there are spatial samples already submitted to the GPU. The shutdown flow will be run to ensure already
	* completed work is written to disk. This is a non-blocking operation, use Shutdown() instead if you need to
	* block until it is fully shut down.
	*
	* @param bIsError - Whether this is a request for early shut down due to an error
	*
	* This function is thread safe.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	void RequestShutdown(bool bIsError = false)
	{
		RequestShutdownImpl(bIsError);
	}

	/**
	* Abandons any future work on this Movie Pipeline and runs through the shutdown flow to ensure already
	* completed work is written to disk. This is a blocking-operation and will not return until all outstanding
	* work has been completed.
	*
	* @param bIsError - Whether this is an early shut down due to an error
	*
	* This function should only be called from the game thread.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	void Shutdown(bool bIsError = false)
	{
		ShutdownImpl(bIsError);
	}

	/**
	* Has RequestShutdown() been called?
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	bool IsShutdownRequested() const
	{
		return IsShutdownRequestedImpl();
	}

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	EMovieRenderPipelineState GetPipelineState() const
	{
		return GetPipelineStateImpl();
	}

	/**
	* Called when we have completely finished this pipeline. This means that all frames have been rendered,
	* all files written to disk, and any post-finalize exports have finished. This Pipeline will call
	* Shutdown() on itself before calling this delegate to ensure we've unregistered from all delegates
	* and are no longer trying to do anything (even if we still exist).
	*
	* The params struct in the return will have metadata about files written to disk for each shot.
	*/
	FMoviePipelineWorkFinishedNative& OnMoviePipelineWorkFinished() { return OnMoviePipelineWorkFinishedImpl(); }
	
	/**
	* Only called if `IsFlushDiskWritesPerShot()` is set!
	* Called after each shot is finished and files have been flushed to disk. The returned data in
	* the params struct will have only the per-shot metadata for the just finished shot. Use
	* OnMoviePipelineFinished() if you need all of the metadata.
	*/
	FMoviePipelineWorkFinishedNative& OnMoviePipelineShotWorkFinished() { return OnMoviePipelineShotWorkFinishedImpl(); }

	/**
	* Called when we have completely finished this pipeline. This means that all frames have been rendered,
	* all files written to disk, and any post-finalize exports have finished. This Pipeline will call
	* Shutdown() on itself before calling this delegate to ensure we've unregistered from all delegates
	* and are no longer trying to do anything (even if we still exist).
	*
	* The params struct in the return will have metadata about files written to disk for each shot.
	*/
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FMoviePipelineWorkFinished OnMoviePipelineWorkFinishedDelegate;



	/**
	* This callback will not be called by default due to performance reasons. You need to opt into this (via scripting
	* in MoviePipeline or in the node in Movie Graph) by setting FlushDiskWritesPerShot to true in the output setting
	* for this job's configuration.
	*
	* Called after each shot is finished and files have been flushed to disk. The returned data in
	* the params struct will have only the per-shot metadata for the just finished shot. Use
	* OnMoviePipelineWorkFinishedDelegate if you need all of the metadata.
	*/
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FMoviePipelineWorkFinished OnMoviePipelineShotWorkFinishedDelegate;

protected:
	virtual void RequestShutdownImpl(bool bIsError) {}
	virtual void ShutdownImpl(bool bIsError) {}
	virtual bool IsShutdownRequestedImpl() const { return false; }
	virtual EMovieRenderPipelineState GetPipelineStateImpl() const { return EMovieRenderPipelineState::Uninitialized; }
	virtual FMoviePipelineWorkFinishedNative& OnMoviePipelineWorkFinishedImpl() { return OnMoviePipelineWorkFinishedDelegateNative; }
	virtual FMoviePipelineWorkFinishedNative& OnMoviePipelineShotWorkFinishedImpl() { return OnMoviePipelineShotWorkFinishedDelegateNative; }
	virtual bool IsPostShotCallbackNeeded() const { return false; }

	/** Called when we have completely finished. This object will call Shutdown before this and stop ticking. */
	FMoviePipelineWorkFinishedNative OnMoviePipelineWorkFinishedDelegateNative;

	/** Called when each shot has finished work if IsFlushDiskWritesPerShot() is set. */
	FMoviePipelineWorkFinishedNative OnMoviePipelineShotWorkFinishedDelegateNative;
};
