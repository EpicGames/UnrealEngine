// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Scene.h"
#include "Graph/MovieGraphNamedResolution.h"
#include "Graph/Nodes/MovieGraphFileOutputNode.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"

#include "MoviePipelineBasicConfig.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMovieGraphConfig;

/** Denoiser type for the Path Traced Renderer in Basic configuration mode. */
UENUM(BlueprintType)
enum class EMoviePipelineBasicDenoiserType : uint8
{
	Spatial = 0,
	Temporal = 1
};

/**
 * A simplified configuration for Movie Render Queue jobs. Provides a pared-down set of rendering
 * options that generates a UMovieGraphConfig just-in-time before rendering. This allows most users
 * to configure renders without touching Presets or Graphs, while still using the full graph pipeline
 * under the hood.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineBasicConfig : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMoviePipelineBasicConfig();

	/**
	 * Merge a shot's basic config with this config, providing the result in "OutMerged".
	 * For each bOverride_* that is true within InShotOverrides, use the shot's value; otherwise, use the value from this config.
	 */
	UE_API void MergeOverrides(const UMoviePipelineBasicConfig* InShotOverrides, UMoviePipelineBasicConfig& OutMerged) const;

	/**
	 * Generate a UMovieGraphConfig from this config (or a merged config).
	 * Note that EnforceInvariants() will be called on InConfig to ensure the generated graph is valid.
	 */
	static UE_API UMovieGraphConfig* GenerateGraph(UMoviePipelineBasicConfig* InConfig, UObject* InOuter);

	/**
	 * Returns true if at least one bOverride_* flag is set on this config.
	 */
	UE_API bool HasAnyOverrides() const;

	/**
	 * Enforce minimum-valid-state invariants:
	 *   - At least one renderer (Deferred or Path Traced) must be enabled; if neither is, Deferred is re-enabled.
	 *   - EnabledOutputTypes must be non-empty; if it is, PNG is restored as the default.
	 */
	UE_API void EnforceInvariants();

#if WITH_EDITOR
	virtual UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Save InConfig as the user's default Basic config (persisted to a local uasset in the project). */
	static UE_API void SaveAsDefault(const UMoviePipelineBasicConfig* InConfig);
#endif

	/** Return the user's saved default Basic config, or nullptr if none exists. */
	static UE_API const UMoviePipelineBasicConfig* GetSavedDefault();

public:
	// --- Override flags. Some are shot-only gates, while others also act as Basic config enable flags. ---

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputDirectory : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_FileNameFormat : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputResolution : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_EnabledOutputTypes : 1;

	/** Whether to override the sequence's playback range start. Playback range overrides are only job-level settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_CustomStartFrame : 1;

	/** Whether to override the sequence's playback range end. Playback range overrides are only job-level settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_CustomEndFrame : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bUseDeferredRenderer : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DeferredSpatialSampleCount : 1;

	/** Whether to override the project's anti-aliasing method. For shots, this overrides the job's anti-aliasing method. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DeferredAntiAliasingMethod : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bUsePathTracedRenderer : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_PathTracedSpatialSampleCount : 1;

	/** Whether to enable the path traced denoiser. For shots, this overrides the job's denoiser setting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_PathTracedDenoiserType : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_NumWarmUpFrames : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_TemporalSampleCount : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInClass : 1;

	// --- Properties ---

	/** The directory all the output files are relative to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output")
	FDirectoryPath OutputDirectory;

	/** The format string for file names. Can include folder prefixes, and format string tokens ({shot_name}, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output")
	FString FileNameFormat;

	/** The resolution that output files are exported at. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output")
	FMovieGraphNamedResolution OutputResolution;

	/** The output types that rendered media should be saved to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output")
	TArray<TSoftClassPtr<UMovieGraphFileOutputNode>> EnabledOutputTypes;

	/** A custom start frame, overriding the sequence's playback range start. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output", meta = (EditCondition = "bOverride_CustomStartFrame"))
	int32 CustomStartFrame;

	/** A custom end frame, overriding the sequence's playback range end. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output", meta = (EditCondition = "bOverride_CustomEndFrame"))
	int32 CustomEndFrame;

	/** Whether the deferred renderer should be enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool bUseDeferredRenderer;

	/** Spatial sample count for the Deferred Renderer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Spatial Sample Count", EditCondition = "bUseDeferredRenderer", UIMin = 1, ClampMin = 1))
	int32 DeferredSpatialSampleCount;

	/** Anti-aliasing method for the Deferred Renderer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Anti Aliasing Method", EditCondition = "bOverride_DeferredAntiAliasingMethod"))
	TEnumAsByte<EAntiAliasingMethod> DeferredAntiAliasingMethod;

	/** Whether the path tracer should be enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool bUsePathTracedRenderer;

	/** Spatial sample count for the Path Traced Renderer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Spatial Sample Count", EditCondition = "bUsePathTracedRenderer", UIMin = 1, ClampMin = 1))
	int32 PathTracedSpatialSampleCount;

	/** If denoising is enabled, this specifies the specific denoiser that path tracer results should be passed to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Denoiser", EditCondition = "bOverride_PathTracedDenoiserType"))
	EMoviePipelineBasicDenoiserType PathTracedDenoiserType;

	/** The number of frames to run (without producing output) at the start of each shot to warm up systems like Niagara, Chaos, etc. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Num Warm Up Frames", UIMin = 0, ClampMin = 0))
	int32 NumWarmUpFrames;

	/** The number of temporal sub-samples per output frame. Higher values will reduce temporal aliasing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Temporal Sample Count", UIMin = 1, ClampMin = 1))
	int32 TemporalSampleCount;

	/**
	 * The burn-in widget class to use on all output types that support burn-in. Only applies if the burn-in override is enabled.
	 * Burn-ins are enabled for all output types that support them, and compositing is enabled for all output types that support burn-in compositing.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output", meta = (EditCondition = "bOverride_BurnInClass", MetaClass = "/Script/MovieRenderPipelineCore.MovieGraphBurnInWidget"))
	FSoftClassPath BurnInClass;
};

#undef UE_API
