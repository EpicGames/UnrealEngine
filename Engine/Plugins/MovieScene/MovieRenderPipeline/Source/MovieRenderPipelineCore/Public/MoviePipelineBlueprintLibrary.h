// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MovieRenderPipelineDataTypes.h"

#include "MoviePipelineBlueprintLibrary.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declare
class UCineCameraComponent;
class UMoviePipeline;
class UMovieSceneSequence;
class UMoviePipelineSetting;

UCLASS(MinimalAPI, meta = (ScriptName = "MoviePipelineLibrary"))
class UMoviePipelineBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	* Duplicates the specified sequence using a medium depth copy. Standard duplication will only duplicate
	* the top level Sequence (since shots and sub-sequences are other standalone assets) so this function
	* recursively duplicates the given sequence, shot and subsequence and then fixes up the references to
	* point to newly duplicated sequences.
	*
	* Use at your own risk. Some features may not work when duplicated (complex object binding arrangements,
	* blueprint GetSequenceBinding nodes, etc.) but can be useful when wanting to create a bunch of variations
	* with minor differences (such as swapping out an actor, track, etc.)
	*
	* This does not duplicate any assets that the sequence points to outside of Shots/Subsequences.
	*
	* @param	Outer		- The Outer of the newly duplicated object. Leave null for TransientPackage();
	* @param	InSequence	- The sequence to recursively duplicate.
	* @return				- The duplicated sequence, or null if no sequence was provided to duplicate.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API UMovieSceneSequence* DuplicateSequence(UObject* Outer, UMovieSceneSequence* InSequence);

	/**
	* Resolves the provided InFormatString by converting {format_strings} into settings provided by the primary config.
	* @param	InFormatString		A format string (in the form of "{format_key1}_{format_key2}") to resolve.
	* @param	InParams			The parameters to resolve the format string with. See FMoviePipelineFilenameResolveParams properties for details. 
	*								Expected that you fill out all of the parameters so that they can be used to resolve strings, otherwise default
	*								values may be used.
	* @param	OutFinalPath		The final filepath based on a combination of the format string and the Resolve Params.
	* @return	OutMergedFormatArgs	A merged set of Key/Value pairs for both Filename Arguments and Metadata that merges all the sources.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API void ResolveFilenameFormatArguments(const FString& InFormatString, const FMoviePipelineFilenameResolveParams& InParams, FString& OutFinalPath, FMoviePipelineFormatArgs& OutMergedFormatArgs);


	/**
	* Get the estimated amount of time remaining for the current pipeline. Based on looking at the total
	* amount of samples to render vs. how many have been completed so far. Inaccurate when Time Dilation
	* is used, and gets more accurate over the course of the render.
	*
	* @param	InPipeline	The pipeline to get the time estimate from.
	* @param	OutEstimate	The resulting estimate, or FTimespan() if estimate is not valid.
	* @return				True if a valid estimate can be calculated, or false if it is not ready yet (ie: not enough samples rendered)
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API bool GetEstimatedTimeRemaining(const UMoviePipeline* InPipeline, FTimespan& OutEstimate);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FDateTime GetJobInitializationTime(const UMoviePipeline* InMoviePipeline);

	/**
	* Get the current state of the specified Pipeline. See EMovieRenderPipelineState for more detail about each state.
	*
	* @param	InPipeline	The pipeline to get the state for.
	* @return				Current State.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API EMovieRenderPipelineState GetPipelineState(const UMoviePipeline* InPipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API EMovieRenderShotState GetCurrentSegmentState(UMoviePipeline* InMoviePipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FText GetJobName(UMoviePipeline* InMoviePipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FText GetJobAuthor(UMoviePipeline* InMoviePipeline);


	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API void GetOverallOutputFrames(const UMoviePipeline* InMoviePipeline, int32& OutCurrentIndex, int32& OutTotalCount);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API void GetCurrentSegmentName(UMoviePipeline* InMoviePipeline, FText& OutOuterName, FText& OutInnerName);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API void GetOverallSegmentCounts(const UMoviePipeline* InMoviePipeline, int32& OutCurrentIndex, int32& OutTotalCount);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FMoviePipelineSegmentWorkMetrics GetCurrentSegmentWorkMetrics(const UMoviePipeline* InMoviePipeline);

	/** Gets the completion percent of the Pipeline in 0-1 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API float GetCompletionPercentage(const UMoviePipeline* InPipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FTimecode GetRootTimecode(const UMoviePipeline* InMoviePipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FFrameNumber GetRootFrameNumber(const UMoviePipeline* InMoviePipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FTimecode GetCurrentShotTimecode(const UMoviePipeline* InMoviePipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FFrameNumber GetCurrentShotFrameNumber(const UMoviePipeline* InMoviePipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API float GetCurrentFocusDistance(const UMoviePipeline* InMoviePipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API float GetCurrentFocalLength(const UMoviePipeline* InMoviePipeline);
	
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API float GetCurrentAperture(const UMoviePipeline* InMoviePipeline);

	/** Get the package name for the map in this job. The level travel command requires the package path and not the asset path. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FString GetMapPackageName(UMoviePipelineExecutorJob* InJob);

	/** Loads the specified manifest file and converts it into an UMoviePipelineQueue. Use in combination with SaveQueueToManifestFile. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API class UMoviePipelineQueue* LoadManifestFileFromString(const FString& InManifestFilePath);

	/** Scan the provided sequence in the job to see which camera cut sections we would try to render and update the job's shotlist. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API void UpdateJobShotListFromSequence(ULevelSequence* InSequence, UMoviePipelineExecutorJob* InJob, bool& bShotsChanged);
	
	/**  If version number is manually specified by the Job, returns that. Otherwise search the Output Directory for the highest version already existing (and increment it by one if bGetNextVersion is true). */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API int32 ResolveVersionNumber(FMoviePipelineFilenameResolveParams InParams, bool bGetNextVersion = true);

	/**
	* Retrieves the cached version number calculated for the current shot, which depends on where the version token was used in the File Name Output
	* ie: If {version} comes before {shot_name} then all shots will use the same version number, but if it comes afterwards then each shot may
	* have a different version (which is the highest number found of that particular shot). This function should retrieve what is used in the
	* filename writing step either way.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API int32 GetCurrentVersionNumber(const UMoviePipeline* InMoviePipeline);
	
	/** In case of Overscan percentage being higher than 0 we render additional pixels. This function returns the resolution with overscan taken into account. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline", meta=(DeprecatedFunction, DeprecationMessage = "Use GetOverscannedResolution instead"))
	UE_DEPRECATED(5.6, "GetEffectiveOutputResolution is deprecated, use GetOverscannedResolution to get the overscanned resolution instead")
	static UE_API FIntPoint GetEffectiveOutputResolution(UMoviePipelinePrimaryConfig* InPrimaryConfig, UMoviePipelineExecutorShot* InPipelineExecutorShot, float DefaultOverscan = 0.0f);

	/** 
	 * Gets the desired output resolution for the movie render queue as specified by the user. Does not include overscan, tiling, or aspect ratio constraints,
	 * and is the target resolution that the pipeline will generally output (e.g. when cropping overscan in non-EXR formats). Use GetOverscannedResolution or
	 * GetBackbufferResolution to get resolutions that factor in overscan and tiling, respectively
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API FIntPoint GetDesiredOutputResolution(const UMoviePipelinePrimaryConfig* InPrimaryConfig);

	/** Gets the overscanned resolution, which is the target output resolution scaled by any configured overscan amount */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API FIntPoint GetOverscannedResolution(const UMoviePipelinePrimaryConfig* InPrimaryConfig, const UMoviePipelineExecutorShot* InPipelineExecutorShot, float InDefaultOverscan = 0.0f);
	
	/** Gets the resolution that will be used by the engine when rendering a frame. Includes overscan and tiling */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API FIntPoint GetBackbufferResolution(const UMoviePipelinePrimaryConfig* InPrimaryConfig, const UMoviePipelineExecutorShot* InPipelineExecutorShot, float InDefaultOverscan = 0.0f);
	
	/**
	 * Gets a rectangle that will crop out any overscan applied to the scene. If there is no overscan, the rectangle returned will match the backbuffer rectangle
	 * @param InPrimaryConfig			- The pipeline configuration that will provide context for resolving the resolution
	 * @param InPipelineExecutorShot	- The shot that is being generated by the pipeline
	 * @param DefaultOverscan			- The default overscan to use if there are no camera settings that provide an overscan override value, from 0.0 to 1.0
	 * @return							- The crop rectangle, which will have a resolution matching the requested output resolution
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API FIntRect GetOverscanCropRectangle(const UMoviePipelinePrimaryConfig* InPrimaryConfig, const UMoviePipelineExecutorShot* InPipelineExecutorShot, float DefaultOverscan = 0.0f);
	
	/** Allows access to a setting of provided type for specific shot. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline", meta = (DeterminesOutputType = "InSettingType"))
	static UE_API UMoviePipelineSetting* FindOrGetDefaultSettingForShot(TSubclassOf<UMoviePipelineSetting> InSettingType, const UMoviePipelinePrimaryConfig* InPrimaryConfig, const UMoviePipelineExecutorShot* InShot);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API ULevelSequence* GetCurrentSequence(const UMoviePipeline* InMoviePipeline);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API UMoviePipelineExecutorShot* GetCurrentExecutorShot(const UMoviePipeline* InMoviePipeline);

	/** Get a string to represent the Changelist Number for the burn in. This can be driven by a Modular Feature if you want to permanently replace it with different information. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API FText GetMoviePipelineEngineChangelistLabel(const UMoviePipeline* InMoviePipeline);

private:
	// Give the graph BP library access to shared utility methods
	friend class UMovieGraphBlueprintLibrary;


	/** Get the current cine camera in use, or nullptr if there is none. */
	static UE_API UCineCameraComponent* Utility_GetCurrentCineCamera(const UWorld* InWorld);
};

#undef UE_API
