// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineSetting.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineCameraSetting.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API


UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineCameraSetting : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:
	UMoviePipelineCameraSetting()
		: ShutterTiming(EMoviePipelineShutterTiming::FrameCenter)
	{}

public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "CameraSettingDisplayName", "Camera"); }
#endif
protected:
	virtual bool IsValidOnShots() const override { return true; }
	virtual bool IsValidOnPrimary() const override { return true; }
	
	UE_API virtual void GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:	
	/**
	* Shutter Timing allows you to bias the timing of your shutter angle to either be before, during, or after
	* a frame. When set to FrameClose, it means that the motion gathered up to produce frame N comes from 
	* before and right up to frame N. When set to FrameCenter, the motion represents half the time before the
	* frame and half the time after the frame. When set to FrameOpen, the motion represents the time from 
	* Frame N onwards.
	*
	* As an example, here are 4 temporal subsamples with 180 degree shutter angle (50% motion blur) between frames A & B:
	*
	* FrameClose
	* F-------F-------F
	* ----1234---------
	* ----AAAA---------
	*
	* FrameCenter
	* F-------F-------F
	* ------1234-------
	* ------AABB-------
	*
	* FrameOpen
	* F-------F-------F
	* --------1234-----
	* --------BBBB-----
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings")
	EMoviePipelineShutterTiming ShutterTiming;

	/**
	 * If true, the camera settings overscan value will override any overscan on the cameras when rendering;
	 * otherwise, the overscan value on the cameras will be used.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (InlineEditConditionToggle))
	bool bOverrideCameraOverscan;
	
	/**
	* Overscan percent enables rendering additional pixels beyond the current target resolution, and can be used in conjunction 
	* with the EXR output format to add post-processing effects such as lens distortion. Please note that using this feature may cause
	* the render to appear different in some scenarios (for example, auto-exposure may be impacted due to the increased FoV picking
	* up additional lighting).
	* 
	* With EXR outputs, the non-overscanned pixel data is contained in the EXR "display window", and the overscanned pixel data is
	* contained in the "data window". For example, a 1920x1080 render with 0.1 overscan will result in a 2112x1188 render; the display
	* window will be 1920x1080, and the data window will be 2112x1188. For all other formats, the pixels are cropped to the
	* original target resolution (ie, no extra pixel data is included).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Overscan Percentage Override", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", EditCondition = "bOverrideCameraOverscan"), Category = "Camera Settings")
	float OverscanPercentage;
	
	/**
	* If true, when a Camera Cut section is found we will also render any other cameras within the same sequence (not parent, nor child sequences though).
	* These cameras are rendered at the same time as the primary camera meaning all cameras capture the same world state. Do note that this multiplies
	* render times and memory requirements!
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings")
	bool bRenderAllCameras;
};

#undef UE_API
