// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineSetting.h"
#include "Engine/EngineTypes.h"
#include "Misc/FrameRate.h"
#include "OpenColorIOColorSpace.h"
#include "MoviePipelineColorSetting.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API


UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineColorSetting : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:
	UE_API UMoviePipelineColorSetting();
public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "ColorSetting", "Color Output"); }
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
	virtual bool IsValidOnShots() const override { return false; }
	virtual bool IsValidOnPrimary() const override { return true; }

public:
	/**
	* OCIO config to be passed to OCIO View Extension. If this is enabled the Tone Curve will be disabled.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Misc")
	FOpenColorIODisplayConfiguration OCIOConfiguration;

	/** If true the Filmic Tone Curve will not be applied. Disabling this will allow you to export linear data for EXRs. Force Disabled if Open Color IO is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Misc")
	bool bDisableToneCurve;
};

#undef UE_API
