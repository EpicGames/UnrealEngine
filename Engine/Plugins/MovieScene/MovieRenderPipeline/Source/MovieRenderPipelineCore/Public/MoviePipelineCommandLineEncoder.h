// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineSetting.h"
#include "Engine/EngineTypes.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineCommandLineEncoder.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

UENUM(BlueprintType)
enum class EMoviePipelineEncodeQuality : uint8
{
	Low = 0,
	Medium = 1,
	High = 2,
	Epic = 3
};

UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineCommandLineEncoder : public UMoviePipelineSetting
{
	struct FEncoderParams
	{
		FStringFormatNamedArguments NamedArguments;
		TMap<FString, TArray<FString>> FilesByExtensionType;
		TWeakObjectPtr<class UMoviePipelineExecutorShot> Shot;
	};

	GENERATED_BODY()
public:
	UE_API UMoviePipelineCommandLineEncoder();
	UE_API void StartEncodingProcess(TArray<FMoviePipelineShotOutputData>& InOutData, const bool bInIsShotEncode);
public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "CommandLineEncode_DisplayText", "Command Line Encoder"); }
	virtual FText GetCategoryText() const override { return NSLOCTEXT("MovieRenderPipeline", "ExportsCategoryName_Text", "Exports"); }
	virtual bool CanBeDisabled() const override { return true; }
#endif
	UE_API virtual void SetupForPipelineImpl(UMoviePipeline* InPipeline);
	UE_API virtual void ValidateStateImpl() override;
	virtual bool IsValidOnShots() const override { return false; }
	virtual bool IsValidOnPrimary() const override { return true; }
	UE_API virtual bool HasFinishedExportingImpl() override;
	UE_API virtual void BeginExportImpl() override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	
protected:
	UE_API bool NeedsPerShotFlushing() const;
	UE_API void LaunchEncoder(const FEncoderParams& InParams);
	UE_API void OnTick();
	UE_API FString GetQualitySettingString() const;

public:
	/** 
	* File name format string override. If specified it will override the FileNameFormat from the Output setting.
	* If {shot_name} or {camera_name} is used, encoding will begin after each shot finishes rendering.
	* Can be different from the main one in the Output setting so you can render out frames to individual
	* shot folders but encode to one file.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Encoder")
	FString FileNameFormatOverride;

	/** What encoding quality to use for this job? Exact command line arguments for each one are specified in Project Settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Encoder")
	EMoviePipelineEncodeQuality Quality;
	
	/** Any additional arguments to pass to the CLI encode for this particular job. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Encoder")
	FString AdditionalCommandLineArgs;
	
	/** Should we delete the source files from disk after encoding? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Encoder")
	bool bDeleteSourceFiles;

	/** If a render was canceled (via hitting escape mid render) should we skip trying to encode the files we did produce? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Encoder")
	bool bSkipEncodeOnRenderCanceled;

	/** Write the duration for each frame into the generated text file. Needed for some input types on some CLI encoding software. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Command Line Encoder")
	bool bWriteEachFrameDuration;
	
private:
	struct FActiveJob
	{
		FActiveJob()
			: ReadPipe(nullptr)
			, WritePipe(nullptr)
		{}

		FProcHandle ProcessHandle;
		void* ReadPipe;
		void* WritePipe;

		TArray<FString> FilesToDelete;
	};

	TArray<FActiveJob> ActiveEncodeJobs;
};

#undef UE_API
