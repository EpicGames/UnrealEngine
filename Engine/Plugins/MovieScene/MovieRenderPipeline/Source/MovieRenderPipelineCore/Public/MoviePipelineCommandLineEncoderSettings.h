// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"
#include "MoviePipelineCommandLineEncoderSettings.generated.h"

UCLASS(MinimalAPI, BlueprintType, config = Engine, defaultconfig, meta=(DisplayName = "Movie Pipeline CLI Encoder") )
class UMoviePipelineCommandLineEncoderSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UMoviePipelineCommandLineEncoderSettings()
		: Super()
	{
		CodecHelpText = NSLOCTEXT("MovieRenderPipeline", "CommandLineEncode_HelpText", "Run 'MovieRenderPipeline.DumpCLIEncoderCodecs' in Console to see available codecs.");
		EncodeSettings_Low = TEXT("-crf 28");
		EncodeSettings_Med = TEXT("-crf 23");
		EncodeSettings_High = TEXT("-crf 20");
		EncodeSettings_Epic = TEXT("-crf 16");
		CommandLineFormat = TEXT("-hide_banner -y -loglevel error {AdditionalLocalArgs} {VideoInputs} {AudioInputs} -acodec {AudioCodec} -vcodec {VideoCodec} {Quality} \"{OutputPath}\"");
		VideoInputStringFormat = TEXT("-f concat -safe 0 -i \"{InputFile}\" -r {FrameRate}");
		AudioInputStringFormat = TEXT("-f concat -safe 0 -i \"{InputFile}\"");
	}

	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

public:
	/** Path to the executable (including extension). Can just be "ffmpeg.exe" if it can be located via PATH directories. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Encoder")
	FString ExecutablePath;

	UPROPERTY(EditAnywhere, Category = "Command Line Encoder")
	FText CodecHelpText;

	/** Which video codec should we use? Run 'MovieRenderPipeline.DumpCLIEncoderCodecs' for options. */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Encoder")
	FString VideoCodec;

	/** Which audio codec should we use? Run 'MovieRenderPipeline.DumpCLIEncoderCodecs' for options. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Encoder")
	FString AudioCodec;

	/** Extension for the output files. Many encoders use this to determine the container type they are placed in. Should be without dot, ie: "webm". */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Encoder")
	FString OutputFileExtension;

	/** The format string used when building the final command line argument to launch. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Arguments")
	FString CommandLineFormat;
	
	/** Format string used for each video input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Arguments")
	FString VideoInputStringFormat;
	
	/** Format string used for each audio input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Arguments")
	FString AudioInputStringFormat;
	
	/** The flags used for low quality encoding. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Arguments")
	FString EncodeSettings_Low;
	
	/** The flags used for medium quality encoding. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Arguments")
	FString EncodeSettings_Med;
	
	/** The flags used for high quality encoding. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Arguments")
	FString EncodeSettings_High;
	
	/** The flags used for epic quality encoding. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Command Line Arguments")
	FString EncodeSettings_Epic;
};
