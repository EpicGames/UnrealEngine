// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "MoviePipelineSetting.h"
#include "MoviePipelineConfigBase.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API


UCLASS(MinimalAPI, BlueprintType, Abstract)
class UMoviePipelineConfigBase : public UObject
{
	GENERATED_BODY()

public:
	UMoviePipelineConfigBase()
	{
		SettingsSerialNumber = -1;
		DisplayName = TEXT("Unsaved Config");
	}

	virtual void PostRename(UObject* OldOuter, const FName OldName) override
	{
		DisplayName = GetFName().ToString();
		Super::PostRename(OldOuter, OldName);
	}

	virtual void PostDuplicate(bool bDuplicateForPIE) override
	{
		DisplayName = GetFName().ToString();
		Super::PostDuplicate(bDuplicateForPIE);
	}

public:
	/** Removes the specific instance from our Setting list. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API virtual void RemoveSetting(UMoviePipelineSetting* InSetting);

	/** Copy this configuration from another existing configuration. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API virtual void CopyFrom(UMoviePipelineConfigBase* InConfig);

	int32 GetSettingsSerialNumber() const { return SettingsSerialNumber; }

	/** Returns an array of all settings in this config that the user has added via the UI or via Scripting. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	virtual TArray<UMoviePipelineSetting*> GetUserSettings() const { return Settings; }

	/**
	 * Gets the config that this config was originally based on (if any). The origin will only be set on transient
	 * configs; the origin will be nullptr for non-transient configs because the origin will be this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UMoviePipelineConfigBase* GetConfigOrigin() const { return ConfigOrigin.LoadSynchronous(); }

	/**
	 * Sets the config that this config originated from (if any). The origin should only be set for transient configs.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	void SetConfigOrigin(UMoviePipelineConfigBase* InConfig) { ConfigOrigin = InConfig; }

public:
	template<typename SettingType>
	TArray<SettingType*> FindSettings(const bool bIncludeDisabledSettings = false) const
	{
		TArray<UMoviePipelineSetting*> AllSettings = GetUserSettings();
		TArray<SettingType*> FoundSettings;

		for (UMoviePipelineSetting* Setting : AllSettings)
		{
			if (Setting && Setting->IsA<SettingType>() && (Setting->IsEnabled() || bIncludeDisabledSettings))
			{
				FoundSettings.Add(Cast<SettingType>(Setting));
			}
		}

		return FoundSettings;
	}

	template<typename SettingType>
	SettingType* FindSetting(const bool bIncludeDisabledSettings = false) const
	{
		return Cast<SettingType>(FindSettingByClass(SettingType::StaticClass(), bIncludeDisabledSettings));
	}

	/**
	* Find all settings of a particular type for this config.
	* @param InClass - Class that you wish to find the setting object for.
	* @param bIncludeDisabledSettings - if true, disabled settings will be included in the search
	* @param bExactMatch - if true, only exact matches of the specified class will be found, otherwise subclasses of the specified class will also be found
	* @return An array of instances of this class if it already exists as a setting on this config
	*/
	UFUNCTION(BlueprintPure, meta = (DeterminesOutputType = "InClass"), Category = "Movie Render Pipeline")
	TArray<UMoviePipelineSetting*> FindSettingsByClass(TSubclassOf<UMoviePipelineSetting> InClass, const bool bIncludeDisabledSettings = false, const bool bExactMatch = false) const
	{
		TArray<UMoviePipelineSetting*> AllSettings = GetUserSettings();
		TArray<UMoviePipelineSetting*> MatchingSettings;
		for (UMoviePipelineSetting* Setting : AllSettings)
		{
			if (!Setting)
			{
				continue;
			}

			const bool IsTypeMatch = bExactMatch ? (InClass.Get() == Setting->GetClass()) : Setting->IsA(InClass.Get());
			if (IsTypeMatch && (Setting->IsEnabled() || bIncludeDisabledSettings))
			{
				MatchingSettings.Add(Setting);
			}
		}

		return MatchingSettings;
	}

	/**
	* Find a setting of a particular type for this config.
	* @param InClass - Class that you wish to find the setting object for.
	* @param bIncludeDisabledSettings - if true, disabled settings will be included in the search
	* @param bExactMatch - if true, only exact matches of the specified class will be found, otherwise subclasses of the specified class will also be found
	* @return An instance of this class if it already exists as a setting on this config, otherwise null.
	*/
	UFUNCTION(BlueprintPure, meta = (DeterminesOutputType = "InClass"), Category = "Movie Render Pipeline")
	UMoviePipelineSetting* FindSettingByClass(TSubclassOf<UMoviePipelineSetting> InClass, const bool bIncludeDisabledSettings = false, const bool bExactMatch = false) const
	{
		TArray<UMoviePipelineSetting*> AllInstances = FindSettingsByClass(InClass, bIncludeDisabledSettings, bExactMatch);
		if (AllInstances.Num() > 0)
		{
			return AllInstances[0];
		}

		return nullptr;
	}


	/**
	* Finds a setting of a particular type for this pipeline config, adding it if it doesn't already exist.
	* @param InClass - Class you wish to find or create the setting object for.
	* @param bIncludeDisabledSettings - if true, disabled settings will be included in the search
	* @param bExactMatch - if true, only exact matches of the specified class will be found, otherwise subclasses of the specified class will also be found
	* @return An instance of this class as a setting on this config.
	*/
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "InClass"), Category = "Movie Render Pipeline")
	UMoviePipelineSetting* FindOrAddSettingByClass(TSubclassOf<UMoviePipelineSetting> InClass, const bool bIncludeDisabledSettings = false, const bool bExactMatch = false)
	{
		UMoviePipelineSetting* Found = FindSettingByClass(InClass, bIncludeDisabledSettings, bExactMatch);
		if (!Found)
		{
			Modify();
			
			Found = NewObject<UMoviePipelineSetting>(this, InClass);

			if (CanSettingBeAdded(Found))
			{
				Settings.Add(Found);
				OnSettingAdded(Found);
				++SettingsSerialNumber;
			}
			else
			{
				FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Setting %s is not compatible with this Config Type and was not added."), *InClass->GetName()), ELogVerbosity::Error);
				return nullptr;
			}
		}

		return Found;
	}


	
public:
	virtual bool CanSettingBeAdded(const UMoviePipelineSetting* InSetting) const PURE_VIRTUAL( UMoviePipelineConfigBase::CanSettingBeAdded, return false; );

protected:
	virtual void OnSettingAdded(UMoviePipelineSetting* InSetting) { InSetting->ValidateState(); }
	virtual void OnSettingRemoved(UMoviePipelineSetting* InSetting) {}
public:
	UPROPERTY()
	FString DisplayName;

protected:
	/** Array of settings classes that affect various parts of the output pipeline. */
	UPROPERTY(VisibleAnywhere, Instanced, Category = "Movie Pipeline")
	TArray<TObjectPtr<UMoviePipelineSetting>> Settings;

private:
	/** The config that this config originated from. Helpful for transient configs. */
	UPROPERTY()
	TSoftObjectPtr<UMoviePipelineConfigBase> ConfigOrigin;
	
	int32 SettingsSerialNumber;
};

#undef UE_API
