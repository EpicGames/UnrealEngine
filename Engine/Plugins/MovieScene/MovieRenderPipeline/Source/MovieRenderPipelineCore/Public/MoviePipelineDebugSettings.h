// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineSetting.h"
#include "Engine/EngineTypes.h"

#include "MoviePipelineDebugSettings.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineDebugSettings : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:
	UE_API UMoviePipelineDebugSettings();
public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DebugSettingDisplayName", "Debug Options"); }
	UE_API virtual FText GetFooterText(UMoviePipelineExecutorJob* InJob) const override;
	virtual bool CanBeDisabled() const override { return true; }
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
	virtual bool IsValidOnShots() const override { return false; }
	virtual bool IsValidOnPrimary() const override { return true; }

	/**
	* If true, we will write all samples that get generated to disk individually. This can be useful for debugging or if you need to accumulate
	* render passes differently than provided.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Render Settings")
	bool bWriteAllSamples;

	/** Returns true if both the RenderCapture modular feature is available and the user has enabled capturing with RenderDoc */
	bool IsRenderDocEnabled() const { return bIsRenderDebugCaptureAvailable && bCaptureFramesWithRenderDoc; }

	/** If true, automatically trigger RenderDoc to capture rendering information for frames from CaptureStartFrame to CaptureEndFrame, inclusive */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RenderDoc")
	bool bCaptureFramesWithRenderDoc;
	
	/** Used when capturing rendering information with RenderDoc. In Display Rate frames.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RenderDoc")
	int32 CaptureFrame;

	/** If true, automatically capture an Unreal Insights trace file for the duration of the render. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Profiling")
	bool bCaptureUnrealInsightsTrace;
	
private:
	bool bIsRenderDebugCaptureAvailable;
};

#undef UE_API
