// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "MovieRenderPipelineDataTypes.h"
#include "Http.h"
#include "Misc/CoreDelegates.h"
#include "Serialization/ArrayReader.h"
#include "MovieRenderDebugWidget.h"
#include "MoviePipelineExecutor.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMoviePipelinePrimaryConfig;
class UMoviePipelineExecutorBase;
class UMoviePipelineExecutorJob;
class UMoviePipeline;
class UMoviePipelineQueue;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMoviePipelineExecutorFinishedNative, UMoviePipelineExecutorBase* /*PipelineExecutor*/, bool /*bSuccess*/);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMoviePipelineExecutorFinished, UMoviePipelineExecutorBase*, PipelineExecutor, bool, bSuccess);

DECLARE_MULTICAST_DELEGATE_FourParams(FOnMoviePipelineExecutorErroredNative, UMoviePipelineExecutorBase* /*PipelineExecutor*/, UMoviePipeline* /*PipelineWithError*/, bool /*bIsFatal*/, FText /*ErrorText*/);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams(FOnMoviePipelineExecutorErrored, UMoviePipelineExecutorBase*, PipelineExecutor, UMoviePipeline*, PipelineWithError, bool, bIsFatal, FText, ErrorText);

/** Called when a socket message is recieved. String is UTF8 encoded and has had the length byte stripped from it. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMoviePipelineSocketMessageRecieved, const FString&, Message);

/** Called when the response for an HTTP request comes in. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FMoviePipelineHttpResponseRecieved, int32, RequestIndex, int32, ResponseCode, const FString&, Message);


/**
* A Movie Pipeline Executor is responsible for executing an array of Movie Pipelines,
* and (optionally) reporting progress back for the movie pipelines. The entire array
* is passed at once to allow the implementations to choose how to split up the work.
* By default we provide a local execution (UMoviePipelineLocalExecutor) which works
* on them serially, but you can create an implementation of this class, change the 
* default in the Project Settings and use your own distribution logic. For example,
* you may want to distribute the work to multiple computers over a network, which
* may involve running command line options on each machine to sync the latest content
* from the project before the execution starts.
*/
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMoviePipelineExecutorBase : public UObject
{
	GENERATED_BODY()
public:
	UMoviePipelineExecutorBase()
		: bAnyJobHadFatalError(false)
		, PostRenderActionType(EMoviePipelinePostRenderActionType::DoNothing)
		, TargetPipelineClass(nullptr)
		, ExternalSocket(nullptr)
		, RecvMessageDataRemaining(0)
	{
		// Don't register for callbacks on the CDO.
		if (!HasAnyFlags(RF_ArchetypeObject))
		{
			FCoreDelegates::OnBeginFrame.AddUObject(this, &UMoviePipelineExecutorBase::OnBeginFrame);
		}
	}

	/**
	* Execute the provided Queue. You are responsible for deciding how to handle each job
	* in the queue and processing them. OnExecutorFinished should be called when all jobs
	* are completed, which can report both success, warning, cancel, or error. 
	*
	* For C++ implementations override `virtual void Execute_Implementation() const override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def execute(self):
	*
	* @param InPipelineQueue The queue that this should process all jobs for. This can be null
							 when using certain combination of command line render flags/scripting.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void Execute(UMoviePipelineQueue* InPipelineQueue);

	/**
	* Report the current state of the executor. This is used to know if we can call Execute again.
	*
	* For C++ implementations override `virtual bool IsRendering_Implementation() const override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def is_rendering(self):
	*		return ?
	* 
	* @return True if the executor is currently working on a queue to produce a render.
	*/
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API bool IsRendering() const;

	/**
	* Called once at the beginning of each engine frame. 
	*
	* For C++ implementations override `virtual bool OnBeginFrame_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def on_begin_frame(self):
	*		super().on_begin_frame()
	*
	*/
	UFUNCTION(BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void OnBeginFrame();

	/**
	* Set the status of this Executor. Does nothing in default implementations, but a useful shorthand
	* for implementations to broadcast status updates, ie: You can call SetStatusMessage as your executor
	* changes state, and override the SetStatusMessage function to make it actually do something (such as
	* printing to a log or updating a third party API).
	*
	* For C++ implementations override `virtual bool SetStatusMessage_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def set_status_message(self, inStatus):
	*
	* @param InStatus	The status message you wish the executor to have.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void SetStatusMessage(const FString& InStatus);

		/**
	* Get the current status message for this job. May be an empty string.
	*
	* For C++ implementations override `virtual FString GetStatusMessage_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def get_status_message(self):
	*		return ?
	*/
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API FString GetStatusMessage() const;

	/**
	* Set the progress of this Executor. Does nothing in default implementations, but a useful shorthand
	* for implementations to broadcast progress updates, ie: You can call SetStatusProgress as your executor
	* changes progress, and override the SetStatusProgress function to make it actually do something (such as
	* printing to a log or updating a third party API).
	*
	* Does not necessairly reflect the overall progress of the work, it is an arbitrary 0-1 value that
	* can be used to indicate different things (depending on implementation).
	*
	* For C++ implementations override `virtual bool SetStatusProgress_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def set_status_progress(self, inStatus):
	*
	* @param InProgress	The progress (0-1 range) the executor should have.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void SetStatusProgress(const float InProgress);

	/**
	* Get the current progress as last set by SetStatusProgress. 0 by default.
	*
	* For C++ implementations override `virtual float GetStatusProgress_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def get_status_progress(self):
	*		return ?
	*/
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API float GetStatusProgress() const;

	/**
	* Abort the currently executing job.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void CancelCurrentJob();

	/**
	* Abort the currently executing job and skip all other jobs.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void CancelAllJobs();
	
	/**
	* Set the post-render action that will be taken when all the renders in the queue finish.
	*
	* For C++ implementations override `virtual void SetPostRenderAction_Implementation(const EMoviePipelinePostRenderActionType PostRenderAction) override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def set_post_render_action(self, postRenderAction):
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void SetPostRenderAction(const EMoviePipelinePostRenderActionType PostRenderAction);

	/**
	* Get the post-render action that will be taken when all the renders in the queue finish.
	*
	* For C++ implementations override `virtual EMoviePipelinePostRenderActionType GetPostRenderAction_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def get_post_render_action(self):
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API EMoviePipelinePostRenderActionType GetPostRenderAction();

	/**
	* Get whether this executor supports performing a post-render action.
	*
	* The base executor classes do not implement post-render actions. Executors returning "true" here must implement support for post-render
	* actions themselves.
	*
	* For C++ implementations override `virtual bool GetSupportsPostRenderActions_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def get_supports_post_render_actions(self):
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API bool GetSupportsPostRenderActions();

	/**
	* Specify which MoviePipeline class type should be created by this executor when processing jobs.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	void SetMoviePipelineClass(UClass* InPipelineClass)
	{
		TargetPipelineClass = InPipelineClass;
	}

	/** Native C++ event to listen to for when this Executor has finished. */
	FOnMoviePipelineExecutorFinishedNative& OnExecutorFinished()
	{
		return OnExecutorFinishedDelegateNative;
	}

	FOnMoviePipelineExecutorErroredNative& OnExecutorErrored()
	{
		return OnExecutorErroredDelegateNative;
	}

protected:
	// UObject Interface
	virtual void BeginDestroy() override
	{
		DisconnectSocket();
		Super::BeginDestroy();
	}
	// ~UObject Interface


protected:
	/** 
	* This should be called when the Executor has finished executing all of the things
	* it has been asked to execute. This should be called in the event of a failure as 
	* well, and passing in false for success to allow the caller to know failure. Errors
	* should be broadcast on the error delegate, so this is just a handy way to know at
	* the end without having to track it yourself.
	*
	* @param bInSuccess	True if the pipeline successfully executed all jobs. False if there was an error. 
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	virtual void OnExecutorFinishedImpl()
	{
		// Broadcast to both Native and Python/BP
		OnExecutorFinishedDelegateNative.Broadcast(this, !bAnyJobHadFatalError);
		OnExecutorFinishedDelegate.Broadcast(this, !bAnyJobHadFatalError);
	}

	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	virtual void OnExecutorErroredImpl(UMoviePipeline* ErroredPipeline, bool bFatal, FText ErrorReason)
	{
		if (bFatal)
		{
			bAnyJobHadFatalError = true;
		}

		// Broadcast to both Native and Python/BP
		OnExecutorFinishedDelegateNative.Broadcast(this, !bAnyJobHadFatalError);
		OnExecutorFinishedDelegate.Broadcast(this, !bAnyJobHadFatalError);
	}

	bool IsAnyJobErrored() const { return bAnyJobHadFatalError; }

	// UMoviePipelineExecutorBase Interface
	virtual void Execute_Implementation(UMoviePipelineQueue* InPipelineQueue) PURE_VIRTUAL(UMoviePipelineExecutorBase::ExecuteImpl, );
	virtual bool IsRendering_Implementation() const PURE_VIRTUAL(UMoviePipelineExecutorBase::IsRenderingImpl, return false; );
	UE_API virtual void OnBeginFrame_Implementation();
	virtual void SetStatusMessage_Implementation(const FString& InMessage) { StatusMessage = InMessage; }
	virtual void SetStatusProgress_Implementation(const float InProgress) { StatusProgress = InProgress; }
	virtual FString GetStatusMessage_Implementation() const { return StatusMessage; }
	virtual float GetStatusProgress_Implementation() const { return StatusProgress; }
	virtual void CancelCurrentJob_Implementation() PURE_VIRTUAL(UMoviePipelineExecutorBase::CancelCurrentJobImpl, );
	virtual void CancelAllJobs_Implementation() PURE_VIRTUAL(UMoviePipelineExecutorBase::CancelAllJobsImpl, );
	UE_API virtual void SetPostRenderAction_Implementation(const EMoviePipelinePostRenderActionType PostRenderAction);
	virtual EMoviePipelinePostRenderActionType GetPostRenderAction_Implementation() { return PostRenderActionType; }
	virtual bool GetSupportsPostRenderActions_Implementation() { return false; }
	// ~UMoviePipelineExecutorBase

private:
	/** 
	* Called when the Executor has finished all jobs. Reports success if no jobs
	* had fatal errors. Subscribe to the error delegate for more information about
	* any errors.
	*
	* Exposed for Blueprints/Python. Called at the same time as the native one.
	*/
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FOnMoviePipelineExecutorFinished OnExecutorFinishedDelegate;

	/** For native C++ code. Called at the same time as the Blueprint/Python one. */
	FOnMoviePipelineExecutorFinishedNative OnExecutorFinishedDelegateNative;

	/**
	* Called when an individual job reports a warning/error. Jobs are considered fatal
	* if the severity was bad enough to abort the job (missing sequence, write failure, etc.)
	*
	* Exposed for Blueprints/Python. Called at the same time as the native one.
	*/
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FOnMoviePipelineExecutorErrored OnExecutorErroredDelegate;

	/** For native C++ code. Called at the same time as the Blueprint/Python one. */
	FOnMoviePipelineExecutorErroredNative OnExecutorErroredDelegateNative;

	/** Set automatically when the error delegate gets broadcast (if fatal). */
	bool bAnyJobHadFatalError;

	/** The type of action that will be taken after all renders in the queue finish. */
	EMoviePipelinePostRenderActionType PostRenderActionType;

protected:
	/** 
	* Attempts to connect a socket to the specified host and port. This is a blocking call.
	* @param InHostName	The host name as to connect to such as "127.0.0.1"
	* @param InPort		The port to attempt to connect to the host on.
	* @return True if the socket was succesfully connected to the given host and port.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API bool ConnectSocket(const FString& InHostName, const int32 InPort);

	/*
	* Disconnects the socket (if currently connected.)
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API void DisconnectSocket();

	/**
	Sends a socket message if the socket is currently connected. Messages back will happen in the OnSocketMessageRecieved event.
	@param InMessage	The message to send. This will be sent over the socket (if connected) with a 4 byte (int32) size prefix on the
						message so the recieving end knows how much data to recieve before considering it done. This prevents accidentally
						chopping json strings in half.
	@return True if the message was sent succesfully. 
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API bool SendSocketMessage(const FString& InMessage);

	/** Returns true if the socket is currently connected, false otherwise. Call ConnectSocket to attempt a connection. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UE_API bool IsSocketConnected() const;

	/** 
	* If this executor has been configured to connect to a socket, this event will be called each time the socket recieves something. 
	* The message response expected from the server should have a 4 byte (int32) size prefix for the string to specify how much data
	* we should expect. This delegate will not get invoked until we recieve that many bytes from the socket connection to avoid broadcasting
	* partial messages.
	*/
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FMoviePipelineSocketMessageRecieved SocketMessageRecievedDelegate;

	/**
	* Sends a asynchronous HTTP request. Responses will be returned in the the OnHTTPResponseRecieved event.
	* 
	* @param InURL		The URL to send the request to.
	* @param InVerb		The HTTP verb for the request. GET, PUT, POST, etc.
	* @param InMessage	The content of the request.
	* @param InHeaders	Headers that should be set on the request.
	*
	* @return Returns an index for the request. This index will be provided in the OnHTTPResponseRecieved event so you can
	*		  make multiple HTTP requests at once and tell them apart when you recieve them, even if they come out of order.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API int32 SendHTTPRequest(const FString& InURL, const FString& InVerb, const FString& InMessage, const TMap<FString, FString>& InHeaders);

	/** If an HTTP Request has been made and a response returned, the returned response will be broadcast on this event. */
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FMoviePipelineHttpResponseRecieved HTTPResponseRecievedDelegate;

private:
	UE_API bool BlockingSendSocketMessageImpl(const uint8* Data, int32 BytesToSend);
	UE_API void OnHttpRequestCompletedImpl(FHttpRequestPtr InRequest, FHttpResponsePtr InResponse, bool bWasSuccessful);
	UE_API bool ProcessIncomingSocketData();

public:
	/** Optional widget for feedback during render */
	UE_DEPRECATED(5.1, "Use SetViewportInitArgs instead.")
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	TSubclassOf<UMovieRenderDebugWidget> DebugWidgetClass;

	void SetViewportInitArgs(const UE::MoviePipeline::FViewportArgs& InArgs) { ViewportInitArgs = InArgs; }
	/**
	* Some global initialization args that get passed to the UMoviePipeline before the call to ::Initialize()
	*/
	UE::MoviePipeline::FViewportArgs ViewportInitArgs;

	/**
	* Arbitrary data that can be associated with the executor. Not used by default implementations, nor read.
	* This can be used to attach third party metadata such as job ids from remote farms.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	FString UserData;
protected:
	/** Which Pipeline Class should be created by this Executor. May be null. */
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	TSubclassOf<UMoviePipeline> TargetPipelineClass;

	FString StatusMessage;
	float StatusProgress;

private:
	/** The socket connection if we currently have one. */
	class FSocket* ExternalSocket;

	/** Message data we're currently in the process of receiving, if any */
	TSharedPtr<FArrayReader, ESPMode::ThreadSafe> RecvMessageData;

	/** The number of bytes of incoming message data we're still waiting on before we have a complete message */
	int32 RecvMessageDataRemaining;

	struct FOutstandingRequest
	{
		FOutstandingRequest()
			: RequestIndex(-1)
			, Request(nullptr)
		{}

		int32 RequestIndex;
		FHttpRequestPtr Request;
	};

	TArray<FOutstandingRequest> OutstandingRequests;
};

#undef UE_API
