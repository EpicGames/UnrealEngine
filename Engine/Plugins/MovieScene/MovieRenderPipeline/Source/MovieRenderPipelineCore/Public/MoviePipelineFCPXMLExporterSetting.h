// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineOutputBase.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineFCPXMLExporterSetting.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

UENUM(BlueprintType)
enum class FCPXMLExportDataSource : uint8
{
	OutputMetadata,
	SequenceData
};

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineFCPXMLExporter : public UMoviePipelineOutputBase
{
	GENERATED_BODY()
public:
	UMoviePipelineFCPXMLExporter()
		: DataSource(FCPXMLExportDataSource::SequenceData)
		, bHasFinishedExporting(false)
	{}

public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "FCPXMLExporterDisplayName", "Final Cut Pro XML"); }
#endif
protected:
	virtual bool HasFinishedExportingImpl() { return bHasFinishedExporting; }
	UE_API virtual void BeginExportImpl() override;

	UE_API bool EnsureWritableFile();
	bool bOverwriteFile;
public:
	/** File name format string override. If specified it will override the FileNameFormat from the Output setting. Can include folder prefixes, and format string tags ({sequence_name}, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output")
	FString FileNameFormatOverride;
	
	/** Whether to build the FCPXML from sequence data directly (for reimporting) or from actual frame output data (for post processing) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output")
	FCPXMLExportDataSource DataSource;

protected:
	/** The file to write to */
	FString FilePath;

	bool bHasFinishedExporting;
};

#undef UE_API
