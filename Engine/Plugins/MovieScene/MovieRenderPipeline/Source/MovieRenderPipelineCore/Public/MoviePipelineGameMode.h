// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameFramework/GameModeBase.h"
#include "GameFramework/PlayerController.h"
#include "Kismet/GameplayStatics.h"
#include "MoviePipelineGameMode.generated.h"

UCLASS(MinimalAPI)
class AMoviePipelineGameMode : public AGameModeBase
{
	GENERATED_BODY()
public:
	virtual void BeginPlay() override
	{
		APlayerController* PlayerController = UGameplayStatics::GetPlayerController(this, 0);
		if(PlayerController)
		{
			const bool bCinematicMode = true;
			const bool bHidePlayer = true;
			const bool bHideHUD = true;
			const bool bPreventMovement = true;
			const bool bPreventTurning = true;
			PlayerController->SetCinematicMode(bCinematicMode, bHidePlayer, bHideHUD, bPreventMovement, bPreventTurning);
		}
	}
};
