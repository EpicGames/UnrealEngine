// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineSetting.h"
#include "GameFramework/GameModeBase.h"
#include "Templates/SubclassOf.h"
#include "MoviePipelineGameMode.h"
#include "Scalability.h"
#include "MoviePipelineGameOverrideSetting.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

UENUM(BlueprintType)
enum class EMoviePipelineTextureStreamingMethod : uint8
{
	/** This will not change the texture streaming method / cvars the users has set. */
	None UMETA(DisplayName="Don't Override" ),
	/** Disable the Texture Streaming system. Requires the highest amount of VRAM, but helps if Fully Load Used Textures still has blurry textures. */
	Disabled UMETA(DisplayName = "Disable Streaming"),
	/**  Fully load used textures instead of progressively streaming them in over multiple frames. Requires less VRAM but can occasionally still results in blurry textures. */
	FullyLoad UMETA(DisplayName = "Fully Load Used Textures")
};

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineGameOverrideSetting : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:
	UMoviePipelineGameOverrideSetting()
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: GameModeOverride(nullptr)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, SoftGameModeOverride(AMoviePipelineGameMode::StaticClass())
		, bCinematicQualitySettings(true)
		, TextureStreaming(EMoviePipelineTextureStreamingMethod::Disabled)
		, bUseLODZero(true)
		, bDisableHLODs(true)
		, bUseHighQualityShadows(true)
		, ShadowDistanceScale(10)
		, ShadowRadiusThreshold(0.001f)
		, bOverrideViewDistanceScale(true)
		, ViewDistanceScale(50)
		, bFlushGrassStreaming(true)
		, bOverrideGrassCullDistanceScale(true)
		, GrassCullDistanceScale(50.0f)
		, bOverrideGrassDensityScale(false)
		, GrassDensityScale(1.0f)
		, bFlushStreamingManagers(true)
		, bOverrideVirtualTextureFeedbackFactor(true)
		, VirtualTextureFeedbackFactor(1)
		, bDisableGPUTimeout(true)
	{
	}

public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "GameOverrideSettingDisplayName", "Game Overrides"); }
#endif
	virtual bool IsValidOnShots() const override { return false; }
	virtual bool IsValidOnPrimary() const override { return true; }
	UE_API virtual void BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const override;
	UE_API virtual void SetupForPipelineImpl(UMoviePipeline* InPipeline) override;
	UE_API virtual void TeardownForPipelineImpl(UMoviePipeline* InPipeline) override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	// Used to ensure we set the default values even if the user forgets to add it.
	virtual bool IgnoreTransientFilters() const override { return true; }

	//~ Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

protected:
	UE_API void ApplyCVarSettings(const bool bOverrideValues);

public:
	/** Optional Game Mode to override the map's default game mode with. This can be useful if the game's normal mode displays UI elements or loading screens that you don't want captured. */
	UE_DEPRECATED(5.5, "Please use the SoftGameModeOverride property instead.")
	UPROPERTY(BlueprintReadWrite, Category = "Game", meta = (DeprecatedProperty, DeprecationMessage = "Please use the SoftGameModeOverride property instead."))
	TSubclassOf<AGameModeBase> GameModeOverride;

	/** Optional Game Mode to override the map's default game mode with. This can be useful if the game's normal mode displays UI elements or loading screens that you don't want captured. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, DisplayName = "Game Mode Override", Category = "Game")
	TSoftClassPtr<AGameModeBase> SoftGameModeOverride;

	/** If true, automatically set the engine to the Cinematic Scalability quality settings during render. See the Scalability Reference documentation for information on how to edit cvars to add/change default quality values.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Game")
	bool bCinematicQualitySettings;

	/** Defines which If true, when using texture streaming fully load the required textures each frame instead of loading them in over time. This solves objects being blurry after camera cuts. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	EMoviePipelineTextureStreamingMethod TextureStreaming;

	/** Should we try to use the highest quality LOD for meshes and particle systems regardless of distance? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool bUseLODZero;

	/** Should we disable Hierarchical LODs and instead use their real meshes regardless of distance? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta=(ScriptName="DisableHlods;DisableHLODs"))
	bool bDisableHLODs;

	/** Should we override shadow-related CVars with some high quality preset settings? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool bUseHighQualityShadows;
	
	/** Scalability option to trade shadow distance versus performance for directional lights  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta=(EditCondition=bUseHighQualityShadows, ClampMin=0.1, UIMin=0.1, UIMax=10))
	int32 ShadowDistanceScale;

	/** Cull shadow casters if they are too small, value is the minimal screen space bounding sphere radius */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = bUseHighQualityShadows, UIMin=0.001, ClampMin=0.001))
	float ShadowRadiusThreshold;

	/** Should we override the View Distance Scale? Can be used in situations where MaxDrawDistance has been set before for in-game performance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (InlineEditConditionToggle))
	bool bOverrideViewDistanceScale;

	/** Controls the view distance scale. A primitive's MaxDrawDistance is scaled by this value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = bOverrideViewDistanceScale))
	int32 ViewDistanceScale;
	
	/** Flushing grass streaming (combined with override view distance scale) prevents visible pop-in/culling of grass instances. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool bFlushGrassStreaming;

	/** Should we override the Landscape Grass Cull Distance Scale? Can be used in situations where grass.CullDistanceScale has been set before for in-game performance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (InlineEditConditionToggle))
	bool bOverrideGrassCullDistanceScale;

	/** Controls the grass cull distance scale. See grass.CullDistanceScale */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = bOverrideGrassCullDistanceScale))
	float GrassCullDistanceScale;

	/** Should we override the Landscape Grass Density Scale? Can be used in situations where grass.densityScale has been set before for in-game performance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (InlineEditConditionToggle))
	bool bOverrideGrassDensityScale;

	/** Controls the grass density scale. See grass.densityScale */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (EditCondition = bOverrideGrassDensityScale))
	float GrassDensityScale;

	/** Experimental. If true flush the streaming managers (Texture Streaming) each frame. Allows Texture Streaming to not have visible pop-in in final frames. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool bFlushStreamingManagers;

	/** If true then override the virtual texture feedback resolution factor. Otherwise the value from the project renderer settings will be used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (InlineEditConditionToggle))
	bool bOverrideVirtualTextureFeedbackFactor;

	/** The virtual texture feedback resolution factor. A lower factor will increase virtual texture feedback resolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (editcondition = "bOverrideVirtualTextureFeedbackFactor"))
	int32 VirtualTextureFeedbackFactor;

	/** Should we disable the GPU Timeout? Currently only applicable when using D3D12 renderer. */
	bool bDisableGPUTimeout;

private:
	// To restore previous choices when we modify these at runtime. These will be unset if the user doesn't have the override enabled.
	Scalability::FQualityLevels PreviousQualityLevels;
	int32 PreviousFramesForFullUpdate;
	int32 PreviousFullyLoadUsedTextures;
	int32 PreviousTextureStreaming;
	int32 PreviousForceLOD;
	int32 PreviousSkeletalMeshBias;
	int32 PreviousParticleLODBias;
	float PreviousShadowDistanceScale;
	int32 PreviousShadowQuality;
	float PreviousShadowRadiusThreshold;
	float PreviousViewDistanceScale;
	float PreviousGrassCullDistanceScale;
	float PreviousGrassDensityScale;
	int32 PreviousGPUTimeout;
	int32 PreviousAnimationUROEnabled;
	int32 PreviousFoliageDitheredLOD;
	int32 PreviousFoliageForceLOD;
	int32 PreviousNeverMuteNonRealtimeAudio;
	bool PreviousNeverDisableSubmixes;
	int32 PreviousSkyLightRealTimeReflectionCaptureTimeSlice;
	int32 PreviousVolumetricRenderTarget;
	int32 PreviousVolumetricRenderTargetMode;
	int32 PreviousIgnoreStreamingPerformance;
	int32 PreviousStreamingManagerSyncState;
#if WITH_EDITOR
	int32 PreviousGeoCacheStreamerShowNotification;
	int32 PreviousGeoCacheStreamerBlockTillFinish;
#endif
	float PreviousChaosImmPhysicsMinStepTime;
	int32 PreviousSkipRedundantTransformUpdate;
	int32 PreviousChaosClothUseTimeStepSmoothing;
	int32 PreviousSkipWaterInfoTextureRenderWhenWorldRenderingDisabled;
	int32 PreviousNaniteVSMInvalidateOnLODDelta;
	int32 PreviousAllowSlateThrottling;
	bool PreviousAlphaOutput;
	int32 PreviousLUTSize;
};

#undef UE_API
