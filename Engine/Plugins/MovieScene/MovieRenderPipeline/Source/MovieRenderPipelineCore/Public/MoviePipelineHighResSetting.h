// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineSetting.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineUtils.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineAntiAliasingSetting.h"
#include "MoviePipelineTelemetry.h"

#include "MoviePipelineHighResSetting.generated.h"

UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineHighResSetting : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:
	UMoviePipelineHighResSetting()
		: TileCount(1)
		, TextureSharpnessBias(0.f)
		, OverlapRatio(0.f)
		, bOverrideSubSurfaceScattering(false)
		, BurleySampleCount(64)
		, bAllocateHistoryPerTile(true)
		, bPageToSystemMemory(false)
	{
	}
	
	FIntPoint CalculatePaddedBackbufferSize(const FIntPoint& InTileSize) const
	{
		int32 OverlappedPadX = FMath::CeilToInt(InTileSize.X * OverlapRatio);
		int32 OverlappedPadY = FMath::CeilToInt(InTileSize.Y * OverlapRatio);

		return InTileSize + FIntPoint(2 * OverlappedPadX, 2 * OverlappedPadY);
	}

public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "HighResSettingDisplayName", "High Resolution"); }
#endif
	virtual bool IsValidOnShots() const override { return true; }
	virtual bool IsValidOnPrimary() const override { return true; }
	virtual void ValidateStateImpl() override
	{
		Super::ValidateStateImpl();

		if (TileCount > 1)
		{
			ValidationResults.Add(NSLOCTEXT("MovieRenderPipeline", "HighRes_UnsupportedFeatures", "Tiling does not support all rendering features (bloom, some screen-space effects). Additionally, TAA is not supported and will be forced off when rendering. Use Spatial/Temporal sampling instead. Tiling does not support reference motion blur with the Path Tracer."));
			if (FMath::IsNearlyEqual(OverlapRatio, 0.f))
			{
				ValidationResults.Add(NSLOCTEXT("MovieRenderPipeline", "HighRes_OverlapNeeded", "Increase the overlap amount to avoid seams in the final image. More overlap results in longer renders so start at 0.1 and increase as needed."));
			}
			ValidationState = EMoviePipelineValidationState::Warnings;
		}
	}

	virtual void GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const override
	{
		Super::GetFormatArguments(InOutFormatArgs);

		InOutFormatArgs.FilenameArguments.Add(TEXT("tile_count"), FString::FromInt(TileCount));
		InOutFormatArgs.FilenameArguments.Add(TEXT("overlap_percent"), FString::SanitizeFloat(OverlapRatio));
		InOutFormatArgs.FileMetadata.Add(TEXT("unreal/highres/tileCount"), FString::FromInt(TileCount));
		InOutFormatArgs.FileMetadata.Add(TEXT("unreal/highres/overlapPercent"), FString::SanitizeFloat(OverlapRatio));
	}

	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override
	{
		InTelemetry->bUsesHighResTiling = true;
	}

#if WITH_EDITOR
	virtual FText GetFooterText(UMoviePipelineExecutorJob* InJob) const override
	{
		if (!InJob || !InJob->GetConfiguration())
		{
			return FText();
		}

		if (TileCount > 1)
		{
			// Get the settings not from the job (those are the cached values) but from our owning configuration, which may
			// exist transiently in the UI.
			UMoviePipelineConfigBase* OwningConfig = GetTypedOuter<UMoviePipelineConfigBase>();
			if (OwningConfig)
			{
				UMoviePipelineOutputSetting* OutputSettings = OwningConfig->FindSetting<UMoviePipelineOutputSetting>();
				UMoviePipelineAntiAliasingSetting* AASettings = OwningConfig->FindSetting<UMoviePipelineAntiAliasingSetting>();

				// If we don't have an output setting we know this is a shot override. If this is a shot override we'll fall back
				// to the cached primary configuration values. We don't do this for non-shot overrides as it would pull the cached
				// values, not the transient ones in the UI.
				if (!OutputSettings)
				{
					if (!InJob || !InJob->GetConfiguration())
					{
						return FText();
					}

					UMoviePipelinePrimaryConfig* PrimaryConfig = InJob->GetConfiguration();
					if (!OutputSettings)
					{
						OutputSettings = PrimaryConfig->FindSetting<UMoviePipelineOutputSetting>();
					}
					if (!AASettings)
					{
						AASettings = PrimaryConfig->FindSetting<UMoviePipelineAntiAliasingSetting>();
					}
				}

				// Shouldn't happen but better to have no footer text than crash.
				if (!OutputSettings)
				{
					return FText();
				}

				int32 NumTiles = TileCount * TileCount;
				int32 NumSamplesPerTick = 1;
				int32 NumSamplesPerTemporal = 1;
			
				if (AASettings)
				{
					NumSamplesPerTick = AASettings->SpatialSampleCount;
					NumSamplesPerTemporal = AASettings->TemporalSampleCount;
				}

				FIntPoint TileResolution = FIntPoint(
					FMath::CeilToInt((float)OutputSettings->OutputResolution.X / (float)TileCount),
					FMath::CeilToInt((float)OutputSettings->OutputResolution.Y / (float)TileCount));

				TileResolution = CalculatePaddedBackbufferSize(TileResolution);

				FFormatNamedArguments FormatArgs;
				FormatArgs.Add(TEXT("NumTiles")) = NumTiles;
				FormatArgs.Add(TEXT("NumPerFrameSamples")) = NumTiles * NumSamplesPerTick;
				FormatArgs.Add(TEXT("NumTotalFrameSamples")) = NumTiles * NumSamplesPerTick * NumSamplesPerTemporal;
				FormatArgs.Add(TEXT("ResolutionX")) = TileResolution.X;
				FormatArgs.Add(TEXT("ResolutionY")) = TileResolution.Y;

				FText OutputFormatString = NSLOCTEXT("MovieRenderPipeline", "HighRes_OutputInfoFmt", "This will render {NumTiles} tiles, each with an individual resolution of {ResolutionX}x{ResolutionY}. There will be a total of {NumPerFrameSamples} renders per tick, and {NumTotalFrameSamples} renders per output frame. Large per-tick sample counts may freeze the editor for long periods of time.");
				return FText::Format(OutputFormatString, FormatArgs);
			}
		}

		return FText();
	}
#endif

	virtual void SetupForPipelineImpl(UMoviePipeline* InPipeline) override
	{
		int32 NumSamples = bOverrideSubSurfaceScattering ? BurleySampleCount : 0;
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PrevBurleyOverride, TEXT("r.SSS.Burley.NumSamplesOverride"), NumSamples, true);
	}

	virtual void TeardownForPipelineImpl(UMoviePipeline* InPipeline) override
	{
		int32 NumSamples = 0; // Dummy
		MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(PrevBurleyOverride, TEXT("r.SSS.Burley.NumSamplesOverride"), NumSamples, false);
	}

public:
	/**
	* How many tiles should the resulting movie render be broken into? A tile should be no larger than
	* the maximum texture resolution supported by your graphics card (likely 16k), so NumTiles should be
	* ceil(Width/MaxTextureSize). More tiles mean more individual passes over the whole scene at a smaller
	* resolution which may help with gpu timeouts. Requires at least 1 tile. Tiling is applied evenly to
	* both X and Y.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = "1", ClampMin = "1", UIMax = "16"), Category = "Render Settings")
	int32 TileCount;
	
	/**
	* This bias encourages the engine to use a higher detail texture when it would normally use a lower detail
	* texture due to the size of the texture on screen. A more negative number means overall sharper images
	* (up to the detail resolution of your texture). Too much sharpness will cause visual grain/noise in the
	* resulting image, but this can be mitigated with more spatial samples.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = "-1", UIMax = "0"), Category = "Render Settings")
	float TextureSharpnessBias;
	
	/**
	* How much should each tile overlap each other (0-0.5). Decreasing the overlap will result in smaller individual
	* tiles (which means faster renders) but increases the likelyhood of edge-of-screen artifacts showing up which
	* will become visible in the final image as a "grid" of repeated problem areas.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = "0", ClampMin = "0", UIMax = "0.5", ClampMax = "1"), Category = "Render Settings")
	float OverlapRatio;

	/**
	* Sub Surface Scattering relies on history which is not available when using tiling. This can be overriden to use more samples
	* to improve the quality.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Render Settings")
	bool bOverrideSubSurfaceScattering;

	/*
	* How many samples should the Burley Sub Surface Scattering use?
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = "64", ClampMin = "0", UIMax = "1024", EditCondition="bOverrideSubSurfaceScattering"), Category = "Render Settings")
	int32 BurleySampleCount;
	
	/*
	* If true, allocate a unique history for each tile. This is needed to make some render features work, including auto-exposure and Lumen,
	* but should be disabled when dealing with extremely large resolutions as you will spend all of your GPU memory on history buffers.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Render Settings")
	bool bAllocateHistoryPerTile;

	/*
	* If true, persistent GPU data per high resolution tile is mirrored to system memory, allowing higher resolutions, but increasing render time.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Render Settings")
	bool bPageToSystemMemory;


private:
	/** What was the Burley CVar before we overrode it? */
	int32 PrevBurleyOverride;
};
