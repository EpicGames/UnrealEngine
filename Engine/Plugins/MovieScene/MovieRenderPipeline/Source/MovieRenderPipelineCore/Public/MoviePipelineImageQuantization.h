// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Engine/EngineTypes.h"
#include "ImagePixelData.h"

namespace UE
{
namespace MoviePipeline
{
	MOVIERENDERPIPELINECORE_API TUniquePtr<FImagePixelData> QuantizeImagePixelDataToBitDepth(const FImagePixelData* InData, const int32 TargetBitDepth, FImagePixelPayloadPtr InPayload = nullptr, bool bConvertToSrgb = true);
}
}
	  