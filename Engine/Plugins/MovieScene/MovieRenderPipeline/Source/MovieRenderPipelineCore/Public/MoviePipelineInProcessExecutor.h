// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineLinearExecutor.h"
#include "MoviePipeline.h"
#include "MoviePipelineInProcessExecutor.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UWorld;

/**
* This executor implementation can process an array of movie pipelines and
* run them inside the currently running process. This is intended for usage
* outside of the editor (ie. -game mode) as it will take over the currently
* running world/game instance instead of launching a new world instance like 
* the editor only PIE.
*/
UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineInProcessExecutor : public UMoviePipelineLinearExecutorBase
{
	GENERATED_BODY()

public:
	UMoviePipelineInProcessExecutor()
		: UMoviePipelineLinearExecutorBase()
		, bUseCurrentLevel(false)
		, RemainingInitializationFrames(-1)
	{
	}

	/** Use current level instead of opening new level */
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	bool bUseCurrentLevel;

	FMoviePipelineWorkFinishedNative& OnIndividualJobFinished()
	{
		return OnIndividualJobFinishedDelegateNative;
	}

protected:
	UE_API virtual void Start(const UMoviePipelineExecutorJob* InJob) override;
	UE_API virtual void CancelAllJobs_Implementation() override;
	UE_API virtual void CancelCurrentJob_Implementation() override;
private:
	UE_API void OnMapLoadFinished(UWorld* NewWorld);
	UE_API void OnMoviePipelineFinished(FMoviePipelineOutputData InOutputData);
	UE_API void OnApplicationQuit();
	UE_API void OnTick();

	UE_API void BackupState();
	UE_API void ModifyState(const UMoviePipelineExecutorJob* InJob);
	UE_API void RestoreState();

private:
	/** If using delayed initialization, how many frames are left before we call Initialize. Will be -1 if not actively counting down. */
	int32 RemainingInitializationFrames;

	struct FSavedState
	{
		bool bBackedUp = false;

		// PlayerController
		bool bCinematicMode = false;
		bool bHidePlayer = false;

		bool bUseFixedTimeStep = false;
		double FixedDeltaTime = 1.0;

		TOptional<FText> WindowTitle;
	};

	FSavedState SavedState;
	FMoviePipelineWorkFinishedNative OnIndividualJobFinishedDelegateNative;
};

#undef UE_API
