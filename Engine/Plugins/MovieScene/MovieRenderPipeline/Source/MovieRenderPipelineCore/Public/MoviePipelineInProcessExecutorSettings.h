// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"
#include "Misc/CommandLine.h"
#include "MoviePipelineInProcessExecutorSettings.generated.h"

/**
* This is the implementation responsible for executing the rendering of
* multiple movie pipelines after being launched via the command line.
*/
UCLASS(MinimalAPI, BlueprintType, config = Engine, defaultconfig, meta=(DisplayName = "Movie Pipeline New Process") )
class UMoviePipelineInProcessExecutorSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	UMoviePipelineInProcessExecutorSettings()
		: Super()
	{
		bCloseEditor = false;
		// -AudioCallbackBufferFrameSize/-AudioNumBuffersToEnqueue: match the buffer config that PIE applies to its
		// per-world NRT device (LevelEditorPlaySettings::bUseNonRealtimeAudioDevice path passes
		// BufferSizeOverride=32, NumBuffersOverride=2 in PlayLevel.cpp).
		AdditionalCommandLineArguments = TEXT("-NoLoadingScreen -FixedSeed -log -Unattended -MRQInstance -deterministicaudio -audiomixer -AudioCallbackBufferFrameSize=32 -AudioNumBuffersToEnqueue=2");
		InitialDelayFrameCount = 0;

		// Find all arguments from the command line and set them as the InheritedCommandLineArguments.
		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(FCommandLine::Get(), Tokens, Switches);
		for (auto& Switch : Switches)
		{
			InheritedCommandLineArguments.AppendChar('-');
			InheritedCommandLineArguments.Append(Switch);
			InheritedCommandLineArguments.AppendChar(' ');
		}
	}
	
		/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

	/** If enabled the editor will close itself when a new process is started. This can be used to gain some performance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Startup")
	bool bCloseEditor;

	/** A list of additional command line arguments to be appended to the new process startup. In the form of "-foo -bar=baz". Can be useful if your game requires certain arguments to start such as disabling log-in screens. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Startup")
	FString AdditionalCommandLineArguments;

	/** A list of command line arguments which are inherited from the currently running Editor instance that will be automatically appended to the new process. */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, config, Category = "Startup")
	FString InheritedCommandLineArguments;

	/** 
	* How long should we wait after being initialized to start doing any work? This can be used
	* to work around situations where the game is not fully loaded by the time the pipeline
	* is automatically started and it is important that the game is fully loaded before we do
	* any work (such as evaluating frames for warm-up).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, meta = (UIMin = "0", ClampMin = "0", UIMax = "150"), Category = "Startup")
	int32 InitialDelayFrameCount;
};
