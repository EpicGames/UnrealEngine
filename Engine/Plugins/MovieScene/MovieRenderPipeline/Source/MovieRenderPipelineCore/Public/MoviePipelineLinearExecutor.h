// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineExecutor.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineLinearExecutor.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMoviePipelineBase;
class UMoviePipelineQueue;

/**
* This is an abstract base class designed for executing an array of movie pipelines in linear
* fashion. It is generally the case that you only want to execute one at a time on a local machine
* and a different executor approach should be taken for a render farm that distributes out jobs
* to many different machines.
*/
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMoviePipelineLinearExecutorBase : public UMoviePipelineExecutorBase
{
	GENERATED_BODY()
	
public:
	UMoviePipelineLinearExecutorBase()
		: UMoviePipelineExecutorBase()
		, CurrentPipelineIndex(0)
		, JobsStarted(0)
		, bIsRendering(false)
	{
	}

protected:
	// UMoviePipelineExecutorBase Interface
	UE_API virtual void Execute_Implementation(UMoviePipelineQueue* InPipelineQueue) override;
	virtual bool IsRendering_Implementation() const override { return bIsRendering; }
	// ~UMoviePipelineExeuctorBase Interface

	UE_API virtual void StartPipelineByIndex(int32 InPipelineIndex);
	virtual void Start(const UMoviePipelineExecutorJob* InJob) {}
	UE_API virtual FText GetWindowTitle();
	UE_API virtual float GetCompletionPercentageFromActivePipeline();
public:
	/** Note: When using a Movie Graph Pipeline this pointer will always be null. */
	UE_API virtual void OnIndividualPipelineFinished(UMoviePipeline* /* FinishedPipeline */);
	UE_API virtual void OnExecutorFinishedImpl();
	/** Note: When using a Movie Graph Pipeline InPipeline will always be null. */
	UE_API virtual void OnPipelineErrored(UMoviePipeline* InPipeline, bool bIsFatal, FText ErrorText);

	UE_API virtual void CancelCurrentJob_Implementation();
	UE_API virtual void CancelAllJobs_Implementation();

protected:
	
	/** List of Pipeline Configs we've been asked to execute. */
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineQueue> Queue;

	/** A Movie Pipeline that has been spawned and is running (if any) */
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineBase> ActiveMoviePipeline;

	/** Which Pipeline Config are we currently working on. */
	int32 CurrentPipelineIndex;

	/** The number of jobs started by this executor during this execution. */
	int32 JobsStarted;

	/** Have we actually successfully started a render? */
	bool bIsRendering;

	/** Are we in the process of canceling all execution of the queue? Will stop new jobs from being started. */
	bool bIsCanceling;

private:
	/** Used to track total processing duration. */
	FDateTime InitializationTime;
};


#undef UE_API
