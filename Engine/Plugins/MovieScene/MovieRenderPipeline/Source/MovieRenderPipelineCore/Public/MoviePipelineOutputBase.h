// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "MovieRenderPipelineDataTypes.h"
#include "Engine/EngineTypes.h"
#include "MoviePipelineSetting.h"
#include "MoviePipelineOutputBase.generated.h"

class UMoviePipeline;
class UMoviePipelineExecutorShot;

/**
* A base class for different output containers for the data generated by the Movie Pipeline,
* i.e: image sequences, video containers, etc.
*/
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMoviePipelineOutputBase : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:

	/** 
	* Called when a frame is ready for output. This will contain all passes the user has requested for the given output frame. 
	* It also contains metrics about the output frame (such as frame number).
	*/
	void OnReceiveImageData(FMoviePipelineMergerOutputFrame* InMergedOutputFrame) { OnReceiveImageDataImpl(InMergedOutputFrame); }

	/** 
	* Called once when all frames have been produced for the pipeline. Use this as an indicator to start flushing to disk. 
	* Async processes can be started here and HasFinishedProcessing() will be called each frame until all all containers
	* return true, at which point Finalize will be called.
	*/
	void BeginFinalize() { BeginFinalizeImpl(); }

	/**
	* Called each frame to check to see if this output container has finished processing. Finalize won't be called until
	* all output containers return true for this. 
	*/
	bool HasFinishedProcessing() { return HasFinishedProcessingImpl(); }

	/** 
	* Called after all output containers have reported that they are done processing.
	*/
	void Finalize() { FinalizeImpl(); }
	
	/**
	* This is called when a shot ends, right after the last frame is rendered.
	*/
	void OnShotFinished(const UMoviePipelineExecutorShot* InShot, const bool bFlushToDisk) { OnShotFinishedImpl(InShot, bFlushToDisk); }

	/**
	* This is called during the Shutdown process of the Pipeline. This is after finalization.
	*/
	void OnPipelineFinished() { OnPipelineFinishedImpl(); }

	/**
	* Called at the end of the frame, before rendering has happened for that frame.
	*/
	void OnPostTick() { OnPostTickImpl(); }
	
protected:
	// UMoviePipelineOutputBase Interface
	virtual void OnReceiveImageDataImpl(FMoviePipelineMergerOutputFrame* InMergedOutputFrame) {}
	virtual void BeginFinalizeImpl() {}
	virtual bool HasFinishedProcessingImpl() { return true; }
	virtual void FinalizeImpl() {}
	virtual void OnShotFinishedImpl(const UMoviePipelineExecutorShot* InShot, const bool bFlushToDisk = false) {}
	virtual void OnPipelineFinishedImpl() {}
	virtual void OnPostTickImpl() {}
	
	/** Returns whether any overscan should be automatically cropped from the image before compositing and output */
	virtual bool ShouldCropOverscanImpl() const { return true; }
	// ~UMoviePipelineOutputBase

protected:
	virtual bool IsValidOnShots() const override { return false; }
	virtual bool IsValidOnPrimary() const override { return true; }
#if WITH_EDITOR
	virtual FText GetCategoryText() const override { return NSLOCTEXT("MovieRenderPipeline", "ExportsCategoryName_Text", "Exports"); }
#endif
};
