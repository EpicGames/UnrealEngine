// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MoviePipeline.h"
#include "MovieRenderPipelineDataTypes.h"
#include "Containers/Queue.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
struct FImagePixelData;
class UMoviePipeline;



class FMoviePipelineOutputMerger : public MoviePipeline::IMoviePipelineOutputMerger
{
public:
	FMoviePipelineOutputMerger(UMoviePipeline* InOwningMoviePipeline)
		: WeakMoviePipeline(MakeWeakObjectPtr(InOwningMoviePipeline))
	{
	}

public:
	UE_API virtual FMoviePipelineMergerOutputFrame& QueueOutputFrame_GameThread(const FMoviePipelineFrameOutputState& CachedOutputState) override;
	UE_API virtual void OnCompleteRenderPassDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) override;
	UE_API virtual void OnSingleSampleDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) override;
	UE_API virtual void AbandonOutstandingWork() override;
	virtual int32 GetNumOutstandingFrames() const override { return PendingData.Num(); }

public:
	TQueue<FMoviePipelineMergerOutputFrame> FinishedFrames;
private:
	/** The Movie Pipeline that owns us. */
	TWeakObjectPtr<UMoviePipeline> WeakMoviePipeline;

	/** Data that is expected but not fully available yet. Sorted by frame number. */
	TSortedMap<FMoviePipelineFrameOutputState, FMoviePipelineMergerOutputFrame> PendingData;

	/** Mutex that protects adding/updating/removing from ActiveData */
	FCriticalSection ActiveDataMutex;
};


#undef UE_API
