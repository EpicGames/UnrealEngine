// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineSetting.h"
#include "Engine/EngineTypes.h"
#include "Misc/FrameRate.h"
#include "MoviePipelineColorSetting.h"
#include "Sound/SoundSubmix.h"
#include "MoviePipelineOutputSetting.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineOutputSetting : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:
	UE_API UMoviePipelineOutputSetting();

#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "OutputSettingDisplayName", "Output"); }
	UE_API virtual FText GetFooterText(UMoviePipelineExecutorJob* InJob) const override;
	virtual bool CanBeDisabled() const override { return false; }
#endif
	virtual bool IsValidOnShots() const override { return false; }
	virtual bool IsValidOnPrimary() const override { return true; }
	UE_API virtual void GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const override;
	UE_API virtual void SetupForPipelineImpl(UMoviePipeline* InPipeline) override;

	// UObject Interface
	UE_API virtual void PostLoad() override;
	// ~UObject Interface

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:
	/** What directory should all of our output files be relative to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output")
	FDirectoryPath OutputDirectory;
	
	/** What format string should the final files use? Can include folder prefixes, and format string ({shot_name}, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output")
	FString FileNameFormat;

	/** What resolution should our output files be exported at? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output")
	FIntPoint OutputResolution;
	
public:
	/** Should we use the custom frame rate specified by OutputFrameRate? Otherwise defaults to Sequence frame rate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output")
	bool bUseCustomFrameRate;
	
	/** What frame rate should the output files be exported at? This overrides the Display Rate of the target sequence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(EditCondition=bUseCustomFrameRate), Category = "File Output")
	FFrameRate OutputFrameRate;
	
	/** Experimental way to interleave renders between multiple computers. If FrameStep is 5, and each render is offset by 0-4, they'll render a different subset of frames to add up to all. */
	int32 DEBUG_OutputFrameStepOffset;
	
	/** If true, output containers should attempt to override any existing files with the same name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output")
	bool bOverrideExistingOutput;
	
public:

	/**
	* Top level shot track sections will automatically be expanded by this many frames in both directions, and the resulting
	* additional time will be rendered as part of that shot. The inner sequence should have sections long enough to cover
	* this expanded range, otherwise these tracks will not evaluate during handle frames and may produce unexpected results.
	* This can be used to generate handle frames for traditional non linear editing tools.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0), Category = "Frames")
	int32 HandleFrameCount;

	/** Render every Nth frame. ie: Setting this value to 2 renders every other frame. Game Thread is still evaluated on 'skipped' frames for accuracy between renders of different OutputFrameSteps. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = "1", ClampMin = "1"), Category = "Frames")
	int32 OutputFrameStep;

	/** If true, override the Playback Range start/end bounds with the bounds specified below.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Frames")
	bool bUseCustomPlaybackRange;
	
	/** Used when overriding the playback range. In Display Rate frames. If bUseCustomPlaybackRange is false range will come from Sequence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(EditCondition=bUseCustomPlaybackRange), Category = "Frames")
	int32 CustomStartFrame;
	
	/** Used when overriding the playback range. In Display Rate frames. If bUseCustomPlaybackRange is false range will come from Sequence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(EditCondition=bUseCustomPlaybackRange), Category = "Frames")
	int32 CustomEndFrame;

public:
	/** The value to use for the version token if versions are not automatically incremented. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition = "!bAutoVersion", UIMin = 1, UIMax = 10, ClampMin = 1), Category = "Versioning")
	int32 VersionNumber;

	/** If true, version tokens will automatically be incremented with each local render. If false, the custom version number below will be used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Versioning")
	bool bAutoVersion;

	/** Exports additional .wav file for each submix if .wav output is selected, excludes the device's MasterSubmix. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output")
	TArray<TObjectPtr<USoundSubmix>> AdditionalSubmixOutputs;

public:
	/** How many digits should all output frame numbers be padded to? MySequence_1.png -> MySequence_0001.png. Useful for software that struggles to recognize frame ranges when non-padded. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = "1", UIMax = "5", ClampMin = "1"), AdvancedDisplay, Category = "File Output")
	int32 ZeroPadFrameNumbers;

	/** 
	* How many frames should we offset the output frame number by? This is useful when using handle frames on Sequences that start at frame 0,
	* as the output would start in negative numbers. This can be used to offset by a fixed amount to ensure there's no negative numbers. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "File Output")
	int32 FrameNumberOffset;

	/**
	* If true, the game thread will stall at the end of each shot to flush the rendering queue, and then flush any outstanding writes to disk, finalizing any
	* outstanding videos and generally completing the work. This is intentionally not exposed to the user interface as it is only relevant for scripting where
	* scripts may do post-shot callback work.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "File Output")
	bool bFlushDiskWritesPerShot;
};

#undef UE_API
