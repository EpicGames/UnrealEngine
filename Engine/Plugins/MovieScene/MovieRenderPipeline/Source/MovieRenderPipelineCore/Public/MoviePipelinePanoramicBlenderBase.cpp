// Copyright Epic Games, Inc. All Rights Reserved.
#include "MoviePipelinePanoramicBlenderBase.h"
#include "Math/PerspectiveMatrix.h"
#include "HAL/UnrealMemory.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelinePanoramicBlenderBase)

static TAutoConsoleVariable<int32> CVarMoviePipelinePanoramicMaxPoolsPerFrame(
	TEXT("MoviePipeline.Panoramic.MaxConcurrentBlendingPoolCount"),
	4,
	TEXT("When blending panoramic images, this determines the maximum number of concurrent blending pools. "
		"A larger number may result in better CPU usage, but can come at a significant cost to CPU memory. "
		"Lowering this value can reduce the amount of memory needed for panoramic blending, but will result in "
		"blending taking longer."),
	ECVF_Default
);

namespace UE::MoviePipeline
{
	static FLinearColor GetColorBilinearFiltered(const FImagePixelData* InSampleData, const FVector2D& InSamplePixelCoords, bool& OutClipped, bool bInForceAlphaToOpaque = false)
	{
		// Pixel coordinates assume that 0.5, 0.5 is the center of the pixel, so we subtract half to make it indexable.
		const FVector2D PixelCoordinateIndex = InSamplePixelCoords - 0.5f;

		// Get surrounding pixels indices
		FIntPoint LowerLeftPixelIndex = FIntPoint(FMath::FloorToInt(PixelCoordinateIndex.X), FMath::FloorToInt(PixelCoordinateIndex.Y));
		FIntPoint LowerRightPixelIndex = FIntPoint(LowerLeftPixelIndex + FIntPoint(1, 0));
		FIntPoint UpperLeftPixelIndex = FIntPoint(LowerLeftPixelIndex + FIntPoint(0, 1));
		FIntPoint UpperRightPixelIndex = FIntPoint(LowerLeftPixelIndex + FIntPoint(1, 1));

		// Clamp pixels indices to pixels array bounds.
		// ToDo: Is this needed? Should we handle wrap around for the bottom right pixel? What gives
		auto ClampPixelCoords = [&](FIntPoint& InOutPixelCoords, const FIntPoint& InArraySize)
			{
				if (InOutPixelCoords.X > InArraySize.X - 1 ||
					InOutPixelCoords.Y > InArraySize.Y - 1 ||
					InOutPixelCoords.X < 0 ||
					InOutPixelCoords.Y < 0)
				{
					OutClipped = true;
				}
				InOutPixelCoords = FIntPoint(FMath::Clamp(InOutPixelCoords.X, 0, InArraySize.X - 1), FMath::Clamp(InOutPixelCoords.Y, 0, InArraySize.Y - 1));
			};
		ClampPixelCoords(LowerLeftPixelIndex, InSampleData->GetSize());
		ClampPixelCoords(LowerRightPixelIndex, InSampleData->GetSize());
		ClampPixelCoords(UpperLeftPixelIndex, InSampleData->GetSize());
		ClampPixelCoords(UpperRightPixelIndex, InSampleData->GetSize());

		// Fetch the colors for the four pixels. We convert to FLinearColor here so that our accumulation
		// is done in linear space with enough precision. The samples are probably in F16 color right now.
		FLinearColor LowerLeftPixelColor;
		FLinearColor LowerRightPixelColor;
		FLinearColor UpperLeftPixelColor;
		FLinearColor UpperRightPixelColor;

		int64 SizeInBytes = 0;
		const void* SrcRawDataPtr = nullptr;
		InSampleData->GetRawData(SrcRawDataPtr, SizeInBytes);

		switch (InSampleData->GetType())
		{
		case EImagePixelType::Float16:
		{
			const FFloat16Color* ColorDataF16 = static_cast<const FFloat16Color*>(SrcRawDataPtr);
			LowerLeftPixelColor = FLinearColor(ColorDataF16[LowerLeftPixelIndex.X + (LowerLeftPixelIndex.Y * InSampleData->GetSize().X)]);
			LowerRightPixelColor = FLinearColor(ColorDataF16[LowerRightPixelIndex.X + (LowerRightPixelIndex.Y * InSampleData->GetSize().X)]);
			UpperLeftPixelColor = FLinearColor(ColorDataF16[UpperLeftPixelIndex.X + (UpperLeftPixelIndex.Y * InSampleData->GetSize().X)]);
			UpperRightPixelColor = FLinearColor(ColorDataF16[UpperRightPixelIndex.X + (UpperRightPixelIndex.Y * InSampleData->GetSize().X)]);
		}
		break;
		case EImagePixelType::Float32:
		{
			const FLinearColor* ColorDataF32 = static_cast<const FLinearColor*>(SrcRawDataPtr);
			LowerLeftPixelColor = ColorDataF32[LowerLeftPixelIndex.X + (LowerLeftPixelIndex.Y * InSampleData->GetSize().X)];
			LowerRightPixelColor = ColorDataF32[LowerRightPixelIndex.X + (LowerRightPixelIndex.Y * InSampleData->GetSize().X)];
			UpperLeftPixelColor = ColorDataF32[UpperLeftPixelIndex.X + (UpperLeftPixelIndex.Y * InSampleData->GetSize().X)];
			UpperRightPixelColor = ColorDataF32[UpperRightPixelIndex.X + (UpperRightPixelIndex.Y * InSampleData->GetSize().X)];
		}
		break;
		default:
			// Not implemented
			check(0);
		}

		const float FracX = (InSamplePixelCoords.X - LowerLeftPixelIndex.X - 0.5f);
		const float FracY = (InSamplePixelCoords.Y - LowerLeftPixelIndex.Y - 0.5f);

		FLinearColor InterpolatedPixelColor =
			(LowerLeftPixelColor * (1.0f - FracX) + LowerRightPixelColor * FracX) * (1.0f - FracY)
			+ (UpperLeftPixelColor * (1.0f - FracX) + UpperRightPixelColor * FracX) * FracY;

		// Force final color alpha to opaque if requested
		if (bInForceAlphaToOpaque)
		{
			InterpolatedPixelColor.A = 1.0f;
		}

		return InterpolatedPixelColor;
	}


static FLinearColor GetColorCubicFiltered(const FImagePixelData* InSampleData, const FVector2D& InSamplePixelCoords, float CubicBParam, float CubicCParam, bool& OutClipped, bool bInForceAlphaToOpaque = false)
{
	// Pixel coordinates assume that 0.5, 0.5 is the center of the pixel, so we subtract half to make it indexable.
	const FVector2D PixelCoordinateIndex = InSamplePixelCoords - 0.5f;

	// Get surrounding 4x4 pixels indices, because we floored our center is off-center to the lower-left.
	FIntPoint PixelCenter = FIntPoint(FMath::FloorToInt(PixelCoordinateIndex.X), FMath::FloorToInt(PixelCoordinateIndex.Y));

	auto ParameterizedCubic = [](float X, float B, float C)
		{
			float AbsX = FMath::Abs(X);
			float X2 = AbsX * AbsX;
			float X3 = AbsX * AbsX * AbsX;

			if (AbsX <= 1.f)
			{
				return((12 - 9 * B - 6 * C) * X3 +
					(-18 + 12 * B + 6 * C) * X2 +
					(6 - 2 * B)) / 6.0f;
			}
			else if (AbsX < 2.0f)
			{
				return ((-B - 6 * C) * X3 +
					(6 * B + 30 * C) * X2 +
					(-12 * B - 48 * C) * AbsX +
					(8 * B + 24 * C)) / 6.0f;
			}
			else
			{
				return 0.f;
			}
		};
	// The fractional amount we were within the pixel
	float FracX = InSamplePixelCoords.X - PixelCenter.X - 0.5f;
	float FracY = InSamplePixelCoords.Y - PixelCenter.Y - 0.5f;

	float WeightX[4];
	float WeightY[4];
	for (int32 i = 0; i < 4; i++)
	{
		WeightX[i] = ParameterizedCubic((i - 1) - FracX, CubicBParam, CubicCParam);
		WeightY[i] = ParameterizedCubic((i - 1) - FracY, CubicBParam, CubicCParam);
	}

	int64 SizeInBytes = 0;
	const void* SrcRawDataPtr = nullptr;
	InSampleData->GetRawData(SrcRawDataPtr, SizeInBytes);

	OutClipped = false;

	FLinearColor ResultingColor = FLinearColor(0, 0, 0, 0);
	for (int32 i = 0; i < 4; i++)
	{
		int32 yIndex = FMath::Clamp(PixelCenter.Y + i - 1, 0, InSampleData->GetSize().Y - 1);
		for (int32 j = 0; j < 4; j++)
		{
			int32 xIndex = FMath::Clamp(PixelCenter.X + j - 1, 0, InSampleData->GetSize().X - 1);
			int32 PixelIndex = (yIndex * InSampleData->GetSize().X) + xIndex;
			
			FLinearColor Sample = FLinearColor(0, 0, 0, 0);
			switch (InSampleData->GetType())
			{
			case EImagePixelType::Float16:
			{
				const FFloat16Color* ColorDataF16 = static_cast<const FFloat16Color*>(SrcRawDataPtr);
				Sample = FLinearColor(ColorDataF16[PixelIndex]);
				break;
			}
			case EImagePixelType::Float32:
			{
				const FLinearColor* ColorDataF32 = static_cast<const FLinearColor*>(SrcRawDataPtr);
				Sample = ColorDataF32[PixelIndex];
				break;
			}
			}

			ResultingColor += WeightX[j] * WeightY[i] * Sample;
		}
	}

	// Force final color alpha to opaque if requested
	if (bInForceAlphaToOpaque)
	{
		ResultingColor.A = 1.0f;
	}

	return ResultingColor;
}

void FMoviePipelinePanoramicBlenderBase::Initialize(const FIntPoint InOutputResolution)
{
	OutputEquirectangularMapSize = InOutputResolution;

	// If these are being re-used this should be a no-op
	OutputEquirectangularMap.SetNumUninitialized(OutputEquirectangularMapSize.X * OutputEquirectangularMapSize.Y);

	// We need to zero the memory though as the results are accumulated into it.
	FMemory::Memzero(OutputEquirectangularMap.GetData(), OutputEquirectangularMap.Num() * OutputEquirectangularMap.GetTypeSize());

	// Tasks get pushed into a concurrency limiter to avoid allocating too many concurrent blend pools which can take significant RAM on large images.
	// There is a fair amount of parallelism within each task, so high core count machines still get high occupancy even with low concurrency.
	TaskConcurrencyLimiter = MakeUnique<UE::Tasks::FTaskConcurrencyLimiter>(FMath::Max(CVarMoviePipelinePanoramicMaxPoolsPerFrame.GetValueOnAnyThread(), 1));
}

DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_PanoBlend"), STAT_MoviePipeline_PanoBlend, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_PanoBlendActual"), STAT_MoviePipeline_PanoBlendActual, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_PanoBlendJoin"), STAT_MoviePipeline_PanoBlendJoin, STATGROUP_MoviePipeline);

void FMoviePipelinePanoramicBlenderBase::BlendSample_AnyThread(TUniquePtr<FImagePixelData>&& InData, const UE::MoviePipeline::FPanoramicPane& Pane, TUniqueFunction<void(FLinearColor*, FIntPoint)> OnDebugSampleAvailable)
{

	TaskConcurrencyLimiter->Push(UE_SOURCE_LOCATION, [
		this,
		Pane, 
		InData = MoveTemp(InData),
		OnDebugSampleAvailable = MoveTemp(OnDebugSampleAvailable)](uint32 Slot)
		{
			SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_PanoBlend);
			const float BlendStartTime = FPlatformTime::Seconds();

			// The way blending works is that each sample that comes in gets its own memory to do the blending into. 
			// We calculate a bounding box for where the data would end up in the resulting final texture map, and then
			// blend into it. Then after all samples have come in, we can simply add the results from each blended image
			// together to get our final value, without having a lot of per-pixel contention during blending
			//
			// This math below works out the output dimensions for this sample, which will let us calculate the size.
			FIntPoint SampleSize = Pane.Resolution;
			FRotator SampleRotation = FRotator(Pane.bUseLocalRotation ? Pane.CameraLocalRotation : Pane.CameraRotation);

			const float SampleHalfHorizontalFoVDegrees = 0.5f * Pane.HorizontalFieldOfView;
			const float SampleHalfVerticalFoVDegrees = 0.5f * Pane.VerticalFieldOfView; // FMath::RadiansToDegrees((FMath::Atan((0.5f * SampleSize.X / FMath::Tan(DataPayload->Pane.Data.HorizontalFieldOfView / 2.f))))); // CTransformTools::getVFovFromHFovAndARInDegrees(iSampleFovInDegrees, static_cast<float>(iSampleSize.X) / static_cast<float>(iSampleSize.Y));
			const float SampleHalfHorizontalFoVCosine = FMath::Cos(FMath::DegreesToRadians(SampleHalfHorizontalFoVDegrees));
			const float SampleHalfVerticalFoVCosine = FMath::Cos(FMath::DegreesToRadians(SampleHalfVerticalFoVDegrees));

			// Now calculate which direction the Panoramic Pane (that this sample represents) was facing originally.
			const float SampleYawRad = FMath::DegreesToRadians(SampleRotation.Yaw);
			const float SamplePitchRad = FMath::DegreesToRadians(SampleRotation.Pitch);
			const FVector SampleDirectionOnTheta = FVector(FMath::Cos(SampleYawRad), FMath::Sin(SampleYawRad), 0);
			const FVector SampleDirectionOnPhi = FVector(FMath::Cos(SamplePitchRad), 0.f, FMath::Sin(SamplePitchRad));

			// Now construct a projection matrix representing the sample matching the original perspective it was taken from.
			const FMatrix SampleProjectionMatrix = FReversedZPerspectiveMatrix(FMath::DegreesToRadians(SampleHalfHorizontalFoVDegrees), SampleSize.X, SampleSize.Y, Pane.NearClippingPlane);
			// For our given output size, figure out how many degrees each pixel represents.
			const double EquiRectMapThetaStep = 360.0 / (double)OutputEquirectangularMapSize.X;
			const double EquiRectMapPhiStep = 180.0 / (double)OutputEquirectangularMapSize.Y;

			// Compute the index bounds in the equirectangular map corresponding to the sample bounds, so we don't loop over unnecessary pixels.
			// This is approximated according to the weighting function for blending too.
			// This assumes that the origin of the equirectangular map (0,0) has a yaw/pitch equal to -180/-90.
			// Phi evolves in the opposite direction of Y (Y's origin is up-left)
			// Pitch is clamped, because there is no vertical wrapping in the map
			// Yaw is not clamped, because horizontal wrapping is possible. 
			// The MinBound for X can actually be greater than the MaxBound due to wrapping, modulo is applied at eval time to ensure it wraps right.
			// Note that the bounds are computed with doubles. Very high precision is needed here due to the floor operation. With floats, there can
			// be slight errors at high resolutions that cause floor to evaluate to an incorrect bound (eg, could be 12287.999 which evaluates to
			// 12287 instead of the 12288). These slight errors in the bounds can lead to seams in the blended result.
			const double SampleYawMin = SampleRotation.Yaw - SampleHalfHorizontalFoVDegrees;
			const double SampleYawMax = SampleRotation.Yaw + SampleHalfHorizontalFoVDegrees;
			const int32 PixelIndexHorzMinBound = FMath::FloorToInt(((SampleYawMin)+180.0) / EquiRectMapThetaStep);
			const int32 PixelIndexHorzMaxBound = FMath::FloorToInt(((SampleYawMax)+180.0) / EquiRectMapThetaStep);

			const float SamplePitchMin = FMath::Max(SampleRotation.Pitch - SampleHalfVerticalFoVDegrees, -90.f); // Clamped to [-90, 90]
			const float SamplePitchMax = FMath::Min(SampleRotation.Pitch + SampleHalfVerticalFoVDegrees, 90.f); // Clamped to [-90, 90]
			const int32 PixelIndexVertMinBound = FMath::Max((OutputEquirectangularMapSize.Y) - FMath::FloorToInt((SamplePitchMax + 90.f) / EquiRectMapPhiStep), 0);
			const int32 PixelIndexVertMaxBound = FMath::Min((OutputEquirectangularMapSize.Y) - FMath::FloorToInt((SamplePitchMin + 90.f) / EquiRectMapPhiStep), OutputEquirectangularMapSize.Y);

			// Build a rect that describes which part of the output map we'll be rendering into
			const FIntPoint OutputBoundsMin = FIntPoint(PixelIndexHorzMinBound, PixelIndexVertMinBound);
			const FIntPoint OutputBoundsMax = FIntPoint(PixelIndexHorzMaxBound, PixelIndexVertMaxBound);

			const int32 PixelWidth = OutputBoundsMax.X - OutputBoundsMin.X;
			const int32 PixelHeight = OutputBoundsMax.Y - OutputBoundsMin.Y;
			FIntPoint Resolution = FIntPoint(PixelWidth, PixelHeight);

			// We need to find a free entry in the pool that matches our desired size. We pool these because
			// the memory allocation is expensive, and many of them will have similar sizes.
			FPoolEntry* FoundPool = nullptr;

			{
				FScopeLock ScopeLock(&PoolAccessMutex);

				// The concurrent task limiter should prevent us from allocating too many pools, so this will slowly
				// grow to match the max concurrency limiter.
				for (TUniquePtr<FPoolEntry>& Item : TempBufferPool)
				{
					if (!Item->bActive)
					{
						FoundPool = Item.Get();
						//UE_LOGF(LogTemp, Log, "Reused pool entry, size: %d,%d newSize: %d oldSize:%d", 
						//	Resolution.X, Resolution.Y, Resolution.X*Resolution.Y, Item->Data.Max());
						break;
					}
				}

				if (!FoundPool)
				{
					TUniquePtr<FPoolEntry> NewPoolEntry = MakeUnique<FPoolEntry>();
					FoundPool = NewPoolEntry.Get();
					TempBufferPool.Add(MoveTemp(NewPoolEntry));
					// UE_LOGF(LogTemp, Log, "Allocated new pool entry, size: %d,%d NewPoolCount: %d", Resolution.X, Resolution.Y, TempBufferPool.Num());
				}

				// Now that we've either found a pool for reuse we'll initialize enough of it that other threads
				// don't come in and try and use it, then we'll free the scope lock and finish initializing it.
				FoundPool->bActive = true;
				FoundPool->Resolution = Resolution;
				FoundPool->OutputBoundsMin = OutputBoundsMin;
				FoundPool->OutputBoundsMax = OutputBoundsMax;
			}


			check(FoundPool);
			// We ensure we always have room for the resolution as that is what will be blended into it.
			int32 MaxSizeX = FMath::Max(SampleSize.X, FoundPool->Resolution.X);
			int32 MaxSizeY = FMath::Max(SampleSize.Y, FoundPool->Resolution.Y);
			// This should generally avoid reallocations, smaller blended tiles will just use a sub-region of
			// the memory.
			FoundPool->Data.SetNumUninitialized(MaxSizeX * MaxSizeY, EAllowShrinking::No);

			// We need to zero-initialize the data in this patch, especially if it was re-used, because we additively
			// add images together later.
			FMemory::Memzero(FoundPool->Data.GetData(), FoundPool->Data.Num() * FoundPool->Data.GetTypeSize());

			// Finally we can perform our actual blending. We blend into our intermediate buffer
			// instead of the final output array to avoid multiple threads contending for pixels.
			// This uses the resolution of the output (in blended space) as the size, and we pulls
			// from the appropriate place in the incoming data.
			ParallelFor(Resolution.Y,
				[
					this,
					PixelIndexVertMinBound, 
					PixelIndexHorzMinBound,
					EquiRectMapThetaStep,
					EquiRectMapPhiStep,
					SampleDirectionOnPhi,
					SampleDirectionOnTheta,
					SampleHalfHorizontalFoVCosine,
					SampleHalfVerticalFoVCosine,
					&SampleRotation,
					&SampleProjectionMatrix,
					&SampleSize,
					&Pane,
					InDataPtr = InData.Get(),
					FoundPoolPtr = FoundPool
				](int32 RowY)
				{
					SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_PanoBlendActual);
					for (int32 RowX = 0; RowX < FoundPoolPtr->Resolution.X; RowX++)
					{
						int32 Y = PixelIndexVertMinBound + RowY;
						int32 X = PixelIndexHorzMinBound + RowX;

						// These X, Y coordinates are in output resolution space which is where we want to blend to.
						// Our X bounds may go OOB, but we wrap horizontally so we need to figure out the proper X index.
						const int32 OutputPixelX = ((X % OutputEquirectangularMapSize.X) + OutputEquirectangularMapSize.X) % OutputEquirectangularMapSize.X;
						const int32 OutputPixelY = Y;

						// Get the spherical coordinates (Theta and Phi) corresponding to the X and Y of the equirectangular map coordinates, converted to
						// [-180, 180] and [-90, 90] coordinate space respectively. The half pixel offset is used to make the center of a pixel be considered
						// that coordinate, and Phi increments in the opposite direction of Y.
						const float Theta = EquiRectMapThetaStep * (((float)OutputPixelX) + 0.5f) - 180.f;
						const float Phi = EquiRectMapPhiStep * (((float)OutputEquirectangularMapSize.Y - OutputPixelY) + 0.5f) - 90.f;
						// Now convert the spherical coordinates into an actual direction (on the output map)
						const float ThetaDeg = FMath::DegreesToRadians(Theta);
						const float PhiDeg = FMath::DegreesToRadians(Phi);
						const FVector OutputDirection(FMath::Cos(PhiDeg) * FMath::Cos(ThetaDeg), FMath::Cos(PhiDeg) * FMath::Sin(ThetaDeg), FMath::Sin(PhiDeg));
						const FVector OutputDirectionTheta = FVector(FMath::Cos(ThetaDeg), FMath::Sin(ThetaDeg), 0);
						const FVector OutputDirectionPhi = FVector(FMath::Cos(PhiDeg), 0.f, FMath::Sin(PhiDeg));

						// Now we can compute how much the sample should influence this pixel. It is weighted by angular distance to the direction
						// so that the edges have less influence (where they'd be more distorted anyways).
						const float DirectionPhiDot = FVector::DotProduct(OutputDirectionPhi, SampleDirectionOnPhi);
						const float DirectionThetaDot = FVector::DotProduct(OutputDirectionTheta, SampleDirectionOnTheta);

						// The divide is important, as otherwise at large resolutions the individual weights become really small for the whole image.
						const float WeightTheta = FMath::Max(DirectionThetaDot - SampleHalfHorizontalFoVCosine, 0.0f) / (1.0f - SampleHalfHorizontalFoVCosine);
						const float WeightPhi = FMath::Max(DirectionPhiDot - SampleHalfVerticalFoVCosine, 0.0f) / (1.0f - SampleHalfVerticalFoVCosine);

						const float SampleWeight = WeightTheta * WeightPhi;
						const float SampleWeightSquared = SampleWeight * SampleWeight; // Exponential falloff produces a nicer blending result.

						// The sample weight may be very small and not worth influencing this pixel.
						if (SampleWeightSquared > KINDA_SMALL_NUMBER)
						{
							// Transform the direction vector from the equirectangular map world space to sample world space
							FVector4 DirectionInSampleWorldSpace = FVector4(SampleRotation.UnrotateVector(OutputDirection), 1.0f);

							static const FMatrix UnrealCoordinateConversion = FMatrix(
								FPlane(0, 0, 1, 0),
								FPlane(1, 0, 0, 0),
								FPlane(0, 1, 0, 0),
								FPlane(0, 0, 0, 1));
							DirectionInSampleWorldSpace = UnrealCoordinateConversion.TransformFVector4(DirectionInSampleWorldSpace);

							// Then project that direction into sample clip space
							FVector4 DirectionInSampleClipSpace = SampleProjectionMatrix.TransformFVector4(DirectionInSampleWorldSpace);

							// Converted into normalized device space (Divide by w for perspective)
							FVector DirectionInSampleNDSpace = FVector(DirectionInSampleClipSpace) / DirectionInSampleClipSpace.W;

							// Get the final pixel coordinates (direction in screen space)
							FVector2D DirectionInSampleScreenSpace = ((FVector2D(DirectionInSampleNDSpace) + 1.0f) / 2.0f) * FVector2D(SampleSize.X, SampleSize.Y);

							DirectionInSampleScreenSpace.Y = ((float)SampleSize.Y - DirectionInSampleScreenSpace.Y) - 1.0f;

							// Do a bilinear color sample at the pixel coordinates (from the sample), weight it, and add it to the output map.
							bool bClipped = false;
							FLinearColor SampleColor;
							if (Pane.FilterType == EMoviePipelinePanoramicFilterType::Bilinear)
							{
								SampleColor = GetColorBilinearFiltered(InDataPtr, DirectionInSampleScreenSpace, bClipped, true);
							}
							else
							{
								UE::MoviePipeline::FPanoramicPane::FCubicInterpolationParams CubicParams = UE::MoviePipeline::FPanoramicPane::FCubicInterpolationParams::GetParamsForType(Pane.FilterType);
								SampleColor = GetColorCubicFiltered(InDataPtr, DirectionInSampleScreenSpace, CubicParams.ParamB, CubicParams.ParamC, bClipped, true);
							}

							if (!bClipped)
							{
								// When we calculate the actual output location we need to shift the X/Y. This is because up until now the math has been done in
								// output resolution space, but each sample only allocates a color map big enough for itself. It'll get shifted back out to the
								// right location later.
								int32 SampleOutputX = OutputPixelX - FoundPoolPtr->OutputBoundsMin.X;
								// Mod this again by our output map so we don't OOB on it. It'll wrap weirdly in the output map but should restore fine.
								SampleOutputX = ((SampleOutputX % (FoundPoolPtr->Resolution.X)) + (FoundPoolPtr->Resolution.X)) % (FoundPoolPtr->Resolution.X); // Positive Mod
								int32 SampleOutputY = Y;
								SampleOutputY -= FoundPoolPtr->OutputBoundsMin.Y;

								const int32 FinalIndex = SampleOutputX + (SampleOutputY * (FoundPoolPtr->Resolution.X));

								FoundPoolPtr->Data[FinalIndex] += SampleColor * SampleWeightSquared;
							}
						}
					}
				});

			// We don't have a great way to make this abstract between the two systems (as they have different payloads they want in the output merger)
			// and we need the work to be done mid-cycle before this function returns, so we call an event and let them COPY the data if they want, and
			// then we assume the data still exist and blend it into our output image before we release the pooled data.
			// Once we have finished doing the blending, we optionally pass it along to the output merger as a debug sample.
			if (OnDebugSampleAvailable.IsSet())
			{
				OnDebugSampleAvailable(FoundPool->Data.GetData(), FoundPool->Resolution);
			}

			// Now that blending is complete on this sample, we can place it in the output map. We want this to be fast, because other threads are probably waiting
			// to do the same thing, but we want to do this as soon as possible to return the blended-space array to the pool so that subsequent samples could
			// potentially reuse it within the same frame.
			{
				FScopeLock ScopeLock(&OutputMapAccessMutex);
				ParallelFor(FoundPool->Resolution.Y, 
					[this, FoundPoolPtr = FoundPool] (int32 SampleY) {
					SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_PanoBlendJoin);

					for (int32 SampleX = 0; SampleX < FoundPoolPtr->Resolution.X; SampleX++)
					{
						int32 OriginalX = SampleX + FoundPoolPtr->OutputBoundsMin.X;
						int32 OriginalY = SampleY + FoundPoolPtr->OutputBoundsMin.Y;
						const int32 OutputPixelX = ((OriginalX % OutputEquirectangularMapSize.X) + OutputEquirectangularMapSize.X) % OutputEquirectangularMapSize.X;
						const int32 OutputPixelY = OriginalY;

						int32 SourceIndex = SampleX + (SampleY * (FoundPoolPtr->Resolution.X));
						int32 DestIndex = OutputPixelX + (OutputPixelY * OutputEquirectangularMapSize.X);
						OutputEquirectangularMap[DestIndex] += FoundPoolPtr->Data[SourceIndex];
					}
					});
			}

			// Finally we release the pool entry back to the pool.
			{
				// We don't take the scope lock because there can be threads above in a busy loop waiting on the lock,
				// and we need to release this entry back to the pool so they can consume it. bActive is guarded an
				// atomic so it's okay to write to it even as other threads are trying to read to see if it's no longer active.
				FoundPool->bActive = false;

				// We no longer have control over this pool entry as we've released it to the pool, null it out
				// so it doesn't accidentally get used.
				FoundPool = nullptr;
			}
		});
}

void FMoviePipelinePanoramicBlenderBase::FetchFinalPixelDataHalfFloat(TArray64<FFloat16Color>& OutPixelData) const
{
	OutPixelData.SetNumUninitialized(OutputEquirectangularMap.Num());

	ParallelFor(OutputEquirectangularMapSize.Y, [&](int32 InIndexY)
		{
			for (int32 FullX = 0L; FullX < OutputEquirectangularMapSize.X; FullX++)
			{
				// Be careful with this index, make sure to use 64bit math, not 32bit
				int64 Index = (int64(InIndexY) * int64(OutputEquirectangularMapSize.X)) + int64(FullX);
				FLinearColor Color = OutputEquirectangularMap[Index];

				// Normalize by the weight in the alpha channel
				Color.R /= Color.A;
				Color.G /= Color.A;
				Color.B /= Color.A;
				Color.A = 1.f;

				OutPixelData[Index] = FFloat16Color(Color);
			}
		});
}

void FMoviePipelinePanoramicBlenderBase::FetchFinalPixelDataLinearColor(TArray64<FLinearColor>& OutPixelData) const
{
	OutPixelData.SetNumUninitialized(OutputEquirectangularMap.Num());

	ParallelFor(OutputEquirectangularMapSize.Y, [&](int32 InIndexY)
		{
			for (int32 FullX = 0L; FullX < OutputEquirectangularMapSize.X; FullX++)
			{
				// Be careful with this index, make sure to use 64bit math, not 32bit
				int64 Index = (int64(InIndexY) * int64(OutputEquirectangularMapSize.X)) + int64(FullX);
				FLinearColor Color = OutputEquirectangularMap[Index];

				// Normalize by the weight in the alpha channel
				Color.R /= Color.A;
				Color.G /= Color.A;
				Color.B /= Color.A;
				Color.A = 1.f;

				OutPixelData[Index] = Color;
			}
		});
}
	
} // UE::MoviePipeline 
