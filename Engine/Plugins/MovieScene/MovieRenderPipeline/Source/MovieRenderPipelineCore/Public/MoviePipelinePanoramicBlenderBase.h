// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MovieRenderPipelineDataTypes.h"
#include "Tasks/TaskConcurrencyLimiter.h"
#include "MoviePipelinePanoramicBlenderBase.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declares
struct FImagePixelData;

UENUM(BlueprintType)
enum class EMoviePipelinePanoramicFilterType : uint8
{
	/** 2x2 Bilinear Interpolation. Fastest, nearly the same quality as other options. */
	Bilinear,
	/** Cubic Catmull-Rom interpolation. Slightly sharper than other results. Uses B=0, C=1/2 in parameterized cubic equation. */
	Catmull,
	/** Cubic Mitchell-Netravali interpolation. More neutral look. Uses B=0.33, C=0.33 in parameterized cubic equation. */
	Mitchell
};

namespace UE::MoviePipeline
{
	struct FPanoramicPane
	{
		// The camera location as defined by the actual sequence, consistent for all panes.
		FVector OriginalCameraLocation;
		// The camera location last frame, used to ensure camera motion vectors are right.
		FVector PrevOriginalCameraLocation;
		// The camera rotation as defined by the actual sequence
		FRotator OriginalCameraRotation;
		// The camera rotation last frame, used to ensure camera motion vectors are right.
		FRotator PrevOriginalCameraRotation;
		// The near clip plane distance from the camera.
		float NearClippingPlane;

		// How far apart are the eyes (total) for stereo?
		float EyeSeparation;
		float EyeConvergenceDistance;

		// The horizontal field of view this pane was rendered with
		float HorizontalFieldOfView;
		float VerticalFieldOfView;

		FIntPoint Resolution;

		// The actual rendering location for this pane, offset by the stereo eye if needed.
		FVector CameraLocation;
		FVector PrevCameraLocation;
		FRotator CameraRotation;
		FRotator CameraLocalRotation;
		FRotator PrevCameraRotation;

		// If true, uses only the CameraLocalRotation which means that if the camera yaws, so will the resulting blended image.
		bool bUseLocalRotation;

		// How many horizontal segments are there total.
		int32 NumHorizontalSteps;
		int32 NumVerticalSteps;

		// Which horizontal segment are we?
		int32 HorizontalStepIndex;
		// Which vertical segment are we?
		int32 VerticalStepIndex;

		struct FCubicInterpolationParams
		{
			float ParamB;
			float ParamC;

			static FCubicInterpolationParams GetParamsForType(const EMoviePipelinePanoramicFilterType InType)
			{
				FCubicInterpolationParams Params;
				switch (InType)
				{
				case EMoviePipelinePanoramicFilterType::Catmull:
					Params.ParamB = 0.0f;
					Params.ParamC = 0.5f;
					break;
				case EMoviePipelinePanoramicFilterType::Mitchell:
					Params.ParamB = 0.33f;
					Params.ParamC = 0.33f;
					break;
				default:
					ensureMsgf(false, TEXT("GetParamsForType shouldn't be called for non-cubic interpolations."));
					Params.ParamB = 0.0f;
					Params.ParamC = 0.0f;
					break;
				}
				return Params;
			}

		};
		
		EMoviePipelinePanoramicFilterType FilterType = EMoviePipelinePanoramicFilterType::Bilinear;

		// When indexing into arrays of Panes, which index is this?
		int32 GetAbsoluteIndex() const
		{
			const int32 EyeOffset = EyeIndex == -1 ? 0 : EyeIndex;
			const int32 NumEyeRenders = EyeIndex == -1 ? 1 : 2;
			return  (VerticalStepIndex * NumHorizontalSteps * NumEyeRenders) + HorizontalStepIndex + EyeOffset;
		}

		// -1 if no stereo, 0 left eye, 1 right eye.
		int32 EyeIndex;
	};
		
	/**
	* This class is responsible for blending a together a single Equirectangular Image from a series of individual
	* renders, and can blend multiple samples in a threadsafe way. For each incoming sample the pixel data should
	* contain information about panoramic pane (orientation, index, etc.), and one all samples have been called
	* BlendSample_AnyThread, it is safe for the owner of this instance to fetch the image data, which returns a
	* copy of the output image. You can then call Initialize() on it to reset the output image without reallocating
	* memory, which allows for reusing a given output blender for multiple frames.
	* 
	* This implementation works by allocating memory for each incoming sample that is the size of the data
	* once blended. Depending on where in the projection it is, different samples will take up different parts
	* of the output image (with different resolutions) so the pool stores available buffers by resolution.
	* Once a pool is either found or allocated for the sample, the incoming data is read from. Instead of taking
	* each sample in the incoming data and figuring out where it would go in the output image, we instead work backwards,
	* and calculate from each output pixel in the range that the sample would affect, sample the source image with filtering.
	* 
	* Once the blending into the temporary buffer is complete, a lock on the output array is taken and the data is added to
	* the output, and the buffer is returned to the pool.
	*/
	class FMoviePipelinePanoramicBlenderBase
	{
	public:
		UE_API void Initialize(const FIntPoint InOutputResolution);
	public:
		UE_API void BlendSample_AnyThread(TUniquePtr<FImagePixelData>&& InData, const UE::MoviePipeline::FPanoramicPane& Pane, TUniqueFunction<void(FLinearColor*, FIntPoint)> OnDebugSampleAvailable);
		UE_API void FetchFinalPixelDataHalfFloat(TArray64<FFloat16Color>& OutPixelData) const;
		UE_API void FetchFinalPixelDataLinearColor(TArray64<FLinearColor>& OutPixelData) const;

		TUniquePtr<UE::Tasks::FTaskConcurrencyLimiter> TaskConcurrencyLimiter;
	private:
		struct FPoolEntry
		{
			std::atomic<bool> bActive;
			TArray64<FLinearColor> Data;

			FIntPoint Resolution;
			FIntPoint OutputBoundsMin;
			FIntPoint OutputBoundsMax;
		};

		TArray64<TUniquePtr<FPoolEntry>> TempBufferPool;
		FCriticalSection PoolAccessMutex;
		FCriticalSection OutputMapAccessMutex;

		TArray64<FLinearColor> OutputEquirectangularMap;
		FIntPoint OutputEquirectangularMapSize;

	};
}

#undef UE_API
