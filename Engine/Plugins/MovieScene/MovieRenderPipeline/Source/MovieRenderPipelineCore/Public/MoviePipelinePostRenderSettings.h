// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#include "MoviePipelinePostRenderSettings.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Note: There are a number of enums in here which are named with a MovieGraph prefix. This is no longer accurate; these are
// now usable in the broader movie pipeline with the legacy system as well, but the naming is kept as-is for backwards compatibility.

/** The type of viewer that should be used to play back media. */
UENUM(BlueprintType)
enum class EMovieGraphPlaybackMethod : uint8
{
	/** Use the operating system's default application for playing back media of the type that's being viewed. */
	OperatingSystem,

	/** Use a specific application for playing back media. */
	CustomViewer
};

/** The frame range for image sequences that should be used during playback. */
UENUM(BlueprintType)
enum class EMovieGraphImageSequencePlaybackRange : uint8
{
	/** Only play back the first frame. */
	FirstFrameOnly,

	/** Play back all available frames. */
	FullRange
};

/** Dictates which jobs should be used to get media from for playback if more than one job generated media. */
UENUM(BlueprintType)
enum class EMovieGraphJobPlaybackRange : uint8
{
	/** Only play back media from the first job that was rendered. */
	FirstJobOnly,

	/** Play back media from all of the jobs that were rendered. */
	AllJobs
};

/** Dictates the selection of render layers that should be used in playback if more than one was generated. */
UENUM(BlueprintType)
enum class EMovieGraphRenderLayerPlaybackRange : uint8
{
	/** Only play back media from the first render layer that was rendered. */
	FirstRenderLayerOnly,

	/** Play back media from all render layers that were rendered. */
	AllRenderLayers
};

/** The symbol that should be used to substitute a frame number within image paths. */
UENUM(BlueprintType)
enum class EMovieGraphFrameRangeNotation : uint8
{
	/**
	 * Hash symbol. Includes the rendered frame range after the file path(s) are given on the command line.
	 * For example, "Layer1.#.exr 100-150" for an image sequence that included the frame number in the file name, and a frame range of 100-150 was rendered. 
	 */
	HashWithStartEndFrame UMETA(DisplayName = "# with Start/End Frame"),

	/**
	 * Hash symbol. Does not include the rendered frame range on the command line.
	 * For example, "Layer1.#.exr" for an image sequence that included the frame number in the file name. 
	 */
	Hash UMETA(DisplayName = "# without Start/End Frame"),

	/**
	 * $F. Includes the rendered frame range before the file paths(s) are given on the command line.
	 * For example, "-f 100 150 1 Layer1.$F.exr" for an image sequence that included the frame number in the file name, and a frame range of 100-150 was rendered. 
	 */
	DollarF UMETA(DisplayName = "$F"),

	/**
	 * Use the path for the first rendered frame only.
	 * For example, "Layer1.100.exr", if frame 100 was the first frame rendered. 
	 */
	StartFrame
};

/** How media should be selected for playback when more than one type was generated by a render. */
UENUM(BlueprintType)
enum class EMovieGraphOutputTypePlayback : uint8
{
	/**
	 * Use the priority order when determining the media to play (the media with the extension that has the highest priority is played).
	 * The priority order is specified in Editor Preferences.
	 */
	UsePriorityOrder,

	/**
	 * Play all media types that were generated from the render. Note that the highest-priority media dictates the player that will be used. In order
	 * for lower-priority media to also be played back, the lower-priority media must be configured to use the same player that the highest-priority
	 * is configured to use.
	 */
	PlayAllOutputTypes
};

/** When opening directories via the Play After Render feature, this dictates how they're opened. */
UENUM(BlueprintType)
enum class EMovieGraphDirectoryOpenBehavior : uint8
{
	/** Open the output directory for the first job in the queue only. */
	OpenDirectoryForFirstJob,

	/** Open the output directory for all jobs in the queue. */
	OpenDirectoryForAllJobs
};

/** Options that dictate how videos should be played back after a render completes. */
USTRUCT(BlueprintType)
struct FMovieGraphPostRenderVideoPlayOptions
{
	GENERATED_BODY()
	
	UE_API FMovieGraphPostRenderVideoPlayOptions();
	
	/** Determines which application is used to open rendered media for playback. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	EMovieGraphPlaybackMethod PlaybackMethod;

	/** If using a custom viewer, this is the path to the executable to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "PlaybackMethod == EMovieGraphPlaybackMethod::CustomViewer", EditConditionHides))
	FFilePath PlayerExecutable;

	/** If using a custom viewer, these command line arguments are passed along. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "PlaybackMethod == EMovieGraphPlaybackMethod::CustomViewer", EditConditionHides))
	FString AdditionalCommandLineArguments;

	/** The jobs that should have their media played. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "PlaybackMethod == EMovieGraphPlaybackMethod::CustomViewer"))
	EMovieGraphJobPlaybackRange JobPlayback;

	/** The render layers that should have their media played. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "PlaybackMethod == EMovieGraphPlaybackMethod::CustomViewer"))
	EMovieGraphRenderLayerPlaybackRange RenderLayerPlayback;
};

/** Options that dictate how images should be played back after a render completes. */
USTRUCT(BlueprintType)
struct FMovieGraphPostRenderImageSequencePlayOptions : public FMovieGraphPostRenderVideoPlayOptions
{
	GENERATED_BODY()

	UE_API FMovieGraphPostRenderImageSequencePlayOptions();

	/** The symbol that will be used to replace the frame number in image paths when providing those paths to external viewers. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (DisplayAfter = "AdditionalCommandLineArguments", EditCondition = "PlaybackMethod == EMovieGraphPlaybackMethod::CustomViewer", EditConditionHides))
	EMovieGraphFrameRangeNotation FrameRangeNotation;

	/** The range of frames to play back (only applicable to image sequences). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (DisplayAfter = "FrameRangeNotation", EditCondition = "PlaybackMethod == EMovieGraphPlaybackMethod::CustomViewer"))
	EMovieGraphImageSequencePlaybackRange PlaybackRange;
};

/** Settings that dictate how media is played back after a render finishes. */
USTRUCT(BlueprintType)
struct FMovieGraphPostRenderSettings
{
	GENERATED_BODY()

	UE_API FMovieGraphPostRenderSettings();

	/**
	 * When deciding which media to play, media with extensions ranked higher in this list will play instead of media ranked lower.
	 * For example, if EXR is listed above PNG, and both EXRs and PNGs were rendered, the EXRs will play instead of the PNGs.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Play Render Output")
	TArray<FString> OutputTypePriorityOrder;

	/**
	 * If set to Use Priority Order, the Output Type Priority Order list will be used to determine which media to play (if there were multiple types
	 * of images and/or videos generated). If set to Play All Output Types, all media will be opened up in the player if possible. The media that
	 * is highest in the Output Type Priority Order will have its player used, and the other media will also be opened if it has been assigned this
	 * player as well (ie, two or more players will not be opened). If using the Operating System to open media, this setting is effectively ignored
	 * because only the highest-ranked media in Output Type Priority Order will be opened.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Play Render Output")
	EMovieGraphOutputTypePlayback OutputTypePlayback;

	/** When opening a directory instead of playing back media, this dictates which directories will be opened. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Play Render Output")
	EMovieGraphDirectoryOpenBehavior DirectoryOpenBehavior;

	/** These options are used to determine how to play the rendered images. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Play Render Output", DisplayName = "Image Sequences")
	FMovieGraphPostRenderImageSequencePlayOptions PostRenderImageSequencePlayOptions;

	/** These options are used to determine how to play Apple ProRes movies. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Play Render Output", DisplayName = "Apple ProRes Movies")
	FMovieGraphPostRenderVideoPlayOptions PostRenderAppleProResPlayOptions;

	/** These options are used to determine how to play Avid DNxHR movies. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Play Render Output", DisplayName = "Avid DNxHR Movies")
	FMovieGraphPostRenderVideoPlayOptions PostRenderAvidDNxHRPlayOptions;

	/** These options are used to determine how to play H.264 MP4 movies. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Play Render Output", DisplayName = "H.264 MP4 Movies")
	FMovieGraphPostRenderVideoPlayOptions PostRenderMP4PlayOptions;

	// TODO: No CLE support yet.
	/** If PostRenderBehavior is set to PlayRenderOutput, these options are used to determine how to play Command Line Encoder output. */
	// UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Play Render Output", DisplayName = "Command Line Encoder")
	// FMovieGraphPostRenderVideoPlayOptions PostRenderCommandLineEncoderOptions;
};

#undef UE_API
