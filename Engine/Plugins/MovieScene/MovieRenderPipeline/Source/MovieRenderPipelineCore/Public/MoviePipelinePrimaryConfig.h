// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineConfigBase.h"
#include "Misc/FrameRate.h"

#include "MoviePipelinePrimaryConfig.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API


// Forward Declares
class ULevelSequence;
class UMoviePipelineSetting;
class UMoviePipelineRenderPass;
class UMoviePipelineOutputBase;
class UMoviePipelineShotConfig;
class UMoviePipelineOutputSetting;
class UMoviePipelineExecutorJob;

/**
* This class describes the main configuration for a Movie Render Pipeline.
* Only settings that apply to the entire output should be stored here,
* anything that is changed on a per-shot basis should be stored inside of 
* UMovieRenderShotConfig instead.
*
* THIS CLASS SHOULD BE IMMUTABLE ONCE PASSED TO THE PIPELINE FOR PROCESSING.
* (Otherwise you will be modifying the instance that exists in the UI)
*/
UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelinePrimaryConfig : public UMoviePipelineConfigBase
{
	GENERATED_BODY()
	
public:
	UE_API UMoviePipelinePrimaryConfig();

public:
	UE_API TArray<UMoviePipelineOutputBase*> GetOutputContainers() const;
	
	// UMoviePipelineConfigBase interface
	UE_API virtual TArray<UMoviePipelineSetting*> GetUserSettings() const override;
	UE_API virtual void CopyFrom(UMoviePipelineConfigBase* InConfig) override;
	// ~UMoviePipelineConfigBase interface


	/** Initializes a single instance of every setting so that even non-user-configured settings have a chance to apply their default values. Does nothing if they're already instanced for this configuration. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API void InitializeTransientSettings();

	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	TArray<UMoviePipelineSetting*> GetTransientSettings() const { return TransientSettings; }

	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API TArray<UMoviePipelineSetting*> GetAllSettings(const bool bIncludeDisabledSettings = false, const bool bIncludeTransientSettings = false) const;
public:

	/** Returns a pointer to the config specified for the shot, otherwise the default for this pipeline. */
	UE_API UMoviePipelineShotConfig* GetConfigForShot(const FString& ShotName) const;

	UE_API void GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs, const bool bIncludeAllSettings = false) const;


	/**
	* Returns the frame rate override from the Primary Configuration (if any) or the Sequence frame rate if no override is specified.
	* This should be treated as the actual output framerate of the overall Pipeline.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API FFrameRate GetEffectiveFrameRate(const ULevelSequence* InSequence) const;

	UE_API TRange<FFrameNumber> GetEffectivePlaybackRange(const ULevelSequence* InSequence) const;

protected:
	virtual bool CanSettingBeAdded(const UMoviePipelineSetting* InSetting) const override
	{
		check(InSetting);
		return InSetting->IsValidOnPrimary();
	}

	UE_API virtual void OnSettingAdded(UMoviePipelineSetting* InSetting) override;
	UE_API virtual void OnSettingRemoved(UMoviePipelineSetting* InSetting) override;

	UE_API void AddTransientSettingByClass(const UClass* InSettingClass);
public:	
	/** A mapping of Shot Name -> Shot Config to use for rendering specific shots with specific configs. */
	UPROPERTY(Instanced)
	TMap<FString, TObjectPtr<UMoviePipelineShotConfig>> PerShotConfigMapping;

private:
	UPROPERTY(Instanced)
	TObjectPtr<UMoviePipelineOutputSetting> OutputSetting;

	/** An array of settings that are available in the engine and have not been edited by the user. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMoviePipelineSetting>> TransientSettings;
};

#undef UE_API
