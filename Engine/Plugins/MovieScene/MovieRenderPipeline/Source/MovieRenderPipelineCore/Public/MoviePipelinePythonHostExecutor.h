// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineExecutor.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelinePythonHostExecutor.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API


/**
* This is a dummy executor that is designed to host a executor implemented in
* python. Python defined UClasses are not available when the executor is initialized
* and not all callbacks are available in Python. By inheriting from this in Python
* and overriding which UClass to latently spawn, this class can just forward certain
* events onto Python (by overriding the relevant function).
*/
UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelinePythonHostExecutor : public UMoviePipelineExecutorBase
{
	GENERATED_BODY()

public:
	UMoviePipelinePythonHostExecutor()
		: UMoviePipelineExecutorBase()
	{
	}

	// UMoviePipelineExecutorBase Interface
	UE_API virtual void Execute_Implementation(UMoviePipelineQueue* InPipelineQueue) override;
	virtual bool IsRendering_Implementation() const override { return false; }

	/** You should override this method to implement canceling jobs. */
	UE_API virtual void CancelAllJobs_Implementation() override;
	/** You only need to override this methond if your executor supports multiple queued jobs. */
	virtual void CancelCurrentJob_Implementation() override { CancelAllJobs_Implementation(); };

	// ~UMoviePipelineExecutorBase Interface

	// Python/Blueprint API
	UFUNCTION(BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void ExecuteDelayed(UMoviePipelineQueue* InPipelineQueue);
	virtual void ExecuteDelayed_Implementation(UMoviePipelineQueue* InPipelineQueue) {}

	UFUNCTION(BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void OnMapLoad(UWorld* InWorld);
	virtual void OnMapLoad_Implementation(UWorld* InWorld) {}
	// ~Python/Blueprint API

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UWorld* GetLastLoadedWorld() const { return LastLoadedWorld; }
private:
	UE_API void OnMapLoadFinished(UWorld* NewWorld);

public:
	/** You should override this class type on the CDO of the object with your Python type when Python is initialized. */
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	TSubclassOf<UMoviePipelinePythonHostExecutor> ExecutorClass;
	
	UPROPERTY(BlueprintReadWrite, Transient, Category = "Movie Render Pipeline")
	TObjectPtr<UMoviePipelineQueue> PipelineQueue;
private:
	UPROPERTY(Transient)
	TObjectPtr<UWorld> LastLoadedWorld;
};

#undef UE_API
