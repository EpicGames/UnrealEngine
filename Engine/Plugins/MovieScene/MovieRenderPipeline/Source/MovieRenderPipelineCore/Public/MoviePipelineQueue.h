// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MovieJobVariableAssignmentContainer.h"
#include "MoviePipelineBasicConfig.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineShotConfig.h"
#include "MoviePipelineConfigBase.h"
#include "MovieSceneSequenceID.h"
#include "Graph/MovieGraphConfig.h"

#include "MoviePipelineQueue.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMoviePipelinePrimaryConfig;
class ULevel;
class ULevelSequence;

USTRUCT(BlueprintType)
struct FMoviePipelineSidecarCamera
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Render Pipeline")
	FGuid BindingId;

	// FMovieSceneSequenceID isn't exposed to Blueprints and we need the full support
	// of a FMovieSceneSequenceID so we can't just store a USequence* like the scripting
	// layer does - we need to be able to handle the same sequence being in a sequence
	// multiple times (which scripting does not). This data structure gets regenerated
	// each time a render starts, so the property should be valid when we want to use
	// it, even if it's not exposed to scripting.
	// UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Render Pipeline")
	FMovieSceneSequenceID SequenceId;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Render Pipeline")
	FString Name;
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMoviePipelineShotGraphPresetChanged, UMoviePipelineExecutorShot*, UMovieGraphConfig*);

/** Project-based settings for the queue. */
UCLASS(MinimalAPI, Config=EditorPerProjectUserSettings)
class UMoviePipelineQueueProjectSettings : public UObject
{
	GENERATED_BODY()

private:
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif	// WITH_EDITOR

public:
	/** When a queue finishes rendering, this determines the action that should be taken (if any). */
	UPROPERTY(Config, BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	EMoviePipelinePostRenderActionType PostRenderActionType = EMoviePipelinePostRenderActionType::DoNothing;
};

/** The active configuration mode for a Movie Render Queue job. */
UENUM(BlueprintType)
enum class EMoviePipelineConfigMode : uint8
{
	/** Legacy UMoviePipelinePrimaryConfig preset. */
	Preset,

	/** Graph-based UMovieGraphConfig configuration. */
	Graph,

	/** Simplified UMoviePipelineBasicConfig configuration. */
	Basic
};

/**
* This class represents a segment of work within the Executor Job. This should be owned
* by the UMoviePipelineExecutorJob and can be created before the movie pipeline starts to
* configure some aspects about the segment (such as disabling it). When the movie pipeline
* starts, it will use the already existing ones, or generate new ones as needed.
*/
UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineExecutorShot : public UObject
{
	GENERATED_BODY()
public:
	UMoviePipelineExecutorShot()
		: bEnabled(true)
	{
		Progress = 0.f;
	}

public:
	/**
	* Set the status of this shot to the given value. This will be shown on the UI if progress
	* is set to a value less than zero. If progress is > 0 then the progress bar will be shown
	* on the UI instead. Progress and Status Message are cosmetic.
	*
	* For C++ implementations override `virtual void SetStatusMessage_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def set_status_message(self, inStatus):
	*
	* @param InStatus	The status message you wish the executor to have.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void SetStatusMessage(const FString& InStatus);

	/**
	* Get the current status message for this shot. May be an empty string.
	*
	* For C++ implementations override `virtual FString GetStatusMessage_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def get_status_message(self):
	*		return ?
	*/
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API FString GetStatusMessage() const;

	/**
	* Set the progress of this shot to the given value. If a positive value is provided
	* the UI will show the progress bar, while a negative value will make the UI show the
	* status message instead. Progress and Status Message are cosmetic and dependent on the
	* executor to update. Similar to the UMoviePipelineExecutor::SetStatusProgress function,
	* but at a per-job level basis instead.
	*
	* For C++ implementations override `virtual void SetStatusProgress_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def set_status_progress(self, inStatus):
	*
	* @param InProgress	The progress (0-1 range) the executor should have.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void SetStatusProgress(const float InProgress);

	/**
	* Get the current progress as last set by SetStatusProgress. 0 by default.
	*
	* For C++ implementations override `virtual float GetStatusProgress_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def get_status_progress(self):
	*		return ?
	*/
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API float GetStatusProgress() const;

	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "InConfigType"), Category = "Movie Render Pipeline", meta=(InConfigType="/Script/MovieRenderPipelineCore.MoviePipelineShotConfig"))
	UE_API UMoviePipelineShotConfig* AllocateNewShotOverrideConfig(TSubclassOf<UMoviePipelineShotConfig> InConfigType);

	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API void SetShotOverrideConfiguration(UMoviePipelineShotConfig* InPreset);

	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API void SetShotOverridePresetOrigin(UMoviePipelineShotConfig* InPreset);

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelineShotConfig* GetShotOverrideConfiguration() const
	{
		return ShotOverrideConfig;
	}

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelineShotConfig* GetShotOverridePresetOrigin() const
	{
		return ShotOverridePresetOrigin.Get();
	}

	/**
	 * Returns true if this job is using graph-style configuration, else false.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UE_API bool IsUsingGraphConfiguration() const;

	/**
	 * Gets the graph-style preset that this shot is using.
	 * A non-null graph config does not imply the graph config is the config that's in use;
	 * reference GetConfigMode() on the job for the config type that's active.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMovieGraphConfig* GetGraphPreset() const
	{
		return GraphPreset.LoadSynchronous();
	}

	/**
	 * Sets the graph-style preset that this job will use. Note that this will cause the graph to switch over to using
	 * graph-style configuration if it is not already using it.
	 *
	 * @param InGraphPreset - The graph preset to assign to the shot.
	 * @param bUpdateVariableAssignments - Set to false if variable assignments should NOT be automatically updated to reflect the graph preset being used. This is normally not what you want and should be used with caution.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API void SetGraphPreset(const UMovieGraphConfig* InGraphPreset, const bool bUpdateVariableAssignments = true);

	/**
	* This method will return you the object which contains variable overrides for either the Job's Primary or the Shot's GraphPreset. UMoviePipelineExecutorShot
	* has two separate sets of overrides. You can use the shot to override a variable on the Primary Graph (ie: the one assigned to the whole job),
	* but a graph can also have an entirely separate UMovieGraph config asset to run (though at runtime some variables will only be read from the
	* Primary Graph, ie: Custom Frame Range due to it applying to the entire sequence). 
	* 
	* If you specify true for bIsForPrimaryOverrides it returns an object that allows this shot to override a variable that comes from the primary 
	* graph. If you return false, then it returns an object that allows overriding a variable for this shot's override config (see: GetGraphPreset).
	* See UMoviePipelineExecutorJob's version of this function for more details.
	* 
	* @param InGraph - The graph asset to return the config for. If this shot has its own Graph Preset override, you should return GetGraphPreset()
	* or one of it's sub-graph pointers. If this shot is just trying to override the Primary Graph from the parent UMoviePipelineExecutorJob then
	* you should return a pointer to the Job's GetGraphPreset() (or one of it's sub-graphs). Each graph/sub-graph gets its own set of overrides
	* since sub-graphs can have different variables than the parents, so you have to provide the pointer to the one you want to override variables for.
	* 
	* @param bIsForPrimaryOverride - Default false. If true, tries to override variables on the parent UMoviePipelineExecutorJob's graphs. If false,
	* tries to override variables on the Graph Preset assigned to this shot.
	* 
	* @return A container object which holds a copy of the variables for the specified Graph Asset that can be used to override their values
	* on jobs without actually editing the default asset.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline", DisplayName = "Get or Create Variable Overrides", meta = (ScriptName = "GetOrCreateVariableOverrides"))
	UE_API UMovieJobVariableAssignmentContainer* GetOrCreateJobVariableAssignmentsForGraph(const UMovieGraphConfig* InGraph, const bool bIsForPrimaryOverrides = false);

	/** Returns true if the parent job is using basic configuration mode. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UE_API bool IsUsingBasicConfiguration() const;

	/**
	 * Gets the basic shot config overrides. Returns nullptr if no shot overrides are set.
	 * A non-null Basic config override does not imply the Basic config is the config that's in use;
	 * reference the job's GetConfigMode() for the config type that's active.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelineBasicConfig* GetBasicShotConfig() const { return BasicShotConfig; }

	/** Creates and returns basic shot overrides for this shot. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API UMoviePipelineBasicConfig* SetupBasicShotOverrides();

	/** Returns true if console variable overrides can be used in the active configuration. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UE_API bool CanUseConsoleVariableOverrides() const;

	/** Returns whether this should be rendered. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	bool ShouldRender() const
	{
		return bEnabled;
	}

	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	FString GetCameraName(int32 InCameraIndex) const
	{
		if (InCameraIndex >= 0 && InCameraIndex < SidecarCameras.Num())
		{
			return SidecarCameras[InCameraIndex].Name;
		}
		return InnerName;
	}

	// UObject Interface
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	// ~UObject Interface

	/**
	 * Gets overrides on the variables in graph presets associated with this job. A job can have multiple graphs associated with it if the job's
	 * assigned graph contains subgraphs. The job's graph and each subgraph will have an entry in the returned array.
	 * 
	 * The assignments are updated before they are returned. To opt-out of this, set bUpdateAssignments to false (generally not recommended unless
	 * there is very specific behavior that is needed).
	 */
	UE_API TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& GetGraphVariableAssignments(const bool bUpdateAssignments = true);

	/**
	 * Gets overrides on the variables in the primary graph (and its subgraphs) associated with this job.
	 *
	 * The assignments are updated before they are returned. To opt-out of this, set bUpdateAssignments to false (generally not recommended unless
	 * there is very specific behavior that is needed).
	 */
	UE_API TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& GetPrimaryGraphVariableAssignments(const bool bUpdateAssignments = true);

	/** Refreshes the variable assignments associated with this shot, both for the shot's own graph preset and the associated primary graph. */
	UE_API void RefreshAllVariableAssignments();
	
protected:
	// UMoviePipipelineExecutorShot Interface
	virtual void SetStatusMessage_Implementation(const FString& InMessage) { StatusMessage = InMessage; }
	virtual void SetStatusProgress_Implementation(const float InProgress) { Progress = InProgress; }
	virtual FString GetStatusMessage_Implementation() const { return StatusMessage; }
	virtual float GetStatusProgress_Implementation() const { return Progress; }
	// ~UMoviePipipelineExecutorShot Interface
	
	UE_API void OnGraphPreSave(UObject* InObject, FObjectPreSaveContext InObjectPreSaveContext);

public:

	/** Does the user want to render this shot? */
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	bool bEnabled;

	/** The name of the shot section that contains this shot. Can be empty. */
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	FString OuterName;

	/** The name of the camera cut section that this shot represents. Can be empty. */
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	FString InnerName;

	/** List of cameras to render for this shot. Only used if the setting flag is set in the Camera setting. */
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	TArray<FMoviePipelineSidecarCamera> SidecarCameras;

	/** Called when the graph preset assigned to the shot changes. */
	FOnMoviePipelineShotGraphPresetChanged OnShotGraphPresetChanged;

	/**
	 * (Optional) Console variable overrides which are applied after cvars set via nodes. Applies to graph-based and Basic configurations.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Console Variable Overrides", meta = (DisplayName = "Console Variables", EditCondition="CanUseConsoleVariableOverrides()", EditConditionHides))
	TArray<FMoviePipelineConsoleVariableEntry> ConsoleVariableOverrides;

public:
	/** Transient information used by the active Movie Pipeline working on this shot. */
	FMoviePipelineCameraCutInfo ShotInfo;

protected:
	UPROPERTY(Transient)
	float Progress;
	UPROPERTY(Transient)
	FString StatusMessage;

private:
	UPROPERTY()
	TObjectPtr<UMoviePipelineShotConfig> ShotOverrideConfig;

	UPROPERTY()
	TSoftObjectPtr<UMoviePipelineShotConfig> ShotOverridePresetOrigin;

	/** The graph-based configuration preset that this shot is using. Can be nullptr. */
	UPROPERTY()
	TSoftObjectPtr<UMovieGraphConfig> GraphPreset;

	/** Overrides on the variables in the graph (and subgraphs) associated with this job. */
	UPROPERTY(EditAnywhere, Instanced, Category = "Variable Assignments")
	TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>> GraphVariableAssignments;
	
	/** Overrides on the variables in the primary graph (and its subgraphs) associated with this job. */
	UPROPERTY(EditAnywhere, Instanced, Category = "Variable Assignments")
	TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>> PrimaryGraphVariableAssignments;

	/** The basic configuration overrides for this shot. Non-null indicates shot-level overrides in Basic configuration mode. */
	UPROPERTY(Instanced)
	TObjectPtr<UMoviePipelineBasicConfig> BasicShotConfig;
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMoviePipelineJobGraphPresetChanged, UMoviePipelineExecutorJob*, UMovieGraphConfig*);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMoviePipelineJobConfigModeChanged, UMoviePipelineExecutorJob*, EMoviePipelineConfigMode);

/**
* A particular job within the Queue
*/
UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineExecutorJob : public UObject
{
	GENERATED_BODY()
public:
	UMoviePipelineExecutorJob()
	{
		bEnabled = true;
		StatusProgress = 0.f;
		bIsConsumed = false;
		Configuration = CreateDefaultSubobject<UMoviePipelinePrimaryConfig>("DefaultConfig");
	}

public:	
	/**
	* Set the status of this job to the given value. This will be shown on the UI if progress
	* is set to a value less than zero. If progress is > 0 then the progress bar will be shown
	* on the UI instead. Progress and Status Message are cosmetic and dependent on the
	* executor to update. Similar to the UMoviePipelineExecutor::SetStatusMessage function,
	* but at a per-job level basis instead. 
	*
	* For C++ implementations override `virtual void SetStatusMessage_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def set_status_message(self, inStatus):
	*
	* @param InStatus	The status message you wish the executor to have.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void SetStatusMessage(const FString& InStatus);

	/**
	* Get the current status message for this job. May be an empty string.
	*
	* For C++ implementations override `virtual FString GetStatusMessage_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def get_status_message(self):
	*		return ?
	*/
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API FString GetStatusMessage() const;

	/**
	* Set the progress of this job to the given value. If a positive value is provided
	* the UI will show the progress bar, while a negative value will make the UI show the 
	* status message instead. Progress and Status Message are cosmetic and dependent on the
	* executor to update. Similar to the UMoviePipelineExecutor::SetStatusProgress function,
	* but at a per-job level basis instead.
	*
	* For C++ implementations override `virtual void SetStatusProgress_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def set_status_progress(self, inProgress):
	*
	* @param InProgress	The progress (0-1 range) the executor should have.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void SetStatusProgress(const float InProgress);

	/**
	* Get the current progress as last set by SetStatusProgress. 0 by default.
	*
	* For C++ implementations override `virtual float GetStatusProgress_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def get_status_progress(self):
	*		return ?
	*/
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API float GetStatusProgress() const;

	/**
	* Set the job to be consumed. A consumed job is disabled in the UI and should not be
	* submitted for rendering again. This allows jobs to be added to a queue, the queue
	* submitted to a remote farm (consume the jobs) and then more jobs to be added and
	* the second submission to the farm won't re-submit the already in-progress jobs.
	*
	* Jobs can be unconsumed when the render finishes to re-enable editing.
	*
	* For C++ implementations override `virtual void SetConsumed_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def set_consumed(self, isConsumed):
	*
	* @param bInConsumed	True if the job should be consumed and disabled for editing in the UI.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void SetConsumed(const bool bInConsumed);

	/**
	* Gets whether or not the job has been marked as being consumed. A consumed job is not editable
	* in the UI and should not be submitted for rendering as it is either already finished or
	* already in progress.
	*
	* For C++ implementations override `virtual bool IsConsumed_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def is_consumed(self):
	*		return ?
	*/
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API bool IsConsumed() const;

	/**
	* Set the job to be enabled/disabled. This is exposed to the user in the Queue UI
	* so they can disable a job after loading a queue to skip trying to run it.
	*
	* For C++ implementations override `virtual void SetIsEnabled_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def set_is_enabled(self, isEnabled):
	*
	* @param bInEnabled	True if the job should be enabled and rendered.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void SetIsEnabled(const bool bInEnabled);

	/**
	* Gets whether or not the job has been marked as being enabled. 
	*
	* For C++ implementations override `virtual bool IsEnabled_Implementation() const override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def is_enabled(self):
	*		return ?
	*/
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API bool IsEnabled() const;

	/**
	* Should be called to clear status and user data after duplication so that jobs stay
	* unique and don't pick up ids or other unwanted behavior from the pareant job.
	*
	* For C++ implementations override `virtual bool OnDuplicated_Implementation() override`
	* For Python/BP implementations override
	*	@unreal.ufunction(override=True)
	*	def on_duplicated(self):
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Movie Render Pipeline")
	UE_API void OnDuplicated();

	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API void SetPresetOrigin(UMoviePipelinePrimaryConfig* InPreset);

	/**
	 * Gets the preset for this job, but only if the preset has not been modified. If it has been modified, or the preset
	 * no longer exists, returns nullptr.
	 * @see GetConfiguration()
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelinePrimaryConfig* GetPresetOrigin() const 
	{
		return PresetOrigin.LoadSynchronous();
	}

	/**
	 * Gets the configuration for the job. This configuration is either standalone (not associated with any preset), or
	 * contains a copy of the preset origin plus any modifications made on top of it. If the preset that this
	 * configuration was originally based on no longer exists, this configuration will still be valid.
	 * 
	 * Note that a non-null configuration does not imply that this Preset is the active configuration;
	 * reference GetConfigMode() for the config type that's active.
	 * 
	 * @see GetPresetOrigin()
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelinePrimaryConfig* GetConfiguration() const
	{
		return Configuration;
	}

	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API void SetConfiguration(UMoviePipelinePrimaryConfig* InPreset);

	/** Returns the active configuration mode for this job. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	EMoviePipelineConfigMode GetConfigMode() const { return ConfigMode; }

	/**
	 * Switches the active configuration mode on the job.
	 * For Basic mode, lazily initializes the BasicConfig if it doesn't exist yet.
	 * For Graph mode, requires a graph preset to already be assigned (returns false otherwise). See SetGraphPreset().
	 * @return true if the mode was switched (or was already the requested mode), false if the switch was not possible.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API bool SetConfigMode(EMoviePipelineConfigMode InConfigMode);

	/**
	 * Returns true if this job is using graph-style configuration, else false.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	bool IsUsingGraphConfiguration() const { return ConfigMode == EMoviePipelineConfigMode::Graph; }

	/** Returns true if this job is using basic configuration mode. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	bool IsUsingBasicConfiguration() const { return ConfigMode == EMoviePipelineConfigMode::Basic; }

	/**
	 * Gets the basic configuration object, or nullptr if basic configuration has not been set up.
	 * A non-null Basic config does not imply the Basic config is the config that's in use;
	 * reference GetConfigMode() for the config type that's active.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelineBasicConfig* GetBasicConfig() const { return BasicConfig; }

	/**
	 * Sets up basic configuration mode for this job. Note that this will cause the job to switch over to
	 * using the Basic configuration. Returns the created basic config object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API UMoviePipelineBasicConfig* SetupBasicConfiguration();

	/** Returns true if console variable overrides can be used in the active configuration. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UE_API bool CanUseConsoleVariableOverrides() const;

	/**
	 * Returns true if this job's legacy preset config has been seeded with the project's default
	 * settings. Typically used to gate one-time defaults seeding when switching into Preset mode,
	 * so settings the user has deliberately removed aren't re-added on subsequent mode switches.
	 */
	bool HasInitializedLegacyDefaults() const { return bHasInitializedLegacyPresetDefaults; }

	/**
	 * Marks this job's legacy preset config as having been seeded with the project's default settings,
	 * which disables the one-time defaults seeding that would otherwise run on the next Basic/Graph ->
	 * Preset switch. Typically only called by EnsureJobHasDefaultSettings (which manages the flag as
	 * part of its idempotency gate), but may be useful in niche cases where external code manages the
	 * configuration lifecycle directly and needs to coordinate with that gate.
	 */
	void MarkLegacyPresetDefaultsAsInitialized() { bHasInitializedLegacyPresetDefaults = true; }

	/**
	 * Gets the graph-style preset assigned to this job.
	 * A non-null graph config does not imply the graph config is the config that's in use;
	 * reference GetConfigMode() for the config type that's active.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMovieGraphConfig* GetGraphPreset() const
	{
		return GraphPreset.LoadSynchronous();
	}

	/**
	 * Sets the graph-style preset that this job will use. Note that this will cause the graph to switch over to using
	 * graph-style configuration if it is not already using it.
	 *
	 * @param InGraphPreset - The graph preset to assign to the job.
	 * @param bUpdateVariableAssignments - Set to false if variable assignments should NOT be automatically updated to reflect the graph preset being used. This is normally not what you want and should be used with caution.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API void SetGraphPreset(const UMovieGraphConfig* InGraphPreset, const bool bUpdateVariableAssignments = true);

	UFUNCTION(BlueprintSetter, Category = "Movie Render Pipeline")
	UE_API void SetSequence(FSoftObjectPath InSequence);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSequenceSet, UMoviePipelineExecutorJob* Job, ULevelSequence* Sequence);

	/** Retrieve multicast delegate for when a sequence is set. */
	static UE_API FOnSequenceSet& OnSequenceSetEvent();
	
	/**
	* This method will return you the object which contains variable overrides for the Primary Graph assigned to this job. You need to provide
	* a pointer to the exact graph you want (as the Primary Graph may contain sub-graphs, and those sub-graphs can have their own variables),
	* though this will be the same as GetGraphPreset() if you're not using any sub-graphs, or your variables only exist on the Primary graph.
	* 
	* If you want to override a variable on the primary graph but only for a specific shot, you should get the UMoviePipelineExecutorShot and
	* call that classes version of this function, except passing True for the extra boolean.  See comment on that function for more details.
	* 
	* @param InGraph - The graph asset to get the Job Override values for. Should be the graph the variables you want to edit are defined on,
	* which can either be the primary graph (GetGraphPreset()) or one of the sub-graphs it points to (as sub-graphs can contain their own
	* variables which are all shown at the top level job in the Editor UI).
	* @return A container object which holds a copy of the variables for the specified Graph Asset that can be used to override their values
	* on jobs without actually editing the default asset.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline", DisplayName = "Get or Create Variable Overrides", meta=(ScriptName ="GetOrCreateVariableOverrides"))
	UE_API UMovieJobVariableAssignmentContainer* GetOrCreateJobVariableAssignmentsForGraph(const UMovieGraphConfig* InGraph);

public:
	// UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent);
#endif
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	// ~UObject Interface
	
	/**
	 * Gets overrides on the variables in graph presets associated with this job. A job can have multiple graphs associated with it if the job's
	 * assigned graph contains subgraphs. The job's graph and each subgraph will have an entry in the returned array.
	 *
	 * The assignments are updated before they are returned. To opt-out of this, set bUpdateAssignments to false (generally not recommended unless
	 * there is very specific behavior that is needed).
	 */
	UE_API TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& GetGraphVariableAssignments(const bool bUpdateAssignments = true);

	/** Refreshes the variable assignments associated with this job, both for the jobs's own graph preset and the associated shot graphs. */
	UE_API void RefreshAllVariableAssignments();

protected:
	// UMoviePipelineExecutorJob Interface
	virtual void SetStatusMessage_Implementation(const FString& InMessage) { StatusMessage = InMessage; }
	virtual void SetStatusProgress_Implementation(const float InProgress) { StatusProgress = InProgress; }
	virtual void SetConsumed_Implementation(const bool bInConsumed) { bIsConsumed = bInConsumed; }
	virtual FString GetStatusMessage_Implementation() const { return StatusMessage; }
	virtual float GetStatusProgress_Implementation() const { return StatusProgress; }
	virtual bool IsConsumed_Implementation() const { return bIsConsumed; }
	UE_API virtual void OnDuplicated_Implementation();
	virtual void SetIsEnabled_Implementation(const bool bInEnabled)
	{
		// Call Modify() so OnObjectModified picks up this change.
		Modify();

		bEnabled = bInEnabled;
	}
	virtual bool IsEnabled_Implementation() const { return bEnabled; }
	// ~UMoviePipelineExecutorJob Interface

	UE_API void OnGraphPreSave(UObject* InObject, FObjectPreSaveContext InObjectPreSaveContext);
	
public:
	/** (Optional) Name of the job. Shown on the default burn-in. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	FString JobName;

	/** Which sequence should this job render? */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, BlueprintSetter = "SetSequence", Category = "Movie Render Pipeline", meta = (AllowedClasses = "/Script/LevelSequence.LevelSequence", DisplayThumbnail = false))
	FSoftObjectPath Sequence;

	/** Which map should this job render on */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline", meta = (AllowedClasses = "/Script/Engine.World", DisplayThumbnail = false))
	FSoftObjectPath Map;

	/** (Optional) If left blank, will default to system username. Can be shown in burn in as a first point of contact about the content. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	FString Author;
	
	/** (Optional) If specified, will be shown in the burn in to allow users to keep track of notes about a render. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline", meta = (MultiLine = true))
	FString Comment;

	/** (Optional) Shot specific information. If a shot is missing from this list it will assume to be enabled and will be rendered. */
	UPROPERTY(BlueprintReadWrite, Instanced, Category = "Movie Render Pipeline")
	TArray<TObjectPtr<UMoviePipelineExecutorShot>> ShotInfo;

	/** 
	* Arbitrary data that can be associated with the job. Not used by default implementations, nor read.
	* This can be used to attach third party metadata such as job ids from remote farms. 
	* Not shown in the user interface.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "Movie Render Pipeline")
	FString UserData;

	/** Called when the graph preset assigned to the job changes. */
	FOnMoviePipelineJobGraphPresetChanged OnJobGraphPresetChanged;

	/** Called when the job's configuration mode changes (Basic, Graph, Preset). */
	FOnMoviePipelineJobConfigModeChanged OnJobConfigModeChanged;

	/**
	 * (Optional) Console variable overrides which are applied after cvars set via nodes. Applies to graph-based and Basic configurations.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Console Variable Overrides", meta = (DisplayName = "Console Variables", EditCondition="CanUseConsoleVariableOverrides()", EditConditionHides))
	TArray<FMoviePipelineConsoleVariableEntry> ConsoleVariableOverrides;

private:
	UPROPERTY(Transient)
	FString StatusMessage;
	
	UPROPERTY(Transient)
	float StatusProgress;
	
	UPROPERTY(Transient)
	bool bIsConsumed;

	/** 
	*/
	UPROPERTY(Instanced)
	TObjectPtr<UMoviePipelinePrimaryConfig> Configuration;

	/**
	*/
	UPROPERTY()
	TSoftObjectPtr<UMoviePipelinePrimaryConfig> PresetOrigin;

	/** Whether this job is enabled and should be rendered. */
	UPROPERTY()
	bool bEnabled;

	/**
	 * The graph-based configuration preset that this job is using. Can be nullptr. Typically, this should be set with
	 * SetGraphPreset() in order to keep the ConfigMode in sync. May be non-null even when not in Graph mode
	 * (it's preserved across mode switches). Check ConfigMode to determine the active configuration.
	 */
	UPROPERTY()
	TSoftObjectPtr<UMovieGraphConfig> GraphPreset;

	/** Overrides on the variables in the graph (and subgraphs) associated with this job. */
	UPROPERTY(EditAnywhere, Instanced, Category = "Variable Assignments")
	TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>> GraphVariableAssignments;

	/** The basic configuration for this job. May be non-null even when not in Basic mode (preserved across mode switches). */
	UPROPERTY(Instanced)
	TObjectPtr<UMoviePipelineBasicConfig> BasicConfig;

	/** The active configuration mode for this job. Determines which config object is in use. */
	UPROPERTY()
	EMoviePipelineConfigMode ConfigMode = EMoviePipelineConfigMode::Preset;

	/**
	 * Tracks whether defaults have been applied to this job's legacy configuration. Set when the
	 * defaults are seeded at creation, or on first Basic/Graph -> Preset switch. Prevents re-adding
	 * settings the user has removed on subsequent mode switches. By convention this is written via
	 * MarkLegacyPresetDefaultsAsInitialized().
	 */
	UPROPERTY()
	bool bHasInitializedLegacyPresetDefaults = false;
};

/**
* A queue is a list of jobs that have been executed, are executing and are waiting to be executed. These can be saved
* to specific assets to allow 
*/
UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineQueue : public UObject
{
	GENERATED_BODY()
public:
	
	UE_API UMoviePipelineQueue();
	
	/**
	* Allocates a new Job in this Queue. The Queue owns the jobs for memory management purposes,
	* and this will handle that for you. 
	*
	* @param InJobType	Specify the specific Job type that should be created. Custom Executors can use custom Job types to allow the user to provide more information.
	* @return	The created Executor job instance.
	*/
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "InClass"), Category = "Movie Render Pipeline|Queue", meta=(InJobType="/Script/MovieRenderPipelineCore.MoviePipelineExecutorJob"))
	UE_API UMoviePipelineExecutorJob* AllocateNewJob(TSubclassOf<UMoviePipelineExecutorJob> InJobType);

	/**
	* Deletes the specified job from the Queue. 
	*
	* @param InJob	The job to look for and delete. 
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Queue")
	UE_API void DeleteJob(UMoviePipelineExecutorJob* InJob);

	/**
	* Delete all jobs from the Queue.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Queue", meta = (Keywords = "Clear Empty"))
	UE_API void DeleteAllJobs();

	/**
	* Duplicate the specific job and return the duplicate. Configurations are duplicated and not shared.
	*
	* @param InJob	The job to look for to duplicate.
	* @return The duplicated instance or nullptr if a duplicate could not be made.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Queue")
	UE_API UMoviePipelineExecutorJob* DuplicateJob(UMoviePipelineExecutorJob* InJob);
	
	/**
	* Get all of the Jobs contained in this Queue.
	* @return The jobs contained by this queue.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline|Queue")
	TArray<UMoviePipelineExecutorJob*> GetJobs() const
	{
		return Jobs;
	}

	/**
	 * Gets the queue that this queue was originally based on (if any). The origin will only be set on transient
	 * queues; the origin will be nullptr for non-transient queues because the origin will be this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Queue")
	UMoviePipelineQueue* GetQueueOrigin() const { return QueueOrigin.LoadSynchronous(); }

	/**
	 * Sets the queue that this queue originated from (if any). The origin should only be set for transient queues.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Queue")
	void SetQueueOrigin(UMoviePipelineQueue* InConfig) { QueueOrigin = InConfig; }

	/** 
	* Replace the contents of this queue with a copy of the contents from another queue. 
	* Returns a pointer to this queue if the copy was successful, else nullptr.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Queue")
	UE_API UMoviePipelineQueue* CopyFrom(UMoviePipelineQueue* InQueue);
	
	/* Set the index of the given job */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Queue")
	UE_API void SetJobIndex(UMoviePipelineExecutorJob* InJob, int32 Index);

#if WITH_EDITOR
	/**
	 * Gets the dirty state of this queue. Note that dirty state is only tracked when the editor is active.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Queue")
	bool IsDirty() const { return bIsDirty; }

	/**
	 * Sets the dirty state of this queue. Generally the queue will correctly track the dirty state; however, there are
	 * situations where a queue may need its dirty state reset (eg, it may be appropriate to reset the dirty state after
	 * a call to CopyFrom(), depending on the use case).
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Queue")
	void SetIsDirty(const bool bNewDirtyState) { bIsDirty = bNewDirtyState; }
#endif
	
	/**
	 * Retrieve the serial number that is incremented when a job is added or removed from this list.
	 * @note: This field is not serialized, and not copied along with UObject duplication.
	 */
	uint32 GetQueueSerialNumber() const
	{
		return QueueSerialNumber;
	}

	void InvalidateSerialNumber()
	{
		QueueSerialNumber++;
	}

	// UObject Interface
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	// ~UObject interface

private:
#if WITH_EDITOR
	UE_API void OnAnyObjectModified(UObject* InModifiedObject);
#endif

private:
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UMoviePipelineExecutorJob>> Jobs;

	/* The queue that this queue originated from. Helpful for determining the source of the queue when this queue is transient. */
	UPROPERTY()
	TSoftObjectPtr<UMoviePipelineQueue> QueueOrigin;
	
private:
	int32 QueueSerialNumber;

	/* Dirty state is tracked explicitly so transient queues, which are based on a transient package, know when modifications
	 * have been made. */
	bool bIsDirty;
};

#undef UE_API
