// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineExecutor.h"
#include "MoviePipeline.h"
#include "MoviePipelineQueueEngineSubsystem.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API


/**
* This subsystem is intended for use when rendering in a shipping game (but can also be used in PIE
* during development/testing). See UMoviePipelineQueueSubsystem for the Editor-only queue which is
* bound to the Movie Render Queue UI. To do simple renders at runtime, call AllocateJob(...)
* with the Level Sequence you want to render, then call FindOrAddSettingByClass on the job to add
* the settings (such as render pass, output type, and output directory) that you want for the job.
* Finally, call RenderJob with the job you just configured. Register a delegate to OnRenderFinished 
* to be notified when the render finished. You can optionally call SetConfiguration(...) before
* RenderJob to configure some advanced options.
*/
UCLASS(MinimalAPI, BlueprintType, meta=(DisplayName="MoviePipeline Runtime Subsystem"))
class UMoviePipelineQueueEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
	
public:

	UMoviePipelineQueueEngineSubsystem()
	{
		CurrentQueue = CreateDefaultSubobject<UMoviePipelineQueue>("EngineMoviePipelineQueue");
		const bool bRenderViewport = false;
		SetConfiguration(nullptr, bRenderViewport);
	}
	
	/** Returns the queue of Movie Pipelines that need to be rendered. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelineQueue* GetQueue() const
	{
		return CurrentQueue;
	}

	/** Returns the active executor (if there is one). This can be used to subscribe to events on an already in-progress render. May be null. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelineExecutorBase* GetActiveExecutor() const
	{
		return ActiveExecutor;
	}

	/** 
	* Starts processing the current queue with the supplied executor class. This starts an async process which
	* may or may not run in a separate process (or on separate machines), determined by the executor implementation.
	* The executor should report progress for jobs depending on the implementation.
	*
	* @param InExecutorType	A subclass of UMoviePipelineExecutorBase. An instance of this class is created and started.
	* @return A pointer to the instance of the class created. This instance will be kept alive by the Queue Subsystem
			  until it has finished (or been canceled). Register for progress reports and various callbacks on this instance.
	*/
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "InExecutorType"), Category = "Movie Render Pipeline|Rendering")
	UE_API UMoviePipelineExecutorBase* RenderQueueWithExecutor(TSubclassOf<UMoviePipelineExecutorBase> InExecutorType);

	/** 
	* Starts processing the current queue with the supplied executor. This starts an async process which
	* may or may not run in a separate process (or on separate machines), determined by the executor implementation.
	* The executor should report progress for jobs depending on the implementation.
	*
	* @param InExecutor	Instance of a subclass of UMoviePipelineExecutorBase.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Rendering")
	UE_API void RenderQueueWithExecutorInstance(UMoviePipelineExecutorBase* InExecutor);

	/**
	* Returns true if there is an active executor working on producing a movie. This could be actively rendering frames,
	* or working on post processing (finalizing file writes, etc.). Use GetActiveExecutor() and query it directly for
	* more information, progress updates, etc.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline|Rendering")
	bool IsRendering() const
	{
		return ActiveExecutor ? ActiveExecutor->IsRendering() : false;
	}
	
	/**
	* Sets some advanced configuration options that are used only occasionally to have better control over integrating it into
	* your game/application. This applies to both RenderQueueWithExecutor(Instance) and the simplified RenderJob API. This persists
	* until you call it again with different settings, and needs to be called before the Render* functions.
	*
	* @param ProgressWidget 		- Create a User Widget that inherits from UMovieRenderDebugWidget and it will be created. 
	*						  		  Passing nullptr will use the default widget, if you wish to hide the widget then create a
	*						  		  new widget which inherits from UMovieRenderDebugWidget and put nothing in it.
	* @param bRenderPlayerViewport  - If true, we will render the regular player viewport in addition to the off-screen MRQ render.
	*								  This is significantly performance heavy (doubles render times) but can be useful in the event
	*								  that you want to keep rendering the player viewport to better integrate the render into your
	*								  own application.
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Rendering")
	UE_API void SetConfiguration(TSubclassOf<UMovieRenderDebugWidget> InProgressWidgetClass = {}, const bool bRenderPlayerViewport = false);

	/**
	* Convenience function for creating a UMoviePipelineExecutorJob out of the given Level Sequence asset. The 
	* newly created job will be initialized to render on the current level, and will not have any default settings
	* added to it - instead you will need to call FindOrAddSettingByClass on the job's configuration to add
	* settings such as render passes (UMoviePipelineDeferredPassBase), output types (UMoviePipelineImageSequenceOutput_PNG),
	* and configure the output directory (UMoviePipelineOutputSetting). Once configuration is complete, register
	* a delegate to OnRenderFinished and then call RenderJob.
	*
	* Calling this function will clear the internal queue, see RenderJob for more details.
	* 
	* Using this function while IsRendering() returns true will result in an exception being thrown and no attempt
	* being made to create the job.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Rendering")
	UE_API UMoviePipelineExecutorJob* AllocateJob(ULevelSequence* InSequence);

	/**
	* Convenience function for rendering the specified job. Calling this will render the specified job (if it was
	* allocated using AllocateJob) and then it will reset the queue once finished. If you need to render multiple 
	* jobs (in a queue) then you need to either implement the queue behavior yourself, or use 
	* GetQueue()->AllocateJob(...) instead and use the non-convenience functions.
	*
	* Calling this function will clear the queue (after completion).
	*
	* Using this function while IsRendering() returns true will result in an exception being thrown and no attempt
	* being made to render the job.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Rendering")
	UE_API void RenderJob(UMoviePipelineExecutorJob* InJob);

	/**
	* Assign a function to this delegate to get notified when each individual job is finished.
	*
	* THIS WILL ONLY BE CALLED IF USING THE RENDERJOB CONVENIENCE FUNCTION.
	*
	* Because there can only be one job in the queue when using RenderJob, this will be called when
	* the render is complete, and the executor has been released. This allows you to queue up another
	* job immediately as a result of the OnRenderFinished callback.
	*/
	UPROPERTY(BlueprintAssignable)
	FMoviePipelineWorkFinished OnRenderFinished;


private:
	/** Called when the executor is finished so that we can release it and stop reporting IsRendering() == true. */
	UE_API void OnExecutorFinished(UMoviePipelineExecutorBase* InPipelineExecutor, bool bSuccess);
	UE_API void OnIndividualJobFinished(FMoviePipelineOutputData Results);

private:
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineExecutorBase> ActiveExecutor;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<UMoviePipelineQueue> CurrentQueue;
	
	TOptional<FMoviePipelineOutputData> CachedRenderJobResults;
	TSubclassOf<UMovieRenderDebugWidget> CachedProgressWidgetClass;
	bool bCachedRenderPlayerViewport;
};

#undef UE_API
