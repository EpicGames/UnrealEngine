// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineSetting.h"
#include "ImageWriteStream.h"
#include "MoviePipelineRenderPass.generated.h"

UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMoviePipelineRenderPass : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:
	
	void Setup(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
	{
		SetupImpl(InPassInitSettings);
	}

	void Teardown()
	{
		WaitUntilTasksComplete();
		TeardownImpl();
	}

	/** Called at the start of a new frame to allow the render pass to perform any pre-frame operations */
	void OnFrameStart()
	{
		OnFrameStartImpl();
	}

	/** Called at the start and end of tile rendering, allowing per-tile operations */
	virtual void OnTileStart(FIntPoint TileIndexes)
	{
		OnTileStartImpl(TileIndexes);
	}
	virtual void OnTileEnd(FIntPoint TileIndexes)
	{
		OnTileEndImpl(TileIndexes);
	}
	
	/** An array of identifiers for the output buffers expected as a result of this render pass. */
	void GatherOutputPasses(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses)
	{
		GatherOutputPassesImpl(ExpectedRenderPasses);
	}

	/** This will called for each requested sample. */
	void RenderSample_GameThread(const FMoviePipelineRenderPassMetrics& InSampleState)
	{
		RenderSample_GameThreadImpl(InSampleState);
	}

	UE_DEPRECATED(5.6, "This function is no longer in use.")
	bool IsAlphaInTonemapperRequired() const
	{
		return false;
	}

	virtual bool NeedsFrameThrottle() const { return false; }


protected:
	virtual bool IsValidOnShots() const override { return true; }
	virtual bool IsValidOnPrimary() const override { return true; }
#if WITH_EDITOR
	virtual FText GetCategoryText() const override { return NSLOCTEXT("MovieRenderPipeline", "RenderingCategoryName_Text", "Rendering"); }
#endif
protected:
	virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) {}
	virtual void WaitUntilTasksComplete() {}
	virtual void TeardownImpl() {}
	virtual void OnFrameStartImpl() {}
	virtual void OnTileStartImpl(FIntPoint TileIndexes) {}
	virtual void OnTileEndImpl(FIntPoint TileIndexes) {}
	virtual void GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses) {}
	virtual void RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState) {}
};
