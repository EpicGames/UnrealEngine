// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Internationalization/Text.h"
#include "MoviePipelineSetting.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMoviePipeline;
struct FSlateBrush;
struct FMoviePipelineFormatArgs;
struct FMoviePipelineShotRenderTelemetry;
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;

enum class EMoviePipelineValidationState : uint8
{
	Valid = 0,
	Warnings = 1,
	Errors = 2
};

/**
* A base class for all Movie Render Pipeline settings.
*/
UCLASS(MinimalAPI, BlueprintType, Abstract)
class UMoviePipelineSetting : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMoviePipelineSetting();

	/**
	* This is called once on a setting when the movie pipeline is first set up. If the setting
	* only exists as part of a shot override, it will be called once when the shot is initialized.
	*/
	UE_API void OnMoviePipelineInitialized(UMoviePipeline* InPipeline);

	/**
	* This is called once on a setting when the movie pipeline is shut down. If the setting
	* only exists as part of a shot override, it will be called once the shot is finished.
	* see shot-related callbacks so that they work properly with shot-overrides.
	*/
	void OnMoviePipelineShutdown(UMoviePipeline* InPipeline) { TeardownForPipelineImpl(InPipeline); }

	/**
	* Called only on settings that have been added to the Primary Configuration to let you know that
	* a shot is about to be rendered. Useful if your setting needs to know something about the shot
	* to do something correctly, without using a per-shot override.
	*/
	void OnSetupForShot(UMoviePipelineExecutorShot* InShot) { OnSetupForShotImpl(InShot); }
	/**
	* Called only on settings that have been added to the Primary Configuration to let you know that
	* a shot has finished rendering and is being torn down.
	*/
	void OnTeardownForShot(UMoviePipelineExecutorShot* InShot) { OnTeardownForShotImpl(InShot); }

	/**
	* When rendering in a new process some settings may need to provide command line arguments
	* to affect engine settings that need to be set before most of the engine boots up. This function
	* allows a setting to provide these when the user wants to run in a separate process. This won't
	* be used when running in the current process because it is too late to modify the command line.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	void BuildNewProcessCommandLineArgs(UPARAM(ref) TArray<FString>& InOutUnrealURLParams, UPARAM(ref) TArray<FString>& InOutCommandLineArgs, UPARAM(ref) TArray<FString>& InOutDeviceProfileCvars, UPARAM(ref) TArray<FString>& InOutExecCmds) const { BuildNewProcessCommandLineArgsImpl(InOutUnrealURLParams, InOutCommandLineArgs, InOutDeviceProfileCvars, InOutExecCmds); }

	/**
	* Attempt to validate the configuration the user has chosen for this setting. Caches results for fast lookup in UI later.
	*/
	void ValidateState() { ValidateStateImpl(); }

	// UObject Interface
	UE_API virtual UWorld* GetWorld() const override;
	// ~UObject Interface

	// Post Finalize Export
	bool HasFinishedExporting() { return HasFinishedExportingImpl(); }
	void BeginExport() { BeginExportImpl(); }
	// ~Post Finalize Export

	/** Updates telemetry data for this setting. Should only be used by settings that ship with Movie Render Queue. */
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const { }

protected:
	UE_API UMoviePipeline* GetPipeline() const;

	virtual void SetupForPipelineImpl(UMoviePipeline* InPipeline) {}
	virtual void TeardownForPipelineImpl(UMoviePipeline* InPipeline) {}
	virtual	void OnSetupForShotImpl(UMoviePipelineExecutorShot* InShot) {}
	virtual	void OnTeardownForShotImpl(UMoviePipelineExecutorShot* InShot) {}
	
public:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Warning: This gets called on the CDO of the object */
	virtual FText GetDisplayText() const { return this->GetClass()->GetDisplayNameText(); }
	/** Warning: This gets called on the CDO of the object */
	virtual FText GetCategoryText() const { return NSLOCTEXT("MovieRenderPipeline", "DefaultCategoryName_Text", "Settings"); }

	/** Return a string to show in the footer of the details panel. Will be combined with other selected settings. */
	virtual FText GetFooterText(UMoviePipelineExecutorJob* InJob) const { return FText(); }

	/** Can this setting be disabled? UI only. */
	virtual bool CanBeDisabled() const { return true; }

	/** What icon should this setting use when displayed in the tree list. */
	const FSlateBrush* GetDisplayIcon() { return nullptr; }

	/** What tooltip should be displayed for this setting when hovered in the tree list? */
	FText GetDescriptionText() { return FText(); }
#endif
	/** Can this configuration setting be added to shots? If not, it will throw an error when trying to add it to a shot config. */
	virtual bool IsValidOnShots() const PURE_VIRTUAL(UMoviePipelineSetting::IsValidOnShots, return false; );
	/** Can this configuration setting be added to the primary configuration? If not, it will throw an error when trying to add it to the primary configuration. */
	virtual bool IsValidOnPrimary() const PURE_VIRTUAL(UMoviePipelineSetting::IsValidOnPrimary, return false; );

	/**
	* If true, then this setting will be included when searching for settings even if it was added transiently. This is used for the rare case where a setting
	* needs to be run (to set reasonable default values) even if the user hasn't added it.
	*/
	virtual bool IgnoreTransientFilters() const { return false; }

	/**
	* Higher priority settings are run after lower priority settings when setting up shots during rendering. This is run in reverse during teardown!
	*/
	virtual int32 GetPriority() const { return 0; }

	// Validation
	/** What is the result of the last validation? Only valid if the setting has had ValidateState() called on it. */
	virtual EMoviePipelineValidationState GetValidationState() const { return ValidationState; }

	/** Attempt to validate the configuration the user has chosen for this setting. Caches results for fast lookup in UI later. */
	virtual void ValidateStateImpl() { ValidationResults.Reset(); ValidationState = EMoviePipelineValidationState::Valid; }

	/** Get a human-readable text describing what validation errors (if any) the call to ValidateState() produced. */
	UE_API virtual TArray<FText> GetValidationResults() const;

	/** Return Key/Value pairs that you wish to be usable in the Output File Name format string or file metadata. This allows settings to add format strings based on their values. */
	virtual void GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const {}
	
	/** Modify the Unreal URL and Command Line Arguments when preparing the setting to be run in a new process. */
	virtual void BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const { }

	/** Can only one of these settings objects be active in a valid pipeline? */
	virtual bool IsSolo() const { return true; }
	
	/** Is this setting enabled by the user in the UI? */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	virtual bool IsEnabled() const { return bEnabled; }
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	virtual void SetIsEnabled(bool bInEnabled) { bEnabled = bInEnabled; }

	/** Has this setting finished any export-related things it needs to do post-finalize? */
	virtual bool HasFinishedExportingImpl() { return true; }
	/** Called once when all files have been finalized. */
	virtual void BeginExportImpl() { }
	
private:
	UPROPERTY(Transient)
	TWeakObjectPtr<UMoviePipeline> CachedPipeline;

	/** Is this setting currently enabled? Disabled settings are like they never existed. */
	UPROPERTY()
	bool bEnabled;
protected:
	/** What was the result of the last call to ValidateState() */
	EMoviePipelineValidationState ValidationState;

	/** If ValidationState isn't valid, what text do we want to show the user to explain to them why? */
	TArray<FText> ValidationResults;
};

#undef UE_API
