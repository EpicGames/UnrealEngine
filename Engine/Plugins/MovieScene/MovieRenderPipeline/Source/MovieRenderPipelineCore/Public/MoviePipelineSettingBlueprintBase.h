// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/CoreDelegates.h"
#include "MoviePipelineSetting.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineSettingBlueprintBase.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API


/**
* A base class for all Movie Render Pipeline settings which can be implemented in Blueprints. This features
* a slightly different API than the regular UMoviePipelineSetting to make the Blueprint integration nicer
* without breaking the C++ API backwards compatibility.
*/
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMoviePipelineSetting_BlueprintBase : public UMoviePipelineSetting
{
	GENERATED_BODY()
		
public:
	UMoviePipelineSetting_BlueprintBase()
	{
		CategoryText = NSLOCTEXT("MovieRenderPipelineBP", "DefaultCategoryName_Text", "Custom Settings");
		bIsValidOnPrimary = true;
		bIsValidOnShots = true;
		bCanBeDisabled = true;
	}
	
	// Setup
	virtual void SetupForPipelineImpl(UMoviePipeline* InPipeline) override 
	{
		ReceiveSetupForPipelineImpl(InPipeline);
		FCoreDelegates::OnBeginFrame.AddUObject(this, &UMoviePipelineSetting_BlueprintBase::OnEngineTickBeginFrame);
	}
	
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "SetupForPipeline"))
	UE_API void ReceiveSetupForPipelineImpl(UMoviePipeline* InPipeline);
	
	// Teardown
	virtual void TeardownForPipelineImpl(UMoviePipeline* InPipeline) override 
	{ 
		FCoreDelegates::OnBeginFrame.RemoveAll(this);
		ReceiveTeardownForPipelineImpl(InPipeline); 
	}
	
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="TeardownForPipeline"))
	UE_API void ReceiveTeardownForPipelineImpl(UMoviePipeline* InPipeline);
	
	// Format Arguments & EXR Metadata
	virtual void GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const override { InOutFormatArgs = ReceiveGetFormatArguments(InOutFormatArgs); }
	UFUNCTION(BlueprintNativeEvent, meta=(DisplayName="GetFormatArgumentsAndMetadata"))
	UE_API FMoviePipelineFormatArgs ReceiveGetFormatArguments(UPARAM(ref) FMoviePipelineFormatArgs& InOutFormatArgs) const;
	// Native Implementation so that if a blueprint class doesn't override this function we still pass on the previous arguments to the rest of the chain.
	FMoviePipelineFormatArgs ReceiveGetFormatArguments_Implementation(FMoviePipelineFormatArgs& InOutFormatArgs) const { return InOutFormatArgs; }
	
	// Tick
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnEngineTickBeginFrame();
	
public:
#if WITH_EDITOR
	/** Warning: This gets called on the CDO of the object */
	virtual FText GetCategoryText() const override { return CategoryText; }
	
	// UI Footer Text
	virtual FText GetFooterText(UMoviePipelineExecutorJob* InJob) const override { return ReceiveGetFooterText(InJob); } 
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="GetFooterText"))
	UE_API FText ReceiveGetFooterText(UMoviePipelineExecutorJob* InJob) const;
		
	virtual bool CanBeDisabled() const override { return bCanBeDisabled; }
#endif
	virtual bool IsValidOnPrimary() const override { return bIsValidOnPrimary; }
	virtual bool IsValidOnShots() const override { return bIsValidOnShots; }

	UE_API virtual void PostLoad() override;

protected:
	UPROPERTY(EditDefaultsOnly, Category = "Movie Pipeline")
	FText CategoryText;
	
	UPROPERTY(EditDefaultsOnly, Category = "Movie Pipeline")
	bool bIsValidOnPrimary;
	
	UPROPERTY(EditDefaultsOnly, Category = "Movie Pipeline")
	bool bIsValidOnShots;
	
	UPROPERTY(EditDefaultsOnly, Category = "Movie Pipeline")
	bool bCanBeDisabled;
};

#undef UE_API
