// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineConfigBase.h"

#include "MoviePipelineShotConfig.generated.h"


// Forward Declares
class UMoviePipelineRenderPass;

UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineShotConfig : public UMoviePipelineConfigBase
{
	GENERATED_BODY()
	
public:
	UMoviePipelineShotConfig()
	{
	}
	

protected:
	virtual bool CanSettingBeAdded(const UMoviePipelineSetting* InSetting) const override
	{
		check(InSetting);
		return InSetting->IsValidOnShots();
	}

};
