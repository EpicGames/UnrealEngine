// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PixelFormat.h"
#include "RHI.h"
#include "MovieRenderPipelineDataTypes.h"
#include "RHIGPUReadback.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/**
* When the GPU finishes rendering a view we can copy the resulting data back to the CPU. Because the
* buffer is still in use this requires flushing the RHI and stalling the GPU from doing any additional
* work while the copy is in progress. To resolve this issue, when the data is returned we schedule a copy
* to a surface that isn't in use which allows the GPU to resume work. Then when the surface is written
* to, we can copy it to the CPU without blocking the GPU. This requires ensuring that the surface is
* available, so this SurfaceReader implements a trigger system to ensure we don't try to access the 
* surface until it has been written to, combined with a round-robin of surfaces to avoid stalls.
*/
struct FMoviePipelineSurfaceReader
	: public TSharedFromThis<FMoviePipelineSurfaceReader, ESPMode::ThreadSafe>
{
	/** Construct a surface reader with the given pixel format and size. This will be the output specification. */
	UE_API FMoviePipelineSurfaceReader(EPixelFormat InPixelFormat, bool bInInvertAlpha);

	UE_API ~FMoviePipelineSurfaceReader();

	/** Initialize this reader so that it can be waited on. */
	UE_API void Initialize();

	/** Wait until this surface is available for reuse. Used to stall if we've run out of available surfaces. */
	UE_API void BlockUntilAvailable();

	/**
	* Safely resets the state of the wait event. When doing latent surface reading sometimes we may want to just
	* bail on reading a given frame. Should only be performed after flushing rendering commands.
	*/
	UE_API void Reset();

	bool WasEverQueued() const { return bQueuedForCapture; }
	bool IsAvailable() const { return AvailableEvent == nullptr; }

	/**
	* Issues a command to the GPU to copy the given SourceSurfaceSample to our local ReadbackTexture for this surface.
	*/
	UE_API void ResolveSampleToReadbackTexture_RenderThread(FRHICommandListImmediate& RHICmdList, const FTextureRHIRef& SourceSurfaceSample);

	UE_DEPRECATED(5.8, "ResolveSampleToReadbackTexture_RenderThread now requires the immediate RHI command list.")
	void ResolveSampleToReadbackTexture_RenderThread(const FTextureRHIRef& SourceSurfaceSample)
	{
		ResolveSampleToReadbackTexture_RenderThread(FRHICommandListImmediate::Get(), SourceSurfaceSample);
	}

	/**
	* Maps the ReadbackTexture to the CPU (which should have been resolved to before this point) and copies the data over.
	*/
	UE_API void CopyReadbackTexture_RenderThread(FRHICommandListImmediate& RHICmdList, TUniqueFunction<void(TUniquePtr<FImagePixelData>&&)>&& InFunctionCallback, TSharedPtr<FImagePixelDataPayload, ESPMode::ThreadSafe> InFramePayload);

	UE_DEPRECATED(5.8, "CopyReadbackTexture_RenderThread now requires the immediate RHI command list.")
	void CopyReadbackTexture_RenderThread(TUniqueFunction<void(TUniquePtr<FImagePixelData>&&)>&& InFunctionCallback, TSharedPtr<FImagePixelDataPayload, ESPMode::ThreadSafe> InFramePayload)
	{
		CopyReadbackTexture_RenderThread(FRHICommandListImmediate::Get(), MoveTemp(InFunctionCallback), InFramePayload);
	}

protected:
	friend struct FMoviePipelineSurfaceQueue;
	/** Set up this surface to the specified width/height */
	UE_API void ResizeImpl(uint32 Width, uint32 Height);

protected:
	/** Optional event that is triggered when the surface is no longer in use */
	FEvent* AvailableEvent;

	/** Texture used to store the resolved render target */
	TUniquePtr<FRHIGPUTextureReadback> ReadbackTexture;

	/** The desired pixel format of the resolved textures */
	EPixelFormat PixelFormat;

	/** The desired size for this texture */
	FIntPoint Size;

	bool bQueuedForCapture;
	bool bInvertAlpha;
};

struct FMoviePipelineSurfaceQueue
{
public:
	UE_API FMoviePipelineSurfaceQueue(FIntPoint InSurfaceSize, EPixelFormat InPixelFormat, uint32 InNumSurfaces, bool bInInvertAlpha);
	UE_API ~FMoviePipelineSurfaceQueue();

	// Movable
	FMoviePipelineSurfaceQueue(FMoviePipelineSurfaceQueue&&) = default;
	FMoviePipelineSurfaceQueue& operator=(FMoviePipelineSurfaceQueue&&) = default;

	// Non Copyable
	FMoviePipelineSurfaceQueue(const FMoviePipelineSurfaceQueue&) = delete;
	FMoviePipelineSurfaceQueue& operator=(const FMoviePipelineSurfaceQueue&) = delete;
public:

	UE_API void OnRenderTargetReady_RenderThread(const FTextureRHIRef InRenderTarget, TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> InPayload, TUniqueFunction<void(TUniquePtr<FImagePixelData>&&)>&& InFunctionCallback);

	UE_API void BlockUntilAnyAvailable();
	UE_API void Shutdown();

	/** Updates the stored last used frame to be the current frame */
	UE_API void UpdateLastUsedFrame();

	/** Returns true if the amount of frames between the current frame and the last used frame is greater than a specific threshold */
	UE_API bool IsStale() const;
	
private:
	struct FResolveSurface
	{
		UE_NONCOPYABLE(FResolveSurface);

		FResolveSurface(const EPixelFormat InPixelFormat, const FIntPoint InSurfaceSize, const bool bInInvertAlpha)
			: Surface(new FMoviePipelineSurfaceReader(InPixelFormat, bInInvertAlpha))
		{
			Surface->ResizeImpl(InSurfaceSize.X, InSurfaceSize.Y);
		}

		const TSharedRef<FMoviePipelineSurfaceReader, ESPMode::ThreadSafe> Surface;

		TUniqueFunction<void(TUniquePtr<FImagePixelData>&&)> FunctionCallback;
		TSharedPtr<FImagePixelDataPayload, ESPMode::ThreadSafe> FunctionPayload;
	};
	
	/** Which surface is the next target to be copied to. */
	int32 CurrentFrameIndex;

	/** The array of surfaces we can resolve a render target to. More peformant if there's more than one. */
	TArray<FResolveSurface> Surfaces;
	
	/** How many frames do we wait before we resolve a frame? */
	int32 FrameResolveLatency;

	/** The last frame this surface was used. Surfaces that haven't been used for a certain number of frames are cleaned up */
	uint64 LastUsedFrame;
};

#undef UE_API
