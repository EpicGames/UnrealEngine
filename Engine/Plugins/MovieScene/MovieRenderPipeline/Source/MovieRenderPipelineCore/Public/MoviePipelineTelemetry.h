// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Graph/MovieGraphQuickRenderSettings.h"
#include "HAL/Platform.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMovieGraphEvaluatedConfig;
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;

/** Context used to send begin-shot render telemetry. */
struct FMoviePipelineBeginShotRenderTelemetryContext
{
	UMoviePipelineExecutorShot* Shot = nullptr;
	UMovieGraphEvaluatedConfig* EvaluatedConfig = nullptr;
	bool bIsBasicConfig = false;
};

/** Context used to send end-shot render telemetry. */
struct FMoviePipelineEndShotRenderTelemetryContext
{
	/** Merges in relevant properties within InTelemetry into this context instance. */
	UE_API void MergeRenderPassTelemetry(const FMoviePipelineEndShotRenderTelemetryContext& InTelemetry);

	bool bIsGraph = false;
	bool bIsBasicConfig = false;
	bool bWasSuccessful = false;
	bool bWasCanceled = false;
	int32 TotalRenderedFrameCount = 0;
	bool bUsesAccumulationDepthOfField = false;
	int32 AccumulationDepthOfFieldSampleCount = 0;
	float AccumulationDepthOfFieldSplatSize = 0.f;
};

/** Telemetry data that is captured when a shot begins rendering. Only for use with settings/nodes shipped with Movie Render Queue/Graph. */
struct FMoviePipelineShotRenderTelemetry
{
	bool bIsGraph = false;
	bool bIsBasicConfig = false;
	bool bIsQuickRender = false;
	bool bUsesDeferred = false;
	bool bUsesPathTracer = false;
	bool bUsesPanoramic = false;
	bool bUsesHighResTiling = false;
	bool bUsesNDisplay = false;
	bool bUsesObjectID = false;
	bool bUsesUIRenderer = false;
	bool bUsesMultiCamera = false;
	bool bUsesScripting = false;
	bool bUsesSubgraphs = false;
	bool bUsesPPMs = false;
	bool bUsesAudio = false;
	bool bMovieEncodesAudio = false;
	bool bUsesCommandLineEncoder = false;
	bool bSetsConsoleVariables = false;
	bool bSetsQueueConsoleVariables = false;
	bool bUsesConsoleVariablePreset = false;
	bool bUsesModifier = false;
	bool bUsesLightModifier = false;
	bool bUsesMpcModifier = false;
	bool bUsesAvid = false;
	bool bUsesProRes = false;
	bool bUsesMP4 = false;
	bool bUsesJPG = false;
	bool bUsesPNG = false;
	bool bUsesBMP = false;
	bool bUsesEXR = false;
	bool bUsesMultiEXR = false;
	bool bUsesTmvMovie = false;
	bool bUsesTmvImageSequence = false;
	bool bUsesMotionDesignRundown = false;
	int32 ResolutionX = 0;
	int32 ResolutionY = 0;
	int32 HandleFrameCount = 0;
	int32 TemporalSampleCount = 0;
	int32 SpatialSampleCount = 0;
	int32 WarmUpFrames = 0;
	int32 LayerWarmUpFrames = 0;
	int32 RenderLayerCount = 0;
	int32 HighResTileCount = 0;
	float HighResOverlap = 0;
	bool bUsesPageToSystemMemory = false;
	FString ScalabilityQualityLevel;
	FString AntiAliasingType;

	// Note: If adding an entry here, make sure to also update FMoviePipelineTelemetry::SendBeginShotRenderTelemetry()
	// Also remember to track the telemetry in both the graph and legacy, and perform schematization.
};

/** Responsible for sending out telemetry for Movie Render Queue, Movie Render Graph, and Quick Render. */
class FMoviePipelineTelemetry
{
public:
	/** Sends out telemetry that captures queue-level information. Called by either MRQ or MRG, not Quick Render. */
	static UE_API void SendRendersRequestedTelemetry(const bool bIsLocal, TArray<UMoviePipelineExecutorJob*>&& InJobs);

	/** Sends out telemetry that captures Quick Render information for the provided mode. */
	static UE_API void SendQuickRenderRequestedTelemetry(const EMovieGraphQuickRenderMode QuickRenderMode);

	/** Sends out telemetry that includes information about the shot being rendered (the type of rendering being done, which types of settings/nodes are being used, etc). */
	static UE_API void SendBeginShotRenderTelemetry(const FMoviePipelineBeginShotRenderTelemetryContext& InContext);

	/** Sends out telemetry that indicates whether the shot was rendered successfully. */
	static UE_API void SendEndShotRenderTelemetry(const FMoviePipelineEndShotRenderTelemetryContext& InContext);

private:
	/** Tracks whether the current render request originated from Quick Render. If false, the render originated from a queue in either MRQ or MRG. */
	inline static bool bIsCurrentRenderRequestQuickRender = false;
	
	/** Gets the current session type (Editor, DashGame, Shipping). */
	static UE_API FString GetSessionType();

	/** Returns a populated telemetry object for the shot and graph that's specified. */
	static UE_API FMoviePipelineShotRenderTelemetry GatherShotRenderTelemetryForGraph(const UMoviePipelineExecutorShot* InShot, UMovieGraphEvaluatedConfig* InEvaluatedConfig, const bool bIsBasicConfig);

	/** Returns a populated telemetry object for the shot that's specified. For use with the legacy system, not the graph. */
	static UE_API FMoviePipelineShotRenderTelemetry GatherShotRenderTelemetryForLegacy(UMoviePipelineExecutorShot* InShot);
};

#undef UE_API
