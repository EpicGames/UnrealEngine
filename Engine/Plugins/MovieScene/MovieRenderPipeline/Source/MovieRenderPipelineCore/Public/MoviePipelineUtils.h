// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CineCameraComponent.h"
#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "Engine/Scene.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "Evaluation/MovieSceneTimeTransform.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "MoviePipelinePanoramicBlenderBase.h"
#include "MoviePipelineQueue.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MovieSceneSequenceID.h"

// Forward Declare
class UClass;
class UMoviePipelineAntiAliasingSetting;
class UMoviePipelineExecutorShot;
class UMovieSceneSequence;

namespace MoviePipeline
{
static UWorld* FindCurrentWorld()
{
	UWorld* World = nullptr;
	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (WorldContext.WorldType == EWorldType::Game)
		{
			World = WorldContext.World();
		}
#if WITH_EDITOR
		else if (GIsEditor && WorldContext.WorldType == EWorldType::PIE)
		{
			World = WorldContext.World();
			if (World)
			{
				return World;
			}
		}
#endif
	}

	return World;
}

MOVIERENDERPIPELINECORE_API void GetPassCompositeData(FMoviePipelineMergerOutputFrame* InMergedOutputFrame, TArray<MoviePipeline::FCompositePassInfo>& OutCompositedPasses);

}

#define MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_BOOL(InOutVariable, CVarName, OverrideValue, bUseOverride) \
{ \
	const bool bTrackFrequentCalls = false; \
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName, bTrackFrequentCalls); \
	if(ensureMsgf(CVar, TEXT("Failed to find CVar " #CVarName " to override."))) \
	{ \
		if(bUseOverride) \
		{ \
			InOutVariable = CVar->GetBool(); \
			CVar->SetWithCurrentPriority(OverrideValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
		else \
		{ \
			CVar->SetWithCurrentPriority(InOutVariable, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
	} \
}

#define MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT(InOutVariable, CVarName, OverrideValue, bUseOverride) \
{ \
	const bool bTrackFrequentCalls = false; \
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName, bTrackFrequentCalls); \
	if(ensureMsgf(CVar, TEXT("Failed to find CVar " #CVarName " to override."))) \
	{ \
		if(bUseOverride) \
		{ \
			InOutVariable = CVar->GetInt(); \
			CVar->SetWithCurrentPriority(OverrideValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
		else \
		{ \
			CVar->SetWithCurrentPriority(InOutVariable, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
	} \
}

#define MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_FLOAT(InOutVariable, CVarName, OverrideValue, bUseOverride) \
{ \
	const bool bTrackFrequentCalls = false; \
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName, bTrackFrequentCalls); \
	if(ensureMsgf(CVar, TEXT("Failed to find CVar " #CVarName " to override."))) \
	{ \
		if(bUseOverride) \
		{ \
			InOutVariable = CVar->GetFloat(); \
			CVar->SetWithCurrentPriority(OverrideValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
		else \
		{ \
			CVar->SetWithCurrentPriority(InOutVariable, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
	} \
}

#define MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_BOOL_IF_EXIST(InOutVariable, CVarName, OverrideValue, bUseOverride) \
{ \
	const bool bTrackFrequentCalls = false; \
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName, bTrackFrequentCalls); \
	if(CVar) \
	{ \
		if(bUseOverride) \
		{ \
			InOutVariable = CVar->GetBool(); \
			CVar->SetWithCurrentPriority(OverrideValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
		else \
		{ \
			CVar->SetWithCurrentPriority(InOutVariable, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
	} \
}

#define MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_INT_IF_EXIST(InOutVariable, CVarName, OverrideValue, bUseOverride) \
{ \
	const bool bTrackFrequentCalls = false; \
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName, bTrackFrequentCalls); \
	if(CVar) \
	{ \
		if(bUseOverride) \
		{ \
			InOutVariable = CVar->GetInt(); \
			CVar->SetWithCurrentPriority(OverrideValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
		else \
		{ \
			CVar->SetWithCurrentPriority(InOutVariable, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
	} \
}

#define MOVIEPIPELINE_STORE_AND_OVERRIDE_CVAR_FLOAT_IF_EXIST(InOutVariable, CVarName, OverrideValue, bUseOverride) \
{ \
	const bool bTrackFrequentCalls = false; \
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName, bTrackFrequentCalls); \
	if(CVar) \
	{ \
		if(bUseOverride) \
		{ \
			InOutVariable = CVar->GetFloat(); \
			CVar->SetWithCurrentPriority(OverrideValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
		else \
		{ \
			CVar->SetWithCurrentPriority(InOutVariablee, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability); \
		} \
	} \
}

/*
 * Log a message once per render.
 *
 * Note that the arguments to this are the same as for UE_LOG, but also takes an additional `Identifier`. The identifier is what uniquely identifies
 * this specific message within the current SCOPE (identifiers are NOT unique across scopes, so the same log message will be logged more than once per
 * render if it appears in different scopes).
 */
#define MRG_LOG_ONCE_PER_RENDER(Identifier, LogCategory, LogVerbosity, LogFormat, ...)									\
	static UWorld* UE_JOIN(__LastRenderWorld__, Identifier) = nullptr;													\
	static TArray<FString> UE_JOIN(__LoggedMessages__, Identifier);														\
																														\
	/* If this world has not been encountered yet (ie, no renders yet, or this is a new render), clear out */			\
	/* the messages that have been recorded as already logged. */														\
	if (UE_JOIN(__LastRenderWorld__, Identifier) != MoviePipeline::FindCurrentWorld())									\
	{																													\
		UE_JOIN(__LastRenderWorld__, Identifier) = MoviePipeline::FindCurrentWorld();									\
		UE_JOIN(__LoggedMessages__, Identifier).Empty();																\
	}																													\
																														\
	/* If this message hasn't been logged, record that it has been logged so we can skip it for other frames. */		\
	const FString UE_JOIN(__FinalLogMsg__, Identifier) = FString::Printf(LogFormat, ##__VA_ARGS__);						\
	if (!UE_JOIN(__LoggedMessages__, Identifier).Contains(UE_JOIN(__FinalLogMsg__, Identifier)))						\
	{																													\
		UE_LOGF(LogCategory, LogVerbosity, "%ls", *UE_JOIN(__FinalLogMsg__, Identifier));							\
		UE_JOIN(__LoggedMessages__, Identifier).Add(UE_JOIN(__FinalLogMsg__, Identifier));								\
	}																													\

namespace UE
{
	namespace MovieRenderPipeline
	{
		/** Finds UMoviePipelineSetting classes declared in the project using the asset registry, potentially including blueprint implemented classes. */
		MOVIERENDERPIPELINECORE_API TArray<UClass*> FindMoviePipelineSettingClasses(UClass* InBaseClass, const bool bIncludeBlueprints = true);
		/** Anti-aliasing is defined as a Project Setting, and then MRG can optionally override it. Utility function for getting the project setting and optionally applying the MRQ override to the return value. */
		MOVIERENDERPIPELINECORE_API EAntiAliasingMethod GetEffectiveAntiAliasingMethod(bool bOverride, EAntiAliasingMethod OverrideMethod);
		/** Anti-aliasing is defined as a Project Setting, and then MRQ can optionally override it. Utility function for getting the project setting and optionally applying the MRQ override to the return value. */
		MOVIERENDERPIPELINECORE_API EAntiAliasingMethod GetEffectiveAntiAliasingMethod(const UMoviePipelineAntiAliasingSetting* InSetting);
		
		UE_DEPRECATED(5.3, "Do not use, this is here as a temporary workaround for another issue.")
		MOVIERENDERPIPELINECORE_API uint64 GetRendererFrameCount();
		
		/**
		 * Updates the provided scene view in accordance with the show flags that are currently set (eg, changes the specular override color if
		 * the OverrideDiffuseAndSpecular show flag is set). The show flags that are looked at here are usually the ones that are set when the
		 * view mode index is changed.
		 */
		MOVIERENDERPIPELINECORE_API void UpdateSceneViewForShowFlags(FSceneView* View);
	}

	namespace MoviePipeline
	{
		/** Get the value of the MovieRenderPipeline.AlphaOutputOverride console variable. */
		MOVIERENDERPIPELINECORE_API bool GetAlphaOutputOverride();
		MOVIERENDERPIPELINECORE_API void ConformOutputFormatStringToken(FString& InOutFilenameFormatString, const FStringView InToken, const FName& InNodeName, const FName& InBranchName);
		MOVIERENDERPIPELINECORE_API void ValidateOutputFormatString(FString& InOutFilenameFormatString, const bool bTestRenderPass, const bool bTestFrameNumber, const bool bIncludeCameraName = false);
		MOVIERENDERPIPELINECORE_API void RemoveFrameNumberFormatStrings(FString& InOutFilenameFormatString, const bool bIncludeShots);
		/** De-duplicates the provided array of strings by appending (1), (2), etc to the end of duplicates. */
		MOVIERENDERPIPELINECORE_API void DeduplicateNameArray(TArray<FString>& InOutNames);

		MOVIERENDERPIPELINECORE_API FString GetJobAuthor(const UMoviePipelineExecutorJob* InJob);
		MOVIERENDERPIPELINECORE_API void GetSharedFormatArguments(TMap<FString, FString>& InFilenameArguments, TMap<FString, FString>& InFileMetadata, const FDateTime& InDateTime, const int32 InVersionNumber, const UMoviePipelineExecutorJob* InJob, const FTimespan& InInitializationTimeOffset = FTimespan());
		MOVIERENDERPIPELINECORE_API void GetHardwareUsageMetadata(TMap<FString, FString>& InFileMetadata, const FString& InOutputDir);
		MOVIERENDERPIPELINECORE_API void GetDiagnosticMetadata(TMap<FString, FString>& InFileMetadata, const bool bIsGraph);
		MOVIERENDERPIPELINECORE_API void GetMetadataFromCineCamera(UCineCameraComponent* InComponent, const FString& InPrefix, TMap<FString, FString>& InOutMetadata);
		MOVIERENDERPIPELINECORE_API void GetMetadataFromCineCamera(UCineCameraComponent* InComponent, const FString& InCameraName, const FString& InRenderPassName, TMap<FString, FString>& InOutMetadata);
		MOVIERENDERPIPELINECORE_API void GetMetadataFromCameraLocRot(const FString& InCameraName, const FString& InRenderPassName, const FVector& InCurLoc, const FRotator& InCurRot, const FVector& InPrevLoc, const FRotator& InPrevRot, TMap<FString, FString>& InOutMetadata);
		MOVIERENDERPIPELINECORE_API void GetMetadataFromCameraLocRot(const FVector& InCurLoc, const FRotator& InCurRot, const FVector& InPrevLoc, const FRotator& InPrevRot, const FString& InPrefix, TMap<FString, FString>& InOutMetadata);

		MOVIERENDERPIPELINECORE_API FMoviePipelineRenderPassMetrics GetRenderPassMetrics(UMoviePipelinePrimaryConfig* InPrimaryConfig, UMoviePipelineExecutorShot* InPipelineExecutorShot, const FMoviePipelineRenderPassMetrics& InRenderPassMetrics, const FIntPoint& InEffectiveOutputResolution);
		MOVIERENDERPIPELINECORE_API bool CanWriteToFile(const TCHAR* InFilename, bool bOverwriteExisting);
		MOVIERENDERPIPELINECORE_API FString GetPaddingFormatString(int32 InZeroPadCount, const int32 InFrameNumber);
		MOVIERENDERPIPELINECORE_API void DoPostProcessBlend(const FVector& InViewLocation, const class UWorld* InWorld, const struct FMinimalViewInfo& InViewInfo, class FSceneView* InOutView);

		MOVIERENDERPIPELINECORE_API void SetSkeletalMeshClothSubSteps(const int32 InSubdivisionCount, UWorld* InWorld, TMap<TWeakObjectPtr<UObject>, TArray<::MoviePipeline::FClothSimSettingsCache>>& InClothSimCache);
		MOVIERENDERPIPELINECORE_API void RestoreSkeletalMeshClothSubSteps(const TMap<TWeakObjectPtr<UObject>, TArray<::MoviePipeline::FClothSimSettingsCache>>& InClothSimCache);


		/** When using spatial/temporal samples without anti-aliasing, get the sub-pixel jitter for the given frame index. FrameIndex is modded by InSamplesPerFrame so that the aa jitter pattern repeats every output frame. */
		MOVIERENDERPIPELINECORE_API FVector2f GetSubPixelJitter(int32 InFrameIndex, int32 InSamplesPerFrame);

		/**
		 * Scales the specified resolution by the specified overscan amount, performing appropriate clamping and rounding.
		*/
		MOVIERENDERPIPELINECORE_API FIntPoint ScaleResolutionByOverscan(const float OverscanPercentage, const FIntPoint& InOutputResolution);
		namespace Panoramic
		{
			/** Returns an array of values distributed throughout the given range, used to calculate which angles the Panoramic renderers should orient the camera to. */
			MOVIERENDERPIPELINECORE_API TArray<float> DistributeValuesInInterval(float InMin, float InMax, int32 InNumDivisions, bool bInInclusiveMax);
			/** Given the primary camera rotation and a pane describing the current pane, calculate a new rotation / location for the camera. If stereo was supported, the location would change with the rotation. */
			MOVIERENDERPIPELINECORE_API void GetCameraOrientationForStereo(FVector& OutLocation, FRotator& OutRotation, FRotator& OutLocalRotation, const UE::MoviePipeline::FPanoramicPane& InPane, const int32 InStereoIndex, const bool bInPrevPosition);
			/** Auto-exposure cubemap passes are laid out in a 3x2 arrangement, and rounded down to a multiple of 8 pixels. */
			MOVIERENDERPIPELINECORE_API int32 ComputeAutoExposureCubeCaptureSize(FIntPoint Resolution);
		}

		// When using EXR metadata what is the prefix that metadata tokens should go under? This returns "unreal/<layerName>/<rendererName>/<cameraName>" but be aware that the metadata keys will be potentially
		// renamed when actually writing the EXR to disk so that the per-layer data goes into a key that matches the EXR layer names (which makes the lookup in external software much easier).
		MOVIERENDERPIPELINECORE_API FString GetMetadataPrefixPath(const FMovieGraphRenderDataIdentifier& Identifier);
	}
}

namespace MoviePipeline
{
	MOVIERENDERPIPELINECORE_API void GetOutputStateFormatArgs(TMap<FString, FString>& InFilenameArguments, TMap<FString, FString>& InFileMetadata, const FString FrameNumber, const FString FrameNumberShot, const FString FrameNumberRel, const FString FrameNumberShotRel, const FString CameraName, const FString ShotName);

	/**
	 * Given a non-templated file name/path, returns the "frame-templated" file path. A frame-templated file path is one that has had its
	 * {frame_*} token, if any, replaced with the {frame_placeholder} token. Returns the original file path if no modifications were done.
	 */
	MOVIERENDERPIPELINECORE_API FString GetFrameTemplatedFilePath(const FString& InFilePath);

	/**
	 * Gets the job variable assignments for a specific graph. Creates a new variable assignments container if one was not found for the given graph.
	 * The owner of the variable assignments must be provided in case a new assignment needs to be created.
	 */
	MOVIERENDERPIPELINECORE_API TObjectPtr<UMovieJobVariableAssignmentContainer> GetOrCreateJobVariableAssignmentsForGraph(const UMovieGraphConfig* InGraph, TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& InVariableAssignments, UObject* InAssignmentsOwner);

	/** Refreshes the variable assignments for the given graph and its associated subgraphs. */
	MOVIERENDERPIPELINECORE_API void RefreshVariableAssignments(UMovieGraphConfig* InRootGraph, TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& InVariableAssignments, UObject* InAssignmentsOwner);
	
	/**
	 * Helps duplicate graph configs. Prevents re-duplications, duplicates sub-graphs (updates subgraph nodes accordingly), and potentially more.
	 * Returns the duplicated graph.
	 */
	MOVIERENDERPIPELINECORE_API UMovieGraphConfig* DuplicateConfigRecursive(UMovieGraphConfig* InGraphToDuplicate, TMap<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>>& OutDuplicatedGraphs);
	
	/** Iterate root-to-tails and generate a HierarchyNode for each level. Caches the sub-section range, playback range, camera cut range, etc. */
	void CacheCompleteSequenceHierarchy(UMovieSceneSequence* InSequence, TSharedPtr<FCameraCutSubSectionHierarchyNode> InRootNode);
	/** Matching function to Cache. Restores the sequence properties when given the root node. */
	void RestoreCompleteSequenceHierarchy(UMovieSceneSequence* InSequence, TSharedPtr<FCameraCutSubSectionHierarchyNode> InRootNode);
	/** Iterates tail to root building Hierarchy Nodes while correctly keeping track of which sub-section by GUID for correct enabling/disabling later. */
	void BuildSectionHierarchyRecursive(const FMovieSceneSequenceHierarchy& InHierarchy, UMovieSceneSequence* InRootSequence, const FMovieSceneSequenceID InSequenceId, const FMovieSceneSequenceID InChildId, TSharedPtr<FCameraCutSubSectionHierarchyNode> OutSubsectionHierarchy);
	/** Gets the inner and outer names for the shot by resolving camera bindings/shot names, etc. Not neccessairly the final names. */
	TTuple<FString, FString> GetNameForShot(const FMovieSceneSequenceHierarchy& InHierarchy, UMovieSceneSequence* InRootSequence, TSharedPtr<FCameraCutSubSectionHierarchyNode> InSubSectionHierarch);
	/** Given a leaf node, either caches the value of the hierarchy or restores it. Used to save the state before soloing a shot. */
	void SaveOrRestoreSubSectionHierarchy(TSharedPtr<FCameraCutSubSectionHierarchyNode> InLeaf, const bool bInSave);
	/** Given a leaf node, appropriately sets the IsActive flags for the whole hierarchy chain up to root for soloing a shot. */
	void SetSubSectionHierarchyActive(TSharedPtr<FCameraCutSubSectionHierarchyNode> InRoot, bool bInActive);
	/** Given a leaf node, searches for sections that will be partially evaluated when using temporal sub-sampling and prints a warning. */
	void CheckPartialSectionEvaluationAndWarn(const FFrameNumber& LeftDeltaTicks, TSharedPtr<FCameraCutSubSectionHierarchyNode> Node, UMoviePipelineExecutorShot* InShot, const FFrameRate& InRootDisplayRate);
}