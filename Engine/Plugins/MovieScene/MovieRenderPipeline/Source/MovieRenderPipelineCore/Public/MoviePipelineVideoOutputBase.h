// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineOutputBase.h"
#include "MovieRenderPipelineDataTypes.h"
#include "ImagePixelData.h"
#include "Async/AsyncWork.h"
#include "Async/TaskGraphInterfaces.h"
#include "Templates/Function.h"
#include "Stats/Stats.h"
#include "MoviePipelineVideoOutputBase.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

namespace MovieRenderPipeline
{
	struct IVideoCodecWriter
	{
		IVideoCodecWriter()
			: bConvertToSrgb(true)
		{}

		// The filename to actually write to disk with. This may include "de-duplication" numbers (MyFile (1).ext etc.)
		FString FileName;
		// The filename without de-duplication numbers, used to match up multiple incoming frames back to the same writer.
		// We use this when looking for existing writers so that we can avoid the de-duplication numbers perpetually increasing
		// due to the file existing on disk after the first frame comes in, and then the next one de-duplicating to one more than that.
		FString StableFileName;
		FMoviePipelineFormatArgs FormatArgs;
		bool bConvertToSrgb;

		// This array contains the indexes of each shot that contributed to this writer. This is needed so that after a shot/render, when
		// all we have is the completed blocks of audio samples, we can figure out which blocks go to which video clips. Because of how you can
		// split videos up by either shot, or by the whole sequence, there isn't an obvious mapping, so this array will tell the audio writing
		// system which audio blocks should be sent to the shot.
		struct FLightweightSourceData
		{
			int32 SubmittedFrameCount;
		};
		TMap<int32, FLightweightSourceData> LightweightSourceData;
	};
}

class FMoviePipelineBackgroundMediaTasks
{
private:
	FGraphEventRef LastCompletionEvent;

public:
	FGraphEventRef Execute(TUniqueFunction<void()> InFunctor)
	{
		if (LastCompletionEvent)
		{
			LastCompletionEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(InFunctor), GetStatId(), LastCompletionEvent);
		}
		else
		{
			LastCompletionEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(InFunctor), GetStatId());
		}
		return LastCompletionEvent;
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMoviePipelineBackgroundMediaTasks, STATGROUP_ThreadPoolAsyncTasks);
	}
};

typedef TTuple<TUniquePtr<MovieRenderPipeline::IVideoCodecWriter>, TPromise<bool>> FMoviePipelineCodecWriter;
/**
* A base class for video codec outputs for the Movie Pipeline system. To simplify encoder implementations
* this handles multi-threading for you and will call all of the encoding functions on a dedicated thread.
* This allows an encoder to do more expensive operations (such as image quantization) without implementing
* threading yourself, nor having to worry about blocking the game thread.
*/
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMoviePipelineVideoOutputBase : public UMoviePipelineOutputBase
{
	GENERATED_BODY()

public:
	UE_API UMoviePipelineVideoOutputBase();

protected:
	// UMoviePipelineOutputBase Interface
	UE_API virtual void OnReceiveImageDataImpl(FMoviePipelineMergerOutputFrame* InMergedOutputFrame) override;
	UE_API virtual bool HasFinishedProcessingImpl() override;
	UE_API virtual void BeginFinalizeImpl() override;
	UE_API virtual void FinalizeImpl() override;
	UE_API virtual void OnShotFinishedImpl(const UMoviePipelineExecutorShot* InShot, const bool bFlushToDisk) override;
#if WITH_EDITOR
	UE_API virtual FText GetFooterText(UMoviePipelineExecutorJob* InJob) const override;
#endif
	// ~UMoviePipelineOutputBase Interface

	// UMoviePipelineVideoOutputBase Interface
	virtual TUniquePtr<MovieRenderPipeline::IVideoCodecWriter> Initialize_GameThread(const FString& InFileName, FIntPoint InResolution, EImagePixelType InPixelType, ERGBFormat InPixelFormat, uint8 InBitDepth, uint8 InNumChannels)  PURE_VIRTUAL(UMoviePipelineVideoOutputBase::Initialize_GameThread, return nullptr; );
	virtual bool Initialize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter) PURE_VIRTUAL(UMoviePipelineVideoOutputBase::Initialize_EncodeThread, return true;);
	virtual void WriteFrame_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<MoviePipeline::FCompositePassInfo>&& InCompositePasses) PURE_VIRTUAL(UMoviePipelineVideoOutputBase::WriteFrame_EncodeThread);
	virtual void BeginFinalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter) PURE_VIRTUAL(UMoviePipelineVideoOutputBase::BeginFinalize_EncodeThread);
	virtual void Finalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter) PURE_VIRTUAL(UMoviePipelineVideoOutputBase::Finalize_EncodeThread);
	virtual const TCHAR* GetFilenameExtension() const PURE_VIRTUAL(UMoviePipelineVideoOutputBase::GetFilenameExtension, return TEXT(""););
	virtual bool IsAudioSupported() const PURE_VIRTUAL(UMoviePipelineVideoOutputBase::IsAudioSupported, return false;);
	// ~UMoviePipelineVideoOutputBase Interface

private:
	TArray<FMoviePipelineCodecWriter> AllWriters;
	bool bHasError;

	FGraphEventArray OutstandingTasks;
};

#undef UE_API
