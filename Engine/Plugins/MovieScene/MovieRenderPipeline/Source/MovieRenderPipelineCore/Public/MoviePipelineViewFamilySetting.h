// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineSetting.h"
#include "MoviePipelineViewFamilySetting.generated.h"

class FSceneViewFamily;

UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMoviePipelineViewFamilySetting : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:
	virtual bool IsValidOnShots() const override { return true; }
	virtual bool IsValidOnPrimary() const override { return true; }

	/** Allows all Output settings to affect the ViewFamily's Render settings before Rendering starts. */
	virtual void SetupViewFamily(FSceneViewFamily& ViewFamily) PURE_VIRTUAL(UMoviePipelineViewFamilySetting::SetupViewFamily, );
};
