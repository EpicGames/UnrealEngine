// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Blueprint/UserWidget.h"
#include "MovieRenderDebugWidget.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMoviePipeline;

/**
* C++ Base Class for the debug widget that is drawn onto the game viewport
* (but not burned into the output files) that allow us to easily visualize
* the current state of the pipeline.
*/
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMovieRenderDebugWidget : public UUserWidget
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnInitializedForPipeline(UMoviePipeline* ForPipeline);
};

/**
* C++ Base Class for the preview widget that is drawn onto the game viewport
* (but not burned into the output files) that allow us to easily visualize
* the current state of the pipeline. Used for Graph Based pipelines.
*/
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMovieGraphRenderPreviewWidget : public UUserWidget
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnInitializedForPipeline(UMovieGraphPipeline* InPipeline);
};

#undef UE_API
