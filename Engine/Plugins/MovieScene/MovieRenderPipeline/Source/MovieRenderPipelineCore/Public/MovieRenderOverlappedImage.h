// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "CoreMinimal.h"

#include "ImagePixelData.h"

#include "MovieRenderPipelineDataTypes.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/* Types
 *****************************************************************************/

/**
 * Structure for a single overlapped image.
 */
struct FImageOverlappedPlane
{
public:
	/**
	 * Default constructor.
	 */
	FImageOverlappedPlane()
		: Size(0,0)
	{
	}

	/**
	 * Initialize the memory. Before using the memory, we also will need a call to ZeroPlane.
	 *
	 * @param InSize - Size of the tile in pixels.
	 */
	UE_API void Init(FIntPoint InSize);

	/**
	 * Zeroes the accumulation. Assumes the data is already allocated.
	 */
	UE_API void ZeroPlane();

	/**
	 * Frees the memory and resets the sizes
	 */
	UE_API void Reset();

	/**
	 * Accumulate a single tile to this plane. The raw data will in general be smaller than the full plane.
	 * Addtionally, we are only going to be using part of the input plane because the input plane will
	 * have unused border areas. The SubRect parameters describe the area inside the source plane.
	 *
	 * @param InRawData - Raw data to accumulate
	 * @param InWeightData - Mask to apply to the raw data for blending the overlapped areas.
	 * @param InSize - Size of the tile. Must exactly match. InSize.X
	 * @param InOffset - The (x,y) offset to the overlapped image.
	 * @param SubpixelOffsetX - Subpixeoffset of the tile (in X), goes from [0,1] with 0.5 meaning it is right in the center.
	 * @param SubpixelOffsetY - Subixel offset of the tile (in Y), goes from [0,1] with 0.5 meaning it is right in the center
	 * @param SubRectOffset - The offset of the SubRect inside the raw data (InRawData) that actually has a weight > 0.0.
	 * @param SubRectSize - The size of the SubRect inside the raw data (InRawData) that actually has a weight > 0.0.
	 */
	UE_API void AccumulateSinglePlane(const TArray64<float>& InRawData, FIntPoint InSize, FIntPoint InOffset,
								float SubpixelOffsetX, float SubpixelOffsetY,
								FIntPoint SubRectOffset,
								FIntPoint SubRectSize,
								const TArray<float> & WeightDataX,
								const TArray<float> & WeightDataY);

	/** Actual channel data*/
	TArray64<float> ChannelData;

	/** Width and Height of the image. */
	FIntPoint Size;
};

/**
 * Contains all the image planes for the tiles.
 */
struct FImageOverlappedAccumulator : MoviePipeline::IMoviePipelineOverlappedAccumulator
{
public:
	/**
	 * Default constructor.
	 */
	FImageOverlappedAccumulator()
		: PlaneSize(0,0)
		, NumChannels(0)
		, AccumulationGamma(1.0f)
	{
	}

	static FName GetName()
	{
		static FName Name = FName("ImageOverlappedAccumulator");
		return Name;
	}

	/**
	 * Allocates memory.
	 *
	 * @param InTileSizeX - Horizontal tile size.
	 * @param InTileSizeY - Vertical tile size.
	 * @param InNumTilesX - Num horizontal tiles.
	 * @param InNumTilesY - Num vertical tiles.
	 * @param InNumChannels - Num Channels.
	 */
	UE_API void InitMemory(FIntPoint InPlaneSize, int32 InNumChannels);

	/**
	 * Initializes memory.
	 *
	 * Resets the memory to 0s so that we can start a new frame.
	 */
	UE_API void ZeroPlanes();

	/**
	 * Resets the PlaneSize and NumChannels but does NOT free the memory
	 * because most of the time we're going to just need the same resolution
	 * next frame anyways when InitMemory is called again.
	 */
	UE_API void Reset();

#if 0
	/**
	 * Given the size of a tile, calculate its weight mask.
	 * 
	 * @param OutWeights - Raw pixel data.
	 * @param Size - Size of the Tile.
	 */
	static UE_API void GenerateTileWeight(TArray64<float>& OutWeights, FIntPoint Size);

	/**
	 * Given a tile size and weights, calculate the SubRect that contains all nonzere weights.
	 * 
	 * @param OutSubRectOffset - Offset of the found subrect.
	 * @param OutSubRectSize - Size of the found subrect.
	 * @param Weights - Input weights mask data.
	 * @param Size - Input weights mask dimensions.
	 */
	static UE_API void GetTileSubRect(FIntPoint & OutSubRectOffset, FIntPoint & OutSubRectSize, const TArray64<float>& Weights, const FIntPoint Size);

	/**
	 * Given a tile size and weights, check() that every non-zero weight is in the subrect for debugging.
	 * 
	 * @param Weights - Input weights mask data.
	 * @param Size - Input weights mask dimensions.
	 * @param SubRectOffset - Offset of the found subrect.
	 * @param SubRectSize - Size of the found subrect.
	 */
	static UE_API void CheckTileSubRect(const TArray64<float>& Weights, const FIntPoint Size, FIntPoint SubRectOffset, FIntPoint SubRectSize);
#endif

	/**
	 * Given a rendered tile, accumulate the data to the full size image.
	 * 
	 * @param InPixelData - Raw pixel data.
	 * @param InTileOffset - Tile offset in pixels.
	 * @param InSubPixelOffset - SubPixel offset, should be in the range [0,1)
	 * @param WeightX - 1D Weights in X
	 * @param WeightY - 1D Weights in Y
	 */
	UE_API void AccumulatePixelData(const FImagePixelData& InPixelData, FIntPoint InTileOffset, FVector2D InSubpixelOffset, const MoviePipeline::FTileWeight1D & WeightX, const MoviePipeline::FTileWeight1D & WeightY);

	/**
	 * After accumulation is finished, fetch the final image as bytes. In theory we don't need this, because we could
	 * just fetch as LinearColor and convert to bytes. But the largest size asked for is 45k x 22.5k, which is 1B pixels.
	 * So fetching as LinearColor would create a 16GB intermediary image, so it's worth having an option to fetch
	 * straight to FColors.
	 * 
	 * @param OutPixelData - Finished pixel data.
	 */
	UE_API void FetchFinalPixelDataByte(TArray64<FColor>& OutPixelData) const;

	/**
	 * After accumulation is finished, fetch the final image as linear colors
	 * 
	 * @param OutPixelData - Finished pixel data.
	 */
	UE_API void FetchFinalPixelDataHalfFloat(TArray64<FFloat16Color>& OutPixelData) const;

	/**
	 * After accumulation is finished, fetch the final image as linear colors
	 * 
	 * @param OutPixelData - Finished pixel data.
	 */
	UE_API void FetchFinalPixelDataLinearColor(TArray64<FLinearColor>& OutPixelData) const;

	/**
	 * Grab a single pixel from the full res tile and scale it by the appropriate Scale value.
	 * 
	 * @param Rgba - Found pixel value.
	 * @param PlaneScale - The scales of each channel plane.
	 * @param FullX - X position of the full res image.
	 * @param FullY - Y position of the full res image.
	 */
	UE_API void FetchFullImageValue(float Rgba[4], int32 FullX, int32 FullY) const;

public:
	/** Width and height of each tile in pixels */
	FIntPoint PlaneSize;

	/** Number of channels in the tiles. Typical will be 3 (RGB). */
	int32 NumChannels;

	/** Gamma for accumulation. Typical values are 1.0 and 2.2. */
	float AccumulationGamma;


	/** 
	* A temporary buffer that is only initialized to the size of a single sample used during RGBA unpacking.
	* We persist it as a member level variable to avoid having to allocate/free it for every sample (which
	* is about a 32mb allocation at 4k) as that is quite slow.
	*/
	TArray64<float> UnpackedDataStorage[4];
	/** 
	* A temporary buffer that is only initialized to the size of a single sample used during weight plane accumulation.
	* It is currently filled with 1.0f for all weights, and exists so that we can use the same code as we do for RGBA accumulation.
	*/
	TArray64<float> WeightChannelStorage;

	TArray64<FImageOverlappedPlane> ChannelPlanes;
	
	FImageOverlappedPlane WeightPlane;
};


#undef UE_API
