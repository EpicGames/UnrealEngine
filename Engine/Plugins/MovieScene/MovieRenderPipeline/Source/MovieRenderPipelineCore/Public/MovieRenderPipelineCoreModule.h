// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NamingTokens.h"
#include "Stats/Stats.h"
#include "Modules/ModuleInterface.h"
#include "NamingTokensEvaluationData.h"
#include "Features/IModularFeature.h"

#include "MovieRenderPipelineCoreModule.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

// Forward Declare
class ULevelSequence;
class UMoviePipeline;
class UMoviePipelineExecutorBase;
class UMoviePipelineQueue;
struct FMoviePipelineFrameOutputState;

// Declare a stat-group for our performance stats to be counted under, readable in game by "stat MovieRenderPipeline".
DECLARE_STATS_GROUP(TEXT("MovieRenderPipeline"), STATGROUP_MoviePipeline, STATCAT_Advanced);

/**
 * Naming Tokens for Movie Render Pipeline.
 */
UCLASS(NotBlueprintable)
class UMovieRenderPipelineNamingTokens final: public UNamingTokens
{
	GENERATED_BODY()

public:
	/** The namespace that all MRG tokens are included under within the Naming Tokens system. */
	static inline const FString TokenNamespace = TEXT("mrg");
	
	UMovieRenderPipelineNamingTokens()
	{
		Namespace = TokenNamespace;
		NamespaceDisplayName = FText::FromString(TEXT("Movie Render Graph"));
		bIsPrivateNamespace = true;
	}
	
	virtual ~UMovieRenderPipelineNamingTokens() override
	{
	}
};

/**
* Movie pipeline tick information
* Used by external consumers, like Niagara, to query information
*/
struct FMoviePipelineLightweightTickInfo
{
	FMoviePipelineLightweightTickInfo()
		: bIsActive(false)
		, TemporalSampleCount(1)
		, TemporalSampleIndex(0)
		, SequenceFPS(0.0)
	{
	}

	bool	bIsActive;
	int32	TemporalSampleCount;
	int32	TemporalSampleIndex;
	double	SequenceFPS;
};

namespace MoviePipelineErrorCodes
{
	/** Everything completed as expected or we (unfortunately) couldn't detect the error. */
	constexpr uint8 Success = 0;
	/** Fallback for any generic critical failure. This should be used for "Unreal concepts aren't working as expected" severity errors. */
	constexpr uint8 Critical = 1;
	/** The specified level sequence asset could not be found. Check the logs for details on what it looked for. */
	constexpr uint8 NoAsset = 2;
	/** The specified pipeline configuration asset could not be found. Check the logs for details on what it looked for. */
	constexpr uint8 NoConfig = 3;
}

class FMovieRenderPipelineCoreModule : public IModuleInterface
{
public:
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	static UE_API void SetTickInfo(const FMoviePipelineLightweightTickInfo& InTickInfo);
	const FMoviePipelineLightweightTickInfo& GetTickInfo() const { return TickInfo; }

private:
	UE_API bool IsTryingToRenderMovieFromCommandLine(FString& OutSequenceAssetPath, FString& OutConfigAssetPath, FString& OutExecutorType, FString& OutPipelineType) const;
	UE_API void OnMapLoadFinished(class UWorld* InWorld);
	UE_API void QueueInitialize(class UWorld* InWorld);
	UE_API void InitializeCommandLineMovieRender();

	UE_API void OnCommandLineMovieRenderCompleted(UMoviePipelineExecutorBase* InExecutor, bool bSuccess);
	UE_API void OnCommandLineMovieRenderErrored(UMoviePipelineExecutorBase* InExecutor, UMoviePipeline* InPipelineWithError, bool bIsFatal, FText ErrorText);

	UE_API uint8 ParseMovieRenderData(const FString& InSequenceAssetPath, const FString& InConfigAssetPath, const FString& InExecutorType, const FString& InPipelineType,
		UMoviePipelineQueue*& OutQueue, UMoviePipelineExecutorBase*& OutExecutor) const;
	
	/** Setup bindings to MRQ Naming Tokens. */
	void BindToNamingTokenEvents();

	/** Executes every time Naming Tokens are fetched. */
	void OnMovieRenderPipelineNamingTokensPreEvaluate(const FNamingTokensEvaluationData& InEvaluationData);

private:
	/** The GUID that the Naming Tokens system has assigned to the MRQ naming tokens. */
	static FGuid NamingTokensGuid;

	/** Naming Tokens object for MRQ. */
	static TWeakObjectPtr<UNamingTokens> NamingTokens;
	
	FMoviePipelineLightweightTickInfo TickInfo;
	FString MoviePipelineLocalExecutorClassType;
	FString MoviePipelineClassType;
	FString SequenceAssetValue;
	FString SettingsAssetValue;
};

class IMoviePipelineBurnInExtension : public IModularFeature
{
public:
	virtual ~IMoviePipelineBurnInExtension() = default;
	static UE_API FName ModularFeatureName;

	virtual FText GetEngineChangelistLabel() const { return FText(); }
};

MOVIERENDERPIPELINECORE_API DECLARE_LOG_CATEGORY_EXTERN(LogMovieRenderPipeline, Log, All);
MOVIERENDERPIPELINECORE_API DECLARE_LOG_CATEGORY_EXTERN(LogMovieRenderPipelineIO, Log, All);

#undef UE_API
