// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderPipelineCoreObjectVersion.h"

#include "Serialization/CustomVersion.h"

const FGuid FMovieRenderPipelineCoreObjectVersion::GUID = FGuid(0x81A1B2B3, 0xAF127711, 0xE211B0B2, 0x1D235409);
FCustomVersionRegistration GRegisterMovieRenderPipelineCoreObjectVersion(FMovieRenderPipelineCoreObjectVersion::GUID, FMovieRenderPipelineCoreObjectVersion::LatestVersion, TEXT("MovieRenderPipelineCoreObjectVersion"));