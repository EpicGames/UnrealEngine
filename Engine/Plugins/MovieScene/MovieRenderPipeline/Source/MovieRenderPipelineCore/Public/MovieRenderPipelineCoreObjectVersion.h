// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/** Custom serialization version for changes made to Movie Render Pipeline Core objects. */
struct FMovieRenderPipelineCoreObjectVersion
{
	enum Type : int32
	{
		PreVersioning = 0,

		/** Added bOnlyMatchComponents to some conditions within graph collections, defaulting to true. */
		OnlyMatchComponentsAdded,

		/** Added EMoviePipelineConfigMode to UMoviePipelineExecutorJob. Pre-existing assets derive the mode from pointer state in Serialize(). */
		JobConfigModeAdded,

		/** Added bHasInitializedLegacyPresetDefaults to UMoviePipelineExecutorJob. Pre-existing assets backfill the flag to "true". */
		JobInitializedLegacyDefaultsTrackingAdded,

		// --- Add new versions above this line ---
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static UE_API const FGuid GUID;

	FMovieRenderPipelineCoreObjectVersion() = delete;
};

#undef UE_API
