// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipeline.h"
#include "MovieRenderPipelineCoreModule.h"
#include "RHI.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Sections/MovieSceneCinematicShotSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieRenderPipelineDataTypes)

FFrameNumber FMoviePipelineCameraCutInfo::GetOutputFrameCountEstimate() const
{
	const FFrameRate SourceRate = CachedFrameRate;
	const FFrameRate DestinationRate = CachedTickResolution;
	
	// TotalRange is stored in Tick Resolution, so we convert 1 frame of Frame Rate to the number of ticks.
	FFrameNumber OneFrameInTicks = FFrameRate::TransformTime(FFrameTime(FFrameNumber(1)), SourceRate, DestinationRate).FloorToFrame();

	// Find out how many ticks long our total output range is.
	FFrameNumber TotalOutputRangeTicks = TotalOutputRangeRoot.Size<FFrameNumber>();
	int32 NumFrames = FMath::CeilToInt(TotalOutputRangeTicks.Value / (double)OneFrameInTicks.Value);

	return FFrameNumber(NumFrames);
}

void FMoviePipelineCameraCutInfo::CalculateWorkMetrics()
{
	// Initial Range + Handle Frames
	FFrameNumber OutputFrameCount = GetOutputFrameCountEstimate();
	WorkMetrics.TotalOutputFrameCount = OutputFrameCount.Value;
	WorkMetrics.TotalSubSampleCount = NumSpatialSamples * NumTemporalSamples * NumTiles.X * NumTiles.Y; // Samples to generate an output frame.
	WorkMetrics.TotalEngineWarmUpFrameCount = NumEngineWarmUpFramesRemaining;
}

FMoviePipelineConsoleVariableEntry::FMoviePipelineConsoleVariableEntry(const FString& InName, const float InValue, const bool bInIsEnabled)
	: Name(InName)
	, Value(InValue)
	, bIsEnabled(bInIsEnabled)
{

}

FMoviePipelineConsoleVariableEntry::FMoviePipelineConsoleVariableEntry()
	: Value(0), bIsEnabled(true)
{
}

void FMoviePipelineCameraCutInfo::SetNextStateAfter(const EMovieRenderShotState InCurrentState)
{
	switch (InCurrentState)
	{
		// This may get called multiple times so we just do nothing until it's appropriate to move on from WarmingUp.
	case EMovieRenderShotState::WarmingUp:
		// Warming Up can jump directly to either Rendering or to MotionBlur depending on if fixes are applied.
		if (NumEngineWarmUpFramesRemaining == 0)
		{
			if (bEmulateFirstFrameMotionBlur)
			{
				UE_LOGF(LogMovieRenderPipeline, Verbose, "[%lld] Shot WarmUp finished. Setting state to MotionBlur.", GFrameCounter);
				State = EMovieRenderShotState::MotionBlur;
			}
			else
			{
				UE_LOGF(LogMovieRenderPipeline, Verbose, "[%lld] Shot WarmUp finished. Setting state to Rendering due to no MotionBlur pre-frames.", GFrameCounter);
				State = EMovieRenderShotState::Rendering;
			}
		}
		break;
		// This should only be called once with the Uninitialized state.
	case EMovieRenderShotState::Uninitialized:
		// Uninitialized can either jump to WarmUp, MotionBlur, or straight to Rendering if no fixes are applied.
		if (NumEngineWarmUpFramesRemaining > 0)
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "[%lld] Initialization set state to WarmingUp due to having %d warm up frames.", GFrameCounter, NumEngineWarmUpFramesRemaining);
			State = EMovieRenderShotState::WarmingUp;
		}
		// If they didn't want any warm-up frames we'll still check to see if they want to fix motion blur on the first frame.
		else if (bEmulateFirstFrameMotionBlur)
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "[%lld] Initialization set state to MotionBlur due to having no warm up frames.", GFrameCounter);
			State = EMovieRenderShotState::MotionBlur;
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Verbose, "[%lld] Initialization set state to Rendering due to no MotionBlur pre-frames.", GFrameCounter);
			State = EMovieRenderShotState::Rendering;
		}
		break;
	}

}

MoviePipeline::FMoviePipelineRenderPassInitSettings::FMoviePipelineRenderPassInitSettings()
{
	FeatureLevel = GMaxRHIFeatureLevel;
}

bool MoviePipeline::FTileWeight1D::operator==(const FTileWeight1D& Rhs) const
{
	const bool bIsSame =
		(X0 == Rhs.X0) &&
		(X1 == Rhs.X1) &&
		(X2 == Rhs.X2) &&
		(X3 == Rhs.X3);

	return bIsSame;
}

void MoviePipeline::FTileWeight1D::InitHelper(int32 PadLeft, int32 SizeCenter, int32 PadRight)
{
	check(PadLeft >= 0);
	check(SizeCenter > 0);
	check(PadRight >= 0);

	int32 Size = PadLeft + SizeCenter + PadRight;
	int32 Midpoint = PadLeft + SizeCenter / 2;

	X0 = PadLeft / 2;
	X1 = (3 * PadLeft) / 2;
	X2 = Size - ((3 * PadLeft) / 2);
	X3 = Size - (PadRight / 2);

	X1 = FMath::Min<int32>(X1, Midpoint);
	X2 = FMath::Max<int32>(X2, Midpoint);
}

float MoviePipeline::FTileWeight1D::CalculateWeight(int32 Pixel) const
{
	// the order of the if is important, because if X0=X1, then
	// we take the Pixel < X0 path, not the Pixel < X1 path which would cause a
	// divide by zero. Same for X2 and X3.
	if (Pixel < X0)
	{
		return 0.0f;
	}
	else if (Pixel < X1)
	{
		// note that it's impossible for X0==X1, because the X0 if is earlier
		return float(Pixel - X0) / float(X1 - X0);
	}
	else if (Pixel < X2)
	{
		return 1.0f;
	}
	else if (Pixel < X3)
	{
		// note that it's impossible for X2==X3
		return float(X3 - Pixel) / float(X3 - X2);
	}

	return 0.0f;
}

void MoviePipeline::FTileWeight1D::CalculateArrayWeight(TArray<float>& WeightData, int Size) const
{
	WeightData.SetNum(Size);
	for (int32 Index = 0; Index < X0; Index++)
	{
		WeightData[Index] = 0.0f;
	}

	float ScaleLhs = (X0 != X1) ? 1.0f / float(X1 - X0) : 0.0f;
	float ScaleRhs = (X2 != X3) ? 1.0f / float(X3 - X2) : 0.0f;

	for (int32 Index = X0; Index < X1; Index++)
	{
		float W = FMath::Clamp<float>(((float(Index) + .5f) - float(X0)) * ScaleLhs, 0.0f, 1.0f);
		WeightData[Index] = W;
	}

	for (int32 Index = X1; Index < X2; Index++)
	{
		WeightData[Index] = 1.0f;
	}

	for (int32 Index = X2; Index < X3; Index++)
	{
		float W = FMath::Clamp<float>(1.0f - ((float(Index) + .5f) - float(X2)) * ScaleRhs, 0.0f, 1.0f);
		WeightData[Index] = W;
	}

	for (int32 Index = X3; Index < Size; Index++)
	{
		WeightData[Index] = 0.0f;
	}
}

