// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Timecode.h"
#include "ImagePixelData.h"
#include "ImageWriteTask.h"
#include "Containers/UnrealString.h"
#include "Engine/Scene.h"
#include "Engine/EngineTypes.h"
#include "DSP/BufferVectorOperations.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Sections/MovieSceneSubSection.h"
#include "OpenColorIOColorSpace.h"
#include "Async/ParallelFor.h"
#include "MovieSceneSequenceID.h"
#include "MovieRenderDebugWidget.h"
#include "Graph/MovieGraphRenderDataIdentifier.h"
#include "MovieRenderPipelineDataTypes.generated.h"

#define UE_API MOVIERENDERPIPELINECORE_API

class UMovieSceneCinematicShotSection;

template<typename ElementType> class TRange;
template<class T, class TWeakObjectPtrBase> struct TWeakObjectPtr;

class UWorld;
class ULevelSequence;
class UMovieSceneCinematicShotSection;
class UMoviePipelinePrimaryConfig;
class UMoviePipelineShotConfig;
class UMovieSceneCameraCutSection;
class UMoviePipelineRenderPass;
struct FImagePixelData;
struct FImageOverlappedAccumulator;
class FMoviePipelineOutputMerger;
class FRenderTarget;
class UMoviePipeline;
struct FMoviePipelineFormatArgs;
class UMoviePipelineExecutorShot;
class UMovieGraphDataSourceBase;

namespace Audio { class FMixerSubmix; }

/**
* What is the current overall state of the Pipeline? States are processed in order from first to
* last and all states will be hit (though there is no guarantee the state will not be transitioned
* away from on the same frame it entered it). Used to help track overall progress and validate
* code flow.
*/
UENUM(BlueprintType)
enum class EMovieRenderPipelineState : uint8
{
	/** The pipeline has not been initialized yet. Only valid operation is to call Initialize. */
	Uninitialized = 0,
	/** The pipeline has been initialized and is now controlling time and working on producing frames. */
	ProducingFrames = 1,
	/** All desired frames have been produced. Audio is already finalized. Outputs have a chance to finish long processing tasks. */
	Finalize = 2,
	/** All outputs have finished writing to disk or otherwise processing. Additional exports that may have needed information about the produced file can now be run. */
	Export = 3,
	/** The pipeline has been shut down. It is an error to shut it down again. */
	Finished = 4,
};

/**
* What is the current state of a shot? States are processed in order from first to last but not
* all states are required, ie: WarmUp and MotionBlur can be disabled and the shot will never
* pass through this state.
*/
UENUM(BlueprintType)
enum class EMovieRenderShotState : uint8
{
	/** The shot has not been initialized yet.*/
	Uninitialized = 0,

	/** The shot is warming up. Engine ticks are passing but no frames are being produced. */
	WarmingUp = 1,
	/*
	* The shot is doing additional pre-roll for motion blur. No frames are being produced,
	* but the rendering pipeline is being run to seed histories.
	*/
	MotionBlur = 2,
	/*
	* The shot is working on producing frames and may be currently doing a sub-frame or
	* a whole frame.
	*/
	Rendering = 3,
	/** 
	* The shot is cooling down. Engine ticks are passing and frames are being rendered, but
	* not saved to disk. This is needed because temporal-based denoisers need to look at the
	* future (which would be after a shot) to finish the current frames of the shot.
	*/
	CoolingDown = 4,
	/*
	* The shot has produced all frames it will produce. No more evaluation should be
	* done for this shot once it reaches this state.
	*/
	Finished = 5
};

USTRUCT(BlueprintType)
struct FMoviePipelinePassIdentifier
{
	GENERATED_BODY()

	FMoviePipelinePassIdentifier()
	{}

	// Name defaults to "camera" for backwards compatiblity with non-multicam metadata
	FMoviePipelinePassIdentifier(const FString& InPassName, const FString& InCameraName = TEXT("camera"))
		: Name(InPassName)
		, CameraName(InCameraName)
	{
	}

	bool operator == (const FMoviePipelinePassIdentifier& InRHS) const
	{
		return Name == InRHS.Name && CameraName == InRHS.CameraName;
	}

	bool operator != (const FMoviePipelinePassIdentifier& InRHS) const
	{
		return !(*this == InRHS);
	}

	friend uint32 GetTypeHash(FMoviePipelinePassIdentifier OutputState)
	{
		return HashCombine(GetTypeHash(OutputState.Name), GetTypeHash(OutputState.CameraName));
	}

	friend FString LexToString(const FMoviePipelinePassIdentifier InIdentifier)
	{
		return FString::Printf(TEXT("Name: %s, Camera: %s"), *InIdentifier.Name, *InIdentifier.CameraName);
	}

public:
	// The name of the pass such as "FinalImage" or "ObjectId", etc.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Pipeline")
	FString Name;

	// The name of the camera that this pass is for. Stored here so we can differentiate between 
	// multiple cameras within a single pass.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Pipeline")
	FString CameraName;
};

/**
 * Represents a console variable override that can be enabled/disabled.
 */
USTRUCT(BlueprintType)
struct FMoviePipelineConsoleVariableEntry
{
	GENERATED_BODY()
	
	UE_API FMoviePipelineConsoleVariableEntry(const FString& InName, const float InValue, const bool bInIsEnabled = true);
	UE_API FMoviePipelineConsoleVariableEntry();

	/* The name of the console variable. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	FString Name;

	/* The value of the console variable. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	float Value;

	/* Enable state. If disabled, this cvar entry will be ignored when resolving the final value of the cvar. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	bool bIsEnabled;
};

namespace MoviePipeline
{
	/**
	* Frame info needed for creating the FSceneView for each tile.
	*/
	struct FMoviePipelineFrameInfo
	{
	public:
		FMoviePipelineFrameInfo() {}
		virtual ~FMoviePipelineFrameInfo() = default;

		/**
		 * We have to apply camera motion vectors manually. So we keep the current and previous fram'es camera view and rotation.
		 * Then we render a sequence of the same movement, and update after running the game sim.
		 **/

		/** Current frame camera view location **/
		FVector CurrViewLocation;

		/** Current frame camera view rotation **/
		FRotator CurrViewRotation;

		/** Previous frame camera view location **/
		FVector PrevViewLocation;

		/** Previous frame camera view rotation **/
		FRotator PrevViewRotation;

		TArray<FVector> CurrSidecarViewLocations;
		TArray<FRotator> CurrSidecarViewRotations;
		TArray<FVector> PrevSidecarViewLocations;
		TArray<FRotator> PrevSidecarViewRotations;
	};

	/**
	* Utility function for the Pipeline to calculate the time metrics for the current
	* frame that can then be used by the custom time step to avoid custom time logic.
	*/
	struct FFrameTimeStepCache
	{
		FFrameTimeStepCache()
			: UndilatedDeltaTime(0.0)
		{}

		FFrameTimeStepCache(double InDeltaTime)
			: UndilatedDeltaTime(InDeltaTime)
		{}

		double UndilatedDeltaTime;
	};

	struct FOutputFrameData
	{
		FIntPoint Resolution;

		TArray<FColor> ColorBuffer;

		FString PassName;
	};

	struct FTileWeight1D
	{
	public:
		/**
		 * For tiles, this struct stores the weight function in one dimension.
		 *
		 *                X1           X2
		 * 1.0              |------------|
		 *                 /              \
		 *                /                \
		 *               /                  \
		 * 0.0 |--------|                    |----------------|
		 *     0       X0                    X3             Size
		 *
		 *  For a tile, the weight goes from:
		 *  [0 ,X0]    : 0.0
		 *  [X0,X1]    : 0.0 to 1.0
		 *  [X1,X2]    : 1.0
		 *  [X2,X3]    : 1.0 to 0.0
		 *  [X3,SizeX] : 0.0
		 **/

		/** Constructor. Sets to invalid values and must be initialized to reasonable values before using. **/
		FTileWeight1D()
		{
			X0 = 0;
			X1 = 0;
			X2 = 0;
			X3 = 0;
		}

		/** Make sure this map has valid values. **/
		void CheckValid() const
		{
			check(0 <= X0);
			check(X0 <= X1);
			check(X1 <= X2);
			check(X2 <= X3);
		}

		/** Is equal operator.**/
		UE_API bool operator==(const FTileWeight1D& Rhs) const;

		/**
		 * The full tile is of size PadLeft + SizeCenter + PadRight
		 *
		 *  |------PadLeft-------|------------SizeCenter-----------|------PadRight-----|
		 *  
		 * This function puts X0 in the middle of PadLeft, X3 in the middle of PadRight.
		 * And X1 is "reflected" around PadLeft, and X2 is "reflected" around PadRight.
		 *
		 *  |------PadLeft-------|------------SizeCenter-----------|------PadRight-----|
		 *           X0                  X1                  X2              X3
		 * Also note that PadLeft and PadRight are allowed to be zero, but SizeCenter must be > 0;
		 **/
		UE_API void InitHelper(int32 PadLeft, int32 SizeCenter, int32 PadRight);

		UE_API float CalculateWeight(int32 Pixel) const;

		UE_API void CalculateArrayWeight(TArray<float>& WeightData, int Size) const;

		int32 X0;
		int32 X1;
		int32 X2;
		int32 X3;
	};

	struct FFrameConstantMetrics
	{
		/** What is the tick resolution of the root sequence */
		FFrameRate TickResolution;
		/** What is the tick resolution of the current shot */
		FFrameRate ShotTickResolution;
		/** What is the effective frame rate of the output */
		FFrameRate FrameRate;
		/** How many ticks per output frame. */
		FFrameTime TicksPerOutputFrame;
		/** How many ticks per sub-frame sample. */
		FFrameTime TicksPerSample;
		/** How many ticks while the shutter is closed */
		FFrameTime TicksWhileShutterClosed;
		/** How many ticks while the shutter is opened */
		FFrameTime TicksWhileShutterOpen;

		/** Fraction of the output frame that the accumulation frames should cover. */
		double ShutterAnglePercentage;

		/** Fraction of the output frame that the shutter is closed (that accumulation frames should NOT cover). */
		double ShutterClosedFraction;

		/**
		* Our time samples are offset by this many ticks. For a given output frame N, we can bias the time N represents
		* to be before, during or after the data stored on time N.
		*/
		FFrameTime ShutterOffsetTicks;

		/** 
		* How many ticks do we have to go to center our motion blur? This makes it so the blurred area matches the object
		* position during the time it represents, and not before.
		*/
		FFrameTime MotionBlurCenteringOffsetTicks;

		FFrameTime GetFinalEvalTime(const FFrameNumber InTime) const
		{
			// We just use a consistent offset from the given time to take motion blur centering and
			// shutter timing offsets into account. Written here just to consolidate various usages.
			return FFrameTime(InTime) + MotionBlurCenteringOffsetTicks + ShutterOffsetTicks;
		}
	};

	struct FCameraCutSubSectionHierarchyNode : TSharedFromThis<FCameraCutSubSectionHierarchyNode>
	{
		FCameraCutSubSectionHierarchyNode()
			: bOriginalMovieSceneReadOnly(false)
			, bOriginalMovieScenePlaybackRangeLocked(false)
			, bOriginalMovieScenePackageDirty(false)
			, bOriginalShotSectionIsLocked(false)
			, bOriginalCameraCutIsActive(false)
			, bOriginalShotSectionIsActive(false)
			, OriginalSequenceFlags(EMovieSceneSequenceFlags::None)
			, EvaluationType(EMovieSceneEvaluationType::WithSubFrames)
			, NodeID(MovieSceneSequenceID::Invalid)
		{
		}

		/** The UMovieScene that this node has data for. */
		TWeakObjectPtr<class UMovieScene> MovieScene;
		/** The UMovieSceneSubSection within the movie scene (if any) */
		TWeakObjectPtr<class UMovieSceneSubSection> Section;
		/** The UMovieSceneCameraCutSection within the movie scene (if any) */
		TWeakObjectPtr<class UMovieSceneCameraCutSection> CameraCutSection;
		
		// These are used to restore this node back to its proper shape post render.
		TRange<FFrameNumber> OriginalMovieScenePlaybackRange;
		bool bOriginalMovieSceneReadOnly;
		bool bOriginalMovieScenePlaybackRangeLocked;
		bool bOriginalMovieScenePackageDirty;

		bool bOriginalShotSectionIsLocked;
		TRange<FFrameNumber> OriginalShotSectionRange;
		TRange<FFrameNumber> OriginalCameraCutSectionRange;

		bool bOriginalCameraCutIsActive;
		bool bOriginalShotSectionIsActive;
		EMovieSceneSequenceFlags OriginalSequenceFlags;

		/** An array of sections that we should expand, as well as their original range for restoration later. */
		TArray<TTuple<UMovieSceneSection*, TRange<FFrameNumber>>> AdditionalSectionsToExpand;

		EMovieSceneEvaluationType EvaluationType;
		FMovieSceneSequenceID NodeID;

		void AddChild(TSharedPtr<FCameraCutSubSectionHierarchyNode> InChild)
		{
			if (!InChild)
			{
				return;
			}

			InChild->Parent = AsShared();
			Children.AddUnique(InChild);
		}

		TArray<TSharedPtr<FCameraCutSubSectionHierarchyNode>> GetChildren() const
		{
			return Children;
		}

		void SetParent(TSharedPtr<FCameraCutSubSectionHierarchyNode> InParent)
		{
			if (!InParent)
			{
				return;
			}

			if (InParent == Parent)
			{
				return;
			}

			ensureAlwaysMsgf(!Parent, TEXT("Cannot switch parents of FCameraCutSubSectionHierarchy nodes."));

			// Automatically adds us as a child of the parent and assigns out parent.
			InParent->AddChild(AsShared());
		}

		bool IsEmpty() const
		{
			return *this == FCameraCutSubSectionHierarchyNode();
		}

		TSharedPtr<FCameraCutSubSectionHierarchyNode> GetParent() const { return Parent; }

	private:
		/** A pointer to the next node in the tree who actually includes us in the hierarchy. Different than outers as each moviescene is outered to its own package. */
		TSharedPtr<FCameraCutSubSectionHierarchyNode> Parent;
		/** An array of pointers to our children for downwards traversal. */
		TArray<TSharedPtr<FCameraCutSubSectionHierarchyNode>> Children;

		bool operator == (const FCameraCutSubSectionHierarchyNode& InRHS) const
		{
			return MovieScene == InRHS.MovieScene
				&& Section == InRHS.Section
				&& CameraCutSection == InRHS.CameraCutSection
				&& OriginalMovieScenePlaybackRange == InRHS.OriginalMovieScenePlaybackRange
				&& bOriginalMovieSceneReadOnly == InRHS.bOriginalMovieSceneReadOnly
				&& bOriginalMovieScenePlaybackRangeLocked == InRHS.bOriginalMovieScenePlaybackRangeLocked
				&& bOriginalMovieScenePackageDirty == InRHS.bOriginalMovieScenePackageDirty
				&& bOriginalShotSectionIsLocked == InRHS.bOriginalShotSectionIsLocked
				&& OriginalShotSectionRange == InRHS.OriginalShotSectionRange
				&& OriginalCameraCutSectionRange == InRHS.OriginalCameraCutSectionRange
				&& bOriginalCameraCutIsActive == InRHS.bOriginalCameraCutIsActive
				&& bOriginalShotSectionIsActive == InRHS.bOriginalShotSectionIsActive
				&& EvaluationType == InRHS.EvaluationType
				&& NodeID == InRHS.NodeID
				&& Children == InRHS.Children
				&& Parent == InRHS.Parent;
		}

		bool operator != (const FCameraCutSubSectionHierarchyNode& InRHS) const
		{
			return !(*this == InRHS);
		}
	};

	struct FClothSimSettingsCache
	{
		int32 NumSubSteps;
		float DynamicSubstepDeltaTime;
	};
}

USTRUCT(BlueprintType)
struct FMoviePipelineSegmentWorkMetrics
{
	GENERATED_BODY()

public:
	FMoviePipelineSegmentWorkMetrics()
		: SegmentName()
		, OutputFrameIndex(-1)
		, TotalOutputFrameCount(-1)
		, OutputSubSampleIndex(-1)
		, TotalSubSampleCount(-1)
		, EngineWarmUpFrameIndex(-1)
		, TotalEngineWarmUpFrameCount(-1)
	{ }	
	
	// This information is used to be presented in the UI.
	
	/** The name of the segment (if any) */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Pipeline Segment")
	FString SegmentName;
	
	/** Index of the output frame we are working on right now. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Pipeline Segment")
	int32 OutputFrameIndex;
	/** The number of output frames we expect to make for this segment. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Pipeline Segment")
	int32 TotalOutputFrameCount;


	/** Which temporal/spatial sub sample are we working on right now. This counts temporal, spatial, and tile samples to accurately reflect how much work is needed for this output frame. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Pipeline Segment")
	int32 OutputSubSampleIndex;
	
	/** The total number of samples we will have to build to render this output frame. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Pipeline Segment")
	int32 TotalSubSampleCount;
	
	/** The index of the engine warm up frame that we are working on. No rendering samples are produced for these. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Pipeline Segment")
	int32 EngineWarmUpFrameIndex;
	
	/** The total number of engine warm up frames for this segment. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Pipeline Segment")
	int32 TotalEngineWarmUpFrameCount;
};


UENUM(BlueprintType)
enum class EMoviePipelineShutterTiming : uint8
{
	FrameOpen,
	FrameCenter,
	FrameClose
};

// ToDo: Rename this to segment.
USTRUCT(BlueprintType)
struct FMoviePipelineCameraCutInfo
{
	GENERATED_BODY()
public:
	FMoviePipelineCameraCutInfo()
		: bEmulateFirstFrameMotionBlur(true)
		, NumTemporalSamples(0)
		, NumSpatialSamples(0)
		, NumTiles(0, 0)
		, State(EMovieRenderShotState::Uninitialized)
		, bHasEvaluatedMotionBlurFrame(false)
		, NumEngineWarmUpFramesRemaining(0)
		, NumEngineCoolDownFramesRemaining(0)
		, bNeedsAudioResetOnRenderStart(false)
		, VersionNumber(0)
	{
	}

	bool IsInitialized() const { return State != EMovieRenderShotState::Uninitialized; }
	void SetNextStateAfter(const EMovieRenderShotState InCurrentState);
	void CalculateWorkMetrics();

private:
	FFrameNumber GetOutputFrameCountEstimate() const;

public:
	
	TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> SubSectionHierarchy;


	/** 
	* Should we evaluate/render an extra frame at the start of this shot to show correct motion blur on the first frame? 
	* This emulates motion blur by evaluating forward one frame and then going backwards which is a approximation.
	*/
	bool bEmulateFirstFrameMotionBlur;

	/** How many temporal samples is each frame broken up into? */
	int32 NumTemporalSamples;

	/** For each temporal sample, how many spatial samples do we take? */
	int32 NumSpatialSamples;

	/** How many image tiles are going to be rendered per temporal frame. */
	FIntPoint NumTiles;

	/** Cached Frame Rate these are being rendered at. Simplifies some APIs. */
	FFrameRate CachedFrameRate;

	/** Cached Tick Resolution our numbers are in. Simplifies some APIs. */
	FFrameRate CachedTickResolution;
	
	/** Cached Tick Resolution the movie scene this range was generated for is in. Can be different than the root due to mixed tick resolutions. */
	FFrameRate CachedShotTickResolution;

	/** The current state of processing this Shot is in. Not all states will be passed through. */
	EMovieRenderShotState State;

	/** The current tick of this shot that we're on in root space */
	FFrameNumber CurrentTickInRoot;

	/** The current time of this shot that we're on in root space (for the Graph version. Tracks sub-frames too.) */
	FFrameTime CurrentTimeInRoot;

	/** The initial time in the root space for this shot when we started evaluating. Only assigned in the Graph version of MRQ. Effectively the same as CurrentTimeInRoot, but is immutable during the shot. */
	FFrameTime InitialTimeInRoot;
	/** The initial time in the shot time space for this shot when we started evaluating. Only assigned in the Graph version of MRQ. Effectively the same as CurrentTimeInRoot * OuterToInnerTransform, but is immutable during the shot. */
	FFrameTime InitialTimeInShot;

	/** Converts from the outermost space into the innermost space. Only works with linear transforms. */
	FMovieSceneSequenceTransform OuterToInnerTransform;
	
	/** The total range of output frames in root space */
	TRange<FFrameNumber> TotalOutputRangeRoot;
	
	/** The total range of warmup frames in root space */
	TRange<FFrameNumber> WarmupRangeRoot;

	/** Metrics - How much work has been done for this particular shot and how much do we estimate we have to do? */
	FMoviePipelineSegmentWorkMetrics WorkMetrics;

	/** Have we evaluated the motion blur frame? Only used if bEvaluateMotionBlurOnFirstFrame is set */
	bool bHasEvaluatedMotionBlurFrame;

	/** How many engine warm up frames are left to process for this shot. May be zero. */
	int32 NumEngineWarmUpFramesRemaining;

	/** How many cool down frames for this shot. May be zero.*/
	int32 NumEngineCoolDownFramesRemaining;

	/** If true, the first rendering frame will issue a sequencer jump to reset procedural audio
	    generators (e.g. MetaSound) that advanced their internal state during non-emulated warm-up. */
	bool bNeedsAudioResetOnRenderStart;


	/** What version number should this shot use when resolving format arguments. This is the highest version number found across all branches. */
	int32 VersionNumber;
};

/**
* The Tick/Render loops are decoupled from the actual desired output. In some cases
* we may render but not desire an output (such as filling temporal histories) or
* we may be accumulating the results into a target which does not produce an output.
* Finally, we may be running the Tick/Render loop but not wanting to do anything!
*/
struct FMoviePipelineFrameOutputState
{
public:
	struct FTimeData
	{
		FTimeData()
			: MotionBlurFraction(0.f)
			, FrameDeltaTime(0.0)
			, WorldSeconds(0.0)
			, TimeDilation(1.f)
		{
		}

		/** A 0-1 fraction of how much motion blur should be applied. This is tied to the shutter angle but more complicated when using sub-sampling */
		float MotionBlurFraction;

		/** The delta time in seconds used for this frame. */
		double FrameDeltaTime;

		/** The total elapsed time since the pipeline started.*/
		double WorldSeconds;

		/**
		* Check if there was a non-1.0 Time Dilation in effect when this frame was produced. This indicates that there
		* may be duplicate frame Source/Effective frame numbers as they find the closest ideal time to the current.
		*/
		float TimeDilation;

		void ResetPerFrameData()
		{
			MotionBlurFraction = 0.f;
			FrameDeltaTime = 0.0;
			TimeDilation = 1.f;
		}

		inline bool IsTimeDilated() const { return !FMath::IsNearlyEqual(TimeDilation, 1.f); }
	};

	FMoviePipelineFrameOutputState()
		: OutputFrameNumber(-1)
		, ShotCount(0)
		, TemporalSampleIndex(-1)
		, TemporalSampleCount(0)
	{
		// There is other code which relies on this starting on -1.
		check(OutputFrameNumber == -1);

		// Likewise, the code assumes you start on sample index -1.
		check(TemporalSampleIndex == -1);

		ResetPerFrameData();
		ResetPerShotData();
	}

	/** Is this the first temporal sample for the output frame? */
	inline bool IsFirstTemporalSample() const { return TemporalSampleIndex == 0; }
	inline bool IsLastTemporalSample() const { return TemporalSampleIndex == (TemporalSampleCount - 1); }

	/**
	* The expected output frame count that the render is working towards creating.
	* This number accurately tracks the number of frames we have produced even if
	* the file written to disk uses a different number (due to relative frame numbers
	* or offset frames being added.
	*/
	int32 OutputFrameNumber;

	/** Which shot is this output state for? */
	int32 ShotIndex;

	/** How many shots total will we be outputting? */
	int32 ShotCount;

	/** Which sub-frame are we on when using Accumulation Frame rendering. */
	int32 TemporalSampleIndex;

	/** How many temporal samples do we add together to produce one Output Frame? */
	int32 TemporalSampleCount;

	/** 
	* The expected output frame count for this current shot that we're working towards
	* creating. Like OutputFrameNumber but relative to this shot. This should get reset between shots. 
	*/
	int32 ShotOutputFrameNumber;

	/** The total number of samples (including warm ups) that have been sent to the GPU for this shot. */
	int32 ShotSamplesRendered;

	/** What time data should this frame use? Can vary between samples when TemporalSampleCount > 1. */
	FTimeData TimeData;

	/** The name of the currently active camera being rendered. May be empty. */
	FString CameraName;

	/** Name used by the {camera_name} format tag. May be empty */
	FString CameraNameOverride;

	/** THe name of the currently active shot. May be empty if there is no shot track. */
	FString ShotName;

	/** INFORMATION BELOW HERE SHOULD NOT GET PERSISTED BETWEEN FRAMES */

	void ResetPerFrameData()
	{
		TimeData.ResetPerFrameData();
		bSkipRendering = false;
		bCaptureRendering = false;
		bDiscardRenderResult = false;
		SourceFrameNumber = 0;
		SourceTimeCode = FTimecode();
		EffectiveFrameNumber = 0;
		EffectiveTimeCode = FTimecode();
		CurrentShotSourceFrameNumber = 0;
		CurrentShotSourceTimeCode = FTimecode();
		FileMetadata.Reset();
	}

	void ResetPerShotData()
	{
		ShotOutputFrameNumber = -1;
		ShotSamplesRendered = 0;
		ShotIndex = 0;
		TemporalSampleCount = 0;
		CameraName.Reset();
		ShotName.Reset();
	}

	/**
	* If true, then the rendering for this frame should be skipped (ie: nothing submitted to the gpu, and the output merger not told to expect this frame).
	* This is used for rendering every Nth frame for rendering drafts. We still run the game thread logic for the skipped frames (which is relatively cheap)
	* and simply omit rendering them. This increases consistency with non-skipped renders, and will be useful for consistency when rendering on a farm.
	*/
	bool bSkipRendering;

	/**
	* If true, and a IRenderCaptureProvider is available, trigger a capture of the rendering process of this frame.
	*/
	bool bCaptureRendering;

	/**
	* If this is true, then the frame will be rendered but the results discarded and not sent to the accumulator. This is used for render warmup frames
	* or gpu-based feedback loops. Ignored if bSkipRendering is true.
	*/
	bool bDiscardRenderResult;


	/** The closest frame number (in Display Rate) on the Sequence. May be duplicates in the case of different output framerate or Play Rate tracks. */
	int32 SourceFrameNumber;

	/** The closest time code version of the SourceFrameNumber on the Sequence. May be a duplicate in the case of different output framerate or Play Rate tracks. */
	FTimecode SourceTimeCode;

	/** 
	* The closest frame number (in Display Rate) on the Sequence adjusted for the effective output rate. These numbers will not line up with the frame
	* in the source Sequence if the output frame rate differs from the Sequence display rate. May be a duplicate in the event of Play Rate tracks.
	*/
	int32 EffectiveFrameNumber;

	/** The closest time code version of the EffectiveFrameNumber. May be a duplicate in the event of Play Rate tracks. */
	FTimecode EffectiveTimeCode;

	/** Metadata to attach to the output file (if supported by the output container) */
	TMap<FString, FString> FileMetadata;


	int32 CurrentShotSourceFrameNumber;
	
	FTimecode CurrentShotSourceTimeCode;

	int32 CameraIndex;


	bool operator == (const FMoviePipelineFrameOutputState& InRHS) const
	{
		// ToDo: I don't think this is used anymore. Was used to make it so that the output state
		// for the first temporal sample (which is what the expected output was built off of) could
		// be equality against the last temporal sample when used in a TSet but better to do that explicitly.
		return
			OutputFrameNumber == InRHS.OutputFrameNumber &&
			// TemporalSampleIndex == InRHS.TemporalSampleIndex &&
			// FMath::IsNearlyEqual(MotionBlurFraction,InRHS.MotionBlurFraction) &&
			// FMath::IsNearlyEqual(FrameDeltaTime, InRHS.FrameDeltaTime) &&
			// FMath::IsNearlyEqual(WorldSeconds, InRHS.WorldSeconds) &&
			SourceFrameNumber == InRHS.SourceFrameNumber &&
			SourceTimeCode == InRHS.SourceTimeCode &&
			EffectiveFrameNumber == InRHS.EffectiveFrameNumber &&
			EffectiveTimeCode == InRHS.EffectiveTimeCode;
	}

	bool operator != (const FMoviePipelineFrameOutputState& InRHS) const
	{
		return !(*this == InRHS);
	}

	bool operator < (const FMoviePipelineFrameOutputState& InRHS) const
	{
		return this->OutputFrameNumber < InRHS.OutputFrameNumber;
	}

	friend uint32 GetTypeHash(FMoviePipelineFrameOutputState OutputState)
	{
		return GetTypeHash(OutputState.OutputFrameNumber);
	}
};

USTRUCT(BlueprintType)
struct FMoviePipelineFormatArgs
{
	GENERATED_BODY()

	FMoviePipelineFormatArgs()
		: InJob(nullptr)
	{
	}

	/** A set of Key/Value pairs for output filename format strings (without {}) and their values. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	TMap<FString, FString> FilenameArguments;

	/** A set of Key/Value pairs for file metadata for file formats that support metadata. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	TMap<FString, FString> FileMetadata;

	/** Which job is this for? Some settings are specific to the level sequence being rendered. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	TObjectPtr<class UMoviePipelineExecutorJob> InJob;
};

USTRUCT(BlueprintType)
struct FMoviePipelineFilenameResolveParams
{
	GENERATED_BODY()

	FMoviePipelineFilenameResolveParams()
		: FrameNumber(0)
		, FrameNumberShot(0)
		, FrameNumberRel(0)
		, FrameNumberShotRel(0)
		, ZeroPadFrameNumberCount(0)
		, bForceRelativeFrameNumbers(false)
		, InitializationTime(0)
		, InitializationVersion(0)
		, Job(nullptr)
		, CameraIndex(-1)
		, ShotOverride(nullptr)
		, AdditionalFrameNumberOffset(0)
	{
	}
	
	/** Frame Number for the Root (matching what you see in the Sequencer timeline. ie: If the Sequence PlaybackRange starts on 50, this value would be 50 on the first frame.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	int32 FrameNumber;
	
	/** Frame Number for the Shot (matching what you would see in Sequencer at the sub-sequence level. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	int32 FrameNumberShot;
	
	/** Frame Number for the Root (relative to 0, not what you would see in the Sequencer timeline. ie: If sequence PlaybackRange starts on 50, this value would be 0 on the first frame. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	int32 FrameNumberRel;
	
	/** Frame Number for the Shot (relative to 0, not what you would see in the Sequencer timeline. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	int32 FrameNumberShotRel;
	
	/** Name used by the {camera_name} format tag. If specified, this will override the camera name (which is normally pulled from the ShotOverride object). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	FString CameraNameOverride;
	
	/** Name used by the {shot_name} format tag. If specified, this will override the shot name (which is normally pulled from the ShotOverride object) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	FString ShotNameOverride;

	/** When converitng frame numbers to strings, how many digits should we pad them up to? ie: 5 => 0005 with a count of 4. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	int32 ZeroPadFrameNumberCount;

	/** If true, force format strings (like {frame_number}) to resolve using the relative version. Used when slow-mo is detected as frame numbers would overlap. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	bool bForceRelativeFrameNumbers;

	/** Optional. If specified this is the filename that will be used instead of automatically building it from the Job's Output Setting. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	FString FileNameOverride;

	/** 
	* A map between "{format}" tokens and their values. These are applied after the auto-generated ones from the system,
	* which allows the caller to override things like {.ext} depending or {render_pass} which have dummy names by default.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	TMap<FString, FString> FileNameFormatOverrides;

	/** A key/value pair that maps metadata names to their values. Output is only supported in exr formats at the moment. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	TMap<FString, FString> FileMetadata;

	/** The initialization time for this job. Used to resolve time-based format arguments. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	FDateTime InitializationTime;

	/** What offset should be applied to InitializationTime when generating {time} related filename tokens? Likely your offset from UTC if you want local time. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	FTimespan InitializationTimeOffset;

	/** The version for this job. Used to resolve version format arguments. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	int32 InitializationVersion;

	/** Required. This is the job all of the settings should be pulled from.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	TObjectPtr<UMoviePipelineExecutorJob> Job;

	/** If this shot has multiple cameras in the sidecar array, which camera is this for? -1 will return the InnerName/Main camera for the shot. */
	int32 CameraIndex;

	/** Optional. If specified, settings will be pulled from this shot (if overriden by the shot). If null, always use the root configuration in the job. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	TObjectPtr<UMoviePipelineExecutorShot> ShotOverride;

	/** Additional offset added onto the offset provided by the Output Settings in the Job. Required for some internal things (FCPXML). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Render Pipeline")
	int32 AdditionalFrameNumberOffset;
};

/**
* These parameters define a single sample that a render pass should render with.
*/
struct FMoviePipelineRenderPassMetrics
{
public:
	// Manual default constructors to avoid deprecation warnings/errors on EffectiveOutputResolution
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMoviePipelineRenderPassMetrics() = default;
	~FMoviePipelineRenderPassMetrics() = default;
	FMoviePipelineRenderPassMetrics(const FMoviePipelineRenderPassMetrics&) = default;
	FMoviePipelineRenderPassMetrics(FMoviePipelineRenderPassMetrics&&) = default;
	FMoviePipelineRenderPassMetrics& operator=(const FMoviePipelineRenderPassMetrics&) = default;
	FMoviePipelineRenderPassMetrics& operator=(FMoviePipelineRenderPassMetrics&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** What FrameIndex should the FSceneView use for its internal index. Used to ensure samples have unique TAA/Raytracing/etc. sequences */
	int32 FrameIndex;

	/** Is the Scene View frozen so no changes are made as a result of this sample? */
	bool bWorldIsPaused;

	/** Do we pass the camera cut flag to the renderer to clear histories? */
	bool bCameraCut;

	/** Which anti-aliasing method should this view use? */
	EAntiAliasingMethod AntiAliasingMethod;

	/** What type of output should the Post Processing output? (HDR /w ToneCurve, HDR w/o ToneCurve) */
	ESceneCaptureSource SceneCaptureSource;

	/** How many tiles on X and Y are there total. */
	FIntPoint OriginalTileCounts;

	/** How many tiles on X and Y are there total. */
	FIntPoint TileCounts;

	/** Of the TileCount, which X/Y Tile is this sample for. */
	FIntPoint TileIndexes;

	/** SpatialJitter offset X. */
	float SpatialShiftX;

	/** SpatialJitter offset Y. */
	float SpatialShiftY;


	/** Get a 0-(TileCount-1) version of TileIndex. */
	inline int32 GetTileIndex() const
	{
		return (TileIndexes.Y * TileCounts.Y) + TileIndexes.X;
	}

	inline int32 GetTileCount() const
	{
		return (TileCounts.X * TileCounts.Y);
	}

	/** How big is the back buffer for this sample? This is the final target size divided by the number of tiles with padding added. */
	FIntPoint BackbufferSize;

	/** Full Output image resolution for this shot. */
	UE_DEPRECATED(5.6, "EffectiveOutputResolution is deprecated. Use OverscannedResolution instead")
	FIntPoint EffectiveOutputResolution;

	/** The overscanned resolution, computed by scaling the desired output resolution by any configured overscan */
	FIntPoint OverscannedResolution;
	
	/** How big is an individual tile for this sample? This is the final target size divided by the number of tiles (without padding added) */
	FIntPoint TileSize;

	/** If true, we will discard this sample after rendering. Used to get history set up correctly. */
	bool bDiscardResult;

	/** If true, we will write this sample to disk (and also send it to the accumulator.) */
	bool bWriteSampleToDisk;

	/** How many spatial jitters will there be total for this particular temporal accumulation frame? */
	int32 SpatialSampleCount;
	
	/** Of the SpatialSampleCount, which index is this? */
	int32 SpatialSampleIndex;

	/** Offset to apply to random number generator seed. */
	int32 SeedOffset;

	/** How many temporal jitters will there be total for this particular output frame? */
	int32 TemporalSampleCount;

	/** Of the TemporalSampleCount, which index is this? */
	int32 TemporalSampleIndex;

	/** How many pixels are we overlapping with adjacent tiles (on each side) */
	FIntPoint OverlappedPad;

	/** How much is this sample offset, taking padding into account. */
	FIntPoint OverlappedOffset;

	/** Use overscan percentage to extend render region beyond the set resolution.  */
	float OverscanPercentage;

	/** Whether to override the camera's overscan value when rendering */
	bool bOverrideCameraOverscan;

	/**
	 * The rectangle to use to crop the image prior to compositing and output. Used to crop out overscan from appropriate formats, and will
	 * generally have size equal to the originally requested resolution, whereas internal resolutions such as Backbuffer and Accumulator
	 * resolutions will be scaled up to include overscan
	 */
	FIntRect CropRectangle;
	
	/** 
	* The gamma space to apply accumulation in. During accumulation, pow(x,AccumulationGamma) is applied
	* and pow(x,1/AccumulationGamma) is applied after accumulation is finished. 1.0 means no change."
	*/
	float AccumulationGamma;

	/**
	* The amount of screen-space shift used to replace TAA Projection Jitter, modified for each spatial sample
	* of the render.
	*/
	FVector2D ProjectionMatrixJitterAmount;

	/**
	* Any additional texture mip map bias that should be added when rendering. Can be used to force extra sharpness. A more negative number means more likely to use a higher quality mip map.
	*/
	float TextureSharpnessBias;

	/**
	* What screen percentage should each view be rendered at? 1.0 = standard size. Can gain more detail (at the cost of longer renders, more device timeouts), consider trying to use TextureSharpnessBias if possible instead.
	*/
	float GlobalScreenPercentageFraction;

	/**
	* Whether the auto exposure pass is an omnidirectional cube capture pass, useful for panoramic rendering.  Only set for the actual exposure pass!
	*/
	bool bAutoExposureCubePass;

	/**
	* Transient value set when generating an autoexposure cube face.
	*/
	int8 AutoExposureCubeFace;

	FMoviePipelineFrameOutputState OutputState;

	FVector2D OverlappedSubpixelShift;

	MoviePipeline::FMoviePipelineFrameInfo FrameInfo;

	FOpenColorIODisplayConfiguration* OCIOConfiguration;
};

namespace MoviePipeline
{
	struct FMoviePipelineRenderPassInitSettings
	{
	public:
		UE_DEPRECATED(5.0, "FMoviePipelineRenderPassInitSettings must be constructed with arguments")
		FMoviePipelineRenderPassInitSettings();

		FMoviePipelineRenderPassInitSettings(ERHIFeatureLevel::Type InFeatureLevel, const FIntPoint& InBackbufferResolution, const FIntPoint& InTileCount)
			:	BackbufferResolution(InBackbufferResolution)
			,	TileCount(InTileCount)
			,	FeatureLevel(InFeatureLevel)
		{
		}

	public:
		/** This takes into account any padding needed for tiled rendering overlap. Different than the output resolution of the final image. */
		FIntPoint BackbufferResolution = FIntPoint(0, 0);

		/** How many tiles (in each direction) are we rendering with. */
		FIntPoint TileCount = FIntPoint(0, 0);

		ERHIFeatureLevel::Type FeatureLevel;
	};

	struct FCompositePassInfo
	{
		FCompositePassInfo() {}

		FMoviePipelinePassIdentifier PassIdentifier;
		TUniquePtr<FImagePixelData> PixelData;
	};


}

struct FImagePixelDataPayload : IImagePixelDataPayload, public TSharedFromThis<FImagePixelDataPayload, ESPMode::ThreadSafe>
{
	FMoviePipelineRenderPassMetrics SampleState;

	FMoviePipelinePassIdentifier PassIdentifier;
	
	/** Does this output data have to be transparent to be useful? Overrides output format to one that supports transparency. */
	bool bRequireTransparentOutput;

	int32 SortingOrder;
	bool bCompositeToFinalImage;

	/** If specified, use this as the output filename (not including output directory) when using debug write samples to disk */
	FString Debug_OverrideFilename;

	FImagePixelDataPayload()
		: bRequireTransparentOutput(false)
		, SortingOrder(TNumericLimits<int32>::Max())
		, bCompositeToFinalImage(false)
	{}

	virtual TSharedRef<FImagePixelDataPayload> Copy() const
	{
		return MakeShared<FImagePixelDataPayload>(*this);
	}

	virtual FIntPoint GetAccumulatorSize() const
	{
		return FIntPoint(SampleState.TileSize.X * SampleState.TileCounts.X, SampleState.TileSize.Y * SampleState.TileCounts.Y);
	}

	virtual FIntPoint GetOverlappedOffset() const
	{
		return SampleState.OverlappedOffset;
	}

	virtual FVector2D GetOverlappedSubpixelShift() const
	{
		return SampleState.OverlappedSubpixelShift;
	}

	virtual void GetWeightFunctionParams(MoviePipeline::FTileWeight1D& WeightFunctionX, MoviePipeline::FTileWeight1D& WeightFunctionY) const
	{
		WeightFunctionX.InitHelper(SampleState.OverlappedPad.X, SampleState.TileSize.X, SampleState.OverlappedPad.X);
		WeightFunctionY.InitHelper(SampleState.OverlappedPad.Y, SampleState.TileSize.Y, SampleState.OverlappedPad.Y);
	}

	virtual FIntPoint GetOverlapPaddedSize() const
	{
		return FIntPoint(
			(SampleState.TileSize.X + 2 * SampleState.OverlappedPad.X),
			(SampleState.TileSize.Y + 2 * SampleState.OverlappedPad.Y));
	}

	virtual bool GetOverlapPaddedSizeIsValid(const FIntPoint InRawSize) const
	{
		return (SampleState.TileSize.X + 2 * SampleState.OverlappedPad.X == InRawSize.X)
			&& (SampleState.TileSize.Y + 2 * SampleState.OverlappedPad.Y == InRawSize.Y);
	}

	/** Is this the first tile of an image and we should start accumulating? */
	inline bool IsFirstTile() const
	{
		return SampleState.TileIndexes.X == 0 && SampleState.TileIndexes.Y == 0 && SampleState.SpatialSampleIndex == 0;
	}

	/** Is this the last tile of an image and we should finish accumulating? */
	inline bool IsLastTile() const
	{
		return SampleState.TileIndexes.X == SampleState.TileCounts.X - 1 &&
			   SampleState.TileIndexes.Y == SampleState.TileCounts.Y - 1 &&
			   SampleState.SpatialSampleIndex == SampleState.SpatialSampleCount - 1;
	}

	inline bool IsFirstTemporalSample() const
	{
		return SampleState.TemporalSampleIndex == 0;
	}

	inline bool IsLastTemporalSample() const
	{
		return SampleState.TemporalSampleIndex == SampleState.TemporalSampleCount - 1;
	}
};

struct FMoviePipelineMergerOutputFrame
{
public:
	FMoviePipelineMergerOutputFrame() {}
	virtual ~FMoviePipelineMergerOutputFrame() = default;

	FMoviePipelineMergerOutputFrame& operator=(FMoviePipelineMergerOutputFrame&& InOther)
	{
		FrameOutputState = InOther.FrameOutputState;
		ExpectedRenderPasses = InOther.ExpectedRenderPasses;
		ImageOutputData = MoveTemp(InOther.ImageOutputData);
	
		return *this;
	}
	FMoviePipelineMergerOutputFrame(FMoviePipelineMergerOutputFrame&& InOther)
		: FrameOutputState(InOther.FrameOutputState)
		, ExpectedRenderPasses(InOther.ExpectedRenderPasses)
		, ImageOutputData(MoveTemp(InOther.ImageOutputData))
	{
	}

	bool HasDataFromMultipleCameras() const
	{
		TMap<FString, int32> CameraNameUseCounts;
		for(const FMoviePipelinePassIdentifier& PassIdentifier : ExpectedRenderPasses)
		{
			CameraNameUseCounts.FindOrAdd(PassIdentifier.CameraName) += 1;
		}

		// ToDo: This logic might get more complicated because of burn ins, etc. that don't have camera names, 
		// need to check.
		return CameraNameUseCounts.Num() > 1;
	}

	bool HasDataFromMultipleRenderPasses(const TArray<MoviePipeline::FCompositePassInfo>& InCompositedPasses) const
	{
		TMap<FString, int32> RenderPassUseCounts;
		for (const FMoviePipelinePassIdentifier& PassIdentifier : ExpectedRenderPasses)
		{
			RenderPassUseCounts.FindOrAdd(PassIdentifier.Name) += 1;
		}

		// Remove any render passes that will be composited on later
		for (const MoviePipeline::FCompositePassInfo& CompositePass : InCompositedPasses)
		{
			RenderPassUseCounts.Remove(CompositePass.PassIdentifier.Name);
		}

		return RenderPassUseCounts.Num() > 1;
	}

private:
	// Explicitly delete copy operators since we own unique data.
	// void operator=(const FMoviePipelineMergerOutputFrame&);
	// FMoviePipelineMergerOutputFrame(const FMoviePipelineMergerOutputFrame&);

public:
	FMoviePipelineFrameOutputState FrameOutputState;

	TArray<FMoviePipelinePassIdentifier> ExpectedRenderPasses;

	TMap<FMoviePipelinePassIdentifier, TUniquePtr<FImagePixelData>> ImageOutputData;
};


namespace MoviePipeline
{
	/** The interface that all accumulators should derive from. */
	struct IMoviePipelineAccumulator
	{
	};

	/** The interface that all accumulation args should derive from. */
	struct IMoviePipelineAccumulationArgs
	{
	};
	
	struct IMoviePipelineOverlappedAccumulator : IMoviePipelineAccumulator, TSharedFromThis<IMoviePipelineOverlappedAccumulator>
	{
	};

	struct IMoviePipelineOutputMerger : public TSharedFromThis<IMoviePipelineOutputMerger>
	{
		virtual FMoviePipelineMergerOutputFrame& QueueOutputFrame_GameThread(const FMoviePipelineFrameOutputState& CachedOutputState) = 0;
		virtual void OnCompleteRenderPassDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) = 0;
		virtual void OnSingleSampleDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) = 0;
		virtual void AbandonOutstandingWork() = 0;
		virtual int32 GetNumOutstandingFrames() const = 0;

		virtual ~IMoviePipelineOutputMerger() = default;
	};

	struct FAudioState
	{
		struct FAudioSegment
		{
			FAudioSegment()
			{
				Id = FGuid::NewGuid();
			}

			FGuid Id;
			FMoviePipelineFrameOutputState OutputState;

			FString SubmixName;

			float NumChannels;
			float SampleRate;

			Audio::AlignedFloatBuffer SegmentData;
		};

		FAudioState()
			: bIsRecordingAudio(false)
			, PrevUnfocusedAudioMultiplier(1.f)
			, PrevRenderEveryTickValue(1)
			, PrevNeverMuteNRTAudioValue(0)
			, PrevAudioLatencyCompensationFrames(0)
			, bPrevAllowBackgroundAudio(false)
		{}

		bool bIsRecordingAudio;

		float PrevUnfocusedAudioMultiplier;
		int32 PrevRenderEveryTickValue;
		int32 PrevNeverMuteNRTAudioValue;
		int32 PrevAudioLatencyCompensationFrames;

		/** The previous value of ULevelEditorMiscSettings::bAllowBackgroundAudio. */
		bool bPrevAllowBackgroundAudio;

		/** Float Buffers for Movie Segments we've finished rendering. Stored until shutdown. Fully available during Finalize stage. */
		TArray<FAudioSegment> FinishedSegments;

		/** An array of active submixes we are recording for this shot. Gets cleared when recording stops on a shot. */
		TArray<TWeakPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe>> ActiveSubmixes;
	};
}

/** A simple struct wrapper around a string array so it can be included in a TMap. */
USTRUCT(BlueprintType)
struct FMoviePipelineStringArray
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TArray<FString> Array;
};

USTRUCT(BlueprintType)
struct FMoviePipelineRenderPassOutputData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TArray<FString> FilePaths;

	/**
	 * The "frame templated" file paths. This groups together a set of output frames by frame number, eg "Seq.Shot.{frame_placeholder}.exr".
	 * If the render generated more than one output type (eg, EXR and JPEG), there may be more than one path here, but there should only
	 * be one per output type.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TArray<FString> FrameTemplatedFilePaths;

	/** Maps a setting type to all of the files that the setting generated. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TMap<FSoftClassPath, FMoviePipelineStringArray> SettingTypeToFilePaths;
};

USTRUCT(BlueprintType)
struct FMoviePipelineShotOutputData
{
	GENERATED_BODY()

	/** Which shot was this output data for? */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, Category = "Movie Pipeline")
	TWeakObjectPtr<UMoviePipelineExecutorShot> Shot;

	/** 
	* A mapping between render passes (such as 'FinalImage') and an array containing the files written for that shot.
	* Will be multiple files if using image sequences.
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TMap<FMoviePipelinePassIdentifier, FMoviePipelineRenderPassOutputData> RenderPassData;
};

namespace MoviePipeline
{
	struct FMoviePipelineOutputFutureData
	{
		FMoviePipelineOutputFutureData()
			: Shot(nullptr)
			, OriginSettingClass(nullptr)
		{}

		UMoviePipelineExecutorShot* Shot;
		FString FilePath;
		FMoviePipelinePassIdentifier PassIdentifier;

		/** The "frame templated" filepath. This groups together a set of output frames by frame number, eg "Seq.Shot.{frame_placeholder}.exr". */
		FString FrameTemplatedFilePath;

		/** The setting class that this file originated from. */
		UClass* OriginSettingClass;
	};
}

namespace UE
{
	namespace MoviePipeline
	{
		struct FViewportArgs
		{
			TSubclassOf<UMovieRenderDebugWidget> DebugWidgetClass;
			bool bRenderViewport = false;
		};
	}
}
/**
* Contains information about the to-disk output generated by a movie pipeline. This structure is used both for per-shot work finished
* callbacks and for the final render finished callback. When used as a per-shot callback ShotData will only have one entry (for the
* shot that was just finished), and for the final render callback it will have data for all shots that managed to render. Can be empty
* if the job failed to produce any files.
*/
USTRUCT(BlueprintType)
struct FMoviePipelineOutputData
{
	GENERATED_BODY()

	FMoviePipelineOutputData()
	: Pipeline(nullptr)
	, Job(nullptr)
	, bSuccess(false)
	{}
	
	/** 
	* The UMoviePipeline instance that generated this data. This is only provided as an id (in the event you were the one who created
	* the UMoviePipeline instance. DO NOT CALL FUNCTIONS ON THIS (unless you know what you're doing)
	*
	* Provided here for backwards compatibility.
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, Category = "Movie Pipeline")
	TObjectPtr<class UMoviePipelineBase> Pipeline;
	
	/** 
	* Job the data is for. Job may still be in progress (if a shot callback) so be careful about modifying properties on it 
	* When using the Movie Render Graph this will point to the duplicated job created during execution.
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, Category = "Movie Pipeline")
	TObjectPtr<UMoviePipelineExecutorJob> Job;
	
	/** Did the job succeed, or was it canceled early due to an error (such as failure to write file to disk)? */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	bool bSuccess;

	/** 
	* The file data for each shot that was rendered. If no files were written this will be empty. If this is from the per-shot work
	* finished callback it will only have one entry (for the just finished shot). Will not include shots that did not get rendered
	* due to the pipeline encountering an error.
	*
	* This will be empty when using the Movie Render Graph. If a job is a Movie Render Graph job, use GraphData instead, which has
	* a similar layout but with different data types.
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TArray<FMoviePipelineShotOutputData> ShotData;

	/** 
	* The file data for each shot that was rendered. If no files were written this will be empty. If this is from the per-shot work
	* finished callback it will only have one entry (for the just finished shot). Will not include shots that did not get rendered
	* due to the pipeline encountering an error.
	*
	* This will be empty when using the legacy Movie Pipeline Configurations. If a job is a legacy configuration, use ShotData instead, which has
	* a similar layout but with different data types.
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TArray<FMovieGraphRenderOutputData> GraphData;
};

/** The action that should be taken after finishing the render. */
UENUM(BlueprintType)
enum class EMoviePipelinePostRenderActionType : uint8
{
	/** Do nothing after the render. */
	DoNothing				= 0,

	/** Open the media that was generated from the render. The application used for this is defined in Editor Preferences for Movie Render Graph. */
	PlayRenderOutput		= 1 << 0,

	/** Open the directory that contains the files generated by the render. */
	OpenOutputDirectory		= 1 << 1
};
// Note: The values above are treated as flags in the possible future scenario where multiple actions can be taken
// at the same time (and the actions could be OR'd together).

/**
 * A pixel preprocessor for use with FImageWriteTask::PixelPreProcessor that does a simple alpha blend of the provided image onto the
 * target pixel data. This isn't very general purpose.
 */
template<typename PixelType> struct TAsyncCompositeImage;

template<>
struct TAsyncCompositeImage<FColor>
{
	TAsyncCompositeImage(TUniquePtr<FImagePixelData>&& InPixelData, FIntPoint InOffset = FIntPoint())
		: ImageToComposite(MoveTemp(InPixelData))
		, Offset(InOffset)
	{}

	void operator()(FImagePixelData* PixelData)
	{
		check(PixelData->GetType() == EImagePixelType::Color);
		check(ImageToComposite && ImageToComposite->GetType() == EImagePixelType::Color);

		// Offset within source and destination images to start compositing from and to
		const FIntPoint SrcOffset = FIntPoint(FMath::Max(-Offset.X, 0), FMath::Max(-Offset.Y, 0));
		const FIntPoint DestOffset = FIntPoint(FMath::Max(Offset.X, 0), FMath::Max(Offset.Y, 0));
		
		const FIntPoint SrcSize = ImageToComposite->GetSize();
		const FIntPoint DestSize = PixelData->GetSize();
		
		// The size of the region being composited
		const FIntPoint CompositeSize = FIntPoint(
			FMath::Min(SrcSize.X - SrcOffset.X, DestSize.X - DestOffset.X),
			FMath::Min(SrcSize.Y - SrcOffset.Y, DestSize.Y - DestOffset.Y));

		// If either dimension of our composite size is less than or equal to zero, it means that one or more axes of our source image is completely outside the bounds of
		// the destination image when offset to the specified position, and can't be properly composited onto it
		if (!ensureMsgf(CompositeSize.X > 0 && CompositeSize.Y > 0, TEXT("Source of size (%d,%d) does not overlap destination of size (%d,%d) when offset to (%d,%d)!"),
			SrcSize.X, SrcSize.Y, DestSize.X, DestSize.Y, Offset.X, Offset.Y))
		{
			return;
		}
		
		TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline::TAsyncCompositeImageFColor);

		TImagePixelData<FColor>* DestColorData = static_cast<TImagePixelData<FColor>*>(PixelData);
		TImagePixelData<FColor>* SrcColorData = static_cast<TImagePixelData<FColor>*>(ImageToComposite.Get());

		ParallelFor(CompositeSize.Y,
			[&](int32 ScanlineIndex = 0)
			{
				for (int64 ColumnIndex = 0; ColumnIndex < CompositeSize.X; ColumnIndex++)
				{
					int64 SrcIndex = (int64(ScanlineIndex) + int64(SrcOffset.Y)) * int64(SrcColorData->GetSize().X) + int64(ColumnIndex) + int64(SrcOffset.X);
					int64 DstIndex = (int64(ScanlineIndex) + int64(DestOffset.Y)) * int64(DestColorData->GetSize().X) + int64(ColumnIndex) + int64(DestOffset.X);
					
					FColor& Dst = DestColorData->Pixels[DstIndex];
					FColor& Src = SrcColorData->Pixels[SrcIndex];

					if (Src.A == 255)
					{
						// If the source is fully opaque then we can just stomp the destination without any calculation
						Dst = Src;
					}
					else if (Src.A == 0)
					{
						// If the source is fully transparent, there's no composite needed (which is true for most of our burn-in)
					}
					else
					{
						// If a pixel is partially transparent then we need to change how we comp.
						auto Lin2sRGB = [](float InSRGB) -> float
						{
							return InSRGB < 0.0031308 ? InSRGB * 12.92 : (1.055f * FMath::Pow(InSRGB, 1 / 2.4f) - 0.055f);
						};

						// To match the composite done by UMG/Viewport we need to convert from sRGB to linear and
						// undo the premultiply, then convert back to sRGB and re-premultiply,  before comping.
						// The operation done by the viewport is done in the incorrect order, so we must match
						// the incorrect behavior here.
	
						// Convert sRGB -> Linear
						FLinearColor SrcAsLinear = FLinearColor(Src);

						// Unpre-mult. The divide should be safe here as we should have a non-zero alpha to be in this branch.
						FLinearColor SrcLinearUnPremult;
						SrcLinearUnPremult.R = SrcAsLinear.R / SrcAsLinear.A;
						SrcLinearUnPremult.G = SrcAsLinear.G / SrcAsLinear.A;
						SrcLinearUnPremult.B = SrcAsLinear.B / SrcAsLinear.A;
						SrcLinearUnPremult.A = SrcAsLinear.A;

						// Convert Linear -> sRGB
						FLinearColor SrcAsSRGBUnPremult;
						SrcAsSRGBUnPremult.R = Lin2sRGB(SrcLinearUnPremult.R);
						SrcAsSRGBUnPremult.G = Lin2sRGB(SrcLinearUnPremult.G);
						SrcAsSRGBUnPremult.B = Lin2sRGB(SrcLinearUnPremult.B);
						SrcAsSRGBUnPremult.A = SrcLinearUnPremult.A;

						// Re premult now that it is in sRGB
						FLinearColor SrcAsSRGBPremult;
						SrcAsSRGBPremult.R = SrcAsSRGBUnPremult.R * SrcAsSRGBUnPremult.A;
						SrcAsSRGBPremult.G = SrcAsSRGBUnPremult.G * SrcAsSRGBUnPremult.A;
						SrcAsSRGBPremult.B = SrcAsSRGBUnPremult.B * SrcAsSRGBUnPremult.A;
						SrcAsSRGBPremult.A = SrcAsSRGBUnPremult.A;

						// Comp in sRGB (even though they're in floats)
						FLinearColor SrcL = SrcAsSRGBPremult;
						FLinearColor DstL = Dst.ReinterpretAsLinear();

						FLinearColor OutL;
						OutL.R = SrcL.R + (DstL.R * (1.f - SrcL.A));
						OutL.G = SrcL.G + (DstL.G * (1.f - SrcL.A));
						OutL.B = SrcL.B + (DstL.B * (1.f - SrcL.A));
						OutL.A = SrcL.A + (DstL.A * (1.f - SrcL.A));

						// Quantize back to 0-255, without applying sRGB. This also clamps to 0-255.
						Dst = OutL.QuantizeRound();
					}
				}
			});
	}

	TUniquePtr<FImagePixelData> ImageToComposite;
	
	/**
	 * Amount in pixels from the destination's top left to offset the source when compositing onto the destination.
	 * Can be negative, in which case a portion of the source's bottom right will be composited onto the destination's top left
	 */
	FIntPoint Offset;
};

template<>
struct TAsyncCompositeImage<FFloat16Color>
{
	TAsyncCompositeImage(TUniquePtr<FImagePixelData>&& InPixelData, FIntPoint InOffset = FIntPoint())
		: ImageToComposite(MoveTemp(InPixelData))
		, Offset(InOffset)
	{}

	void operator()(FImagePixelData* PixelData)
	{
		check(PixelData->GetType() == EImagePixelType::Float16);
		check(ImageToComposite && ImageToComposite->GetType() == EImagePixelType::Color);
		
		// Offset within source and destination images to start compositing from and to
        const FIntPoint SrcOffset = FIntPoint(FMath::Max(-Offset.X, 0), FMath::Max(-Offset.Y, 0));
        const FIntPoint DestOffset = FIntPoint(FMath::Max(Offset.X, 0), FMath::Max(Offset.Y, 0));
        
        const FIntPoint SrcSize = ImageToComposite->GetSize();
        const FIntPoint DestSize = PixelData->GetSize();
        
        // The size of the region being composited
        const FIntPoint CompositeSize = FIntPoint(
        	FMath::Min(SrcSize.X - SrcOffset.X, DestSize.X - DestOffset.X),
        	FMath::Min(SrcSize.Y - SrcOffset.Y, DestSize.Y - DestOffset.Y));

		// If either dimension of our composite size is less than or equal to zero, it means that one or more axes of our source image is completely outside the bounds of
		// the destination image when offset to the specified position, and can't be properly composited onto it
		if (!ensureMsgf(CompositeSize.X > 0 && CompositeSize.Y > 0, TEXT("Source of size (%d,%d) does not overlap destination of size (%d,%d) when offset to (%d,%d)!"),
			SrcSize.X, SrcSize.Y, DestSize.X, DestSize.Y, Offset.X, Offset.Y))
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline::TAsyncCompositeImageFFloat16Color);
		
		TImagePixelData<FFloat16Color>* DestColorData = static_cast<TImagePixelData<FFloat16Color>*>(PixelData);
		TImagePixelData<FColor>* SrcColorData = static_cast<TImagePixelData<FColor>*>(ImageToComposite.Get());
		ParallelFor(CompositeSize.Y,
			[&](int32 ScanlineIndex = 0)
			{
				for (int64 ColumnIndex = 0; ColumnIndex < CompositeSize.X; ColumnIndex++)
				{
					int64 SrcIndex = (int64(ScanlineIndex) + int64(SrcOffset.Y)) * int64(SrcColorData->GetSize().X) + int64(ColumnIndex) + int64(SrcOffset.X);
					int64 DstIndex = (int64(ScanlineIndex) + int64(DestOffset.Y)) * int64(DestColorData->GetSize().X) + int64(ColumnIndex) + int64(DestOffset.X);
					
					FFloat16Color& Dst = DestColorData->Pixels[DstIndex];
					FColor& Src = SrcColorData->Pixels[SrcIndex];

					float SourceAlpha = Src.A / 255.f;
					FFloat16Color Out;
					Out.A = (Src.A / 255.f) + (Dst.A * (1.f - SourceAlpha));
					Out.R = (Src.R / 255.f) + (Dst.R * (1.f - SourceAlpha));
					Out.G = (Src.G / 255.f) + (Dst.G * (1.f - SourceAlpha));
					Out.B = (Src.B / 255.f) + (Dst.B * (1.f - SourceAlpha));
					Dst = Out;
				}
			});
	}

	TUniquePtr<FImagePixelData> ImageToComposite;
	
	/**
	 * Amount in pixels from the destination's top left to offset the source when compositing onto the destination.
	 * Can be negative, in which case a portion of the source's bottom right will be composited onto the destination's top left
	 */
	FIntPoint Offset;
};

template<>
struct TAsyncCompositeImage<FLinearColor>
{
	TAsyncCompositeImage(TUniquePtr<FImagePixelData>&& InPixelData, FIntPoint InOffset = FIntPoint())
		: ImageToComposite(MoveTemp(InPixelData))
		, Offset(InOffset)
	{}

	void operator()(FImagePixelData* PixelData)
	{
		check(PixelData->GetType() == EImagePixelType::Float32);
		check(ImageToComposite && ImageToComposite->GetType() == EImagePixelType::Color);

		// Offset within source and destination images to start compositing from and to
		const FIntPoint SrcOffset = FIntPoint(FMath::Max(-Offset.X, 0), FMath::Max(-Offset.Y, 0));
		const FIntPoint DestOffset = FIntPoint(FMath::Max(Offset.X, 0), FMath::Max(Offset.Y, 0));
		
		const FIntPoint SrcSize = ImageToComposite->GetSize();
		const FIntPoint DestSize = PixelData->GetSize();
		
		// The size of the region being composited
		const FIntPoint CompositeSize = FIntPoint(
			FMath::Min(SrcSize.X - SrcOffset.X, DestSize.X - DestOffset.X),
			FMath::Min(SrcSize.Y - SrcOffset.Y, DestSize.Y - DestOffset.Y));

		// If either dimension of our composite size is less than or equal to zero, it means that one or more axes of our source image is completely outside the bounds of
		// the destination image when offset to the specified position, and can't be properly composited onto it
		if (!ensureMsgf(CompositeSize.X > 0 && CompositeSize.Y > 0, TEXT("Source of size (%d,%d) does not overlap destination of size (%d,%d) when offset to (%d,%d)!"),
			SrcSize.X, SrcSize.Y, DestSize.X, DestSize.Y, Offset.X, Offset.Y))
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline::TAsyncCompositeImageFLinearColor);
		
		TImagePixelData<FLinearColor>* DestColorData = static_cast<TImagePixelData<FLinearColor>*>(PixelData);
		TImagePixelData<FColor>* SrcColorData = static_cast<TImagePixelData<FColor>*>(ImageToComposite.Get());

		ParallelFor(CompositeSize.Y,
			[&](int32 ScanlineIndex = 0)
			{
				for (int64 ColumnIndex = 0; ColumnIndex < CompositeSize.X; ColumnIndex++)
				{
					int64 SrcIndex = (int64(ScanlineIndex) + int64(SrcOffset.Y)) * int64(SrcColorData->GetSize().X) + int64(ColumnIndex) + int64(SrcOffset.X);
					int64 DstIndex = (int64(ScanlineIndex) + int64(DestOffset.Y)) * int64(DestColorData->GetSize().X) + int64(ColumnIndex) + int64(DestOffset.X);
					
					FLinearColor& Dst = DestColorData->Pixels[DstIndex];
					FColor& Src = SrcColorData->Pixels[SrcIndex];

					float SourceAlpha = Src.A / 255.f;
					FLinearColor Out;
					Out.A = (Src.A / 255.f) + (Dst.A * (1.f - SourceAlpha));
					Out.R = (Src.R / 255.f) + (Dst.R * (1.f - SourceAlpha));
					Out.G = (Src.G / 255.f) + (Dst.G * (1.f - SourceAlpha));
					Out.B = (Src.B / 255.f) + (Dst.B * (1.f - SourceAlpha));
					Dst = Out;
				}
			});
	}

	TUniquePtr<FImagePixelData> ImageToComposite;
	
	/**
	 * Amount in pixels from the destination's top left to offset the source when compositing onto the destination.
	 * Can be negative, in which case a portion of the source's bottom right will be composited onto the destination's top left
	 */
	FIntPoint Offset;
};

template<typename PixelType>
struct TAsyncCropImage
{
	TAsyncCropImage(FImageWriteTask* InParentWriteTask, const FIntRect& InCropRect)
		: ParentWriteTask(InParentWriteTask)
		, CropRect(InCropRect)
	{ }

	TAsyncCropImage(const FIntRect& InCropRect)
		: ParentWriteTask(nullptr)
		, CropRect(InCropRect)
	{ }
	
	void operator()(FImagePixelData* PixelData)
	{
		// Perhaps a little unintuitively, crop region is allowed to be bigger than the original source image, in which case the output image will be scaled up
		// to the size of the crop region and filled with zeros, while the source image is copied into the subregion it inhabits in the crop region.
		// Still have to make sure there is some overlap between the crop region and the source image (it's not useful to return an entirely blank output)
		const bool bIsValidCropRegion = CropRect.Max.X > 0 && CropRect.Max.Y > 0 && CropRect.Min.X < PixelData->GetSize().X && CropRect.Min.Y < PixelData->GetSize().Y;
		if (!ensureMsgf(bIsValidCropRegion, TEXT("Crop rectangle (%d, %d) to (%d, %d) does not overlap any part of the source image of size (%d, %d)"),
			CropRect.Min.X, CropRect.Min.Y, CropRect.Max.X, CropRect.Max.Y, PixelData->GetSize().X, PixelData->GetSize().Y))
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline::TAsyncCropImage)
		
		// In the case where the crop region extends beyond the top-left corner of the source image, keep track of that offset so that
		// we can read and write valid pixel locations from the source and destination images (e.g. normal crop only reads from the crop rect, but
		// inflationary crop can only read from the top left of the source image even if the crop rect extends beyond it). SrcOffset is the top left
		// of the rectangle we read from in the source image, and DestOffset is the top left of the rectangle we write to in the destination image
		const FIntPoint SrcOffset = FIntPoint(FMath::Max(CropRect.Min.X, 0), FMath::Max(CropRect.Min.Y, 0));
		const FIntPoint DestOffset = FIntPoint(FMath::Max(-CropRect.Min.X, 0), FMath::Max(-CropRect.Min.Y, 0));

		// Allocate enough pixels for the crop region and fill them with zero values
		TArray64<PixelType> DefaultPixels;
		DefaultPixels.AddZeroed(CropRect.Size().X * CropRect.Size().Y);

		TSharedRef<FImagePixelDataPayload> NewPayload = PixelData->GetPayload<FImagePixelDataPayload>()->Copy();
		
		TUniquePtr<FImagePixelData> CroppedImage = MakeUnique<TImagePixelData<PixelType>>(CropRect.Size(), MoveTemp(DefaultPixels), NewPayload);
		TImagePixelData<PixelType>* DestColorData = static_cast<TImagePixelData<PixelType>*>(CroppedImage.Get());
		TImagePixelData<PixelType>* SrcColorData = static_cast<TImagePixelData<PixelType>*>(PixelData);

		// Find the size of the region that actually needs to be copied, which is the intersection of the source image and the crop rectangle.
		// Start reading from the top left of the crop region, or if it extends beyond the source image, the source image top left, and continue
		// to the bottom right corner of the crop rect or the source image, whichever is smaller
		const FIntPoint Size = FIntPoint(
			FMath::Min(CropRect.Max.X - SrcOffset.X, SrcColorData->GetSize().X),
			FMath::Min(CropRect.Max.Y - SrcOffset.Y, SrcColorData->GetSize().Y));
		
		ParallelFor(Size.Y,
			[&](int32 ScanlineIndex = 0)
			{
				int64 DstIndex = (int64(ScanlineIndex) + int64(DestOffset.Y)) * int64(DestColorData->GetSize().X) + int64(DestOffset.X);
				int64 SrcIndex = (int64(ScanlineIndex) + int64(SrcOffset.Y)) * int64(SrcColorData->GetSize().X) + int64(SrcOffset.X);
				
				FMemory::Memcpy(&DestColorData->Pixels[DstIndex], &SrcColorData->Pixels[SrcIndex], Size.X * sizeof(PixelType));
			});

		// If we have a parent write task that is holding the image being processed, write to that. Otherwise,
		// write to a local image that can be used by the caller
		if (ParentWriteTask)
		{
			ParentWriteTask->PixelData = MoveTemp(CroppedImage);
		}
		else
		{
			OutCroppedImage = MoveTemp(CroppedImage);
		}
	}

	/** The parent image write task using this image preprocessor. If not null, the cropped image will be written to the write task's pixel data */
	FImageWriteTask* ParentWriteTask;

	/** If the parent write task is null, the cropped image will be written to this pixel data instead */
	TUniquePtr<FImagePixelData> OutCroppedImage;

	/** the rectangle to crop the source image to */
	FIntRect CropRect;
};

#undef UE_API
