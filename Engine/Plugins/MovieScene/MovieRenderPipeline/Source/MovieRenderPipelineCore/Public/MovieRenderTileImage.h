// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "CoreMinimal.h"

#include "ImagePixelData.h"
#include "MovieRenderPipelineDataTypes.h"

#define UE_API MOVIERENDERPIPELINECORE_API

/* Types
 *****************************************************************************/

/**
 * Structure for a single tile.
 */
struct FImageTilePlane
{
public:
	/**
	 * Default constructor.
	 */
	FImageTilePlane()
		: SizeX(0)
		, SizeY(0)
		, AccumulationWeight(0.0f)
	{
	}

	/**
	 * Initialize the memory. Before using the memory, we also will need a call to ZeroPlane.
	 *
	 * @param InSizeX - Horizontal size of the tile.
	 * @param InSizeY - Vertical size of the tile.
	 */
	UE_API void Init(int32 InSizeX, int32 InSizeY);

	/**
	 * Zeroes the accumulation. Assumes the data is already allocated.
	 */
	UE_API void ZeroPlane();

	/**
	 * Frees the memory and resets the sizes
	 */
	UE_API void Reset();

	/**
	 * Accumulate a single tile to this plane. The raw data must be the exact size of the tile. In general, SampleOffsetX/SampleOffsetY will be 0 or 1.
	 * In most cases it is zero, but but if a sample has a subpixel offset after the last sample, it will affect the first sample of the next pixel.
	 * All accumulating assumes edge clamping.
	 *
	 * @param InRawData - Raw data to accumulate
	 * @param InRawSizeX - Width of the tile. Must exactly match.
	 * @param InRawSizeY - Height of the tile. Must exactly match.
	 * @param InSampleOffsetX - Pixel offset of the tile (in X)
	 * @param InSampleOffsetY - Pixel offset of the tile (in Y)
	 */
	UE_API void AccumulateSinglePlane(const TArray64<float>& InRawData, int32 InSizeX, int32 InSizeY, float InSampleWeight, int InSampleOffsetX, int InSampleOffsetY);

	/** Actual channel data*/
	TArray64<float> ChannelData;

	/** Width of the image. */
	int32 SizeX;

	/** Height of the image. */
	int32 SizeY;

	/** Accumulation weights, uniform for the whole plane. */
	float AccumulationWeight;
};

/**
 * Contains all the image planes for the tiles.
 */
struct FImageTileAccumulator : MoviePipeline::IMoviePipelineAccumulator
{
public:
	/**
	 * Default constructor.
	 */
	FImageTileAccumulator()
		: TileSizeX(0)
		, TileSizeY(0)
		, NumTilesX(0)
		, NumTilesY(0)
		, NumChannels(0)
		, KernelRadius(1.0f)
		, AccumulationGamma(1.0f)
	{
	}

	/**
	 * Allocates memory.
	 *
	 * @param InTileSizeX - Horizontal tile size.
	 * @param InTileSizeY - Vertical tile size.
	 * @param InNumTilesX - Num horizontal tiles.
	 * @param InNumTilesY - Num vertical tiles.
	 * @param InNumChannels - Num Channels.
	 */
	UE_API void InitMemory(int InTileSizeX, int InTileSizeY, int InNumTilesX, int InNumTilesY, int InNumChannels);

	/**
	 * Initializes memory.
	 *
	 * Resets the memory to 0s so that we can start a new frame.
	 */
	UE_API void ZeroPlanes();

	/**
	 * Resets the memory.
	 */
	UE_API void Reset();

	/**
	 * Get the tile index in the ImagePlanes list.
	 *
	 * @param InTileX - Horizontal tile.
	 * @param InTileY - Vertical tile.
	 * @param InChannel - Channel index.
	 */
	UE_API int32 GetPlaneIndex(int32 InTileX, int32 InTileY, int32 InChannel) const;

	/**
	 * Get the data for the appropriate tile.
	 *
	 * @param InTileX - Horizontal tile.
	 * @param InTileY - Vertical tile.
	 * @param InChannel - Channel index.
	 */
	UE_API FImageTilePlane& AtPlaneData(int32 InTileX, int32 InTileY, int32 InChannel);

	/**
	 * Get the data for the appropriate tile.
	 *
	 * @param InTileX - Horizontal tile.
	 * @param InTileY - Vertical tile.
	 * @param InChannel - Channel index.
	 */
	UE_API const FImageTilePlane& AtPlaneData(int32 InTileX, int32 InTileY, int32 InChannel) const;

	/**
	 * Given a tile and its subpixel offset, accumulate this tile to all of the image planes.
	 * 
	 * The raw sizes should exactly match the tile size. The SubpixelOffset should be in the range of [0,1).
	 * AccumulateTile() then figures out which tiles it touches, and accumulates them.
	 *
	 * @param InRawData - Tile to add.
	 * @param InRawSizeX - Width of the tile. Should exactly match the size of the internal channel planes.
	 * @param InRawSizeY - Height of the tile. Should exactly match the size of the internal channel planes.
	 * @param InRawChannel - Which channel to accumulate to.
	 * @param InSubpixelOffset - The offset, which affects which tiles get accumulated to.
	 */
	UE_API void AccumulateTile(const TArray64<float>& InRawData, int32 InRawSizeX, int32 InRawSizeY, int InRawChannel, FVector2D InSubpixelOffset);

	/**
	 * Given the dstance from a sample, calculate the weight.
	 *
	 * @param InDistance - The distance from the sample to the pixel center.
	 */
	UE_API float CalcSampleWeight(float InDistance) const;

	/**
	 * Given a rendered tile, accumulate the data to the full size image.
	 * 
	 * @param InPixelData - Raw pixel data.
	 * @param InTileX - Tile index in X.
	 * @param InTileY - Tile index in Y.
	 * @param InSubPixelOffset - SubPixel offset, should be in the range [0,1)
	 */
	UE_API void AccumulatePixelData(const FImagePixelData& InPixelData, int32 InTileX, int32 InTileY, FVector2D InSubpixelOffset);

	/**
	 * After accumulation is finished, fetch the final image as bytes. In theory we don't need this, because we could
	 * just fetch as LinearColor and convert to bytes. But the largest size asked for is 45k x 22.5k, which is 1B pixels.
	 * So fetching as LinearColor would create a 16GB intermediary image, so it's worth having an option to fetch
	 * straight to FColors.
	 * 
	 * @param FImagePixelData - Finished pixel data.
	 */
	UE_API void FetchFinalPixelDataByte(TArray64<FColor>& OutPixelData) const;

	/**
	 * After accumulation is finished, fetch the final image as linear colors
	 * 
	 * @param FImagePixelData - Finished pixel data.
	 */
	UE_API void FetchFinalPixelDataLinearColor(TArray64<FLinearColor>& OutPixelData) const;

	/**
	 * After accumulation is finished, fetch the final scale of each plane.
	 * 
	 * @param PlaneScale - Final scale for each plane.
	 */
	UE_API void FetchFinalPlaneScale(TArray<float>& PlaneScale) const;

	/**
	 * Grab a single pixel from the full res tile and scale it by the appropriate Scale value.
	 * 
	 * @param Rgba - Found pixel value.
	 * @param PlaneScale - The scales of each channel plane.
	 * @param FullX - X position of the full res image.
	 * @param FullY - Y position of the full res image.
	 */
	UE_API void FetchFullImageValue(float Rgba[4], const TArray<float>& PlaneScale, int32 FullX, int32 FullY) const;

public:
	/** Width of each tile in pixels */
	int32 TileSizeX;

	/** Height of each tile in pixels */
	int32 TileSizeY;

	/** Horizontal tiles. */
	int32 NumTilesX;

	/** Vertical tiles. */
	int32 NumTilesY;

	/** Number of channels in the tiles. Typical will be 3 (RGB). */
	int32 NumChannels;

	/** Radius of the kernel, in units of the final image. A value of 1.0 means a radius of one pixel. */
	float KernelRadius;

	/** When accumulating apply pow(X,AccumulationGamma), and then apply pow(X,1/AccumulationGamma) on output. */
	float AccumulationGamma;

	/**
	 * Actual pixel data. Note total number of planes is NumTilesX * NumTilesY * NumChannels;
	 * The total image size can be up to 32k x 32k. For an RGBA pass, that is 16GB of data. So in the future
	 * we might want to keep a pool of tiles around and share them to avoid allocating/freeing so much memory.
	 *
	 * The layout of tiles is by subsample. So all tiles are the "width/height" of the full image, but describe
	 * different subsamples. So a 4k by 4k with TileSizeX=4 and TileSizeY=3 would have a pixel layout like this.
	 * 
	 * TileIndex:
	 * -----------------------------------------
	 * |  0  1  2  3 | 0  1  2  3 | 0  1  2  3 |
	 * |  4  5  6  7 | 4  5  6  7 | 4  5  6  7 |
	 * |  8  9 10 11 | 8  9 10 11 | 8  9 10 11 |
	 * -----------------------------------------
	 * |  0  1  2  3 | 0  1  2  3 | 0  1  2  3 |
	 * |  4  5  6  7 | 4  5  6  7 | 4  5  6  7 |
	 * |  8  9 10 11 | 8  9 10 11 | 8  9 10 11 |
	 * -----------------------------------------
	 */
	TArray64<FImageTilePlane> ImagePlanes;

};


#undef UE_API
