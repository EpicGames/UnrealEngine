// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MovieRenderPipelineEditor : ModuleRules
{
	public MovieRenderPipelineEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
            }
		);
		 
		PrivateIncludePaths.AddRange(
			new string[] {
            }
        );

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"TimeManagement",
				"MovieScene",
                "MovieSceneTools",
                "MovieSceneTracks",
				"MovieRenderPipelineCore",
                "MovieRenderPipelineRenderPasses",
                "MovieRenderPipelineSettings",
				"Settings",
				"ContentBrowser",
				"PropertyEditor",
				"EditorWidgets",
				"EditorSubsystem",
				"FunctionalTesting",
            }
        );

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"CoreUObject",
				"Engine",
				"Kismet",
				"Slate",
				"SlateCore",
				"InputCore",
				"MovieSceneCaptureDialog",
                "MovieSceneCapture",
                "LevelSequence",
				"EditorFramework",
                "UnrealEd",
                "WorkspaceMenuStructure",				
				"LevelSequenceEditor",
				"DeveloperSettings",
				"MessageLog",
				"Sequencer",
				"ToolMenus",
				"EditorStyle",
				"Json",
				"JsonUtilities",
				"ScreenShotComparisonTools",
				"AutomationMessages",
				"ToolWidgets",
				"ConsoleVariablesEditor",
				"GraphEditor",
				"StructUtilsEditor",
				"ApplicationCore",
				"KismetWidgets",
				"BlueprintGraph",
				"OpenColorIO",
				"OpenColorIOEditor",
				"Imath",		// This and UEOpenExr are only required for use of UMovieGraphImageSequenceOutputNode. Ideally these are refactored
				"UEOpenExr",	// out of the ImageSequenceOutputNode header at some point so these dependencies can be removed.
				"Projects",		// For IPluginManager
				"NamingTokensUI",
				"CinematicCamera",
			}
        );

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				
			}
		);
    }
}
