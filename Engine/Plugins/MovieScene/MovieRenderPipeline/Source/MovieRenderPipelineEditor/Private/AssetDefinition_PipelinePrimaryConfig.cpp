// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PipelinePrimaryConfig.h"

#include "MoviePipelineAssetEditor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "MoviePipelineQueueSubsystem.h"
#include "IMovieRenderPipelineEditorModule.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PipelinePrimaryConfig)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_PipelinePrimaryConfig::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (AssetEditorSubsystem)
	{
		UMoviePipelineAssetEditor* AssetEditor = NewObject<UMoviePipelineAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		if (AssetEditor)
		{
			for (UMoviePipelinePrimaryConfig* PrimaryConfig : OpenArgs.LoadObjects<UMoviePipelinePrimaryConfig>())
			{
				AssetEditor->SetObjectToEdit(PrimaryConfig);
				AssetEditor->Initialize();
			}
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
