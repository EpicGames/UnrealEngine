// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelinePrimaryConfig.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_PipelinePrimaryConfig.generated.h"

UCLASS()
class UAssetDefinition_PipelinePrimaryConfig : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_PipelinePrimaryConfig", "Movie Pipeline Primary Config"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(78, 40, 165)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMoviePipelinePrimaryConfig::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Cinematics };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
