// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PipelineQueue.h"

#include "Editor.h"
#include "LevelEditor.h"
#include "MoviePipelineQueueSubsystem.h"
#include "MovieRenderPipelineEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PipelineQueue)

EAssetCommandResult UAssetDefinition_PipelineQueue::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (UMoviePipelineQueue* ValidQueue = OpenArgs.LoadFirstValid<UMoviePipelineQueue>())
	{
		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);

		if (Subsystem->LoadQueue(ValidQueue))
		{
			const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			if (const TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
			{
				LevelEditorTabManager->TryInvokeTab(FMovieRenderPipelineEditorModule::MoviePipelineQueueTabName);
			}
		}
	}

	return EAssetCommandResult::Handled;
}
