// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineQueue.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_PipelineQueue.generated.h"

UCLASS()
class UAssetDefinition_PipelineQueue : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_PipelineQueue", "Movie Pipeline Queue"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(78, 40, 165)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMoviePipelineQueue::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Cinematics };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
