// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PipelineShotConfig.h"

#include "MoviePipelineAssetEditor.h"
#include "MoviePipelineQueueSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PipelineShotConfig)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_PipelineShotConfig::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (AssetEditorSubsystem)
	{
		UMoviePipelineAssetEditor* AssetEditor = NewObject<UMoviePipelineAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		if (AssetEditor)
		{
			for (UMoviePipelineShotConfig* ShotConfig : OpenArgs.LoadObjects<UMoviePipelineShotConfig>())
			{
				AssetEditor->SetObjectToEdit(ShotConfig);
				AssetEditor->Initialize();
			}
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
