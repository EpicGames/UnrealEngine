// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineShotConfig.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_PipelineShotConfig.generated.h"

UCLASS()
class UAssetDefinition_PipelineShotConfig : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_PipelineShotConfig", "Movie Pipeline Shot Config"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 255, 255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMoviePipelineShotConfig::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Cinematics };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
