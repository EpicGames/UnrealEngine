// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Internationalization.h"
#include "Misc/DefaultValueHelper.h"
#include "MoviePipelineConsoleVariableSetting.h"
#include "PropertyCustomizationHelpers.h"
#include "Views/Widgets/SConsoleVariablesEditorCustomConsoleInputBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineEditor"

/* Customizes how console variables are displayed in the details pane. */
class FConsoleVariablesDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FConsoleVariablesDetailsCustomization>();
	}

protected:
	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		const TSharedRef<IPropertyHandle> NameProp = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMoviePipelineConsoleVariableEntry, Name)).ToSharedRef();
		const TSharedRef<IPropertyHandle> ValueProp = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMoviePipelineConsoleVariableEntry, Value)).ToSharedRef();
		const TSharedRef<IPropertyHandle> IsEnabledProp = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMoviePipelineConsoleVariableEntry, bIsEnabled)).ToSharedRef();

		// Action for deleting a cvar from the parent array
		const FExecuteAction DeleteAction = FExecuteAction::CreateLambda([InStructPropertyHandle]
		{
			const TSharedPtr<IPropertyHandleArray> ParentPropertyHandleArray = InStructPropertyHandle->GetParentHandle()->AsArray();
			const int32 ArrayIndex = InStructPropertyHandle->IsValidHandle() ? InStructPropertyHandle->GetIndexInArray() : INDEX_NONE;
			if (ParentPropertyHandleArray.IsValid() && ArrayIndex >= 0)
			{
				ParentPropertyHandleArray->DeleteItem(ArrayIndex);
			}
		});

		HeaderRow
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 2)
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SCheckBox)
				.IsChecked(this, &FConsoleVariablesDetailsCustomization::GetIsEnabledCheckBoxState, IsEnabledProp)
				.OnCheckStateChanged(this, &FConsoleVariablesDetailsCustomization::OnIsEnabledCheckBoxStateChanged, IsEnabledProp)
			]

			+ SHorizontalBox::Slot()
			.Padding(5, 2)
			.FillWidth(0.75f)
			[
				SNew(SConsoleVariablesEditorCustomConsoleInputBox)
				.IsEnabled(this, &FConsoleVariablesDetailsCustomization::IsEnabled, IsEnabledProp)
				.Font(CustomizationUtils.GetRegularFont())
				.Text(this, &FConsoleVariablesDetailsCustomization::GetConsoleVariableText, NameProp)
				.HideOnFocusLost(false)
				.ClearOnCommit(false)
				.HintText(LOCTEXT("ConsoleVariableInputHintText", "Enter Console Variable"))
				.OnTextChanged(this, &FConsoleVariablesDetailsCustomization::OnConsoleVariableNameTextChanged, NameProp)
			]

			+ SHorizontalBox::Slot()
			.Padding(0, 2)
			.FillWidth(0.25f)
			[
				SNew(SSpinBox<float>)
				.IsEnabled(this, &FConsoleVariablesDetailsCustomization::IsEnabled, IsEnabledProp)
				.MaxFractionalDigits(3)
				.Font(CustomizationUtils.GetRegularFont())
				.Value(this, &FConsoleVariablesDetailsCustomization::GetConsoleVariableValue, ValueProp)
				.OnValueChanged(this, &FConsoleVariablesDetailsCustomization::OnConsoleVariableValueChanged, ValueProp)
				.OnValueCommitted(this, &FConsoleVariablesDetailsCustomization::OnConsoleVariableValueCommitted, ValueProp)
				.ToolTipText(this, &FConsoleVariablesDetailsCustomization::GetConsoleVariableTooltip, InStructPropertyHandle)
			]

			+ SHorizontalBox::Slot()
			.Padding(5, 2)
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeInsertDeleteDuplicateButton({}, DeleteAction, {})
			]
		];
	}
	
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
	}
	//~ End IPropertyTypeCustomization interface

	/* Get a CVar settings object from a property handle. Returned pointer may be invalid if the object does not exist. */
	UMoviePipelineConsoleVariableSetting* GetCVarSettingFromHandle(const TSharedPtr<IPropertyHandle>& PropertyHandle) const
	{
		if (!PropertyHandle.IsValid() || !PropertyHandle->IsValidHandle())
		{
			return nullptr;
		}
		
		TArray<UObject *> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		if (OuterObjects.Num() != 1)
		{
			return nullptr;
		}

		// Get the outermost settings object
		return Cast<UMoviePipelineConsoleVariableSetting>(OuterObjects[0]);
	}

	/* Get a CVar entry object from a property handle. Returned pointer may be invalid if the object does not exist. */
	FMoviePipelineConsoleVariableEntry* GetEntryForHandle(const TSharedPtr<IPropertyHandle>& PropertyHandle) const
	{
		if (!PropertyHandle.IsValid() || !PropertyHandle->IsValidHandle())
		{
			return nullptr;
		}

		TArray<UObject *> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		if (OuterObjects.Num() != 1)
		{
			return nullptr;
		}

		// Get the pointer to the ConsoleVariable struct from the first outer object
		return reinterpret_cast<FMoviePipelineConsoleVariableEntry*>(
			PropertyHandle->GetValueBaseAddress(reinterpret_cast<uint8*>(OuterObjects[0]))
		);
	}

	TWeakPtr<FConsoleVariablesEditorCommandInfo> GetCommandInfoForConsoleVariableEntry(const FMoviePipelineConsoleVariableEntry* InEntry) const
	{
		FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();
		return ConsoleVariablesEditorModule.FindCommandInfoByName(InEntry->Name);
	}

	ECheckBoxState GetIsEnabledCheckBoxState(TSharedRef<IPropertyHandle> PropertyHandle) const
	{
		bool bValue = false;
		if (PropertyHandle->IsValidHandle() && (PropertyHandle->GetValue(bValue) != FPropertyAccess::Fail))
		{
			return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Unchecked;
	}

	void OnIsEnabledCheckBoxStateChanged(ECheckBoxState NewState, TSharedRef<IPropertyHandle> PropertyHandle)
	{
		if (PropertyHandle->IsValidHandle())
		{
			const bool bValue = (NewState == ECheckBoxState::Checked) ? true : false;
			PropertyHandle->SetValue(bValue);
		}
	}

	bool IsEnabled(TSharedRef<IPropertyHandle> InIsEnabledPropertyHandle) const
	{
		bool bValue = false;
		if (InIsEnabledPropertyHandle->IsValidHandle() && (InIsEnabledPropertyHandle->GetValue(bValue) != FPropertyAccess::Fail))
		{
			return bValue;
		}

		return false;
	}

	FText GetConsoleVariableText(TSharedRef<IPropertyHandle> PropertyHandle) const
	{
		FString Value;
		if (PropertyHandle->IsValidHandle() && (PropertyHandle->GetValue(Value) != FPropertyAccess::Fail))
		{
			return FText::FromString(Value);
		}

		return FText();
	}

	void OnConsoleVariableNameTextChanged(const FText& NewText, TSharedRef<IPropertyHandle> PropertyHandle)
	{
		if (PropertyHandle->IsValidHandle())
		{
			PropertyHandle->SetValue(NewText.ToString());
		}
	}

	float GetConsoleVariableValue(TSharedRef<IPropertyHandle> PropertyHandle) const
	{
		// Get the handle to the parent struct property
		const TSharedPtr<IPropertyHandle> ParentPropertyHandle = PropertyHandle->GetParentHandle();
		
		const FMoviePipelineConsoleVariableEntry* CVarEntry = GetEntryForHandle(ParentPropertyHandle);
		if (!CVarEntry)
		{
			return 0.f;
		}

		// If this cvar override is disabled, find what its value would be as if this override did not exist
		if (!CVarEntry->bIsEnabled)
		{
			// Get the non-override value of this cvar
			const UMoviePipelineConsoleVariableSetting* CVarSetting = GetCVarSettingFromHandle(ParentPropertyHandle);
			float CVarFloatValue = 0.f;

			// If we are in a settings object then use that to get the disabled value
			if (CVarSetting)
			{
				FDefaultValueHelper::ParseFloat(CVarSetting->ResolveDisabledValue(*CVarEntry), CVarFloatValue);
			}
			// Fall back to the startup value of the cvar if we're not in a settings object
			else
			{
				const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = GetCommandInfoForConsoleVariableEntry(CVarEntry).Pin();
				if (CommandInfo.IsValid())
				{
					FDefaultValueHelper::ParseFloat(CommandInfo->StartupValueAsString, CVarFloatValue);
				}
			}
			return CVarFloatValue;
		}
		
		float Value;
		if (PropertyHandle->IsValidHandle() && (PropertyHandle->GetValue(Value) != FPropertyAccess::Fail))
		{
			return Value;
		}

		return 0.f;
	}

	void OnConsoleVariableValueChanged(float Value, TSharedRef<IPropertyHandle> PropertyHandle)
	{
		if (PropertyHandle->IsValidHandle())
		{
			PropertyHandle->SetValue(Value);
		}
	}

	void OnConsoleVariableValueCommitted(float Value, ETextCommit::Type CommitType, TSharedRef<IPropertyHandle> PropertyHandle)
	{
		OnConsoleVariableValueChanged(Value, PropertyHandle);
	}

	FText GetConsoleVariableTooltip(TSharedRef<IPropertyHandle> StructPropertyHandle) const
	{
		static const FText ToolTipFormatText = LOCTEXT("CVarToolTipFormatText", "Custom Value: {0}\nPreset Value: {1}\nStartup Value: {2} (Set By {3})");
		static const FText NoPresetValueText = LOCTEXT("NoPresetValue", "None");
		
		const UMoviePipelineConsoleVariableSetting* Setting = GetCVarSettingFromHandle(StructPropertyHandle);
		const FMoviePipelineConsoleVariableEntry* CVarEntry = GetEntryForHandle(StructPropertyHandle);
		if (!CVarEntry)
		{
			return FText();
		}
		
		const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = GetCommandInfoForConsoleVariableEntry(CVarEntry).Pin();
		if (!CommandInfo.IsValid())
		{
			return FText();
		}
		
		const FString CustomValue = FString::SanitizeFloat(CVarEntry->Value);
		const FString PresetValue = Setting ? Setting->ResolvePresetValue(CVarEntry->Name) : FString();
		const FString StartupValue = CommandInfo->StartupValueAsString;
		
		return FText::Format(ToolTipFormatText,
			FText::FromString(CustomValue),
			PresetValue.IsEmpty() ? NoPresetValueText : FText::FromString(PresetValue),
			FText::FromString(StartupValue),
			FConsoleVariablesEditorCommandInfo::ConvertConsoleVariableSetByFlagToText(CommandInfo->StartupSource));
	}
};

#undef LOCTEXT_NAMESPACE