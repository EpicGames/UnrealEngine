// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailGroup.h"
#include "MoviePipelineConsoleVariableSetting.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Sections/MovieSceneConsoleVariableTrackInterface.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineEditor"

/** Delegate that gets the menu widget associated with a specific cvar. */
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FGetCVarMenu, const FString&);

/** Customize how properties in UMoviePipelineConsoleVariableSetting appear in the details panel. */
class FConsoleVariablesSettingDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FConsoleVariablesSettingDetailsCustomization>();
	}

	static void AddConsoleVariablePresetRowsToGroup(const TScriptInterface<IMovieSceneConsoleVariableTrackInterface>& InCVarPreset, IDetailGroup& InGroup, IDetailLayoutBuilder* InDetailBuilder, const TAttribute<bool>& InRowEnabledAttr, const FGetCVarMenu& InGetCVarMenuDelegate)
	{
		static const FText ConsoleVariableDisabledText = LOCTEXT("DisabledConsoleVariable", "This console variable is disabled.");
		
		constexpr bool bOnlyIncludeChecked = false;
		TArray<TTuple<FString, FString>> CVars;
		InCVarPreset->GetConsoleVariablesForTrack(bOnlyIncludeChecked, CVars);

		// Show every console variable that's included in this preset; each console variable gets its own row
		for (const TTuple<FString, FString>& CVar : CVars)
		{
			InGroup.AddWidgetRow()
			.IsEnabled(InRowEnabledAttr)
			.WholeRowContent()
			[
				SNew(SHorizontalBox)
        		
				+ SHorizontalBox::Slot()
				.Padding(5, 0)
				.FillWidth(0.75f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.IsEnabled(InCVarPreset->IsConsoleVariableEnabled(CVar.Key))
					.Font(InDetailBuilder->GetDetailFont())
					.Text(FText::FromString(CVar.Key))
					.ToolTipText(!InCVarPreset->IsConsoleVariableEnabled(CVar.Key) ? ConsoleVariableDisabledText : FText::GetEmpty())
				]
        		
				+ SHorizontalBox::Slot()
				.Padding(0, 0)
				.FillWidth(0.25f)
				.VAlign(VAlign_Center)
				[
					SNew(SEditableTextBox)
					.IsEnabled(false)
					.Font(InDetailBuilder->GetDetailFont())
					.Text(FText::FromString(CVar.Value))
				]
        		
				+ SHorizontalBox::Slot()
				.Padding(5, 0)
				.AutoWidth()
				[
					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
					.ForegroundColor(FSlateColor::UseForeground())
					.HasDownArrow(true)
					.MenuContent()
					[
						InGetCVarMenuDelegate.Execute(CVar.Key)
					]
				]
			];
		}
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		IDetailCategoryBuilder& SettingsCategory = DetailBuilder.EditCategory(TEXT("Settings"));
		const TSharedRef<IPropertyHandle> ConsoleVariablePresetsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMoviePipelineConsoleVariableSetting, ConsoleVariablePresets));

		// Customize how the console variable presets array looks. Each preset is shown as a group, and that group's children
		// are the console variables contained within the preset.
		const TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(ConsoleVariablePresetsHandle));
		PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FConsoleVariablesSettingDetailsCustomization::GeneratePresetGroup, &DetailBuilder));
		SettingsCategory.AddCustomBuilder(PropertyBuilder, false);

		// Regenerate the preset details when the preset array is updated
		ConsoleVariablePresetsHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailBuilder]()
		{
			DetailBuilder.ForceRefreshDetails();
		}));
	}
	//~ End IDetailCustomization interface

	UMoviePipelineConsoleVariableSetting* GetCVarSettingFromHandle(const TSharedPtr<IPropertyHandle>& PropertyHandle) const
	{
		if (!PropertyHandle.IsValid() || !PropertyHandle->IsValidHandle())
		{
			return nullptr;
		}
		
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		if (OuterObjects.Num() != 1)
		{
			return nullptr;
		}

		// Get the outermost cvar setting object
		return Cast<UMoviePipelineConsoleVariableSetting>(OuterObjects[0]);
	}

	void GeneratePresetGroup(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout)
	{
		static const FText ConsoleVariableOverrideText = LOCTEXT("CreateConsoleVariableOverride", "Override Console Variable");
		static const FText ConsoleVariableDisabledText = LOCTEXT("DisabledConsoleVariable", "This console variable is disabled.");

		// Add the preset asset chooser as the group header
		FStringView PropName = ElementProperty->GetPropertyPath();
		IDetailGroup& Group = ChildrenBuilder.AddGroup(FName(PropName), FText::FromStringView(PropName));
		Group.HeaderRow()
		[
			ElementProperty->CreatePropertyValueWidget()
		];

		UMoviePipelineConsoleVariableSetting* CVarSetting = GetCVarSettingFromHandle(ElementProperty);
		if (!CVarSetting)
		{
			return;
		}

		const TArray<TScriptInterface<IMovieSceneConsoleVariableTrackInterface>>& CVarPresets = CVarSetting->ConsoleVariablePresets;
		if (!CVarPresets.IsValidIndex(ElementIndex))
		{
			return;
		}

		const TScriptInterface<IMovieSceneConsoleVariableTrackInterface>& CVarPreset = CVarPresets[ElementIndex];
		if (!CVarPreset)
		{
			return;
		}

		constexpr bool bRowEnabledAttr = true;
		AddConsoleVariablePresetRowsToGroup(CVarPreset, Group, DetailLayout, bRowEnabledAttr, FGetCVarMenu::CreateLambda([CVarSetting](const FString& InCVarName)
		{
			// Each console variable gets a menu that allows the user to create an override outside of the preset
			FMenuBuilder CreateOverrideMenu(true, nullptr, nullptr, true);
			const FUIAction AddOverrideAction(
				FExecuteAction::CreateLambda([InCVarName, CVarSetting]()
				{
					CVarSetting->AddConsoleVariable(InCVarName, 0.f);
				})
			);
			CreateOverrideMenu.AddMenuEntry(ConsoleVariableOverrideText, FText::GetEmpty(), FSlateIcon(), AddOverrideAction);

			return CreateOverrideMenu.MakeWidget();
		}));
	}
};

#undef LOCTEXT_NAMESPACE