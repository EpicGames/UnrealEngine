// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/ConsoleVariableSettingCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Graph/MovieEdGraphNode.h"
#include "Graph/MovieGraphSchema.h"
#include "Graph/Nodes/MovieGraphApplyCVarPresetNode.h"
#include "Graph/Nodes/MovieGraphSetCVarValueNode.h"
#include "IDetailCustomization.h"
#include "IDetailGroup.h"
#include "PropertyHandle.h"
#include "Sections/MovieSceneConsoleVariableTrackInterface.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customize how the Apply Console Variable Preset node appears in the details panel. */
class FMovieGraphApplyCVarPresetNodeCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphApplyCVarPresetNodeCustomization>();
	}

protected:
	/** Creates a new Set CVar Value node (overriding the cvar with the provided name) connected to the given preset node. */
	static void CreateOverrideViaSetCVarValueNode(const TWeakObjectPtr<UMovieGraphApplyCVarPresetNode> InPresetNode, const FString& InCVarName)
	{
		// Putting logic like this into a customization is a little awkward. However, it seems like it's still an OK choice after considering the
		// alternatives. Putting the logic in a new ed graph node would require instantiating a new ed node class JUST to get this behavior. Doesn't
		// belong in the schema because it's too use-specific. Also doesn't belong on the non-ed graph node because Core cannot access Editor.
		
		if (!InPresetNode.IsValid())
		{
			return;
		}
		
		// When creating the new action, since it's only being used to create a node, the category, display name, and tooltip can just be empty
		constexpr int32 Grouping = 0;
		const FText Keywords = InPresetNode->GetKeywords();
		const TSharedPtr<FMovieGraphSchemaAction_NewNode> NewNodeAction =
			MakeShared<FMovieGraphSchemaAction_NewNode>(FText::GetEmpty(), FText::GetEmpty(), FText::GetEmpty(), Grouping, Keywords);
		NewNodeAction->NodeClass = UMovieGraphSetCVarValueNode::StaticClass();

		// Put the new node to the right of this node
		const TObjectPtr<UEdGraphNode> GraphNode = InPresetNode->GraphNode;
		const FVector2f NewLocation(GraphNode->NodePosX + 300.0f, GraphNode->NodePosY);

		// Create the new node (providing FromPin will trigger the action to connect the new node and the preset node)
		UEdGraphPin* FromPin = GraphNode->FindPin(NAME_None, EGPD_Output);
		UEdGraphNode* NewNode = NewNodeAction->PerformAction(GraphNode->GetGraph(), FromPin, NewLocation);

		// Set the node to use the correct cvar
		UMovieGraphNode* RuntimeNode = Cast<UMoviePipelineEdGraphNode>(NewNode)->GetRuntimeNode();
		UMovieGraphSetCVarValueNode* SetCVarNode = Cast<UMovieGraphSetCVarValueNode>(RuntimeNode);
		SetCVarNode->Name = InCVarName;

		// Populate the default value of the override from the cvar in the preset
		if (const TScriptInterface<IMovieSceneConsoleVariableTrackInterface>& PresetAsset = InPresetNode->ConsoleVariablePreset)
		{
			constexpr bool bOnlyIncludeChecked = false;
			TArray<TTuple<FString, FString>> ConsoleVariables;
			PresetAsset->GetConsoleVariablesForTrack(bOnlyIncludeChecked, ConsoleVariables);

			const TTuple<FString, FString>* CVarPair =
				ConsoleVariables.FindByPredicate([&InCVarName](const TTuple<FString, FString>& InPair) { return InPair.Key == InCVarName; });
			if (CVarPair)
			{
				LexFromString(SetCVarNode->Value, *CVarPair->Value);
			}
		}

		// Mark the properties as overridden so they are used in evaluation
		SetCVarNode->bOverride_Name = true;
		SetCVarNode->bOverride_Value = true;
	}

	/** Creates the override menu for a specific cvar row. */
	static TSharedRef<SWidget> CreateCVarOverrideMenu(const FString& InCVarName, TWeakObjectPtr<UMovieGraphApplyCVarPresetNode> InPresetNode)
	{
		static const FText OverrideWithNewNodeText = LOCTEXT("OverrideWithSetConsoleVariableNode", "Override with Set Console Variable Node");
		
		FMenuBuilder CVarMenu(true, nullptr, nullptr, true);
		const FUIAction AddOverrideAction(
			FExecuteAction::CreateLambda([InPresetNode, InCVarName]()
			{
				if (InPresetNode.IsValid())
				{
					CreateOverrideViaSetCVarValueNode(InPresetNode, InCVarName);
				}
			})
		);
		CVarMenu.AddMenuEntry(OverrideWithNewNodeText, FText::GetEmpty(), FSlateIcon(), AddOverrideAction);

		return CVarMenu.MakeWidget();
	}

	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		for (const TWeakObjectPtr<UMovieGraphApplyCVarPresetNode>& ApplyPresetNode : DetailBuilder.GetObjectsOfTypeBeingCustomized<UMovieGraphApplyCVarPresetNode>())
		{
			IDetailCategoryBuilder& SettingsCategory = DetailBuilder.EditCategory(TEXT("Settings"));
			const TSharedRef<IPropertyHandle> ConsoleVariablePresetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphApplyCVarPresetNode, ConsoleVariablePreset));

			// Regenerate the preset details when the preset is updated
			ConsoleVariablePresetHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailBuilder]()
			{
				DetailBuilder.ForceRefreshDetails();
			}));

			// Don't show the dynamic properties (which hold values for cvars which are exposed via pins). For some reason hiding the individual
			// property doesn't work, so the entire category is hidden instead.
			DetailBuilder.HideCategory("Properties");

			// Hide the default preset property since we'll be making a group that contains it instead (and the group's rows will be the individual
			// console variables within the preset)
			ConsoleVariablePresetHandle->MarkHiddenByCustomization();

			const FStringView PresetPropName = ConsoleVariablePresetHandle->GetPropertyPath();
			IDetailGroup& PresetGroup = SettingsCategory.AddGroup(FName(PresetPropName), FText::FromStringView(PresetPropName));

			// Since we're making a group to replace the default property for the preset, the EditCondition checkbox won't come along for the ride for
			// free. Supplying an attr which provides the EditCondition is required.
			TAttribute<bool> PresetEnabledAttr = TAttribute<bool>::Create([ApplyPresetNode]()
			{
				return ApplyPresetNode.IsValid() ? ApplyPresetNode->bOverride_ConsoleVariablePreset > 0 : false;
			});

			// The group's header should look like the default preset property w/ the asset picker widget
			PresetGroup.HeaderRow()
			.FilterString(ConsoleVariablePresetHandle->GetPropertyDisplayName())
			.PropertyHandleList({ConsoleVariablePresetHandle})
			.ShouldAutoExpand(true)
			.EditCondition(
				PresetEnabledAttr,
				FOnBooleanValueChanged::CreateLambda([ApplyPresetNode, &DetailBuilder](const bool bNewValue)
				{
					if (ApplyPresetNode.IsValid())
					{
						FScopedTransaction Transaction(LOCTEXT("PresetEditConditionChanged", "Edit Condition Changed"));
						ApplyPresetNode->Modify();
						ApplyPresetNode->bOverride_ConsoleVariablePreset = bNewValue;

						// This is kinda hack-ish. The details panel doesn't fully refresh in some circumstances when the edit condition is toggled.
						// However, when set the normal way, a refresh seems to be triggered anyway, so maybe there's no better way to accomplish this.
						// Using the IPropertyHandle or FBoolProperty to change the value does not result in different behavior.
						DetailBuilder.ForceRefreshDetails();
					}
				})
			)
			.NameContent()
			[
				ConsoleVariablePresetHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				ConsoleVariablePresetHandle->CreatePropertyValueWidget()
			];

			// Add all of the cvars as rows under the preset group
			if (TScriptInterface<IMovieSceneConsoleVariableTrackInterface> CVarPreset = ApplyPresetNode->ConsoleVariablePreset)
			{
				FConsoleVariablesSettingDetailsCustomization::AddConsoleVariablePresetRowsToGroup(CVarPreset, PresetGroup, &DetailBuilder, PresetEnabledAttr,
					FGetCVarMenu::CreateStatic(&CreateCVarOverrideMenu, ApplyPresetNode));
			}
		}
	}
	//~ End IDetailCustomization interface
};

#undef LOCTEXT_NAMESPACE