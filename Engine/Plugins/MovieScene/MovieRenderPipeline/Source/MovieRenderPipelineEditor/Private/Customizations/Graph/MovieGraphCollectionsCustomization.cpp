// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphCollectionsCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "Graph/Nodes/MovieGraphCollectionNode.h"
#include "IDetailChildrenBuilder.h"
#include "MovieRenderPipelineStyle.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SPositiveActionButton.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditorCollectionCustomization"

FMovieGraphCollectionDragDropOp::FMovieGraphCollectionDragDropOp(const bool bIsConditionGroup, const int32 InitialIndex, TWeakObjectPtr<UMovieGraphConditionGroup> InWeakOwningConditionGroup)
	: bIsConditionGroup(bIsConditionGroup)
	, InitialIndex(InitialIndex)
	, OwningConditionGroup(InWeakOwningConditionGroup)
{
	MouseCursor = EMouseCursor::GrabHandClosed;
}

void FMovieGraphCollectionDragDropOp::Init()
{
	SetValidTarget(false);
	SetupDefaults();
	Construct();
}

void FMovieGraphCollectionDragDropOp::SetValidTarget(const bool bIsValidTarget)
{
	if (bIsValidTarget)
	{
		if (bIsConditionGroup)
		{
			CurrentHoverText = LOCTEXT("DragConditionGroup", "Move 1 condition group here");
		}
		else
		{
			CurrentHoverText = LOCTEXT("DragConditionGroupQuery", "Move 1 condition group query here");
		}
		
		CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.OK");
	}
	else
	{
		if (bIsConditionGroup)
		{
			CurrentHoverText = LOCTEXT("DragConditionGroup_Invalid", "Cannot move condition group here");
		}
		else
		{
			CurrentHoverText = LOCTEXT("DragConditionGroupQuery_Invalid", "Cannot move condition group query here");
		}
		
		CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.Error");
	}
}

int32 FMovieGraphCollectionDragDropOp::GetIndex() const
{
	return InitialIndex;
}

bool FMovieGraphCollectionDragDropOp::IsConditionGroup() const
{
	return bIsConditionGroup;
}

TWeakObjectPtr<UMovieGraphConditionGroup> FMovieGraphCollectionDragDropOp::GetOwningConditionGroup() const
{
	return OwningConditionGroup;
}


FMovieGraphCollectionDragDropHandler::FMovieGraphCollectionDragDropHandler(const bool bIsConditionGroup, IDetailLayoutBuilder* InDetailBuilder, UMovieGraphCollection* InCollection, const int32 ConditionGroupIndex, TWeakObjectPtr<UMovieGraphConditionGroup> InWeakConditionGroup, const int32 ConditionGroupQueryIndex)
	: bIsConditionGroup(bIsConditionGroup)
	, DetailLayoutBuilder(InDetailBuilder)
	, WeakCollection(InCollection)
	, ConditionGroupIndex(ConditionGroupIndex)
	, WeakConditionGroup(InWeakConditionGroup)
	, ConditionGroupQueryIndex(ConditionGroupQueryIndex)
{
}

TSharedPtr<FDragDropOperation> FMovieGraphCollectionDragDropHandler::CreateDragDropOperation() const
{
	TSharedPtr<FMovieGraphCollectionDragDropOp> DragOp = MakeShared<FMovieGraphCollectionDragDropOp>(bIsConditionGroup, bIsConditionGroup ? ConditionGroupIndex : ConditionGroupQueryIndex, WeakConditionGroup);
	DragOp->Init();
	return DragOp;
}

bool FMovieGraphCollectionDragDropHandler::AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const
{
	const TSharedPtr<FMovieGraphCollectionDragDropOp> DragOp = DragDropEvent.GetOperationAs<FMovieGraphCollectionDragDropOp>();
	if (!DragOp.IsValid() || DropZone == EItemDropZone::OntoItem)
	{
		return false;
	}

	// The index of the condition group or query being dragged
	const int32 SourceIndex = DragOp->GetIndex();
	
	if (bIsConditionGroup && WeakCollection.IsValid())
	{
		const TArray<UMovieGraphConditionGroup*>& ConditionGroups = WeakCollection->GetConditionGroups();
		if (!ConditionGroups.IsValidIndex(SourceIndex))
		{
			return false;
		}

		UMovieGraphConditionGroup* TargetConditionGroup = ConditionGroups[SourceIndex]; 

		FScopedTransaction Transaction(LOCTEXT("ReorderConditionGroup", "Reorder Condition Group"));
		const int32 NewIndex = DropZone == EItemDropZone::AboveItem ? ConditionGroupIndex : ConditionGroupIndex + 1;
		WeakCollection->MoveConditionGroupToIndex(TargetConditionGroup, NewIndex);
	}
	else if (!bIsConditionGroup && WeakConditionGroup.IsValid())
	{
		const TArray<UMovieGraphConditionGroupQueryBase*>& ConditionGroupQueries = WeakConditionGroup->GetQueries();
		if (!ConditionGroupQueries.IsValidIndex(SourceIndex))
		{
			return false;
		}

		UMovieGraphConditionGroupQueryBase* TargetQuery = ConditionGroupQueries[SourceIndex]; 

		FScopedTransaction Transaction(LOCTEXT("ReorderConditionGroupQuery", "Reorder Condition Group Query"));
		const int32 NewIndex = DropZone == EItemDropZone::AboveItem ? ConditionGroupQueryIndex : ConditionGroupQueryIndex + 1;
		WeakConditionGroup->MoveQueryToIndex(TargetQuery, NewIndex);
	}

	DetailLayoutBuilder->ForceRefreshDetails();
	return true;
}

TOptional<EItemDropZone> FMovieGraphCollectionDragDropHandler::CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const
{
	TOptional<EItemDropZone> ReturnedDropZone;
	const TSharedPtr<FMovieGraphCollectionDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FMovieGraphCollectionDragDropOp>();

	// Can't drag an element onto another element -- only changing order is allowed (above/below)
	if (!DragDropOp.IsValid() || (DropZone == EItemDropZone::OntoItem))
	{
		return ReturnedDropZone;
	}

	if (bIsConditionGroup)
	{
		// Can't drop a condition group into a query
		const bool bIsTargetValid = DragDropOp->IsConditionGroup();
		
		if (bIsTargetValid && WeakCollection.IsValid() && WeakCollection->GetConditionGroups().IsValidIndex(ConditionGroupIndex))
		{
			ReturnedDropZone = DropZone;
			DragDropOp->SetValidTarget(true);
		}
	}
	else
	{
		// Can't drop a query into a condition group, and can't drop a query into a condition group that didn't originally own it
		const bool bIsTargetValid = !DragDropOp->IsConditionGroup() && (WeakConditionGroup == DragDropOp->GetOwningConditionGroup());
		
		if (bIsTargetValid && WeakConditionGroup.IsValid() && WeakConditionGroup->GetQueries().IsValidIndex(ConditionGroupQueryIndex))
		{
			ReturnedDropZone = DropZone;
			DragDropOp->SetValidTarget(true);
		}
	}

	return ReturnedDropZone;
}

FMovieGraphConditionGroupQueryBuilder::FMovieGraphConditionGroupQueryBuilder(IDetailLayoutBuilder* InDetailBuilder, TSharedRef<IPropertyHandle> InConditionGroupQueryProperty, int32 ConditionGroupQueryIndex, const TWeakObjectPtr<UMovieGraphConditionGroup>& InWeakConditionGroup)
	: DetailLayoutBuilder(InDetailBuilder)
	, ConditionGroupQueryProperty(InConditionGroupQueryProperty)
	, ConditionGroupQueryIndex(ConditionGroupQueryIndex)
	, WeakConditionGroup(InWeakConditionGroup)
{
}

FName FMovieGraphConditionGroupQueryBuilder::GetName() const
{
	// Name doesn't matter here because expansion state doesn't need to be persisted (this node does not have any children nested under it)
	return NAME_None;
}

void FMovieGraphConditionGroupQueryBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{

}

void FMovieGraphConditionGroupQueryBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (!WeakConditionGroup.IsValid())
	{
		return;
	}
	
	const TArray<UMovieGraphConditionGroupQueryBase*>& Queries = WeakConditionGroup->GetQueries();
	
	const TSharedPtr<IPropertyHandleArray> QueriesArrayProperty = ConditionGroupQueryProperty->AsArray();
	check(QueriesArrayProperty);
	
	uint32 NumQueries = 0;
	QueriesArrayProperty->GetNumElements(NumQueries);
	
	for (uint32 QueryIndex = 0; QueryIndex < NumQueries; ++QueryIndex)
	{
		const TSharedRef<IPropertyHandle> ElementHandle = QueriesArrayProperty->GetElement(QueryIndex);
		check(ElementHandle->IsValidHandle());
		check(Queries.IsValidIndex(QueryIndex));

		UMovieGraphConditionGroupQueryBase* Query = Queries[QueryIndex];
		TWeakObjectPtr<UMovieGraphConditionGroupQueryBase> WeakQuery = MakeWeakObjectPtr(Query);

		// Drag-drop handler for condition group queries doesn't provide condition group information
		TWeakObjectPtr<UMovieGraphConditionGroup> InvalidConditionGroup;
		constexpr int32 InvalidConditionGroupIndex = 0;
		constexpr UMovieGraphCollection* InvalidCollection = nullptr;
		constexpr bool bIsConditionGroup = false;

		ChildrenBuilder.AddProperty(ElementHandle)
		.DragDropHandler(MakeShared<FMovieGraphCollectionDragDropHandler>(bIsConditionGroup, DetailLayoutBuilder, InvalidCollection, InvalidConditionGroupIndex, WeakConditionGroup, QueryIndex))
		.ShowPropertyButtons(false)
		.OverrideResetToDefault(FResetToDefaultOverride::Hide())
		.CustomWidget()
		.AddCustomContextMenuAction(FUIAction(
			FExecuteAction::CreateLambda([this, QueryIndex]()
			{
				if (WeakConditionGroup.IsValid())
				{
					const FScopedTransaction Transaction(LOCTEXT("DuplicateConditionGroupQuery_Transaction", "Duplicate Condition Group Query"));
					WeakConditionGroup->DuplicateQuery(QueryIndex);
				}
			})),
			LOCTEXT("DuplicateConditionGroupQuery", "Duplicate"),
			LOCTEXT("DuplicateConditionGroupQueryTooltip", "Duplicate this condition."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"))
		.FilterString(Query->GetClass()->GetDisplayNameText())
		.NameContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 4.f, 0)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([WeakQuery]()
				{
					return WeakQuery.IsValid() && WeakQuery->IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([WeakQuery](ECheckBoxState NewCheckState)
				{
					if (WeakQuery.IsValid())
					{
						WeakQuery->SetEnabled(NewCheckState == ECheckBoxState::Checked);
					}
				})
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.Padding(0, 0, 7.f, 0)
			[
				SNew(SMovieGraphCollectionTreeQueryTypeSelectorWidget)
				.WeakConditionGroup(WeakConditionGroup)
				.WeakQuery(WeakQuery)
				.OnQueryTypeChanged_Lambda([this](TWeakObjectPtr<UMovieGraphConditionGroupQueryBase> NewQuery)
				{
					// The other widgets will be referring to an outdated query object (since changing the type generates a completely new
					// query), so the UI needs to be refreshed
					DetailLayoutBuilder->ForceRefreshDetails();
				})
			]
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(4.f, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SMovieGraphCollectionTreeAddQueryContentWidget)
				.WeakQuery(WeakQuery)
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SComboButton)
				.IsEnabled_Lambda([WeakQuery]()
				{
					return WeakQuery.IsValid() && WeakQuery->IsEnabled();
				})
				.ComboButtonStyle(FMovieRenderPipelineStyle::Get(), "MovieRenderGraph.CollectionsTree.SmallComboButton")
				.ContentPadding(0)
				.ForegroundColor(FSlateColor::UseForeground())
				.HasDownArrow(true)
				.OnGetMenuContent_Lambda([this, WeakQuery]()
				{
					FMenuBuilder ViewOptions(true, nullptr);

					ViewOptions.AddMenuEntry(
						LOCTEXT("DeleteConditionGroupQuery", "Delete"),
						LOCTEXT("DeleteConditionGroupQueryTooltip", "Delete this condition group query."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, WeakQuery]()
							{
								if (WeakConditionGroup.IsValid() && WeakQuery.IsValid())
								{
									const FScopedTransaction Transaction(LOCTEXT("RemoveConditionGroupQuery", "Remove Condition Group Query"));
									WeakConditionGroup->RemoveQuery(WeakQuery.Get());
								}
							}),
							FCanExecuteAction()));

					return ViewOptions.MakeWidget();
				})
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(2.f)
			.FillWidth(1.f)
			[
				SNew(SMovieGraphCollectionTreeOpTypeWidget)
				.IsConditionGroup(false)
				.WeakConditionGroupQuery(WeakQuery)
			]
		];
		
		// Add the widgetry that allows the query's properties to be updated
		AddQueryTypeWidgets(WeakQuery, ChildrenBuilder);
	}

	// Refresh details when the number of queries changes
	IDetailCategoryBuilder& ParentCategory = ChildrenBuilder.GetParentCategory();
	QueriesArrayProperty->SetOnNumElementsChanged(FSimpleDelegate::CreateSPLambda(this, [&ParentCategory]()
	{
		ParentCategory.GetParentLayout().ForceRefreshDetails();
	}));
}

void FMovieGraphConditionGroupQueryBuilder::AddQueryTypeWidgets(TWeakObjectPtr<UMovieGraphConditionGroupQueryBase> WeakQuery, IDetailChildrenBuilder& ChildrenBuilder) const
{
	if (!WeakQuery.IsValid())
	{
		return;
	}

	// If the query defines widgets to display, use those
	const TArray<TSharedRef<SWidget>> QueryWidgets = WeakQuery->GetWidgets();
	if (!QueryWidgets.IsEmpty())
	{
		TSharedPtr<SVerticalBox> WidgetContainer;

		ChildrenBuilder.AddCustomRow(FText())
		.WholeRowContent()
		[
			SAssignNew(WidgetContainer, SVerticalBox)
		];

		// Add all widgets that are generated by the column
		for (const TSharedRef<SWidget>& Widget : WeakQuery->GetWidgets())
		{
			WidgetContainer->AddSlot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.IsEnabled_Lambda([WeakQuery]()
					{
						return WeakQuery.IsValid() && WeakQuery->IsEnabled();
					})
					[
						Widget
					]
				];
		}

		return;
	}

	// If the query does NOT define widgets to display, just display the properties that the query exposes
	for (TFieldIterator<FProperty> PropIt(WeakQuery->GetClass()); PropIt; ++PropIt)
	{
		const FProperty* QueryProperty = *PropIt;
		if (!QueryProperty->HasAllPropertyFlags(CPF_Edit))
		{
			continue;
		}
		
		// Get the property handle for the property within the query. This is needed in order to get the name/value widgets for it.
		TSharedPtr<IPropertyHandle> QueryPropertyHandle =
			DetailLayoutBuilder->AddObjectPropertyData({WeakQuery.Get()}, QueryProperty->GetFName());

		if (!QueryPropertyHandle || !QueryPropertyHandle.IsValid())
		{
			continue;
		}

		ChildrenBuilder.AddProperty(QueryPropertyHandle.ToSharedRef())
		.CustomWidget()
		.NameContent()
		[
			SNew(SBox)
			.Padding(0)
			.IsEnabled_Lambda([WeakQuery]()
			{
				return WeakQuery.IsValid() && WeakQuery->IsEnabled();
			})
			[
				QueryPropertyHandle->CreatePropertyNameWidget()
			]
		]
		.ValueContent()
		[
			SNew(SBox)
			.Padding(5.f, 2.f)
			.IsEnabled_Lambda([WeakQuery]()
			{
				return WeakQuery.IsValid() && WeakQuery->IsEnabled();
			})
			[
				QueryPropertyHandle->CreatePropertyValueWidget()
			]
		];
	}
}

FMovieGraphConditionGroupBuilder::FMovieGraphConditionGroupBuilder(IDetailLayoutBuilder* InDetailBuilder, TSharedRef<IPropertyHandle> InConditionGroupProperty, const uint32 InConditionGroupIndex, TWeakObjectPtr<UMovieGraphCollection>& InWeakCollection)
	: DetailLayoutBuilder(InDetailBuilder)
	, ConditionGroupProperty(InConditionGroupProperty)
	, ConditionGroupIndex(InConditionGroupIndex)
	, WeakCollection(InWeakCollection)
{
}

FName FMovieGraphConditionGroupBuilder::GetName() const
{
	// NAME_None prevents the details panel from restoring (absent) saved expansion state, which would collapse new nodes.
	// This ensures InitiallyCollapsed() == false is respected and condition groups start expanded.
	return NAME_None;
}

void FMovieGraphConditionGroupBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	// Generate the condition group name
	const FString GroupName = FString::Printf(TEXT("Condition Group %i"), (ConditionGroupIndex + 1));

	// Drag-drop handler for condition groups doesn't provide query information
	TWeakObjectPtr<UMovieGraphConditionGroup> InvalidQuery;
	constexpr int32 InvalidQueryIndex = 0;
	constexpr bool bIsConditionGroup = true;
	
	NodeRow
	.DragDropHandler(MakeShared<FMovieGraphCollectionDragDropHandler>(bIsConditionGroup, DetailLayoutBuilder, WeakCollection.Get(), ConditionGroupIndex, InvalidQuery, InvalidQueryIndex))
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.FilterString(FText::FromString(GroupName))
	.PropertyHandleList({ConditionGroupProperty})
	.NameContent()
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0, 0, 2.f, 0)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("ClassIcon.GroupActor"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0, 0, 2.f, 0)
		[
			SNew(STextBlock)
			.Text(FText::FromString(GroupName))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
		]
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SHorizontalBox)
		
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.ToolTip(SNew(SToolTip).Text(LOCTEXT("AddConditionGroupQuery_Tooltip", "Add a condition to this condition group.")))
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked_Lambda([this]()
			{
				if (WeakCollection.IsValid())
				{
					const TArray<UMovieGraphConditionGroup*>& ConditionGroups = WeakCollection->GetConditionGroups();
					if (ConditionGroups.IsValidIndex(ConditionGroupIndex))
					{
						const FScopedTransaction Transaction(LOCTEXT("AddConditionGroupQuery", "Add Condition Group Query"));
						
						// Use the Actor Name query as the default
						ConditionGroups[ConditionGroupIndex]->AddQuery(UMovieGraphConditionGroupQuery_ActorName::StaticClass());
					}
				}

				return FReply::Handled();
			})
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SNew(SComboButton)
			.ComboButtonStyle(FMovieRenderPipelineStyle::Get(), "MovieRenderGraph.CollectionsTree.SmallComboButton")
			.ContentPadding(0)
			.ForegroundColor(FSlateColor::UseForeground())
			.HasDownArrow(true)
			.OnGetMenuContent_Lambda([this]()
			{
				FMenuBuilder ViewOptions(true, nullptr);

				ViewOptions.AddMenuEntry(
					LOCTEXT("DeleteConditionGroup", "Delete"),
					LOCTEXT("DeleteConditionGroupTooltip", "Delete this condition group and the queries in it."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this]()
						{
							if (WeakCollection.IsValid())
							{
								const TArray<UMovieGraphConditionGroup*>& ConditionGroups = WeakCollection->GetConditionGroups();
								if (ConditionGroups.IsValidIndex(ConditionGroupIndex))
								{
									const FScopedTransaction Transaction(LOCTEXT("RemoveConditionGroup", "Remove Condition Group"));
									
									WeakCollection->RemoveConditionGroup(ConditionGroups[ConditionGroupIndex]);
								}
							}
                        }),
						FCanExecuteAction()));

				return ViewOptions.MakeWidget();
			})
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(2.f)
		.FillWidth(1.f)
		[
			SNew(SMovieGraphCollectionTreeOpTypeWidget)
			.IsConditionGroup(true)
			.WeakConditionGroup(WeakCollection->GetConditionGroups()[ConditionGroupIndex])
		]
	];
}

void FMovieGraphConditionGroupBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	UMovieGraphConditionGroup* ConditionGroup = WeakCollection.Get()->GetConditionGroups()[ConditionGroupIndex];
	const TSharedPtr<IPropertyHandle> QueriesProperty = DetailLayoutBuilder->AddObjectPropertyData({ConditionGroup}, FName(TEXT("Queries")));
	
	ChildrenBuilder.AddCustomBuilder(
		MakeShareable(new FMovieGraphConditionGroupQueryBuilder(DetailLayoutBuilder, QueriesProperty.ToSharedRef(), ConditionGroupIndex, MakeWeakObjectPtr(ConditionGroup))));
}

void SMovieGraphCollectionTreeOpTypeWidget::Construct(const FArguments& InArgs)
{
	bIsConditionGroup = InArgs._IsConditionGroup.Get();
	WeakConditionGroup = InArgs._WeakConditionGroup.Get();
	WeakConditionGroupQuery = InArgs._WeakConditionGroupQuery.Get();
	
	ChildSlot
	[
		SNew(SComboBox<FName>)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.ForegroundColor(FSlateColor::UseForeground())
		.IsEnabled(this, &SMovieGraphCollectionTreeOpTypeWidget::IsWidgetEnabled)
		.OptionsSource(bIsConditionGroup ? GetOpTypes<EMovieGraphConditionGroupOpType>() : GetOpTypes<EMovieGraphConditionGroupQueryOpType>())
		.OnSelectionChanged(this, &SMovieGraphCollectionTreeOpTypeWidget::SetOpType)
		.OnGenerateWidget_Lambda([this](const FName& InOpType)
		{
			return GetOpTypeContents(InOpType, bIsConditionGroup);
		})
		.Content()
		[
			GetOpTypeContents(NAME_None, bIsConditionGroup)
		]
	];
}

template <typename T>
TArray<FName>* SMovieGraphCollectionTreeOpTypeWidget::GetOpTypes()
{
	static_assert(TIsEnum<T>::Value, "Provided type must be an enum");
		
	static TArray<FName> OpTypes;

	if (OpTypes.IsEmpty())
	{
		const UEnum* OpTypeEnum = StaticEnum<T>();

		// -1 to skip the implicit "MAX" added at compile time
		for (int32 Index = 0; Index < OpTypeEnum->NumEnums() - 1; ++Index)
		{
			OpTypes.Add(FName(OpTypeEnum->GetDisplayNameTextByIndex(Index).ToString()));
		}
	}

	return &OpTypes;
}

TSharedRef<SWidget> SMovieGraphCollectionTreeOpTypeWidget::GetOpTypeContents(const FName& InOpName, const bool bIsConditionGroupOp) const
{
	TSharedPtr<SWidget> OpTypeTextBlock;

	// This method may only be called once in some scenarios, so provide the option of a text lambda vs. constant text if the
	// text needs to be updated dynamically when the op type changes.
	if (InOpName == NAME_None)
	{
		SAssignNew(OpTypeTextBlock, STextBlock)
		.Text_Lambda([this, bIsConditionGroupOp]()
		{
			return FText::FromName(bIsConditionGroupOp ? GetCurrentConditionGroupOpType() : GetCurrentConditionGroupQueryOpType());
		})
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
	}
	else
	{
		SAssignNew(OpTypeTextBlock, STextBlock)
		.Text(FText::FromName(InOpName))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
	}
		
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(0, 0, 7.f, 0)
		.AutoWidth()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Settings"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			OpTypeTextBlock.ToSharedRef()
		];
}

bool SMovieGraphCollectionTreeOpTypeWidget::IsWidgetEnabled() const
{
	if (bIsConditionGroup && WeakConditionGroup.IsValid())
	{
		// Disable the first condition group
		return !WeakConditionGroup->IsFirstConditionGroup();
	}
	
	if (WeakConditionGroupQuery.IsValid())
	{
		// Disable the first query, or disable it if it was flagged as disabled
		return !WeakConditionGroupQuery->IsFirstConditionGroupQuery() && WeakConditionGroupQuery->IsEnabled();
	}
	
	return true;
}

void SMovieGraphCollectionTreeOpTypeWidget::SetOpType(const FName InNewOpType, ESelectInfo::Type SelectInfo) const
{
	if (bIsConditionGroup && WeakConditionGroup.IsValid())
	{
		const UEnum* OpTypeEnum = StaticEnum<EMovieGraphConditionGroupOpType>();
		WeakConditionGroup->SetOperationType(static_cast<EMovieGraphConditionGroupOpType>(OpTypeEnum->GetValueByName(InNewOpType)));
	}
	else if (WeakConditionGroupQuery.IsValid())
	{
		const UEnum* OpTypeEnum = StaticEnum<EMovieGraphConditionGroupQueryOpType>();
		WeakConditionGroupQuery->SetOperationType(static_cast<EMovieGraphConditionGroupQueryOpType>(OpTypeEnum->GetValueByName(InNewOpType)));
	}
}

FName SMovieGraphCollectionTreeOpTypeWidget::GetCurrentConditionGroupOpType() const
{
	if (WeakConditionGroup.IsValid())
	{
		const UEnum* OpTypeEnum = StaticEnum<EMovieGraphConditionGroupOpType>();
		const EMovieGraphConditionGroupOpType OpType = WeakConditionGroup->GetOperationType();
		const FText DisplayNameText = OpTypeEnum->GetDisplayNameTextByValue(static_cast<__underlying_type(EMovieGraphConditionGroupOpType)>(OpType));
		
		return FName(DisplayNameText.ToString());
	}
		
	return FName();
}

FName SMovieGraphCollectionTreeOpTypeWidget::GetCurrentConditionGroupQueryOpType() const
{
	if (WeakConditionGroupQuery.IsValid())
	{
		const UEnum* OpTypeEnum = StaticEnum<EMovieGraphConditionGroupQueryOpType>();
		const EMovieGraphConditionGroupQueryOpType OpType = WeakConditionGroupQuery->GetOperationType();
		const FText DisplayNameText = OpTypeEnum->GetDisplayNameTextByValue(static_cast<__underlying_type(EMovieGraphConditionGroupQueryOpType)>(OpType));
		
		return FName(DisplayNameText.ToString());
	}
		
	return FName();
}

void SMovieGraphCollectionTreeAddQueryContentWidget::Construct(const FArguments& InArgs)
{
	TWeakObjectPtr<UMovieGraphConditionGroupQueryBase> WeakQuery = InArgs._WeakQuery.Get();

	if (!WeakQuery.IsValid())
	{
		return;
	}

    // Generate a (multi-layered) icon for the "Add" menu
    const TSharedRef<SLayeredImage> AddIcon =
    	SNew(SLayeredImage)
    	.ColorAndOpacity(FSlateColor::UseForeground())
    	.Image(FAppStyle::GetBrush("LevelEditor.OpenAddContent.Background"));
    AddIcon->AddLayer(FAppStyle::GetBrush("LevelEditor.OpenAddContent.Overlay"));

	ChildSlot
	[
		SNew(SComboButton)
		.ToolTipText(LOCTEXT("AddConditionGroupQueryContent", "Add content to this query."))
		.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
		.ContentPadding(0)
		.HasDownArrow(false)
		.IsEnabled_Lambda([WeakQuery]()
		{
			return WeakQuery.IsValid() && WeakQuery->IsEnabled();
		})
		.OnGetMenuContent_Lambda([WeakQuery]()
		{
			if (WeakQuery.IsValid())
			{
				// Call the underlying query to get the contents of the menu. The query will call back to the widget if something was added so the UI has
				// an opportunity to refresh itself.
				return WeakQuery->GetAddMenuContents(
					UMovieGraphConditionGroupQueryBase::FMovieGraphConditionGroupQueryContentsChanged::CreateLambda([]()
					{
						FSlateApplication::Get().DismissAllMenus();
					}));
			}

			return SNullWidget::NullWidget;
		})
		.Visibility_Lambda([WeakQuery]()
		{
			if (WeakQuery.IsValid())
			{
				return WeakQuery->HasAddMenu() ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return EVisibility::Collapsed;
		})
		.ButtonContent()
		[
			SNew(SBox)
			.WidthOverride(16)
			.HeightOverride(16)
			[
				AddIcon
			]
		]
	];
}

void SMovieGraphCollectionTreeQueryTypeSelectorWidget::Construct(const FArguments& InArgs)
{
	WeakConditionGroup = InArgs._WeakConditionGroup.Get();
	WeakQuery = InArgs._WeakQuery.Get();
	OnQueryTypeChanged = InArgs._OnQueryTypeChanged;

	ChildSlot
	[
		SNew(SComboBox<UClass*>)
		.IsEnabled_Lambda([this]()
		{
			return WeakQuery.IsValid() && WeakQuery->IsEnabled();
		})
		.OptionsSource(GetAvailableQueryTypes())
		.OnSelectionChanged(this, &SMovieGraphCollectionTreeQueryTypeSelectorWidget::SetQueryType)
		.OnGenerateWidget_Lambda([this](UClass* InClass)
		{
			return GetQueryTypeContents(InClass);
		})
		.Content()
		[
			GetQueryTypeContents(nullptr)
		]
	];
}

TArray<UClass*>* SMovieGraphCollectionTreeQueryTypeSelectorWidget::GetAvailableQueryTypes()
{
	static TArray<UClass*> QueryTypes;

	if (QueryTypes.IsEmpty())
	{
		for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
		{
			UClass* Class = *ClassIterator;

			if (Class->IsChildOf(UMovieGraphConditionGroupQueryBase::StaticClass()) &&
				!Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_Hidden))
			{
				QueryTypes.Add(Class);
			}
		}

		QueryTypes.Sort([](const UClass& InClass1, const UClass& InClass2)
		{
			return InClass1.GetName().Compare(InClass2.GetName(), ESearchCase::IgnoreCase) < 0;
		});
	}

	return &QueryTypes;
}

TSharedRef<SWidget> SMovieGraphCollectionTreeQueryTypeSelectorWidget::GetQueryTypeContents(UClass* InTypeClass) const
{
	TSharedPtr<SWidget> QueryTypeTextBlock;

	// This method may only be called once in some scenarios, so provide the option of a text lambda vs. constant text if the
	// text needs to be updated dynamically when the query type changes.
	if (InTypeClass == nullptr)
	{
		SAssignNew(QueryTypeTextBlock, STextBlock)
		.Text_Lambda([this]()
		{
			return GetCurrentQueryTypeDisplayName();
		})
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));

		// For below, assign the query type to the current query type
		InTypeClass = GetCurrentQueryType();
	}
	else
	{
		SAssignNew(QueryTypeTextBlock, STextBlock)
		.Text_Lambda([InTypeClass]()
		{
			if (const UMovieGraphConditionGroupQueryBase* Query = Cast<UMovieGraphConditionGroupQueryBase>(InTypeClass->GetDefaultObject()))
			{
				return Query->GetDisplayName();
			}

			// Show the UClass display name as a backup
			return InTypeClass->GetDisplayNameText();
		})
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
	}

	// Defer to the query to get the icon that should be displayed
	const FSlateIcon QueryTypeIcon = InTypeClass
		? GetDefault<UMovieGraphConditionGroupQueryBase>(InTypeClass)->GetIcon()
		: FSlateIcon();
	
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(0, 0, 7.f, 0)
		.AutoWidth()
		[
			SNew(SImage)
			.Image(QueryTypeIcon.GetIcon())
			.ColorAndOpacity(FSlateColor::UseForeground())
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			QueryTypeTextBlock.ToSharedRef()
		];
}

FText SMovieGraphCollectionTreeQueryTypeSelectorWidget::GetCurrentQueryTypeDisplayName() const
{
	if (const UClass* CurrentQueryType = GetCurrentQueryType())
	{
		check(CurrentQueryType->IsChildOf(UMovieGraphConditionGroupQueryBase::StaticClass()));
		
		return Cast<UMovieGraphConditionGroupQueryBase>(CurrentQueryType->GetDefaultObject())->GetDisplayName();
	}
	
	return FText();
}

UClass* SMovieGraphCollectionTreeQueryTypeSelectorWidget::GetCurrentQueryType() const
{
	if (WeakQuery.IsValid())
	{
		return WeakQuery->GetClass();
	}

	return nullptr;
}

void SMovieGraphCollectionTreeQueryTypeSelectorWidget::SetQueryType(UClass* InNewQueryType, ESelectInfo::Type SelectInfo)
{
	if (!WeakConditionGroup.IsValid() || !WeakQuery.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ChangeConditionGroupQueryType", "Change Condition Group Query Type"));
	
	const int32 ExistingQueryIndex = WeakConditionGroup->GetQueries().Find(WeakQuery.Get());

	// Preserve the operation type from the old query so it doesn't reset to Add.
	const EMovieGraphConditionGroupQueryOpType PreviousOpType = WeakQuery->GetOperationType();

	// Swap the old query with a new one.
	// Add the new query at the index of the current query. If the query couldn't be found for some reason, default to the end of the array (-1)
	UMovieGraphConditionGroupQueryBase* NewQuery = WeakConditionGroup->AddQuery(InNewQueryType, ExistingQueryIndex != INDEX_NONE ? ExistingQueryIndex : -1);
	NewQuery->SetOperationType(PreviousOpType);
	WeakConditionGroup->RemoveQuery(WeakQuery.Get());
	WeakQuery = MakeWeakObjectPtr(NewQuery);

	if (OnQueryTypeChanged.IsBound())
	{
		OnQueryTypeChanged.Execute(WeakQuery);
	}
}

TSharedRef<IDetailCustomization> FMovieGraphCollectionsCustomization::MakeInstance()
{
	return MakeShared<FMovieGraphCollectionsCustomization>();
}

void FMovieGraphCollectionsCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TArray<TWeakObjectPtr<UMovieGraphCollectionNode>> CollectionNodes =
		InDetailBuilder.GetObjectsOfTypeBeingCustomized<UMovieGraphCollectionNode>();
	if (CollectionNodes.Num() != 1)
	{
		// Showing more than one collection node is not supported
		return;
	}

	const TWeakObjectPtr<UMovieGraphCollectionNode> CollectionNode = CollectionNodes[0];
	if (!CollectionNode.IsValid())
	{
		return;
	}

	const TObjectPtr<UMovieGraphCollection> Collection = CollectionNode->Collection;
	WeakCollection = Collection;

	// The Collection property is being completely replaced by a custom builder
	const TSharedRef<IPropertyHandle> CollectionProperty = InDetailBuilder.GetProperty("Collection");
	CollectionProperty->MarkHiddenByCustomization();

	IDetailCategoryBuilder& CollectionCategory = InDetailBuilder.EditCategory("Collection");
		
	// Display the "Add Condition Group" button
	CollectionCategory.AddCustomRow(FText())
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SPositiveActionButton)
			.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
			.Text(LOCTEXT("AddNewConditionGroup", "Condition Group"))
			.OnClicked_Lambda([this]
			{
				if (WeakCollection.IsValid())
				{
					const FScopedTransaction Transaction(LOCTEXT("AddConditionGroup", "Add Condition Group"));
					
					UMovieGraphConditionGroup* NewConditionGroup = WeakCollection->AddConditionGroup();
					NewConditionGroup->AddQuery(UMovieGraphConditionGroupQuery_ActorName::StaticClass());
				}

				return FReply::Handled();
			})
		]
	];

	// Add the collection name property
	CollectionCategory.AddExternalObjectProperty({Collection}, FName(TEXT("CollectionName")));

	IDetailCategoryBuilder& ConditionsCategory = InDetailBuilder.EditCategory("Conditions");

	const TSharedPtr<IPropertyHandle> ConditionGroupsProperty = InDetailBuilder.AddObjectPropertyData({Collection}, FName(TEXT("ConditionGroups")));
	const TSharedPtr<IPropertyHandleArray> ConditionGroupsArrayProperty = ConditionGroupsProperty->AsArray();
	check(ConditionGroupsArrayProperty);

	// Add builders for the condition group array
	uint32 NumConditionGroups;
	ConditionGroupsArrayProperty->GetNumElements(NumConditionGroups);
	for (uint32 ConditionGroupIndex = 0; ConditionGroupIndex < NumConditionGroups; ++ConditionGroupIndex)
	{
		const TSharedRef<IPropertyHandle> ConditionGroupProperty = ConditionGroupsArrayProperty->GetElement(ConditionGroupIndex);
		ConditionsCategory.AddCustomBuilder(MakeShareable(new FMovieGraphConditionGroupBuilder(DetailBuilder.Pin().Get(), ConditionGroupProperty, ConditionGroupIndex, WeakCollection)));
	}

	ConditionGroupsArrayProperty->SetOnNumElementsChanged(FSimpleDelegate::CreateSPLambda(this, [&ConditionsCategory]()
	{
		ConditionsCategory.GetParentLayout().ForceRefreshDetails();
	}));
}

void FMovieGraphCollectionsCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailBuilder = InDetailBuilder;
	CustomizeDetails(*InDetailBuilder);
}

#undef LOCTEXT_NAMESPACE
