// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "IDetailCustomization.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailDragDropHandler.h"
#include "Widgets/Views/STreeView.h"

class UMovieGraphCollection;
class UMovieGraphConditionGroup;
class UMovieGraphConditionGroupQueryBase;

/** Drag/drop operation that contains data about a condition group or condition group query being dragged. */
class FMovieGraphCollectionDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FMovieGraphCollectionDragDropOp, FDecoratedDragDropOp);

	FMovieGraphCollectionDragDropOp(const bool bIsConditionGroup, const int32 InitialIndex, TWeakObjectPtr<UMovieGraphConditionGroup> InWeakOwningConditionGroup = nullptr);

	/** Initialize the operation. */
	void Init();

	/** Set whether the operation is currently valid or not. */
	void SetValidTarget(const bool bIsValidTarget);

	/** The index of the condition group or condition group query being dragged. */
	int32 GetIndex() const;

	/** Whether this operation represents a condition group (true) or condition group query (false). */
	bool IsConditionGroup() const;

	/** Get the condition group that owns the query being dragged (nullptr for dragging a condition group). */
	TWeakObjectPtr<UMovieGraphConditionGroup> GetOwningConditionGroup() const;

private:
	/** Whether this is a condition group (true) or condition group query (false) being dragged. */
	bool bIsConditionGroup;
	
	/** The index of the condition group or condition group query being dragged. */
	int32 InitialIndex;

	/** The condition group that owns the query being dragged (this being valid implies !bIsConditionGroup). */
	TWeakObjectPtr<UMovieGraphConditionGroup> OwningConditionGroup;
};

/** Handler for customizing the drag/drop functionality of condition groups and condition group queries. */
class FMovieGraphCollectionDragDropHandler : public IDetailDragDropHandler
{
public:
	FMovieGraphCollectionDragDropHandler(const bool bIsConditionGroup, IDetailLayoutBuilder* InDetailBuilder, UMovieGraphCollection* InCollection, const int32 ConditionGroupIndex, TWeakObjectPtr<UMovieGraphConditionGroup> InWeakConditionGroup, const int32 ConditionGroupQueryIndex);

	//~ Begin IDetailDragDropHandler interface
	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override;
	virtual bool AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override;
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override;
	//~ End IDetailDragDropHandler interface

private:
	/** Whether the handler is targeting a condition group (true) or query (false). */
	bool bIsConditionGroup;

	/** The layout builder that the handler was created from. */
	IDetailLayoutBuilder* DetailLayoutBuilder;

	/** The collection that owns the condition group (or query). */
	TWeakObjectPtr<UMovieGraphCollection> WeakCollection;

	/** The index of the condition group targeted by the handler. */
	int32 ConditionGroupIndex;

	/** The condition group that is targeted by the handler. */
	TWeakObjectPtr<UMovieGraphConditionGroup> WeakConditionGroup;

	/** The index of the query targeted by the handler. */
	int32 ConditionGroupQueryIndex;
};

/** Custom node builder for condition group queries. Displays each query as a child. */
class FMovieGraphConditionGroupQueryBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FMovieGraphConditionGroupQueryBuilder>
{
public:
	FMovieGraphConditionGroupQueryBuilder(IDetailLayoutBuilder* InDetailBuilder, TSharedRef<IPropertyHandle> InConditionGroupQueryProperty, int32 ConditionGroupQueryIndex, const TWeakObjectPtr<UMovieGraphConditionGroup>& InWeakConditionGroup);

	//~ Begin IDetailCustomNodeBuilder interface
	virtual FName GetName() const override;
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual bool RequiresTick() const override { return false; }
	virtual void Tick(float DeltaTime) override {}
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	//~ End IDetailCustomNodeBuilder interface

private:
	void AddQueryTypeWidgets(TWeakObjectPtr<UMovieGraphConditionGroupQueryBase> WeakQuery, IDetailChildrenBuilder& ChildrenBuilder) const;

private:
	/** The layout builder that this node builder was created from. */
	IDetailLayoutBuilder* DetailLayoutBuilder;
	
	/** The property for the condition group query being displayed. */
	TSharedPtr<IPropertyHandle> ConditionGroupQueryProperty;

	/** The index of the query represented by this builder. */
	int32 ConditionGroupQueryIndex;

	/** The condition group that the query belongs to. */
	TWeakObjectPtr<UMovieGraphConditionGroup> WeakConditionGroup;
};

/** Custom node builder for condition groups. The condition group is added as the header, and children are added as FMovieGraphConditionGroupQueryBuilder rows. */
class FMovieGraphConditionGroupBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FMovieGraphConditionGroupBuilder>
{
public:
	FMovieGraphConditionGroupBuilder(IDetailLayoutBuilder* InDetailBuilder, TSharedRef<IPropertyHandle> InConditionGroupProperty, const uint32 InConditionGroupIndex, TWeakObjectPtr<UMovieGraphCollection>& InWeakCollection);

	//~ Begin IDetailCustomNodeBuilder interface
	virtual FName GetName() const override;
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual bool RequiresTick() const override { return false; }
	virtual void Tick(float DeltaTime) override {}
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override { return ConditionGroupProperty; }
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	//~ End IDetailCustomNodeBuilder interface

private:
	/** The layout builder that this node builder was created from. */
	IDetailLayoutBuilder* DetailLayoutBuilder;
	
	/** The property for the condition group being displayed. */
	TSharedPtr<IPropertyHandle> ConditionGroupProperty;
	
	/** The index of the condition group represented by this builder. */
	uint32 ConditionGroupIndex;

	/** The collection that the condition groups belong to. */
	TWeakObjectPtr<UMovieGraphCollection> WeakCollection;
};

/**
 * A widget which controls the op type for a condition group or condition group query.
 */
class SMovieGraphCollectionTreeOpTypeWidget final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMovieGraphCollectionTreeOpTypeWidget) { }
		SLATE_ATTRIBUTE(bool, IsConditionGroup)
		SLATE_ATTRIBUTE(TWeakObjectPtr<UMovieGraphConditionGroup>, WeakConditionGroup)
		SLATE_ATTRIBUTE(TWeakObjectPtr<UMovieGraphConditionGroupQueryBase>, WeakConditionGroupQuery)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Whether this widget is associated with a condition group (else a condition group query). */
	bool bIsConditionGroup = false;

	/** The condition group associated with this widget (if any). */
	TWeakObjectPtr<UMovieGraphConditionGroup> WeakConditionGroup;

	/** The condition group query associated with this widget (if any). */
	TWeakObjectPtr<UMovieGraphConditionGroupQueryBase> WeakConditionGroupQuery;

private:
	/** Gets the names of all op types available. */
	template<typename T>
	static TArray<FName>* GetOpTypes();

	/** Gets a widget (icon + name) representing the given op. */
	TSharedRef<SWidget> GetOpTypeContents(const FName& InOpName, const bool bIsConditionGroupOp) const;

	/** Whether this widget is currently enabled. */
	bool IsWidgetEnabled() const;

	/** Sets the op type for the condition group or condition group query. */
	void SetOpType(const FName InNewOpType, ESelectInfo::Type SelectInfo) const;

	/** Gets the currently-assigned op type for the condition group. */
	FName GetCurrentConditionGroupOpType() const;

	/** Gets the currently-assigned op type for the condition group query. */
	FName GetCurrentConditionGroupQueryOpType() const;
};

/**
 * A widget which displays the "Add" menu for a condition group query, allowing the user to add content to the query (like an actor). The appearance
 * of the widget and behavior of the menu is largely delegated to the query itself.
 */
class SMovieGraphCollectionTreeAddQueryContentWidget final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMovieGraphCollectionTreeAddQueryContentWidget) { }
		SLATE_ATTRIBUTE(TWeakObjectPtr<UMovieGraphConditionGroupQueryBase>, WeakQuery)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};

/**
 * A ComboBox widget which displays the available condition group query types and updates the condition group when one is selected.
 */
class SMovieGraphCollectionTreeQueryTypeSelectorWidget final : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnQueryTypeChanged, TWeakObjectPtr<UMovieGraphConditionGroupQueryBase>);
	
	SLATE_BEGIN_ARGS(SMovieGraphCollectionTreeQueryTypeSelectorWidget) { }
		SLATE_ATTRIBUTE(TWeakObjectPtr<UMovieGraphConditionGroup>, WeakConditionGroup)
		SLATE_ATTRIBUTE(TWeakObjectPtr<UMovieGraphConditionGroupQueryBase>, WeakQuery)
		SLATE_EVENT(FOnQueryTypeChanged, OnQueryTypeChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Gets the query types which are available to be selected and added to the condition group. */
	static TArray<UClass*>* GetAvailableQueryTypes();

	/** Gets the widget that displays an individual query type. */
	TSharedRef<SWidget> GetQueryTypeContents(UClass* InTypeClass) const;

	/** Gets the display name of the currently selected query. */
	FText GetCurrentQueryTypeDisplayName() const;

	/** Gets the UClass associated with the currently selected query. */
	UClass* GetCurrentQueryType() const;

	/** Updates the query to the query type which was selected. */
	void SetQueryType(UClass* InNewQueryType, ESelectInfo::Type SelectInfo);

private:
	/** The condition group that the query belongs to. */
	TWeakObjectPtr<UMovieGraphConditionGroup> WeakConditionGroup = nullptr;

	/** The condition group query associated with this widget. */
	TWeakObjectPtr<UMovieGraphConditionGroupQueryBase> WeakQuery = nullptr;

	/** Called when the query type changes. */
	FOnQueryTypeChanged OnQueryTypeChanged;
};

/** Customize how the Collection node appears in the details panel. */
class FMovieGraphCollectionsCustomization final : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** The collection being displayed. */
	TWeakObjectPtr<UMovieGraphCollection> WeakCollection;

	/** The details builder associated with the customization. */
	TWeakPtr<IDetailLayoutBuilder> DetailBuilder;
};
