// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphCustomizationUtils.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Graph/MovieGraphConfig.h"
#include "IDetailGroup.h"
#include "MovieJobVariableAssignmentContainer.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieRenderPipelineStyle.h"
#include "NamingTokensEngineSubsystem.h"
#include "SNamingTokensEditableTextBox.h"

namespace UE::MovieRenderPipelineEditor::Private
{
	void AddVariableAssignments(TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& InVariableAssignments, IDetailCategoryBuilder& InCategory, IDetailLayoutBuilder* InDetailBuilder)
	{
		// Add a sub-category for each graph (including subgraphs). Each entry in the array represents the assignments for one graph.
		for (TObjectPtr<UMovieJobVariableAssignmentContainer>& VariableAssignment : InVariableAssignments)
		{
			// Skip if the graph associated with this container has no variables in it
			if (VariableAssignment->GetNumAssignments() <= 0)
			{
				continue;
			}

			// If the graph can be found, display its variable assignments under its own category (group)
			TSoftObjectPtr<UMovieGraphConfig> SoftGraphConfig = VariableAssignment->GetGraphConfig();
			if (const UMovieGraphConfig* GraphConfig = SoftGraphConfig.Get())
			{
				constexpr bool bForAdvanced = false;
				constexpr bool bStartExpanded = true;
				IDetailGroup& GraphGroup = InCategory.AddGroup(GraphConfig->GetFName(), FText::FromString(GraphConfig->GetName()), bForAdvanced, bStartExpanded);

				// "Value" is private so we can't use GET_MEMBER_NAME_CHECKED unfortunately
				TSharedPtr<IPropertyHandle> ValueProperty = InDetailBuilder->AddObjectPropertyData({VariableAssignment}, FName("Value"));
				GraphGroup.AddPropertyRow(ValueProperty.ToSharedRef());

				// Un-hide the category if it's currently visible
				InCategory.SetCategoryVisibility(true);
			}
		}
	}

	void AddTokenAutocompleteToPropertyRow(IDetailPropertyRow* InPropertyRow)
	{
		/** Determines if the given property points to a FDirectoryPath struct. */
		auto IsDirectoryPathProperty = [](const TSharedPtr<IPropertyHandle>& InPropertyHandle)
		{
			const FProperty* Property = InPropertyHandle->GetProperty();
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (const TObjectPtr<UScriptStruct> Struct = StructProperty->Struct)
				{
					return Struct->IsChildOf<FDirectoryPath>();
				}
			}

			return false;
		};

		if (!InPropertyRow)
		{
			return;
		}

		// Note: For now, FDirectoryPath support is disabled because it causes the "..." button to disappear. At some point ideally FDirectoryPath
		// also includes token autocomplete, but the "..." button is more important.
		TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyRow->GetPropertyHandle();
		if (IsDirectoryPathProperty(PropertyHandle))
		{
			return;
		}

		// Most token format strings are specified via simple FString properties. However, there are cases where token autocomplete is needed for
		// FDirectoryPath structs too, so those need to be special-cased here.
		if (IsDirectoryPathProperty(PropertyHandle))
		{
			if (const TSharedPtr<IPropertyHandle> PathProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDirectoryPath, Path)))
			{
				PropertyHandle = PathProperty;
			}
		}

		// The MRG namespace in Naming Tokens is private, so we need to explicitly include the namespace so the MRG tokens show up
		FNamingTokenFilterArgs NamingTokenFilterArgs;
		NamingTokenFilterArgs.AdditionalNamespacesToInclude.Add(UMovieRenderPipelineNamingTokens::TokenNamespace);

		InPropertyRow->CustomWidget()
			.PropertyHandleList({PropertyHandle})
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(200.0f)
			[
				SNew(SNamingTokensEditableTextBox)
					.Text_Lambda([WeakAutocompleteProperty = PropertyHandle.ToWeakPtr()]()
					{
						FString InitialTextString;
						
						if (const TSharedPtr<IPropertyHandle> PropertyPin = WeakAutocompleteProperty.Pin())
						{
							PropertyPin->GetValue(InitialTextString);
						}

						return FText::FromString(InitialTextString);
					})
					.OnTextCommitted_Lambda([WeakAutocompleteProperty = PropertyHandle.ToWeakPtr()](const FText& InNewText, ETextCommit::Type CommitType)
					{
						if (const TSharedPtr<IPropertyHandle> PropertyPin = WeakAutocompleteProperty.Pin())
						{
							const FPropertyAccess::Result Result = PropertyPin->SetValueFromFormattedString(InNewText.ToString());
							
							if (Result != FPropertyAccess::Success)
							{
								UE_LOGF(LogMovieRenderPipeline, Warning, "Unable to insert token into format string (could not update FProperty).");
							}
						}
					})
					.Contexts({})
					.FilterArgs(NamingTokenFilterArgs)
					.NamespaceSuggestionPriority({ UMovieRenderPipelineNamingTokens::TokenNamespace })
					.IsReadOnly(false)
					.CanDisplayResolvedText(false)
					.AllowMultiLine(false)
					.EnableSuggestionDropdown(true)
					.TextBoxPadding(FMargin(4))
					.Style(&FMovieRenderPipelineStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("MovieRenderPipeline.AutocompleteTextStyle"))
					.ArgumentStyle(&FMovieRenderPipelineStyle::Get().GetWidgetStyle<FTextBlockStyle>("MovieRenderPipeline.AutocompleteArgumentTextStyle"))
			];
	}
}
