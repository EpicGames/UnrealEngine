// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailCategoryBuilder.h"

class UMovieJobVariableAssignmentContainer;

namespace UE::MovieRenderPipelineEditor::Private
{
	/**
	 * Adds InVariableAssignments to the given InCategory. For each graph that is used within InVariableAssignments, a dedicated sub-category (named
	 * the name of the graph) is added under the given category.
	 */
	void AddVariableAssignments(TArray<TObjectPtr<UMovieJobVariableAssignmentContainer>>& InVariableAssignments, IDetailCategoryBuilder& InCategory, IDetailLayoutBuilder* InDetailBuilder);

	/** Adds the Naming Tokens token autocomplete widget to the given property row. */
	void AddTokenAutocompleteToPropertyRow(IDetailPropertyRow* InPropertyRow);
}
