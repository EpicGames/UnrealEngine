// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailLayoutBuilder.h"
#include "Graph/Nodes/MovieGraphExecuteScriptNode.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customize how the Execute Script node appears in the details panel. */
class FMovieGraphExecuteScriptNodeCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphExecuteScriptNodeCustomization>();
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		const TSharedRef<IPropertyHandle> ModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphExecuteScriptNode, Mode));
		const TSharedRef<IPropertyHandle> EditorOnlyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphExecuteScriptNode, EditorOnlyScript));
		const TSharedRef<IPropertyHandle> EditorAndRuntimeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphExecuteScriptNode, EditorAndRuntimeScript));

		// Gets the enum value of the Mode property.
		auto GetModeValue = [ModeHandle]() -> EMovieGraphExecuteScriptMode
		{
			constexpr EMovieGraphExecuteScriptMode InvalidMode = EMovieGraphExecuteScriptMode::Count;

			if (ModeHandle->IsValidHandle())
			{
				uint8 ModeValue = static_cast<uint8>(InvalidMode);

				if (ModeHandle->GetValue(ModeValue) == FPropertyAccess::Success)
				{
					return static_cast<EMovieGraphExecuteScriptMode>(ModeValue);
				}
			}

			return InvalidMode;
		};

		const TAttribute<EVisibility> EditorOnlyVisibleAttr = TAttribute<EVisibility>::Create([GetModeValue]()
		{
			return (GetModeValue() == EMovieGraphExecuteScriptMode::EditorOnly) ? EVisibility::Visible : EVisibility::Collapsed;
		});

		const TAttribute<EVisibility> EditorAndRuntimeVisibleAttr = TAttribute<EVisibility>::Create([GetModeValue]()
		{
			return (GetModeValue() == EMovieGraphExecuteScriptMode::EditorAndRuntime) ? EVisibility::Visible : EVisibility::Collapsed;
		});

		// Vary the visibility of the script properties depending on the selected mode.
		if (IDetailPropertyRow* EditorOnlyRow = DetailBuilder.EditDefaultProperty(EditorOnlyHandle))
		{
			EditorOnlyRow->Visibility(EditorOnlyVisibleAttr);
		}
		if (IDetailPropertyRow* EditorAndRuntimeRow = DetailBuilder.EditDefaultProperty(EditorAndRuntimeHandle))
		{
			EditorAndRuntimeRow->Visibility(EditorAndRuntimeVisibleAttr);
		}
	}
	//~ End IDetailCustomization interface
};

#undef LOCTEXT_NAMESPACE