// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphLightModifierCustomization.h"

#include "Components/LightComponentBase.h"
#include "Components/LocalLightComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Light.h"
#include "Engine/SkyLight.h"
#include "Graph/Nodes/MovieGraphModifierNode.h"
#include "IPropertyUtilities.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/WeakInterfacePtr.h"
#include "Widgets/Graph/SModifierCollectionsHeader.h"
#include "Widgets/Graph/SMovieGraphModifierCollectionsList.h"

#define LOCTEXT_NAMESPACE "MovieGraphLightModifierCustomization"

TSharedRef<IDetailCustomization> FMovieGraphLightModifierCustomization::MakeInstance()
{
	return MakeShared<FMovieGraphLightModifierCustomization>();
}

void FMovieGraphLightModifierCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	// The customization only supports editing a single Modifier node
	const TWeakInterfacePtr<IMovieGraphModifierNodeInterface> ModifierNode = GetSelectedModifierNode();
	if (!ModifierNode.IsValid())
	{
		return;
	}

	// Replace the "Collection" category row with a custom whole-row widget which includes an add-collection button
	IDetailCategoryBuilder& CollectionsCategory =
		InDetailBuilder.EditCategory(FName("Collections"), FText::GetEmpty(), ECategoryPriority::Uncommon);
	CollectionsCategory.HeaderContent
	(
		SNew(SMovieGraphCollectionsHeaderWidget)
		.WeakModifierInterface(ModifierNode)
		.OnCollectionPicked_Lambda([this](const FName CollectionName)
		{
			CollectionsList->Refresh();
		})
	, /* bWholeRowContent */ true);

	// Add a collections browser
	CollectionsCategory.AddCustomRow(FText::GetEmpty())
	.WholeRowWidget
	[
		SAssignNew(CollectionsList, SMovieGraphModifierCollectionsList)
		.WeakModifierInterface(ModifierNode)
	];

	// Add a "Custom" category and include a "+" button in its header.
	IDetailCategoryBuilder& CustomCategory =
		InDetailBuilder.EditCategory(FName("Custom"), FText::GetEmpty(), ECategoryPriority::Uncommon);
	CustomCategory.HeaderContent
	(
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CustomHeaderText", "Custom"))
			.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(5.f, 0, 0, 0)
		[
			SNew(SComboButton)
			.ToolTipText(LOCTEXT("AddCustomLightPropertyTooltip", "Add a light property that will be updated by this modifier."))
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.ContentPadding(0)
			.HasDownArrow(false)
			.OnGetMenuContent_Lambda([this, ModifierNode]()
			{
				return GetCustomMenuContents(ModifierNode);
			})
			.ButtonContent()
			[
				SNew(SBox)
				.WidthOverride(16)
				.HeightOverride(16)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.AddCircle"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
	, /* bWholeRowContent */ true);

	// Move the node's DynamicProperties (used to store all of the "Custom" lighting properties) to be under the "Custom" category.
	const TSharedRef<IPropertyHandle> LightingValuesProperty = InDetailBuilder.GetProperty(TEXT("DynamicProperties"), UMovieGraphNode::StaticClass());
	if (LightingValuesProperty->IsValidHandle())
	{
		InDetailBuilder.HideProperty(LightingValuesProperty);
		CustomCategory.AddProperty(LightingValuesProperty);
	}

	// Only add the "No Custom Properties" text if the LightingValues value container is empty.
	if (const UMovieGraphLightModifierNode* LightModifierNode = Cast<UMovieGraphLightModifierNode>(ModifierNode.Get()))
	{
		if (LightModifierNode->GetNumCustomLightProperties() == 0)
		{
			CustomCategory.AddCustomRow(FText::GetEmpty())
			.WholeRowWidget
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SBox)
				.Padding(15.f)
				[
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Font(InDetailBuilder.GetDetailFontItalic())
					.Text(FText::FromString(TEXT("Add a Custom Light Property")))
				]
			];
		}
	}

	// Define the sort order of the categories.
	int32 SortOrder = 0;
	InDetailBuilder.EditCategory(FName("Modifier")).SetSortOrder(SortOrder);
	InDetailBuilder.EditCategory(FName("Collections")).SetSortOrder(++SortOrder);
	InDetailBuilder.EditCategory(FName("Light")).SetSortOrder(++SortOrder);
	InDetailBuilder.EditCategory(FName("RayTracing")).SetSortOrder(++SortOrder);
	InDetailBuilder.EditCategory(FName("Custom")).SetSortOrder(++SortOrder);

	const TSharedRef<IPropertyHandle> IntensityMethodProperty = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, IntensityMethod));
	const TSharedRef<IPropertyHandle> IntensityUnitsProperty = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, IntensityUnits));
	const TSharedRef<IPropertyHandle> PointLightIntensityUnitsProperty = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensityUnits));
	const TSharedRef<IPropertyHandle> RectLightIntensityUnitsProperty = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensityUnits));
	const TSharedRef<IPropertyHandle> SpotLightIntensityUnitsProperty = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensityUnits));

	// This is a bit of a heavy hammer, but because many of the intensity-related properties are deeply nested in sub-categories, we need to force
	// refresh the details panel when the Intensity Method changes. This is required to get the visibility lambda on the intensity-related properties
	// to take effect. Without this, the visibility lambdas will still fire, but the layout of the details panel will not update to reflect
	// visibility changes.
	IntensityMethodProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSPLambda(this, [this]()
	{
		if (const TSharedPtr<IDetailLayoutBuilder> DetailBuilderPin = DetailBuilder.Pin())
		{
			DetailBuilderPin->ForceRefreshDetails();
		}
	}));

	// When the Intensity Units property on any of the Intensity properties is about to change, cache the current value of the Intensity Units. 
	IntensityUnitsProperty->SetOnPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FMovieGraphLightModifierCustomization::OnIntensityUnitsValuePreChange, GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, IntensityUnits)));
	PointLightIntensityUnitsProperty->SetOnPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FMovieGraphLightModifierCustomization::OnIntensityUnitsValuePreChange, GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensityUnits)));
	RectLightIntensityUnitsProperty->SetOnPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FMovieGraphLightModifierCustomization::OnIntensityUnitsValuePreChange, GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensityUnits)));
	SpotLightIntensityUnitsProperty->SetOnPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FMovieGraphLightModifierCustomization::OnIntensityUnitsValuePreChange, GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensityUnits)));

	// When the intensity units change, the value of the intensity needs to be updated. The details panel also needs to be force-refreshed to get the
	// units metadata to update properly.
	IntensityUnitsProperty->SetOnPropertyValueChanged(
		FSimpleDelegate::CreateSP(this, &FMovieGraphLightModifierCustomization::OnIntensityUnitsValueChange,
			GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, Intensity),
			GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, IntensityUnits)));
	PointLightIntensityUnitsProperty->SetOnPropertyValueChanged(
		FSimpleDelegate::CreateSP(this, &FMovieGraphLightModifierCustomization::OnIntensityUnitsValueChange,
			GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensity),
			GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensityUnits)));
	RectLightIntensityUnitsProperty->SetOnPropertyValueChanged(
		FSimpleDelegate::CreateSP(this, &FMovieGraphLightModifierCustomization::OnIntensityUnitsValueChange,
			GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensity),
			GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensityUnits)));
	SpotLightIntensityUnitsProperty->SetOnPropertyValueChanged(
		FSimpleDelegate::CreateSP(this, &FMovieGraphLightModifierCustomization::OnIntensityUnitsValueChange,
			GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensity),
			GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensityUnits)));

	// Some of the intensity properties should not be visible depending on the intensity method. EditCondition metadata (+ EditConditionHides)
	// can't be used because EditCondition is already used with the corresponding bOverride_* properties. Instead, use a visibility lambda.
	ShowPropertyConditionally(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, Intensity), EMovieGraphLightModifierIntensityMethod::PointRectSpot);
	ShowPropertyConditionally(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, IntensityUnits), EMovieGraphLightModifierIntensityMethod::PointRectSpot);
	ShowPropertyConditionally(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensity), EMovieGraphLightModifierIntensityMethod::PerLightActor);
	ShowPropertyConditionally(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensityUnits), EMovieGraphLightModifierIntensityMethod::PerLightActor);
	ShowPropertyConditionally(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensity), EMovieGraphLightModifierIntensityMethod::PerLightActor);
	ShowPropertyConditionally(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensityUnits), EMovieGraphLightModifierIntensityMethod::PerLightActor);
	ShowPropertyConditionally(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensity), EMovieGraphLightModifierIntensityMethod::PerLightActor);
	ShowPropertyConditionally(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensityUnits), EMovieGraphLightModifierIntensityMethod::PerLightActor);

	// Initialize the metadata on the Intensity-related properties ("Units", etc -- these depend on the Intensity Units setting).
	InitializeIntensityProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, Intensity), GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, IntensityUnits));
	InitializeIntensityProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensity), GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, PointLightIntensityUnits));
	InitializeIntensityProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensity), GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, RectLightIntensityUnits));
	InitializeIntensityProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensity), GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, SpotLightIntensityUnits));
}

void FMovieGraphLightModifierCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailBuilder = InDetailBuilder;
	CustomizeDetails(*InDetailBuilder);
}

FMovieGraphLightModifierCustomization::FCustomLightingPropertyInfo::FCustomLightingPropertyInfo(
	UClass* LightActorClass, UClass* LightComponentClass, FProperty* LightProperty, const FString& LightPropertyCategory)
	: LightActorClass(LightActorClass)
	, LightComponentClass(LightComponentClass)
	, LightProperty(LightProperty)
	, LightPropertyCategory(LightPropertyCategory)
{
	
}

TWeakInterfacePtr<IMovieGraphModifierNodeInterface> FMovieGraphLightModifierCustomization::GetSelectedModifierNode() const
{
	if (const TSharedPtr<IDetailLayoutBuilder> DetailBuilderPin = DetailBuilder.Pin())
	{
		TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
		DetailBuilderPin->GetObjectsBeingCustomized(CustomizedObjects);

		for (const TWeakObjectPtr<UObject>& CustomizedObject : CustomizedObjects)
		{
			if (CustomizedObject.IsValid() && CustomizedObject->Implements<UMovieGraphModifierNodeInterface>())
			{
				return TWeakInterfacePtr<IMovieGraphModifierNodeInterface>(CustomizedObject.Get());
			}
		}
	}

	return nullptr;
}

void FMovieGraphLightModifierCustomization::InitializeIntensityProperty(const FName& InIntensityPropertyName, const FName& InIntensityUnitsPropertyName)
{
	if (const TSharedPtr<IDetailLayoutBuilder> DetailBuilderPin = DetailBuilder.Pin())
	{
		const TSharedRef<IPropertyHandle> IntensityHandle = DetailBuilderPin->GetProperty(InIntensityPropertyName, UMovieGraphLightModifierNode::StaticClass());
		const TSharedRef<IPropertyHandle> IntensityUnitsHandle = DetailBuilderPin->GetProperty(InIntensityUnitsPropertyName, UMovieGraphLightModifierNode::StaticClass());

		uint8 IntensityUnits = (uint8)ELightUnits::Candelas;
		IntensityUnitsHandle->GetValue(IntensityUnits);
		const ELightUnits IntensityUnitsEnum = static_cast<ELightUnits>(IntensityUnits);

		if (IntensityUnitsEnum == ELightUnits::EV)
		{
			IntensityHandle->SetInstanceMetaData("UIMin",TEXT("-32.0"));
			IntensityHandle->SetInstanceMetaData("UIMax",TEXT("32.0"));
		}
		else
		{
			const float ConversionFactor = ULocalLightComponent::GetUnitsConversionFactor((ELightUnits)0, IntensityUnitsEnum);
	
			IntensityHandle->SetInstanceMetaData("UIMin",TEXT("0.0"));
			IntensityHandle->SetInstanceMetaData("UIMax",  *FString::SanitizeFloat(100000.0f * ConversionFactor));
			IntensityHandle->SetInstanceMetaData("SliderExponent", TEXT("2.0"));
		}

		if (IntensityUnitsEnum == ELightUnits::Lumens)
		{
			IntensityHandle->SetInstanceMetaData("Units", TEXT("lm"));
		}
		else if (IntensityUnitsEnum == ELightUnits::Candelas)
		{
			IntensityHandle->SetInstanceMetaData("Units", TEXT("cd"));
		}
		else if (IntensityUnitsEnum == ELightUnits::EV)
		{
			IntensityHandle->SetInstanceMetaData("Units", TEXT("ev"));
		}
		else if (IntensityUnitsEnum == ELightUnits::Nits)
		{
			IntensityHandle->SetInstanceMetaData("Units", TEXT("nt"));
		}
		else
		{
			IntensityHandle->SetInstanceMetaData("Units", TEXT(""));
		}
	}
}

void FMovieGraphLightModifierCustomization::ShowPropertyConditionally(FName IntensityPropertyName, EMovieGraphLightModifierIntensityMethod ShowWithMethod)
{
	const TSharedPtr<IDetailLayoutBuilder> DetailBuilderPin = DetailBuilder.Pin();
	if (!DetailBuilderPin.IsValid())
	{
		return;
	}

	const TSharedRef<IPropertyHandle> IntensityHandle = DetailBuilderPin->GetProperty(IntensityPropertyName);
	const TSharedRef<IPropertyHandle> IntensityMethodHandle = DetailBuilderPin->GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphLightModifierNode, IntensityMethod));

	if (!IntensityHandle->IsValidHandle() || !IntensityMethodHandle->IsValidHandle())
	{
		return;
	}

	if (IDetailPropertyRow* IntensityPropertyRow = DetailBuilderPin->EditDefaultProperty(IntensityHandle))
	{
		IntensityPropertyRow->Visibility(MakeAttributeLambda([IntensityMethodHandle, ShowWithMethod]()
		{
			uint8 IntensityMethod = 0;
			IntensityMethodHandle->GetValue(IntensityMethod);

			const EVisibility IsVisible = (static_cast<EMovieGraphLightModifierIntensityMethod>(IntensityMethod) == ShowWithMethod)
				? EVisibility::Visible
				: EVisibility::Collapsed;

			return IsVisible;
		}));
	}
}

void FMovieGraphLightModifierCustomization::OnIntensityUnitsValuePreChange(FName IntensityUnitsPropertyName)
{
	if (const TSharedPtr<IDetailLayoutBuilder> DetailBuilderPin = DetailBuilder.Pin())
	{
		const TSharedRef<IPropertyHandle> TargetIntensityUnitsHandle = DetailBuilderPin->GetProperty(IntensityUnitsPropertyName, UMovieGraphLightModifierNode::StaticClass());

		uint8 IntensityUnits = 0;
		TargetIntensityUnitsHandle->GetValue(IntensityUnits);
		PreviousIntensityUnits = static_cast<ELightUnits>(IntensityUnits);
	}
}

void FMovieGraphLightModifierCustomization::OnIntensityUnitsValueChange(const FName IntensityPropertyName, const FName IntensityUnitsPropertyName) const
{
	if (const TSharedPtr<IDetailLayoutBuilder> DetailBuilderPin = DetailBuilder.Pin())
	{
		const TSharedRef<IPropertyHandle> IntensityProperty = DetailBuilderPin->GetProperty(IntensityPropertyName, UMovieGraphLightModifierNode::StaticClass());
		const TSharedRef<IPropertyHandle> IntensityUnitsProperty = DetailBuilderPin->GetProperty(IntensityUnitsPropertyName, UMovieGraphLightModifierNode::StaticClass());

		// Get the current intensity unit
		uint8 IntensityUnits = (uint8)ELightUnits::Candelas;
		IntensityUnitsProperty->GetValue(IntensityUnits);
		const ELightUnits IntensityUnitsEnum = static_cast<ELightUnits>(IntensityUnits);

		// Convert the intensity to a different unit if required
		const float ConversionFactor = ULocalLightComponent::GetUnitsConversionFactor(PreviousIntensityUnits, IntensityUnitsEnum);

		// Set the value of the intensity (only required if the intensity unit changes)
		float CurrentIntensity = 0.f;
		IntensityProperty->GetValue(CurrentIntensity);
		IntensityProperty->SetValue(CurrentIntensity * ConversionFactor);

		// The force refresh is needed when the unit changes (since it's specified via metadata)
		DetailBuilderPin->ForceRefreshDetails();
	}
}

TArray<FMovieGraphLightModifierCustomization::FCustomLightingPropertyInfo> FMovieGraphLightModifierCustomization::GetCustomLightingProperties() const
{
	// Only properties in certain lighting categories should be included.
	static const TArray<FString> AllowedLightingCategories = {
		TEXT("Light"), TEXT("Performance"), TEXT("LightFunction"), TEXT("LightProfiles"), TEXT("DistanceFieldShadows"), TEXT("RayTracing")
	};

	// Some properties are excluded from being returned by this method because they're always displayed outside of the Custom menu.
	static const TArray<FName> ExcludedClassProperties = {
		TEXT("LightColor"), TEXT("bAffectsWorld"), TEXT("CastShadows"), TEXT("IndirectLightingIntensity"), TEXT("VolumetricScatteringIntensity"),
		TEXT("LightingChannels"), TEXT("SamplesPerPixel"), TEXT("Intensity"), TEXT("IntensityUnits")
	};

	TArray<FCustomLightingPropertyInfo> CustomLightingProperties;

	// Get all light classes that are available. Add the Sky Light manually since it doesn't derive from ALight.
	TArray<UClass*> LightClasses;
	constexpr bool bRecursive = false;
	GetDerivedClasses(ALight::StaticClass(), LightClasses, bRecursive);
	LightClasses.Add(ASkyLight::StaticClass());

	// Discover all relevant lighting properties in the light classes.
	for (UClass* LightClass : LightClasses)
	{
		// The light class itself doesn't contain the properties we care about, they're in a component (object property) in the class. Iterate the
		// class until we find this component property.
		const FObjectPropertyBase* LightComponentProperty = nullptr;
		for (TFieldIterator<FProperty> PropertyIt(LightClass, EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated); PropertyIt; ++PropertyIt)
		{
			const FProperty* LightProperty = *PropertyIt;

			const FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(LightProperty);
			if (!ObjectPropertyBase)
			{
				continue;
			}

			if (!ObjectPropertyBase->PropertyClass->IsChildOf(ULightComponentBase::StaticClass()))
			{
				continue;
			}

			LightComponentProperty = ObjectPropertyBase;
			break;
		}

		if (!LightComponentProperty)
		{
			continue;
		}

		// Iterate the properties of the light component.
		for (TFieldIterator<FProperty> ComponentPropertyIt(LightComponentProperty->PropertyClass, EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated); ComponentPropertyIt; ++ComponentPropertyIt)
		{
			FProperty* ComponentProperty = *ComponentPropertyIt;
			const FName PropertyName = ComponentProperty->GetFName();
			
			// Some properties are excluded because they're always shown outside of the Custom properties.
			if (ExcludedClassProperties.Contains(PropertyName))
			{
				continue;
			}
			
			// Only properties showing up in certain categories are included.
			const FString& Category = ComponentProperty->GetMetaData(TEXT("Category"));
			if (!AllowedLightingCategories.Contains(Category))
			{
				continue;
			}

			// Only include properties that show up in the details panel.
			if (!ComponentProperty->HasAnyPropertyFlags(CPF_Edit))
			{
				continue;
			}

			CustomLightingProperties.Add(
				FCustomLightingPropertyInfo(LightClass, LightComponentProperty->PropertyClass, ComponentProperty, Category));
		}
	}

	return CustomLightingProperties;
}

TSharedRef<SWidget> FMovieGraphLightModifierCustomization::GetCustomMenuContents(const TWeakInterfacePtr<IMovieGraphModifierNodeInterface> InWeakModifierNode) const
{
	UMovieGraphLightModifierNode* LightModifier = Cast<UMovieGraphLightModifierNode>(InWeakModifierNode.Get());
	if (!IsValid(LightModifier))
	{
		return SNullWidget::NullWidget;
	}

	constexpr bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder CustomMenu(bShouldCloseWindowAfterMenuSelection, nullptr);

	const TSharedRef<SWidget> CustomMenuHelpTextWidget =
		SNew(STextBlock)
		.Margin(FMargin(10, 5, 10, 5))
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		.Text(LOCTEXT("CustomMenuHelpText", "Add an additional light component property to this modifier."));

	constexpr bool bNoIndent = true;
	CustomMenu.AddWidget(CustomMenuHelpTextWidget, FText::GetEmpty(), bNoIndent);
	
	TArray<FCustomLightingPropertyInfo> PropertiesToDisplay = GetCustomLightingProperties();

	// Group "Custom" properties by light class
	TMap<UClass*, TArray<FCustomLightingPropertyInfo>> PropertiesByLightClass;
	for (const FCustomLightingPropertyInfo& LightPropertyInfo : PropertiesToDisplay)
	{
		PropertiesByLightClass.FindOrAdd(LightPropertyInfo.LightActorClass).Add(LightPropertyInfo);
	}

	CustomMenu.BeginSection(NAME_None, LOCTEXT("LightActorTypesSectionHeader", "Light Component Types"));

	for (const TPair<UClass*, TArray<FCustomLightingPropertyInfo>>& LightClassPropertiesPair : PropertiesByLightClass)
	{
		const UClass* LightClass = LightClassPropertiesPair.Key;
		const TArray<FCustomLightingPropertyInfo>& LightClassProperties = LightClassPropertiesPair.Value;

		CustomMenu.AddSubMenu(
			LightClass->GetDisplayNameText(),
			TAttribute<FText>(),
			FNewMenuDelegate::CreateLambda([InWeakModifierNode, LightClassProperties](FMenuBuilder& MenuBuilder)
			{
				// Group properties by category
				TMap<FString, TArray<FCustomLightingPropertyInfo>> LightPropertiesByCategory;
				for (const FCustomLightingPropertyInfo& LightProperty : LightClassProperties)
				{
					LightPropertiesByCategory.FindOrAdd(LightProperty.LightPropertyCategory).Add(LightProperty);
				}

				// Add properties for each property category
				for (const TPair<FString, TArray<FCustomLightingPropertyInfo>>& LightPropertyPair : LightPropertiesByCategory)
				{
					const FString& LightCategory = LightPropertyPair.Key;
					const TArray<FCustomLightingPropertyInfo>& CategoryProperties = LightPropertyPair.Value;

					MenuBuilder.BeginSection(NAME_None, FText::FromString(FName::NameToDisplayString(LightCategory, false)));
					
					for (const FCustomLightingPropertyInfo& PropertyInfo : CategoryProperties)
					{
						MenuBuilder.AddMenuEntry(
							PropertyInfo.LightProperty->GetDisplayNameText(),
							TAttribute<FText>(),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([InWeakModifierNode, PropertyInfo]()
								{
									if (UMovieGraphLightModifierNode* LightModifierMode = Cast<UMovieGraphLightModifierNode>(InWeakModifierNode.Get()))
									{
										if (LightModifierMode->HasCustomLightProperty(PropertyInfo.LightComponentClass, PropertyInfo.LightProperty))
										{
											const FScopedTransaction Transaction(LOCTEXT("RemoveCustomLightingProperty", "Remove Custom Lighting Property"));
											LightModifierMode->RemoveCustomLightProperty(PropertyInfo.LightComponentClass, PropertyInfo.LightProperty);
										}
										else
										{
											const FScopedTransaction Transaction(LOCTEXT("AddCustomLightingProperty", "Add Custom Lighting Property"));
											LightModifierMode->AddCustomLightProperty(PropertyInfo.LightComponentClass, PropertyInfo.LightProperty);
										}
									}
								}),
								FCanExecuteAction::CreateLambda([]()
								{
									return true;
								}),
								FIsActionChecked::CreateLambda([InWeakModifierNode, PropertyInfo]()
								{
									if (const UMovieGraphLightModifierNode* LightModifierMode = Cast<UMovieGraphLightModifierNode>(InWeakModifierNode.Get()))
									{
										return LightModifierMode->HasCustomLightProperty(PropertyInfo.LightComponentClass, PropertyInfo.LightProperty);
									}

									return false;
								})
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton);
					}

					MenuBuilder.EndSection();
				}
			}),
			false,
			FSlateIconFinder::FindIconForClass(LightClass),
			bShouldCloseWindowAfterMenuSelection);
	}

	CustomMenu.EndSection();	// "Light Actor Types" section

	return CustomMenu.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
