// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Scene.h"
#include "Graph/Nodes/MovieGraphLightModifierNode.h"
#include "IDetailCustomization.h"
#include "UObject/WeakInterfacePtr.h"

class IMovieGraphModifierNodeInterface;
class SMovieGraphModifierCollectionsList;

/** Customize how the Light Modifier node appears in the details panel. */
class FMovieGraphLightModifierCustomization final : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** Stores information about a property returned from GetCustomLightingProperties(). */
	struct FCustomLightingPropertyInfo
	{
		FCustomLightingPropertyInfo() = delete;
		FCustomLightingPropertyInfo(UClass* LightActorClass, UClass* LightComponentClass, FProperty* LightProperty, const FString& LightPropertyCategory);

		UClass* LightActorClass = nullptr;
		UClass* LightComponentClass = nullptr;
		FProperty* LightProperty = nullptr;
		FString LightPropertyCategory;
	};

	/** Gets the selected modifier node that this customization is displaying. */
	TWeakInterfacePtr<IMovieGraphModifierNodeInterface> GetSelectedModifierNode() const;

	/** Gets the contents of the "Custom" menu. */
	TSharedRef<SWidget> GetCustomMenuContents(const TWeakInterfacePtr<IMovieGraphModifierNodeInterface> InWeakModifierNode) const;

	/**
	 * Initializes the Intensity-related property to display correctly with its associated Intensity Units property. This mostly initializes the
	 * metadata on the Intensity-related property, but may initialize other aspects of the property as well.
	 */
	void InitializeIntensityProperty(const FName& InIntensityPropertyName, const FName& InIntensityUnitsPropertyName);

	/**
	 * Adds a Visibility lambda to the given property, which will conditionally show the property only if the node's Intensity Method
	 * matches the value provided in ShowWithMethod (eg, only show the property when the Intensity Method is set to PointRectSpot).
	 */
	void ShowPropertyConditionally(FName IntensityPropertyName, EMovieGraphLightModifierIntensityMethod ShowWithMethod);

	/** Handles value pre-change logic for the Intensity Units property. */
	void OnIntensityUnitsValuePreChange(FName IntensityUnitsPropertyName);

	/** Handles value-change logic for the Intensity Units property. */
	void OnIntensityUnitsValueChange(FName IntensityPropertyName, FName IntensityUnitsPropertyName) const;

	/**
	 * Gets all of the properties that should be displayed in the "Custom" properties menu.
	 */
	TArray<FCustomLightingPropertyInfo> GetCustomLightingProperties() const;

private:
	/** The details builder associated with the customization. */
	TWeakPtr<IDetailLayoutBuilder> DetailBuilder;

	/** Displays the collections which have been chosen. */
	TSharedPtr<SMovieGraphModifierCollectionsList> CollectionsList;

	/** The value of the Intensity Units property before it changes. */
	ELightUnits PreviousIntensityUnits = ELightUnits::Candelas;
};