// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Nodes/MovieGraphMaterialParameterCollectionModifierNode.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailGroup.h"
#include "InstancedPropertyBagStructureDataProvider.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customizes how the Material Parameter Collection Modifier node appears in the details panel. */
class FMovieGraphMaterialParameterCollectionModifierNodeCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphMaterialParameterCollectionModifierNodeCustomization>();
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		static const FName ScalarParamsGroupName = FName(TEXT("ScalarParams"));
		static const FName VectorParamsGroupName = FName(TEXT("VectorParams"));

		static const FText ScalarParamsGroupText = LOCTEXT("MPCNode_ScalarParameters", "Scalar Parameters");
		static const FText VectorParamsGroupText = LOCTEXT("MPCNode_VectorParameters", "Vector Parameters");
		
		for (const TWeakObjectPtr<UMovieGraphMaterialParameterCollectionModifierNode>& MPCNode
				: DetailBuilder.GetObjectsOfTypeBeingCustomized<UMovieGraphMaterialParameterCollectionModifierNode>())
		{
			// Regenerate the details when the MPC asset is changed.
			const TSharedRef<IPropertyHandle> MpcAssetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphMaterialParameterCollectionModifierNode, MaterialParameterCollection));
			MpcAssetHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailBuilder]()
			{
				DetailBuilder.ForceRefreshDetails();
			}));

			// Hide the "Properties" category that is shown by default for the DynamicProperties property that's inherited from UMovieGraphNode. In
			// this customization, we want to display these DynamicProperties in a non-default way.
			DetailBuilder.HideCategory("Properties");

			const TSharedRef<IPropertyHandle> DynamicPropertiesHandle = DetailBuilder.GetProperty(TEXT("DynamicProperties"), UMovieGraphNode::StaticClass());
			if (!DynamicPropertiesHandle->IsValidHandle())
			{
				continue;
			}

			// Get all child handles of the dynamic properties property bag (ie, the scalar/vector properties, plus their associated bOverride_* properties).
			TArray<TSharedPtr<IPropertyHandle>> PropertyBagChildHandles =
				DynamicPropertiesHandle->AddChildStructure(MakeShared<FInstancePropertyBagStructureDataProvider>(*MPCNode->GetMutableDynamicProperties_Unsafe()));

			// For each category (MPC asset name), keep track of the "Scalar"/"Vector" groups that have been added.
			TMap<FString, TMap<FName, IDetailGroup*>> CategoryToGroupMap;

			// Add all of the scalar/vector parameters.
			for (const TSharedPtr<IPropertyHandle>& ChildHandle : PropertyBagChildHandles)
			{
				const FString CategoryName = ChildHandle->GetMetaData(FName(TEXT("Category")));

				// If the category is empty, skip -- this is probably a bOverride_* property.
				if (CategoryName.IsEmpty())
				{
					continue;
				}

				// Add the category (MPC asset name) if it hasn't been added already.
				IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(FName(CategoryName));

				const bool bIsScalar = (ChildHandle->GetPropertyClass() == FFloatProperty::StaticClass());
				const FText& GroupText = bIsScalar ? ScalarParamsGroupText : VectorParamsGroupText;
				const FName& GroupName = bIsScalar ? ScalarParamsGroupName : VectorParamsGroupName;

				// Add the sub-group ("Scalar"/"Vector" parameters) if it hasn't been added already.
				IDetailGroup*& ParamGroup = CategoryToGroupMap.FindOrAdd(CategoryName).FindOrAdd(GroupName);
				if (!ParamGroup)
				{
					constexpr bool bForAdvanced = false;
					constexpr bool bStartExpanded = true;
					ParamGroup = &CategoryBuilder.AddGroup(GroupName, GroupText, bForAdvanced, bStartExpanded);
				}

				// Add the parameter to the details panel as a new row.
				ParamGroup->AddPropertyRow(ChildHandle.ToSharedRef());
			}
		}
	}
	//~ End IDetailCustomization interface
};

#undef LOCTEXT_NAMESPACE