// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailCustomization.h"
#include "IPropertyUtilities.h"
#include "Graph/MovieEdGraphNode.h"
#include "Graph/MovieGraphConfig.h"
#include "PropertyBagDetails.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SPinTypeSelector.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customize how members for a graph appear in the details panel. */
class FMovieGraphMemberCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphMemberCustomization>();
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}

	void OnNameChanged(const FText& InText, UMovieGraphMember* MovieGraphMember) const
	{
		NameEditableTextBox->SetError(FText::GetEmpty());

		FText Error;
		if (!MovieGraphMember->CanRename(InText, Error))
		{
			NameEditableTextBox->SetError(Error);
		}
	}
	
	void OnNameCommitted(const FText& InText, ETextCommit::Type Arg, UMovieGraphMember* MovieGraphMember) const
	{
		const FScopedTransaction Transaction(LOCTEXT("SetMemberName_Transaction", "Set Graph Member Name"));
		
		if (MovieGraphMember->SetMemberName(InText.ToString()))
		{
			NameEditableTextBox->SetError(FText::GetEmpty());
		}
	}

	bool IsPinTypeAllowed(const FEdGraphPinType& InPinType) const
	{
		// Property bags do not support interface types
		if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
		{
			return false;
		}

		return true;
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		static const FText NameLabel = LOCTEXT("MemberPropertyLabel_Name", "Name");
		static const FText CategoryLabel = LOCTEXT("MemberPropertyLabel_Category", "Category");
		static const FText TypeLabel = LOCTEXT("MemberPropertyLabel_Type", "Type");
		
		TSharedRef<IPropertyUtilities> PropUtils = DetailBuilder.GetPropertyUtilities();
		
		auto GetFilteredVariableTypeTree = [this](TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& TypeTree, ETypeTreeFilter TypeTreeFilter)
		{
			// Most types from the schema are allowed
			check(GetDefault<UEdGraphSchema_K2>());
			GetDefault<UPropertyBagSchema>()->GetVariableTypeTree(TypeTree, TypeTreeFilter);

			// Filter out disallowed types
			for (int32 Index = 0; Index < TypeTree.Num(); )
			{
				TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& PinType = TypeTree[Index];
				if (!PinType.IsValid())
				{
					return;
				}

				constexpr bool bForceLoadedSubCategoryObject = false;
				if (!IsPinTypeAllowed(PinType->GetPinType(bForceLoadedSubCategoryObject)))
				{
					TypeTree.RemoveAt(Index);
					continue;
				}

				for (int32 ChildIndex = 0; ChildIndex < PinType->Children.Num(); )
				{
					TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> Child = PinType->Children[ChildIndex];
					if (Child.IsValid())
					{
						if (!IsPinTypeAllowed(Child->GetPinType(bForceLoadedSubCategoryObject)))
						{
							PinType->Children.RemoveAt(ChildIndex);
							continue;
						}
					}
					++ChildIndex;
				}

				++Index;
			}
		};

		auto PinInfoChanged = [PropUtils](const FEdGraphPinType& PinType, const TWeakObjectPtr<UMovieGraphMember>& GraphMember)
		{
			// The SPinTypeSelector popup might outlive this details view, so the member could be invalid
			if (GraphMember.IsValid())
			{
				GraphMember->SetValueType(UMoviePipelineEdGraphNodeBase::GetValueTypeFromPinType(PinType), PinType.PinSubCategoryObject.Get());

				// Need the ForceRefresh to make sure the details panel refreshes immediately after the data type change.
				// Can result in a crash without it.
				PropUtils->ForceRefresh();
			}
		};
		
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

		for (const TWeakObjectPtr<UObject>& CustomizedObject : ObjectsBeingCustomized)
		{
			// Note: The graph members inherit the Value property from a base class, so the enable/disable state cannot
			// be driven by UPROPERTY metadata. Hence why this needs to be done w/ a details customization.
			
			const UMovieGraphInterfaceBase* InterfaceBase = Cast<UMovieGraphInterfaceBase>(CustomizedObject);
			TWeakObjectPtr WeakInterfaceBase(InterfaceBase);
			
			UMovieGraphMember* MemberObject = Cast<UMovieGraphMember>(CustomizedObject);
			if (!MemberObject)
			{
				continue;
			}

			const bool bIsEditable = MemberObject->IsEditable();
			
			// Add a custom row for the Name property (to allow for proper validation)
			IDetailCategoryBuilder& GeneralCategory = DetailBuilder.EditCategory("General");
			GeneralCategory.AddCustomRow(FText::GetEmpty())
			.FilterString(NameLabel)
			.IsEnabled(bIsEditable)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(NameLabel)
				.Font(DetailBuilder.GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(NameEditableTextBox, SEditableTextBox)
				.Text_Lambda([MemberObject]() { return FText::FromString(MemberObject->GetMemberName()); })
				.OnTextChanged(this, &FMovieGraphMemberCustomization::OnNameChanged, MemberObject)
				.OnTextCommitted(this, &FMovieGraphMemberCustomization::OnNameCommitted, MemberObject)
				.SelectAllTextWhenFocused(true)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];

			// Add a custom row for the Category property (its property is not edited directly in order to give the graph a chance to sort the
			// variables correctly). This is only applicable to variables.
			if (UMovieGraphVariable* VariableMember = Cast<UMovieGraphVariable>(MemberObject))
			{
				GeneralCategory.AddCustomRow(FText::GetEmpty())
				.FilterString(CategoryLabel)
				.IsEnabled(bIsEditable)
				.NameContent()
				[
					SNew(STextBlock)
					.Text(CategoryLabel)
					.ToolTipText(LOCTEXT("MemberPropertyTooltip_Category", "The category assigned to the variable. Use a '|' to separate category names to create a category hierarchy (eg, Settings|Resolution)."))
					.Font(DetailBuilder.GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SEditableTextBox)
					.Text_Lambda([VariableMember]()
					{
						return FText::FromString(VariableMember->GetCategory());
					})
					.OnTextCommitted_Lambda([VariableMember](const FText& InNewCategory, ETextCommit::Type TextCommitType)
					{
						const FScopedTransaction Transaction(LOCTEXT("SetVariableCategory", "Set Variable Category"));
						VariableMember->SetCategory(InNewCategory.ToString());
					})
					.Font(IDetailLayoutBuilder::GetDetailFont())
				];
			}
			
			TWeakObjectPtr<UMovieGraphMember> WeakMemberObject = MakeWeakObjectPtr(MemberObject);

			// If this is an interface (eg, input/output) only enable the type selector if it's not a branch and editable.
			// Otherwise, editability is the only factor in enable state.
			TAttribute<bool> IsEnabledAttribute = TAttribute<bool>::CreateLambda([WeakInterfaceBase, WeakMemberObject]()
			{
				if (WeakInterfaceBase.IsValid() && WeakMemberObject.IsValid())
				{
					return !WeakInterfaceBase->bIsBranch && WeakMemberObject->IsEditable();
				}
				if (WeakMemberObject.IsValid())
				{
					return WeakMemberObject->IsEditable();
				}

				return false;
			});

			// Add a PinTypeSelector widget to pick the data type the member uses
			IDetailCategoryBuilder& ValueCategory = DetailBuilder.EditCategory("Value");
			ValueCategory.AddCustomRow(FText::GetEmpty())
			.FilterString(TypeLabel)
			.IsEnabled(IsEnabledAttribute)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(TypeLabel)
				.Font(DetailBuilder.GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SBox)
				.Padding(FMargin(6.f, 0.f))
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SPinTypeSelector, FGetPinTypeTree::CreateLambda(GetFilteredVariableTypeTree))
					.TargetPinType_Lambda([WeakMemberObject]()
					{
						// The SPinTypeSelector popup might outlive this details view, so the member could be invalid
						if (!WeakMemberObject.IsValid())
						{
							return FEdGraphPinType();
						}

						constexpr bool bIsBranch = false;
						constexpr bool bIsWildcard = false;
						return UMoviePipelineEdGraphNodeBase::GetPinType(WeakMemberObject->GetValueType(), bIsBranch, bIsWildcard, WeakMemberObject->GetValueTypeObject());
					})
					.OnPinTypeChanged_Lambda(PinInfoChanged, WeakMemberObject)
					.Schema(GetDefault<UPropertyBagSchema>())
					.bAllowArrays(false)
					.TypeTreeFilter(ETypeTreeFilter::None)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			];

			// Display the property bag Value property. Needs to be added manually so it doesn't display in a multi-nested hierarchy.
			const TSharedRef<IPropertyHandle> ValueProperty = DetailBuilder.GetProperty("MultiValueContainer", UMovieGraphValueContainer::StaticClass());
			if (ValueProperty->IsValidHandle())
			{
				// Get the underlying "Values" property
				const TSharedPtr<IPropertyHandle> ValuesProperty = ValueProperty->GetChildHandle("Values");
				if (ValuesProperty.IsValid() && ValuesProperty->IsValidHandle())
				{
					DetailBuilder.HideProperty(ValueProperty);
					IDetailPropertyRow& ValuePropertyRow = DetailBuilder.EditCategory("Value").AddProperty(ValuesProperty);

					// Enable/disable the value property for inputs/outputs based on whether it is specified as a branch or not
					if (InterfaceBase)
					{
						ValuePropertyRow.IsEnabled(
							TAttribute<bool>::CreateLambda([WeakInterfaceBase]()
							{
								return WeakInterfaceBase.IsValid() ? !WeakInterfaceBase->bIsBranch : false;
							}));
					}

					// Enable/disable the value property for variables based on the editable state
					if (const UMovieGraphVariable* Variable = Cast<UMovieGraphVariable>(CustomizedObject))
					{
						ValuePropertyRow.IsEnabled(Variable->IsEditable());
					}
				}
			}
		}
	}
	//~ End IDetailCustomization interface

private:
	/** Text box for the "Name" property. */
	TSharedPtr<SEditableTextBox> NameEditableTextBox;
};

#undef LOCTEXT_NAMESPACE