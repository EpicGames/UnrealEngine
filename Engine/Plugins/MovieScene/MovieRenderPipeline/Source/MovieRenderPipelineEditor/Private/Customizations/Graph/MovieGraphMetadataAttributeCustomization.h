// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Nodes/MovieGraphSetMetadataAttributesNode.h"

#include "DetailLayoutBuilder.h"
#include "Internationalization/Internationalization.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SMoviePipelineFormatTokenAutoCompleteBox.h"

#define LOCTEXT_NAMESPACE "FMovieGraphMetadataAttributeCustomization"

/** Customize how a metadata attribute (FMovieGraphMetadataAttribute) appears in the details panel. */
class FMovieGraphMetadataAttributeCustomization : public IPropertyTypeCustomization
{

public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphMetadataAttributeCustomization>();
	}

protected:
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		StructPropertyHandle = InStructPropertyHandle;
		
		const TSharedRef<IPropertyHandle> NamePropertyHandle = StructPropertyHandle->GetChildHandle(
			GET_MEMBER_NAME_CHECKED(FMovieGraphMetadataAttribute, Name)).ToSharedRef();
		const TSharedRef<IPropertyHandle> ValuePropertyHandle = StructPropertyHandle->GetChildHandle(
			GET_MEMBER_NAME_CHECKED(FMovieGraphMetadataAttribute, Value)).ToSharedRef();
		const TSharedRef<IPropertyHandle> IsEnabledPropertyHandle = StructPropertyHandle->GetChildHandle(
			GET_MEMBER_NAME_CHECKED(FMovieGraphMetadataAttribute, bIsEnabled)).ToSharedRef();

		HeaderRow
		.NameContent()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 2)
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SCheckBox)
				.IsChecked(this, &FMovieGraphMetadataAttributeCustomization::IsCheckBoxChecked, IsEnabledPropertyHandle)
				.OnCheckStateChanged(this, &FMovieGraphMetadataAttributeCustomization::OnCheckBoxCheckStateChanged, IsEnabledPropertyHandle)
			]

			+ SHorizontalBox::Slot()
			.Padding(5, 2)
			.FillWidth(1.f)
			[
				SNew(SMoviePipelineFormatTokenAutoCompleteBox)
				.TextHandle(NamePropertyHandle)
				.Suggestions(SMoviePipelineFormatTokenAutoCompleteBox::GetFileNameFormatSuggestions())
				.IsEnabled(this, &FMovieGraphMetadataAttributeCustomization::IsEnabled, IsEnabledPropertyHandle)
				.HintText(LOCTEXT("MetadataAttributeNameInputHintText", "Attribute Name"))
			]
		]

		.ValueContent()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 2)
			.FillWidth(1.f)
			[
				SNew(SMoviePipelineFormatTokenAutoCompleteBox)
				.TextHandle(ValuePropertyHandle)
				.Suggestions(SMoviePipelineFormatTokenAutoCompleteBox::GetFileNameFormatSuggestions())
				.IsEnabled(this, &FMovieGraphMetadataAttributeCustomization::IsEnabled, IsEnabledPropertyHandle)
				.HintText(LOCTEXT("MetadataAttributeValueInputHintText", "Attribute Value"))
			]
		];
	}
	
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
	}
	
	ECheckBoxState IsCheckBoxChecked(TSharedRef<IPropertyHandle> PropertyHandle) const
	{
		bool bValue = false;
		if (PropertyHandle->IsValidHandle() && (PropertyHandle->GetValue(bValue) != FPropertyAccess::Fail))
		{
			return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Unchecked;
	}

	void OnCheckBoxCheckStateChanged(ECheckBoxState NewState, TSharedRef<IPropertyHandle> PropertyHandle)
	{
		if (PropertyHandle->IsValidHandle())
		{
			const bool bValue = (NewState == ECheckBoxState::Checked) ? true : false;
			PropertyHandle->SetValue(bValue);
		}
	}

	bool IsEnabled(TSharedRef<IPropertyHandle> InIsEnabledPropertyHandle) const
	{
		bool bValue = false;
		if (InIsEnabledPropertyHandle->IsValidHandle() && (InIsEnabledPropertyHandle->GetValue(bValue) != FPropertyAccess::Fail))
		{
			return bValue;
		}

		return false;
	}
};

#undef LOCTEXT_NAMESPACE
