// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphModifiersCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "Graph/Nodes/MovieGraphCollectionNode.h"
#include "Graph/Nodes/MovieGraphModifierNode.h"
#include "Widgets/Graph/SModifierCollectionsHeader.h"
#include "Widgets/Graph/SMovieGraphModifierCollectionsList.h"

#define LOCTEXT_NAMESPACE "MovieGraphModifiersCustomization"

TSharedRef<IDetailCustomization> FMovieGraphModifiersCustomization::MakeInstance()
{
	return MakeShared<FMovieGraphModifiersCustomization>();
}

void FMovieGraphModifiersCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	// The customization only supports editing a single Modifier node
	const TWeakInterfacePtr<IMovieGraphModifierNodeInterface> ModifierNode = GetSelectedModifierNode();
	if (!ModifierNode.IsValid())
	{
		return;
	}

	// Replace the "Collection" category row with a custom whole-row widget which includes an add-collection button
	IDetailCategoryBuilder& CollectionCategory = InDetailBuilder.EditCategory(FName("Collection"), FText::GetEmpty(), ECategoryPriority::Uncommon);
	CollectionCategory.HeaderContent
	(
		SNew(SMovieGraphCollectionsHeaderWidget)
		.WeakModifierInterface(ModifierNode)
		.OnCollectionPicked_Lambda([this](const FName CollectionName)
		{
			CollectionsList->Refresh();
		})
	, /* bWholeRowContent */ true);

	// Add a collections browser
	CollectionCategory.AddCustomRow(FText::GetEmpty())
	.WholeRowWidget
	[
		SAssignNew(CollectionsList, SMovieGraphModifierCollectionsList)
		.WeakModifierInterface(ModifierNode)
	];

	// For all modifiers added to the node, add a category for each, and add each modifier's EditAnywhere properties to the category
	for (UMovieGraphModifierBase* Modifier : ModifierNode->GetAllModifiers())
	{
		if (!Modifier)
		{
			continue;
		}
		
		const UClass* ModifierClass = Modifier->GetClass();
		const FText DisplayName = Modifier->GetModifierName();

		// Add category as "Uncommon" to display after the general modifier properties
		IDetailCategoryBuilder& Category = InDetailBuilder.EditCategory(FName(DisplayName.ToString()), DisplayName, ECategoryPriority::Uncommon);

		for (TFieldIterator<FProperty> PropertyIterator(ModifierClass); PropertyIterator; ++PropertyIterator)
		{
			// Add any EditAnywhere properties, but skip the bOverride_* properties.
			const FProperty* ModifierProperty = *PropertyIterator;
			if (ModifierProperty && ModifierProperty->HasAnyPropertyFlags(CPF_Edit) && !ModifierProperty->HasMetaData(TEXT("InlineEditConditionToggle")))
			{
				Category.AddExternalObjectProperty({Modifier}, ModifierProperty->GetFName());
			}
		}
	}
}

void FMovieGraphModifiersCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailBuilder = InDetailBuilder;
	CustomizeDetails(*InDetailBuilder);
}

TWeakInterfacePtr<IMovieGraphModifierNodeInterface> FMovieGraphModifiersCustomization::GetSelectedModifierNode() const
{
	if (const TSharedPtr<IDetailLayoutBuilder> DetailBuilderPin = DetailBuilder.Pin())
	{
		TArray<TWeakObjectPtr<UMovieGraphModifierNode>> ModifierNodes =
			DetailBuilderPin->GetObjectsOfTypeBeingCustomized<UMovieGraphModifierNode>();
		if (ModifierNodes.Num() != 1)
		{
			return nullptr;
		}

		const TWeakObjectPtr<UMovieGraphModifierNode> ModifierNode = ModifierNodes[0];
		if (ModifierNode.IsValid() && ModifierNode->Implements<UMovieGraphModifierNodeInterface>())
		{
			return TWeakInterfacePtr<IMovieGraphModifierNodeInterface>(ModifierNode.Get());
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE