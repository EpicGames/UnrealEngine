// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakInterfacePtr.h"

class IMovieGraphModifierNodeInterface;
class SMovieGraphModifierCollectionsList;

/** Customize how the Modifier node appears in the details panel. */
class FMovieGraphModifiersCustomization final : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** Gets the selected modifier node that this customization is displaying. */
	TWeakInterfacePtr<IMovieGraphModifierNodeInterface> GetSelectedModifierNode() const;

private:
	/** The details builder associated with the customization. */
	TWeakPtr<IDetailLayoutBuilder> DetailBuilder;

	/** Displays the collections which have been chosen. */
	TSharedPtr<SMovieGraphModifierCollectionsList> CollectionsList;
};