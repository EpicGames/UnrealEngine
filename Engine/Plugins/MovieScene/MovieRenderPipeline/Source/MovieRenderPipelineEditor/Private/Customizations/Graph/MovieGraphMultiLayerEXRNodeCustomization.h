// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"

#include "MovieGraphImageSequenceOutputNode.h"
#include "MovieRenderPipelineCoreModule.h"

/* Customizes how EXR output node properties appear in the details panel. */
class FMovieGraphMultiLayerEXRNodeCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphMultiLayerEXRNodeCustomization>();
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		const TArray<TWeakObjectPtr<UMovieGraphImageSequenceOutputNode_EXR>> ExrNodes =
			DetailBuilder.GetObjectsOfTypeBeingCustomized<UMovieGraphImageSequenceOutputNode_EXR>();
		if (ExrNodes.IsEmpty())
		{
			return;
		}

		const TSharedRef<IPropertyHandle> CompressionLevelProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphImageSequenceOutputNode_EXR, CompressionLevel));
		const TAttribute<EVisibility> CompressionLevelVisibleAttr = TAttribute<EVisibility>::Create([ExrNodes]()
		{
			for (const TWeakObjectPtr<UMovieGraphImageSequenceOutputNode_EXR>& ExrNode : ExrNodes)
			{
				if (const UMovieGraphImageSequenceOutputNode_EXR* ExrNodePtr = ExrNode.Get())
				{
					if (ExrNodePtr->Compression == EEXRCompressionFormat::DWAA || ExrNodePtr->Compression == EEXRCompressionFormat::DWAB)
					{
						return EVisibility::Visible;
					}
				}
			}

			return EVisibility::Collapsed;
		});
		if (IDetailPropertyRow* CompressionLevelRow = DetailBuilder.EditDefaultProperty(CompressionLevelProperty))
		{
			CompressionLevelRow->Visibility(CompressionLevelVisibleAttr);
		}

		// These properties are only relevant to single-layer EXRs, not multi-layer.
		const TSharedRef<IPropertyHandle> CompositeOntoFinalImageProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphImageSequenceOutputNode_EXR, bCompositeOntoFinalImage));
		const TSharedRef<IPropertyHandle> BurnInFileNameFormatProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphImageSequenceOutputNode_EXR, BurnInFileNameFormat));

		// The multi-layer EXR node inherits from the normal EXR node; this customization should only be applied to multi-layer.
		const TArray<TWeakObjectPtr<UMovieGraphImageSequenceOutputNode_MultiLayerEXR>> MultiLayerExrNodes =
			DetailBuilder.GetObjectsOfTypeBeingCustomized<UMovieGraphImageSequenceOutputNode_MultiLayerEXR>();
		if (MultiLayerExrNodes.IsEmpty())
		{
			return;
		}

		DetailBuilder.HideProperty(CompositeOntoFinalImageProperty);
		DetailBuilder.HideProperty(BurnInFileNameFormatProperty);
	}
	//~ End IDetailCustomization interface
};
