// Copyright Epic Games, Inc. All Rights Reserved.
#include "MovieGraphNamedResolutionCustomization.h"

#include "Graph/MovieGraphNamedResolution.h"
#include "Graph/MovieGraphProjectSettings.h"
#include "Graph/MovieGraphBlueprintLibrary.h"

#include "MovieRenderPipelineCoreModule.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "FMovieGraphNamedResolutionCustomization"

FMovieGraphNamedResolutionCustomization::~FMovieGraphNamedResolutionCustomization()
{
	UMovieGraphProjectSettings* MovieGraphProjectSettings = GetMutableDefault<UMovieGraphProjectSettings>();
	if (ensureAlwaysMsgf(MovieGraphProjectSettings, TEXT("%hs: Failed to find UMovieGraphProjectSettings!"), __FUNCTION__))
	{
		MovieGraphProjectSettings->OnSettingChanged().Remove(OnProjectSettingsModifiedHandle);
	}
}

void FMovieGraphNamedResolutionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;

	const TSharedPtr<IPropertyHandle> ResolutionPropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMovieGraphNamedResolution, Resolution));

	// Cache the property handles as we use them to do all of our math.
	ProfileNamePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMovieGraphNamedResolution, ProfileName));
	ResolutionXPropertyHandle = ResolutionPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, X));
	ResolutionYPropertyHandle = ResolutionPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, Y));

	// Set up option list for first time. If LoadedOption doesn't exist, use the first option in the list.
	UpdateComboBoxOptions();

	ComboBoxWidget = CreateComboBoxWidget();

	// We start with aspect-ratio locking on (it's not stored in the struct but is just a UI element). This could
	// be stored in a config later. We don't update the lock image here because it doesn't exist until the CustomizeChildren is called.
	LockedAspectRatio = CalculateAspectRatio();

	// Repopulate options when the project settings change
	BindToOnProjectSettingsModified();

	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget() 	// Show "Resolution" as the row name
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.Padding(0, 2, 8, 2)
		.HAlign(HAlign_Fill)
		[
			ComboBoxWidget.ToSharedRef()
		]
	];
}

void FMovieGraphNamedResolutionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Override the default CustomizeChildren to only create children for the X/Y (skipping ProfileName and Description)
	AddCustomRowForResolutionAxis(EAxis::X, StructBuilder, ResolutionXPropertyHandle->GeneratePathToProperty(), ResolutionXPropertyHandle->CreatePropertyNameWidget(), MakeAspectRatioLockWidget());
	AddCustomRowForResolutionAxis(EAxis::Y, StructBuilder, ResolutionYPropertyHandle->GeneratePathToProperty(), ResolutionYPropertyHandle->CreatePropertyNameWidget());
	UpdateAspectRatioLockImage();
}

TSharedRef<SWidget> FMovieGraphNamedResolutionCustomization::MakeWidgetForOptionInDropdown(FName InOption) const
{
	// This is called when the drop-down is opened to create a widget for each row in the drop-down
	const bool bSuppressResolution = InOption == FMovieGraphNamedResolution::CustomEntryName;

	return
		SNew(STextBlock)
		.Text(this, &FMovieGraphNamedResolutionCustomization::GetWidgetTextForOption, InOption, bSuppressResolution)
		.ToolTipText(this, &FMovieGraphNamedResolutionCustomization::GetWidgetTooltipTextForOption, InOption)
		.Font(this, &FMovieGraphNamedResolutionCustomization::GetWidgetFontForOptions);
}

TSharedRef<SWidget> FMovieGraphNamedResolutionCustomization::MakeWidgetForSelectedItem() const
{
	// This is called once when the customization is created to create the widget shown at the "root"
	// of the combo-box (not the individual items in the dropdown).
	return
		SNew(STextBlock)
		.Text(this, &FMovieGraphNamedResolutionCustomization::GetWidgetTextForCurrent)
		.ToolTipText(this, &FMovieGraphNamedResolutionCustomization::GetWidgetTooltipTextForCurrent)
		.Font(this, &FMovieGraphNamedResolutionCustomization::GetWidgetFontForOptions);
}

TSharedRef<SWidget> FMovieGraphNamedResolutionCustomization::CreateComboBoxWidget()
{
	FName LoadedOption = NAME_None;
	ProfileNamePropertyHandle->GetValue(LoadedOption);

	return
		SNew(SComboBox<FName>)
		.OnComboBoxOpening(this, &FMovieGraphNamedResolutionCustomization::UpdateComboBoxOptions)
		.OptionsSource(&ComboBoxOptions)
		.OnSelectionChanged(this, &FMovieGraphNamedResolutionCustomization::OnComboBoxSelectionChanged)
		.OnGenerateWidget(this, &FMovieGraphNamedResolutionCustomization::MakeWidgetForOptionInDropdown)
		.InitiallySelectedItem(LoadedOption)
		[
			MakeWidgetForSelectedItem()
		];
}

FText FMovieGraphNamedResolutionCustomization::GetWidgetTextForOption(const FName InOption, const bool bSuppressResolution) const
{
	// We always try to look up the latest named resolution from the project settings to ensure
	// what is displayed in the UI is in sync with the project settings.
	FMovieGraphNamedResolution NamedResolution;

	if (UMovieGraphBlueprintLibrary::IsNamedResolutionValid(InOption))
	{
		NamedResolution = UMovieGraphBlueprintLibrary::NamedResolutionFromProfile(InOption);
	}
	else
	{
		FIntPoint CurrentInternalSize;
		ResolutionXPropertyHandle->GetValue(CurrentInternalSize.X);
		ResolutionYPropertyHandle->GetValue(CurrentInternalSize.Y);

		// Doing this via the BlueprintLibrary just for consistency since it creates with the correct "Custom" name.
		NamedResolution = UMovieGraphBlueprintLibrary::NamedResolutionFromSize(CurrentInternalSize.X, CurrentInternalSize.Y);
	}

	FNumberFormattingOptions FormattingOptions;
	FormattingOptions.UseGrouping = false;

	if (bSuppressResolution)
	{
		return FText::Format(LOCTEXT("NamedResolutionComboboxItemWidgetText_NoResolution", "{0}"),
			FText::FromName(NamedResolution.ProfileName));
	}
	else
	{
		return FText::Format(LOCTEXT("NamedResolutionComboboxItemWidgetText", "{0} - {1}x{2}"),
			FText::FromName(NamedResolution.ProfileName),
			FText::AsNumber(NamedResolution.Resolution.X, &FormattingOptions),
			FText::AsNumber(NamedResolution.Resolution.Y, &FormattingOptions));
	}
}

FText FMovieGraphNamedResolutionCustomization::GetWidgetTooltipTextForOption(const FName InOption) const
{
	// We always try to look up the latest named resolution from the project settings to ensure
	// what is displayed in the UI is in sync with the project settings.
	FMovieGraphNamedResolution NamedResolution;

	if (UMovieGraphBlueprintLibrary::IsNamedResolutionValid(InOption))
	{
		NamedResolution = UMovieGraphBlueprintLibrary::NamedResolutionFromProfile(InOption);
	}
	else
	{
		FIntPoint CurrentInternalSize;
		ResolutionXPropertyHandle->GetValue(CurrentInternalSize.X);
		ResolutionYPropertyHandle->GetValue(CurrentInternalSize.Y);

		// Doing this via the BlueprintLibrary just for consistency since it creates with the correct "Custom" name.
		NamedResolution = UMovieGraphBlueprintLibrary::NamedResolutionFromSize(CurrentInternalSize.X, CurrentInternalSize.Y);
	}

	return FText::FromString(NamedResolution.Description);
}

FText FMovieGraphNamedResolutionCustomization::GetWidgetTextForCurrent() const
{
	FName LoadedOption = NAME_None;
	if (ProfileNamePropertyHandle->GetValue(LoadedOption) == FPropertyAccess::Result::Success)
	{
		// Will automatically handle missing/invalid options.
		const bool bSuppressResolution = false;
		return GetWidgetTextForOption(LoadedOption, bSuppressResolution);
	}

	return FText();
}

FText FMovieGraphNamedResolutionCustomization::GetWidgetTooltipTextForCurrent() const
{
	FName LoadedOption = NAME_None;
	if (ProfileNamePropertyHandle->GetValue(LoadedOption) == FPropertyAccess::Result::Success)
	{
		// Will automatically handle missing/invalid options.
		return GetWidgetTooltipTextForOption(LoadedOption);
	}

	return FText();
}

void FMovieGraphNamedResolutionCustomization::UpdateComboBoxOptions()
{
	ComboBoxOptions.Reset(ComboBoxOptions.Num());

	const UMovieGraphProjectSettings* MovieGraphProjectSettings = GetDefault<UMovieGraphProjectSettings>();
	if (ensureAlwaysMsgf(MovieGraphProjectSettings, TEXT("%hs: Failed to find UMovieGraphProjectSettings!"), __FUNCTION__))
	{
		for (const FMovieGraphNamedResolution& Resolution : MovieGraphProjectSettings->DefaultNamedResolutions)
		{
			ComboBoxOptions.AddUnique(Resolution.ProfileName);
		}
	}

	// We need to always add "Custom" so there's something there if the project settings are empty,
	// and because without it, after changing the value to "Custom" (in the underlying resolution)
	// the drop-down doesn't work the first time you click on it due to a ComboBox issue.
	ComboBoxOptions.AddUnique(FMovieGraphNamedResolution::CustomEntryName);
}

void FMovieGraphNamedResolutionCustomization::BindToOnProjectSettingsModified()
{
	UMovieGraphProjectSettings* MovieGraphProjectSettings = GetMutableDefault<UMovieGraphProjectSettings>();
	if (ensureAlwaysMsgf(MovieGraphProjectSettings, TEXT("%hs: Unable to get UMovieGraphProjectSettings."), __FUNCTION__))
	{
		OnProjectSettingsModifiedHandle = MovieGraphProjectSettings->OnSettingChanged().AddSP(this, &FMovieGraphNamedResolutionCustomization::OnProjectSettingsChanged);
	}
}

void FMovieGraphNamedResolutionCustomization::UpdateAspectRatioLockImage()
{
	// The widget will only exist if the dropdown is expanded, so it's okay
	// to call this without that. But it should be called again after the widget is created.
	if (AspectRatioLockImage)
	{
		AspectRatioLockImage->SetImage(GetAspectRatioLockBrush());
		AspectRatioLockImage->SetToolTipText(GetAspectRatioLockTooltipText());
	}
}

void FMovieGraphNamedResolutionCustomization::ToggleAspectRatioLock()
{
	if (LockedAspectRatio.IsSet())
	{
		LockedAspectRatio.Reset();
	}
	else
	{
		LockedAspectRatio = CalculateAspectRatio();
	}

	UpdateAspectRatioLockImage();
}

const FSlateBrush* FMovieGraphNamedResolutionCustomization::GetAspectRatioLockBrush() const
{
	return LockedAspectRatio.IsSet() ? FAppStyle::GetBrush(TEXT("Icons.Link")) : FAppStyle::GetBrush(TEXT("Icons.Unlink"));
}

FText FMovieGraphNamedResolutionCustomization::GetAspectRatioLockTooltipText() const
{
	return LockedAspectRatio.IsSet() ?
		FText::Format(LOCTEXT("LockedAspectRatioTooltipFormat", "Click to unlock Aspect Ratio ({0})"), FText::AsNumber(LockedAspectRatio.GetValue())) :
		LOCTEXT("UnlockedAspectRatioTooltip", "Click to lock the Aspect Ratio");
}

TSharedRef<SWidget> FMovieGraphNamedResolutionCustomization::MakeAspectRatioLockWidget()
{
	return SNew(SButton)
	.OnClicked_Lambda([this]()
	{
		ToggleAspectRatioLock();
		
		return FReply::Handled();
	})
	.ContentPadding(FMargin(0, 0, 4, 0))
	.ButtonStyle(FAppStyle::Get(), "NoBorder")
	[
		SAssignNew(AspectRatioLockImage, SImage)
		.ColorAndOpacity(FSlateColor::UseForeground())
	];
}

float FMovieGraphNamedResolutionCustomization::CalculateAspectRatio()
{
	float X = 1;
	float Y = 1;
	
	FIntPoint EffectiveResolution = GetEffectiveResolution();
	X = (float)EffectiveResolution.X;
	Y = (float)EffectiveResolution.Y;
	
	return X / Y;
}

namespace MovieGraph::Private
{
	int32 RoundToNearestEvenNumber(double InNumberToRound)
	{
		// Round to nearest even
		const int32 Floored = FMath::FloorToInt(InNumberToRound);
		const int32 Ceiled = FMath::CeilToInt(InNumberToRound);

		return Floored % 2 == 0 ? Floored : Ceiled;
	}
}

FIntPoint FMovieGraphNamedResolutionCustomization::GetEffectiveResolution() const
{
	// To get the resolution that we display, we need to look it up in the Project Settings first,
	// because they might have changed the project setting value since the last time the struct was changed.
	// If we can't find it in the project setting (or it is Custom) then we return the values inside the struct.
	FIntPoint Resolution;
	FName ProfileName;
	ProfileNamePropertyHandle->GetValue(ProfileName);

	if (UMovieGraphBlueprintLibrary::IsNamedResolutionValid(ProfileName))
	{
		FMovieGraphNamedResolution ResolutionFromPreset = UMovieGraphBlueprintLibrary::NamedResolutionFromProfile(ProfileName);
		Resolution = ResolutionFromPreset.Resolution;
	}
	else
	{
		// If it's not a valid profile, we use the stored value.
		ResolutionXPropertyHandle->GetValue(Resolution.X);
		ResolutionYPropertyHandle->GetValue(Resolution.Y);
	}

	return Resolution;
}

int32 FMovieGraphNamedResolutionCustomization::GetCurrentlySelectedResolutionByAxis(const EAxis::Type Axis)
{
	if (!ResolutionXPropertyHandle || !ResolutionYPropertyHandle || !ProfileNamePropertyHandle)
	{
		return INDEX_NONE;
	}

	FIntPoint Resolution = GetEffectiveResolution();
	return Axis == EAxis::X ? Resolution.X : Resolution.Y;
}

void FMovieGraphNamedResolutionCustomization::OnCustomSliderBeginMovement()
{
	// Wrap all interactive changes into a single commit while dragging.
	InteractiveResolutionEditTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("Transaction_EditResolution", "Edit Resolution"));
}

void FMovieGraphNamedResolutionCustomization::OnCustomSliderEndMovement(uint32 NewValue)
{
	// Reset our open transaction to commit it now that they're done interactively changing values.
	InteractiveResolutionEditTransaction.Reset();
}

void FMovieGraphNamedResolutionCustomization::OnCustomSliderValueChanged(uint32 NewValue, EAxis::Type Axis)
{
	if (!ResolutionXPropertyHandle || !ResolutionYPropertyHandle || !ProfileNamePropertyHandle)
	{
		return;
	}

	// If they directly edit the value (via typing it in and hitting enter) OnCustomSliderValueChanged
	// will be called, so we need to make a transaction here. This function is also called when being
	// dragged, so we use OnCustomSliderBeginMovement/OnCustomSliderEndMovement to create a transaction
	// which wraps all the smaller transactions to group them together into one undo/redo event.
	FScopedTransaction Transaction(LOCTEXT("Transaction_EditResolution", "Edit Resolution"));

	// Check to see if we need to convert from an existing profile resolution to a custom one,
	// otherwise the UI won't update with our new values.
	FName ProfileName;
	if (ProfileNamePropertyHandle->GetValue(ProfileName) == FPropertyAccess::Success)
	{
		if (ProfileName != FMovieGraphNamedResolution::CustomEntryName)
		{
			// Convert us from our existing profile to be "Custom".
			ProfileNamePropertyHandle->SetValue(FMovieGraphNamedResolution::CustomEntryName);
		}
	}

	// Get our XY resolution out of the properties
	FIntPoint NewResolution;
	ResolutionXPropertyHandle->GetValue(NewResolution.X);
	ResolutionYPropertyHandle->GetValue(NewResolution.Y);
	
	// Update the correct axis on the Named Resolution struct, potentially constrained by aspect ratio.
	switch (Axis)
	{
		case EAxis::X: 
		{ 
			NewResolution.X = NewValue;
			if (LockedAspectRatio.IsSet())
			{
				const double NewYResolution = (double)NewResolution.X / LockedAspectRatio.GetValue();
				NewResolution.Y = MovieGraph::Private::RoundToNearestEvenNumber(NewYResolution);
			}
			break;
		}
		case EAxis::Y: 
		{ 
			NewResolution.Y = NewValue;
			if (LockedAspectRatio.IsSet())
			{
				const double NewXResolution = (double)NewResolution.Y * LockedAspectRatio.GetValue();
				NewResolution.X = MovieGraph::Private::RoundToNearestEvenNumber(NewXResolution);
			}
			break; 
		}
	}

	// Update both X and Y as they both may have changed due to aspect ratio locking.
	ResolutionXPropertyHandle->SetValue(NewResolution.X);
	ResolutionYPropertyHandle->SetValue(NewResolution.Y);
}

void FMovieGraphNamedResolutionCustomization::AddCustomRowForResolutionAxis(EAxis::Type Axis, IDetailChildrenBuilder& StructBuilder, const FString& FilterString, TSharedRef<SWidget> NameContentWidget, TSharedPtr<SWidget> AspectRatioLockExtensionWidget)
{
	const FSlateFontInfo PropertyFontStyle = FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") );
	const TOptional<uint32> PropertyMaxValue = TOptional<uint32>();

	FDetailWidgetRow& CustomRow = StructBuilder.AddCustomRow(FText::FromString(FilterString))
	.NameContent()
	[
		NameContentWidget
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	[
		// We make our own widget so that we can edit two variables at once in the callback (for aspect-ratio linked X and Y)
		SNew(SBox)
		// Maintain spacing when there would be no extension widget
		.Padding(0, 0, Axis == EAxis::Y ? 24 /* Assuming image is 16x16 + 8 padding */ : 0, 0) 
		[
			SNew(SNumericEntryBox<uint32>)
			.Font(PropertyFontStyle)
			.AllowSpin(true)
			.MinSliderValue(2)
			.MinValue(2)
			.MaxValue(PropertyMaxValue)
			.MaxSliderValue(PropertyMaxValue)
			.Delta(2)
			.Value_Lambda([this, Axis] (){ return GetCurrentlySelectedResolutionByAxis(Axis); }) // Lambda to bypass const requirement
			.OnValueChanged(this, &FMovieGraphNamedResolutionCustomization::OnCustomSliderValueChanged, Axis)
			.OnBeginSliderMovement(this, &FMovieGraphNamedResolutionCustomization::OnCustomSliderBeginMovement)
			.OnEndSliderMovement(this, &FMovieGraphNamedResolutionCustomization::OnCustomSliderEndMovement)
		]
	]
	.OverrideResetToDefault(FResetToDefaultOverride::Hide()); // Prevent showing Reset to Default

	if (AspectRatioLockExtensionWidget.IsValid())
	{
		CustomRow
		.ExtensionContent()
		[
			AspectRatioLockExtensionWidget.ToSharedRef()
		];
	}
}


void FMovieGraphNamedResolutionCustomization::OnProjectSettingsChanged(UObject*, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMovieGraphProjectSettings, DefaultNamedResolutions))
	{
		UpdateComboBoxOptions();
	}
}

void FMovieGraphNamedResolutionCustomization::OnComboBoxSelectionChanged(const FName NewValue, ESelectInfo::Type InType)
{
	// Ensure the value selected was valid, if so, update our internal copy to match.
	FMovieGraphNamedResolution NamedResolution;
	if (UMovieGraphBlueprintLibrary::IsNamedResolutionValid(NewValue))
	{
		// Fetch the latest values from the Project Settings to update the internal copy of the resolution with.
		NamedResolution = UMovieGraphBlueprintLibrary::NamedResolutionFromProfile(NewValue);
	}
	else
	{
		// If it's not a valid resolution, we still want to run this code to switch it to "Custom", otherwise clicking "Custom"
		// in the dropdown doesn't do anything.
		FIntPoint CurrentInternalSize;
		ResolutionXPropertyHandle->GetValue(CurrentInternalSize.X);
		ResolutionYPropertyHandle->GetValue(CurrentInternalSize.Y);

		// Doing this via the BlueprintLibrary just for consistency since it creates with the correct "Custom" name.
		NamedResolution = UMovieGraphBlueprintLibrary::NamedResolutionFromSize(CurrentInternalSize.X, CurrentInternalSize.Y);
	}

	const TSharedPtr<IPropertyHandle> PinnedStructPropertyHandle = StructPropertyHandle.Pin();
	if (!PinnedStructPropertyHandle)
	{
		return;
	}

	void* StructData = nullptr;
	if (PinnedStructPropertyHandle->GetValueData(StructData) == FPropertyAccess::Success && StructData != nullptr)
	{
		// We have to call all of these by hand because we're directly editing the memory and not using property handles.
		FScopedTransaction ScopedTransaction(LOCTEXT("Transaction_AssignPreset", "Assign Preset"));
		PinnedStructPropertyHandle->NotifyPreChange();

		PinnedStructPropertyHandle->EnumerateRawData([NamedResolution](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
			{
				FMovieGraphNamedResolution* OldData = static_cast<FMovieGraphNamedResolution*>(RawData);
				FMovieGraphNamedResolution::StaticStruct()->CopyScriptStruct(RawData, &NamedResolution);
				return true;
			});

		PinnedStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PinnedStructPropertyHandle->NotifyFinishedChangingProperties();
	}
}

#undef LOCTEXT_NAMESPACE
