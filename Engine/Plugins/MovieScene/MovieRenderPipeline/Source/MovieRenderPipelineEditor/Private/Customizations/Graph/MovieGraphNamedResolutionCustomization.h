// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/MovieGraphNamedResolution.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "ScopedTransaction.h"

#define UE_API MOVIERENDERPIPELINEEDITOR_API

/* Customizes how named resolutions are displayed in the details pane. */
class FMovieGraphNamedResolutionCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphNamedResolutionCustomization>();
	}

	UE_API virtual ~FMovieGraphNamedResolutionCustomization() override;

protected:

	/**
	* Update ComboBoxOptions with the latest values in the Project Settings,
	* and adds the "Custom" entry as well.
	*/
	UE_API void UpdateComboBoxOptions();

	/*
	* Gets the text that should be displayed in the widget for the specified option.
	* In the form of "{name} - {x}x{y}" unless bSuppressResolution is true, at which
	* point it will just be in the form of "{name}"
	*/
	UE_API FText GetWidgetTextForOption(const FName InOption, const bool bSuppressResolution) const;
	/** The above, but for the currently selected (ie: "root" of combo box). */
	UE_API FText GetWidgetTextForCurrent() const;
	/** Gets the description property for the specified option */
	UE_API FText GetWidgetTooltipTextForOption(const FName InOption) const;
	/** The above, but for the currently selected (ie: "root" of combo box). */
	UE_API FText GetWidgetTooltipTextForCurrent() const;


	FSlateFontInfo GetWidgetFontForOptions() const
	{
		return IDetailLayoutBuilder::GetDetailFont();
	}

	/** Called to generate a widget for each entry in the dropdown when you open the dropdown. */
	UE_API TSharedRef<SWidget> MakeWidgetForOptionInDropdown(FName InOption) const;
	/** Called to generate the widget for the "root" of the combo box that is always visible. */
	UE_API TSharedRef<SWidget> MakeWidgetForSelectedItem() const;

	/** Create the actual ComboBox Widget */
	UE_API TSharedRef<SWidget> CreateComboBoxWidget();

	/** Binds to the Project Settings to know when a user has added or removed a preset. */
	UE_API void BindToOnProjectSettingsModified();

	/** Get the icon to use for the Aspect Ratio Locked widget, could be a locked or unlocked icon. */
	UE_API const FSlateBrush* GetAspectRatioLockBrush() const;

	/** Get the current tooltip for the aspect ratio lock */
	UE_API FText GetAspectRatioLockTooltipText() const;
	
	//~ Begin IPropertyTypeCustomization interface
	UE_API virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	UE_API virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization interface

	/** Update the aspect ratio lock image based on current lock setting. */
	UE_API void UpdateAspectRatioLockImage();

	/** On click, toggle the current lock state. */
	UE_API void ToggleAspectRatioLock();

	/** Create the Aspect Ratio Lock widget that is added in the customization. */
	UE_API TSharedRef<SWidget> MakeAspectRatioLockWidget();

	/** Calculates the aspect ratio of the current resolution. */
	UE_API float CalculateAspectRatio();

	/** 
	* Gets the effective resolution for the current resolution. For profiles this looks it up in the Project Settings, 
	* so that if the project setting has been changed since the resolution was last updated, we reflect the correct
	* value. MRQ will also look it up from the Project Setting at runtime.
	*/
	UE_API FIntPoint GetEffectiveResolution() const;

	/** Called when dragging the slider starts. Used to open a transaction to encapsulate all changes into one. */
	UE_API void OnCustomSliderBeginMovement();
	/** Called when the slider stops dragging. Closes the transaction. */
	UE_API void OnCustomSliderEndMovement(uint32 NewValue);

	/** Gets the resolution for the given axis. */
	UE_API int32 GetCurrentlySelectedResolutionByAxis(const EAxis::Type Axis);

	/** Called when dragging the slider, or after typing in a number and hitting enter. */
	UE_API void OnCustomSliderValueChanged(uint32 NewValue, EAxis::Type Axis);

	/** Adds a row to the customization for the given axis. */
	UE_API void AddCustomRowForResolutionAxis(
		EAxis::Type Axis, IDetailChildrenBuilder& StructBuilder, const FString& FilterString,
		TSharedRef<SWidget> NameContentWidget, TSharedPtr<SWidget> AspectRatioLockExtensionWidget = nullptr);


	/** Callback bound when project setting changes. */
	UE_API void OnProjectSettingsChanged(UObject*, FPropertyChangedEvent& PropertyChangedEvent);

	/** Called when the user selects a new preset from the combo box dropdown. */
	UE_API void OnComboBoxSelectionChanged(const FName NewValue, ESelectInfo::Type InType);

	/** The widget that allows the end user to select the output resolution */
	TSharedPtr<SWidget> ComboBoxWidget;

	/** The image that indicates whether the aspect ratio is locked. */
	TSharedPtr<SImage> AspectRatioLockImage;

	/** Weak ptr to the handle of the struct which owns this customization */
	TWeakPtr<IPropertyHandle> StructPropertyHandle;

	/** Handle to the "Resolution.X" property on the edited object. */
	TSharedPtr<IPropertyHandle> ResolutionXPropertyHandle;
	/** Handle to the "Resolution.Y" property on the edited object. */
	TSharedPtr<IPropertyHandle> ResolutionYPropertyHandle;
	/** Handle to the "ProfileName" property on the edited object. */
	TSharedPtr<IPropertyHandle> ProfileNamePropertyHandle;

	/**
	 * The aspect ratio of the currently selected resolution.
	 * Updates when the aspect ratio lock is applied, a named resolution is selected,
	 * or when the 'Custom' resolution values are changed. Used to track if Aspect
	 * ratio lock is enabled or not.
	 */
	TOptional<float> LockedAspectRatio;

	/**
	 * The options for the combo box, populated from UMovieGraphProjectSettings::DefaultNamedResolutions.
	 * A 'custom' entry is added to the end of the list, representing an arbitrary user-defined resolution.
	 */
	TArray<FName> ComboBoxOptions;

	/**
	 * A binding that updates the ComboBoxOptions when UMovieGraphProjectSettings::DefaultNamedResolutions changes
	 */
	FDelegateHandle OnProjectSettingsModifiedHandle;

	/**
	* If valid, this means that a slide operation is currently in progress and that the changes will be encapsulated into this transaction. 
	*/
	TUniquePtr<FScopedTransaction> InteractiveResolutionEditTransaction;
};

#undef UE_API
