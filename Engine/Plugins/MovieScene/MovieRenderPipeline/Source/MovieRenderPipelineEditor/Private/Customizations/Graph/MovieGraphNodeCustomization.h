// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Graph/MovieGraphNode.h"
#include "IDetailCustomization.h"
#include "MovieGraphCustomizationUtils.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customize how graph nodes appear in the details panel. */
class FMovieGraphNodeCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphNodeCustomization>();
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		AddTokenAutocompleteWidgetToAllEligibleProperties(DetailBuilder);

		// Hide the "Properties" category, which houses the dynamic properties, if there are no dynamic properties
		// on the node. An empty category in the details panel is messy/confusing.
		//
		// Some nodes may also request that specific properties (most likely inherited) are hidden.
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
		for (const TWeakObjectPtr<UObject>& CustomizedObject : ObjectsBeingCustomized)
		{
			if (const UMovieGraphNode* Node = Cast<UMovieGraphNode>(CustomizedObject))
			{
				if (Node->GetDynamicPropertyDescriptions().IsEmpty())
				{
					DetailBuilder.HideCategory("Properties");
				}

				for (const TPair<FName, UClass*>& HiddenProperty : Node->GetHiddenProperties())
				{
					DetailBuilder.HideProperty(HiddenProperty.Key, HiddenProperty.Value);
				}
			}
		}

		// Also hide the "Properties" category if more than one object is selected. There needs to be some more work done
		// in property bags to be able to handle this condition without crashing in many cases.
		if (ObjectsBeingCustomized.Num() > 1)
		{
			DetailBuilder.HideCategory("Properties");
		}

		// Collapse the Tags category by default because it's not used by most users. Set the sort order to make Tags appear last in the list of
		// categories (1000 * Uncommon == starting index for all categories with "Uncommon" sort order, and the +1000 is to move the category past all
		// other categories with "Uncommon" sort order).
		IDetailCategoryBuilder& TagsCategory = DetailBuilder.EditCategory("Tags");
		TagsCategory.InitiallyCollapsed(true).SetSortOrder(1000 * ECategoryPriority::Uncommon + 1000);

		// Show the "Naming" category, which is on renderer nodes, right before the "Tags" category.
		DetailBuilder.EditCategory("Naming").SetSortOrder(TagsCategory.GetSortOrder() - 1);

		// The File Output category should be shown first in all output nodes that have this category.
		DetailBuilder.EditCategory("File Output").SetSortOrder(ECategoryPriority::Important);
	}
	//~ End IDetailCustomization interface

private:
	/** Adds the token autocomplete widget to all properties that have the MrgTokenAutocomplete UPROPERTY metadata. */
	void AddTokenAutocompleteWidgetToAllEligibleProperties(IDetailLayoutBuilder& DetailBuilder)
	{
		static const FName MrgTokenAutocompleteMetaDataKey(TEXT("MrgTokenAutocomplete"));

		// Find all of the properties that should have the autocomplete widget used on them. Note that this won't find properties in structs.
		// Structs need separate details customizations that add autocomplete if necessary.
		TArray<TSharedPtr<IPropertyHandle>> AutocompleteProperties;
		TArray<FName> CategoryNames;
		DetailBuilder.GetCategoryNames(CategoryNames);
		for (const FName& CategoryName : CategoryNames)
		{
			// Only go through top-level categories. Subcategories are currently not supported here (EditCategory() does not
			// work properly with category names like "Foo|Bar").
			if (CategoryName.ToString().Contains(TEXT("|")))
			{
				continue;
			}

			IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(CategoryName);

			TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
			CategoryBuilder.GetDefaultProperties(CategoryProperties);

			for (const TSharedRef<IPropertyHandle>& CategoryProperty : CategoryProperties)
			{
				if (CategoryProperty->HasMetaData(MrgTokenAutocompleteMetaDataKey))
				{
					AutocompleteProperties.Add(CategoryProperty);
				}
			}
		}

		// Add the autocomplete widget to all properties that have the appropriate UPROPERTY metadata
		for (const TSharedPtr<IPropertyHandle>& PropertyToModify : AutocompleteProperties)
		{
			if (IDetailPropertyRow* PropertyRow = DetailBuilder.EditDefaultProperty(PropertyToModify))
			{
				UE::MovieRenderPipelineEditor::Private::AddTokenAutocompleteToPropertyRow(PropertyRow);
			}
		}
	}
};

#undef LOCTEXT_NAMESPACE