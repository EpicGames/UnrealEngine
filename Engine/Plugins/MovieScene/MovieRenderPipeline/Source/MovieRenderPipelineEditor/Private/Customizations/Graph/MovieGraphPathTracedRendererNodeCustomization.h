// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailLayoutBuilder.h"
#include "Graph/Nodes/MovieGraphPathTracerPassNode.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customize how the Path Traced Renderer node appears in the details panel. */
class FMovieGraphPathTracedRendererNodeCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphPathTracedRendererNodeCustomization>();
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		for (const TWeakObjectPtr<UMovieGraphPathTracerRenderPassNode>& WeakPathTracerNode : DetailBuilder.GetObjectsOfTypeBeingCustomized<UMovieGraphPathTracerRenderPassNode>())
		{
			const TAttribute<bool> FrameCountEnabledAttr = TAttribute<bool>::Create([WeakPathTracerNode]()
			{
				if (const TStrongObjectPtr<UMovieGraphPathTracerRenderPassNode> PathTracerNodePin = WeakPathTracerNode.Pin())
				{
					return PathTracerNodePin->bOverride_FrameCount > 0;
				}
				
				return false;
			});

			const TAttribute<EVisibility> FrameCountVisibleAttr = TAttribute<EVisibility>::Create([WeakPathTracerNode]()
			{
				if (const TStrongObjectPtr<UMovieGraphPathTracerRenderPassNode> PathTracerNodePin = WeakPathTracerNode.Pin())
				{
					return (PathTracerNodePin->DenoiserType == EMovieGraphPathTracerDenoiserType::Temporal) ? EVisibility::Visible : EVisibility::Collapsed;
				}
				
				return EVisibility::Collapsed;
			});

			// Update the "Frame Count" property to only be visible when the Temporal denoiser is active. Note that we need this customization over
			// simple EditCondition and EditConditionHides metadata on the UPROPERTY. In order to get the edit condition checkbox, the EditCondition
			// needs to reference ONLY bOverride_FrameCount (and cannot have any logic around the denoiser type). To get around that, we have to
			// define our own custom conditions here.
			const TSharedRef<IPropertyHandle> FrameCountHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphPathTracerRenderPassNode, FrameCount));
			DetailBuilder.EditDefaultProperty(FrameCountHandle)
				->Visibility(FrameCountVisibleAttr)
				.EditCondition(
					FrameCountEnabledAttr,
					FOnBooleanValueChanged::CreateLambda([WeakPathTracerNode](const bool bNewValue)
					{
						if (const TStrongObjectPtr<UMovieGraphPathTracerRenderPassNode> PathTracerNodePin = WeakPathTracerNode.Pin())
						{
							FScopedTransaction Transaction(LOCTEXT("FrameCountEditConditionChanged", "Edit Condition Changed"));
							PathTracerNodePin->Modify();
							PathTracerNodePin->bOverride_FrameCount = bNewValue;
						}
					})
			);
		}
	}
	//~ End IDetailCustomization interface
};

#undef LOCTEXT_NAMESPACE