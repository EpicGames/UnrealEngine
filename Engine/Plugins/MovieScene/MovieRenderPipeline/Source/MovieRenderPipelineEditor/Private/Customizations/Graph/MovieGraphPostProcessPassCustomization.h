// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "MovieGraphCustomizationUtils.h"
#include "MoviePipelineDeferredPasses.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customize how post process passes appear in the details panel. */
class FMovieGraphPostProcessPassCustomization final : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphPostProcessPassCustomization>();
	}

protected:
	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		HeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				InStructPropertyHandle->CreatePropertyValueWidget()
			];
	}

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		IDetailChildrenBuilder& StructBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		if (!InStructPropertyHandle->IsValidHandle())
		{
			return;
		}

		uint32 NumChildren;
		InStructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 Index = 0; Index < NumChildren; ++Index)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = InStructPropertyHandle->GetChildHandle(Index);
			IDetailPropertyRow& PropertyRow = StructBuilder.AddProperty(ChildHandle.ToSharedRef());

			// Give the "Name" property the token autocomplete widget
			if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FMoviePipelinePostProcessPass, Name))
			{
				UE::MovieRenderPipelineEditor::Private::AddTokenAutocompleteToPropertyRow(&PropertyRow);
			}
		}
	}
	//~ End IPropertyTypeCustomization interface
};

#undef LOCTEXT_NAMESPACE