// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Graph/MovieGraphQuickRenderSettings.h"
#include "IDetailCustomization.h"
#include "IDetailGroup.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "MovieGraphCustomizationUtils.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineQuickRenderUIState.h"
#include "MoviePipelineUtils.h"
#include "PropertyHandle.h"
#include "UObject/Package.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "MoviePipelineQuickRenderSettings"

/** Customize how properties for QuickRenderSettings appear in the details panel. */
class FMovieGraphQuickRenderSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphQuickRenderSettingsCustomization>();
	}

	virtual ~FMovieGraphQuickRenderSettingsCustomization() override = default;

	virtual void PendingDelete() override
	{
		// Unregister delegates. It's important to do this in PendingDelete() vs the destructor because the destructor is not called before the next
		// details panel is created (via ForceRefreshDetails()), leading to an exponential increase in the number of delegates registered.
		
		UPackage::PackageSavedWithContextEvent.RemoveAll(this);

		if (QuickRenderModeSettings.IsValid())
		{
			QuickRenderModeSettings->OnGraphChangedDelegate.RemoveAll(this);
		}
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override
	{
		DetailBuilder = InDetailBuilder.Get();
		CustomizeDetails(*InDetailBuilder);
	}

	void RefreshLayout(const FString&, UPackage*, FObjectPostSaveContext) const
	{
		DetailBuilder->ForceRefreshDetails();

		// Refresh the variable assignments (since the variables within the graph may have changed, thus the variable assignments need to be updated)
		MoviePipeline::RefreshVariableAssignments(QuickRenderModeSettings->GraphPreset.LoadSynchronous(), QuickRenderModeSettings->GraphVariableAssignments, QuickRenderModeSettings.Get());
	}

	void RefreshLayout() const
	{
		// Note: The settings object will take care of refreshing variable assignments on itself in the case where the graph preset asset is updated to be a different asset
		DetailBuilder->ForceRefreshDetails();
	}

	void UpdateFrameRangePropertyEnableState() const
	{
		const TAttribute<bool> IsEnabledAttr = TAttribute<bool>::CreateLambda([]() -> bool
		{
			const EMovieGraphQuickRenderMode CurrentWindowRenderMode = FMoviePipelineQuickRenderUIState::GetWindowQuickRenderMode();
			
			// Enabling the frame range properties only makes sense for modes that can render more than one frame.
			const bool bNewEnableState =
				(CurrentWindowRenderMode == EMovieGraphQuickRenderMode::CurrentSequence)
				|| (CurrentWindowRenderMode == EMovieGraphQuickRenderMode::UseViewportCameraInSequence);

			return bNewEnableState;
		});

		const TSharedRef<IPropertyHandle> FrameRangeTypeProperty =
			DetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphQuickRenderModeSettings, FrameRangeType), UMovieGraphQuickRenderModeSettings::StaticClass());
		const TSharedRef<IPropertyHandle> CustomStartFrameProperty =
			DetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphQuickRenderModeSettings, CustomStartFrame), UMovieGraphQuickRenderModeSettings::StaticClass());
		const TSharedRef<IPropertyHandle> CustomEndFrameProperty =
			DetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphQuickRenderModeSettings, CustomEndFrame), UMovieGraphQuickRenderModeSettings::StaticClass());

		DetailBuilder->EditDefaultProperty(FrameRangeTypeProperty)->IsEnabled(IsEnabledAttr);
		DetailBuilder->EditDefaultProperty(CustomStartFrameProperty)->IsEnabled(IsEnabledAttr);
		DetailBuilder->EditDefaultProperty(CustomEndFrameProperty)->IsEnabled(IsEnabledAttr);
	}

	// Adds a button next to the post-render behavior drop-down to open the editor preferences that dictate how media is played back.
	void AddPostRenderPlayOptionsButton() const
	{
		const TSharedRef<IPropertyHandle> PostRenderBehaviorProperty =
			DetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphQuickRenderModeSettings, PostRenderBehavior), UMovieGraphQuickRenderModeSettings::StaticClass());
		IDetailPropertyRow* PostRenderBehaviorRow = DetailBuilder->EditDefaultProperty(PostRenderBehaviorProperty);

		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		PostRenderBehaviorRow->GetDefaultWidgets(NameWidget, ValueWidget);

		PostRenderBehaviorRow->CustomWidget()
		.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				ValueWidget.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(5.f, 0, 0, 0)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.ContentPadding(0)
				.OnClicked_Lambda([]()
				{
					FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "Plugins", "MovieRenderGraphEditorSettings");
					
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("OpenPlaybackPrefs_Tooltip", "Open up the playback preferences which apply to the 'Play Render Output' option."))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("EditorPreferences.TabIcon"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				]
			]
		];
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override
	{
		// Refresh the customization every time a save happens. Use this opportunity to update the variables in the UI. We could update the UI before
		// a save occurs, but this would be very difficult to get right when multiple subgraphs are involved.
		UPackage::PackageSavedWithContextEvent.AddSP(this, &FMovieGraphQuickRenderSettingsCustomization::RefreshLayout);
		
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		InDetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

		if (ObjectsBeingCustomized.IsEmpty() || (ObjectsBeingCustomized.Num() > 1))
		{
			return;
		}

		QuickRenderModeSettings = Cast<UMovieGraphQuickRenderModeSettings>(ObjectsBeingCustomized[0]);
		if (!QuickRenderModeSettings.IsValid())
		{
			return;
		}

		// Hide the original assignments property (since it presents an asset picker).
		const TSharedRef<IPropertyHandle> GraphVariableAssignmentsProperty =
			InDetailBuilder.GetProperty(TEXT("GraphVariableAssignments"), UMovieGraphQuickRenderModeSettings::StaticClass());
		InDetailBuilder.HideProperty(GraphVariableAssignmentsProperty);

		// Update the enable state of the frame range properties to react to changes in the quick render mode setting
		UpdateFrameRangePropertyEnableState();

		// Refresh the UI if the graph preset changes (so the new variable assignments are displayed).
		QuickRenderModeSettings->OnGraphChangedDelegate.AddSP(this, &FMovieGraphQuickRenderSettingsCustomization::RefreshLayout);

		// Set up the category for variable assignments.
		IDetailCategoryBuilder& PrimaryGraphVariablesCategory = InDetailBuilder.EditCategory(
			"PrimaryGraphVariables", LOCTEXT("PrimaryGraphVariablesCategory", "Primary Graph Variables"));

		// Get the other categories for sort-order purposes.
		IDetailCategoryBuilder& ConfigurationCategory = InDetailBuilder.EditCategory("Configuration");
		IDetailCategoryBuilder& QuickRenderCategory = InDetailBuilder.EditCategory("Quick Render");

		// Set the variable assignments category as hidden by default, and add the variable assignments. Individual categories will be made visible if variables are added under them.
		PrimaryGraphVariablesCategory.SetCategoryVisibility(false);
		UE::MovieRenderPipelineEditor::Private::AddVariableAssignments(QuickRenderModeSettings->GraphVariableAssignments, PrimaryGraphVariablesCategory, DetailBuilder);

		// Give the categories a specific ordering.
		int32 SortOrder = 0;
		ConfigurationCategory.SetSortOrder(SortOrder);
		QuickRenderCategory.SetSortOrder(++SortOrder);
		PrimaryGraphVariablesCategory.SetSortOrder(++SortOrder);

		AddPostRenderPlayOptionsButton();
	}
	//~ End IDetailCustomization interface

private:
	/** The details builder associated with the customization. */
	IDetailLayoutBuilder* DetailBuilder = nullptr;

	/** The quick render mode settings being displayed. */
	TWeakObjectPtr<UMovieGraphQuickRenderModeSettings> QuickRenderModeSettings = nullptr;
};

#undef LOCTEXT_NAMESPACE
