// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Graph/MovieEdGraphNode.h"
#include "Graph/MovieGraphSchema.h"
#include "Graph/Nodes/MovieGraphSelectNode.h"
#include "IDetailCustomization.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "PropertyHandle.h"
#include "SPinTypeSelector.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customize how the Select node appears in the details panel. */
class FMovieGraphSelectNodeCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphSelectNodeCustomization>();
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}

	bool IsTypeAllowed(const FEdGraphPinType& PinType) const
	{
		const FName PinCategory = PinType.PinCategory;
		
		// Only a subset of reliably-compared types are available to be chosen
		return
			(PinCategory == UMovieGraphSchema::PC_Boolean) ||
			(PinCategory == UMovieGraphSchema::PC_Byte) ||
			(PinCategory == UMovieGraphSchema::PC_Integer) ||
			(PinCategory == UMovieGraphSchema::PC_Int64) ||
			(PinCategory == UMovieGraphSchema::PC_Name) ||
			(PinCategory == UMovieGraphSchema::PC_String) ||
			(PinCategory == UMovieGraphSchema::PC_Text) ||
			(PinCategory == UMovieGraphSchema::PC_Enum) ||
			(PinCategory == UMovieGraphSchema::PC_Class) ||
			(PinCategory == UMovieGraphSchema::PC_SoftClass);
	}
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		TSharedRef<IPropertyUtilities> PropUtils = DetailBuilder.GetPropertyUtilities();
		
		auto GetFilteredVariableTypeTree = [this](TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& TypeTree, ETypeTreeFilter TypeTreeFilter)
		{
			check(GetDefault<UEdGraphSchema_K2>());
			GetDefault<UPropertyBagSchema>()->GetVariableTypeTree(TypeTree, TypeTreeFilter);

			// Filter items in the tree down to those that are explicitly allowed
			constexpr bool bForceLoadSubCategoryObject = false;
			for (int32 Index = 0; Index < TypeTree.Num(); )
			{
				TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& PinType = TypeTree[Index];
				if (!PinType.IsValid())
				{
					return;
				}

				if (!IsTypeAllowed(PinType->GetPinType(bForceLoadSubCategoryObject)))
				{
					TypeTree.RemoveAt(Index);
					continue;
				}

				for (int32 ChildIndex = 0; ChildIndex < PinType->Children.Num(); )
				{
					TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> Child = PinType->Children[ChildIndex];
					if (Child.IsValid())
					{
						if (!IsTypeAllowed(PinType->GetPinType(bForceLoadSubCategoryObject)))
						{
							PinType->Children.RemoveAt(ChildIndex);
							continue;
						}
					}
					++ChildIndex;
				}

				++Index;
			}
		};
		
		for (TWeakObjectPtr<UMovieGraphSelectNode>& SelectNodeObject : DetailBuilder.GetObjectsOfTypeBeingCustomized<UMovieGraphSelectNode>())
		{
			if (!SelectNodeObject.IsValid())
			{
				continue;
			}
			
			IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(FName("General"));

			// Add a PinTypeSelector widget to pick the data type the select node uses
			Category.AddCustomRow(LOCTEXT("DataTypeFilterString", "Data Type"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DataType", "Data Type"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SBox)
				.Padding(5, 0, 0, 0)
				[
					SNew(SPinTypeSelector, FGetPinTypeTree::CreateLambda(GetFilteredVariableTypeTree))
					.TargetPinType_Lambda([SelectNodeObject]()
					{
						// The SPinTypeSelector popup might outlive this details view, so the node could be invalid
						if (!SelectNodeObject.IsValid())
						{
							return FEdGraphPinType();
						}

						constexpr bool bIsBranch = false;
						constexpr bool bIsWildcard = false;
						return UMoviePipelineEdGraphNodeBase::GetPinType(SelectNodeObject->GetValueType(), bIsBranch, bIsWildcard, SelectNodeObject->GetValueTypeObject());
					})
					.OnPinTypeChanged_Lambda([PropUtils, SelectNodeObject](const FEdGraphPinType& InPinType)
					{
						// The SPinTypeSelector popup might outlive this details view, so the node could be invalid
						if (SelectNodeObject.IsValid())
						{
							SelectNodeObject->SetDataType(UMoviePipelineEdGraphNodeBase::GetValueTypeFromPinType(InPinType), InPinType.PinSubCategoryObject.Get());

							// Need the ForceRefresh to make sure the details panel refreshes immediately after the data type change.
							// Can result in a crash without it.
							PropUtils->ForceRefresh();
						}
					})
					.Schema(GetDefault<UPropertyBagSchema>())
					.bAllowArrays(false)
					.TypeTreeFilter(ETypeTreeFilter::None)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			];

			const TSharedRef<IPropertyHandle> SelectOptions = DetailBuilder.GetProperty("SelectOptions");
			const TSharedRef<IPropertyHandle> SelectedOption = DetailBuilder.GetProperty("SelectedOption");

			// Add the property bag property ("MultiValueContainer.Values") for both SelectOptions and SelectedOption
			if (SelectOptions->IsValidHandle() && SelectedOption->IsValidHandle())
			{
				if (const TSharedPtr<IPropertyHandle> ContainerProperty = SelectOptions->GetChildHandle("MultiValueContainer"))
				{
					if (const TSharedPtr<IPropertyHandle> ValuesProperty = ContainerProperty->GetChildHandle("Values"))
					{
						Category.AddProperty(ValuesProperty);
						SelectOptions->MarkHiddenByCustomization();
					}
				}

				if (const TSharedPtr<IPropertyHandle> ContainerProperty = SelectedOption->GetChildHandle("MultiValueContainer"))
				{
					if (const TSharedPtr<IPropertyHandle> ValuesProperty = ContainerProperty->GetChildHandle("Values"))
					{
						Category.AddProperty(ValuesProperty);
						SelectedOption->MarkHiddenByCustomization();
					}
				}
			}
		}
	}
	//~ End IDetailCustomization interface
};

#undef LOCTEXT_NAMESPACE