// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/ConsoleVariableCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Graph/Nodes/MovieGraphSetCVarValueNode.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customize how the Set Console Variable Value node appears in the details panel. */
class FMovieGraphSetCVarValueNodeCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphSetCVarValueNodeCustomization>();
	}

protected:	
	FText GetConsoleVariableText(TSharedRef<IPropertyHandle> PropertyHandle) const
	{
		FString Value;
		if (PropertyHandle->IsValidHandle() && (PropertyHandle->GetValue(Value) != FPropertyAccess::Fail))
		{
			return FText::FromString(Value);
		}

		return FText();
	}

	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		for (TWeakObjectPtr<UMovieGraphSetCVarValueNode>& SetCVarValueNode : DetailBuilder.GetObjectsOfTypeBeingCustomized<UMovieGraphSetCVarValueNode>())
		{
			if (!SetCVarValueNode.IsValid())
			{
				continue;
			}

			TSharedRef<IPropertyHandle> CVarNameProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphSetCVarValueNode, Name));

			// Update the default widget for the cvar name to give it autocomplete
			IDetailPropertyRow* CVarNameRow = DetailBuilder.EditDefaultProperty(CVarNameProperty);
			CVarNameRow->CustomWidget(true)
			.NameContent()
			[
				CVarNameProperty->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SConsoleVariablesEditorCustomConsoleInputBox)
				.Font(DetailBuilder.GetDetailFont())
				.Text(this, &FMovieGraphSetCVarValueNodeCustomization::GetConsoleVariableText, CVarNameProperty)
				.HideOnFocusLost(false)
				.CommitOnFocusLost(true)
				.ClearOnCommit(false)
				.HintText(LOCTEXT("ConsoleVariableInputHintText", "Enter Console Variable"))
				.OnTextChanged_Lambda([this, CVarNameProperty](const FText& NewText)
				{
					if (CVarNameProperty->IsValidHandle())
					{
						// Note that the change is marked as Interactive so the graph is not forced to refresh for every character typed
						CVarNameProperty->SetValue(NewText.ToString(), EPropertyValueSetFlags::InteractiveChange);
					}
				})
				.OnTextCommitted_Lambda([this, CVarNameProperty](const FText& NewText)
				{
					// Committing the text will trigger the graph to update
					if (CVarNameProperty->IsValidHandle())
					{
						CVarNameProperty->SetValue(NewText.ToString());
					}
				})
			];
		}
	}
	//~ End IDetailCustomization interface
};

#undef LOCTEXT_NAMESPACE