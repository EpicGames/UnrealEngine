// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailWidgetRow.h"
#include "EditorShowFlags.h"
#include "Graph/Nodes/MovieGraphImagePassBaseNode.h"
#include "Graph/Nodes/MovieGraphPathTracerPassNode.h"
#include "Graph/Renderers/MovieGraphShowFlags.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

/** Customize how show flags appear in the details panel. */
class FMovieGraphShowFlagsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphShowFlagsCustomization>();
	}

protected:
	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		static const TMap<EShowFlagGroup, FText> GroupNames =
		{
			{SFG_Normal, LOCTEXT("GraphNormalSF", "Normal")},
			{SFG_PostProcess, LOCTEXT("GraphPostProcessSF", "Post Processing")},
			{SFG_LightTypes, LOCTEXT("GraphLightTypesSF", "Light Types")},
			{SFG_LightingComponents, LOCTEXT("GraphLightingComponentsSF", "Lighting Components")},
			{SFG_LightingFeatures, LOCTEXT("GraphLightingFeaturesSF", "Lighting Features")},
			{SFG_Lumen, LOCTEXT("GraphLumenSF", "Lumen")},
			{SFG_MegaLights, LOCTEXT("GraphMegaLightsSF", "MegaLights")},
			{SFG_Nanite, LOCTEXT("GraphNaniteSF", "Nanite")},
			{SFG_Developer, LOCTEXT("GraphDeveloperSF", "Developer")},
			{SFG_Visualize, LOCTEXT("GraphVisualizeSF", "Visualize")},
			{SFG_Advanced, LOCTEXT("GraphAdvancedSF", "Advanced")}
		};

		// Show flags that are excluded from showing up for any node
		static const TMap<EShowFlagGroup, TArray<FName>> ExcludedShowFlagsByGroup =
		{
			// Renderer nodes already expose a "Disable Tone Curve" property, so the equivalent show flag is redundant
			{
				SFG_PostProcess, { FName(TEXT("ToneCurve")) }
			}
		};

		check(PropertyHandle->IsValidHandle());

		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		// Some renderer nodes don't allow Show Flags customization
		if (UMovieGraphImagePassBaseNode* ImagePassBaseNode = Cast<UMovieGraphImagePassBaseNode>(OuterObjects[0]))
		{
			if (!ImagePassBaseNode->GetAllowsShowFlagsCustomization())
			{
				return;
			}
		}

		const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(PropertyHandle->GetProperty());
		UMovieGraphShowFlags* ShowFlagObject = Cast<UMovieGraphShowFlags>(
			ObjectProperty->GetObjectPropertyValue(ObjectProperty->ContainerPtrToValuePtr<void>(OuterObjects[0])));

		const UClass* NodeClass = OuterObjects[0]->GetClass();

		// Group together the show flags with the groups they show up under in the UI
		TMap<EShowFlagGroup, TArray<FShowFlagData>> GroupedShowFlags;
		for (FShowFlagData& ShowFlag : GetShowFlagMenuItems())
		{
			// Path Traced Renderer nodes don't show the Lighting Components group because these show flags have no effect on PT renders. Instead,
			// these are controlled by the dedicated Lighting Components settings for the path tracer.
			if ((NodeClass == UMovieGraphPathTracerRenderPassNode::StaticClass()) && (ShowFlag.Group == SFG_LightingComponents))
			{
				continue;
			}
			
			TArray<FShowFlagData>& GroupShowFlags = GroupedShowFlags.FindOrAdd(ShowFlag.Group);
			GroupShowFlags.Add(MoveTemp(ShowFlag));
		}

		// Make each show flag group a group in the UI, and each show flag a row under its respective group
		for (const TTuple<EShowFlagGroup, FText>& ShowFlagGroup : GroupNames)
		{
			const EShowFlagGroup& Group = ShowFlagGroup.Key;
			const FText& GroupName = ShowFlagGroup.Value;
			
			const TArray<FShowFlagData>* ShowFlags = GroupedShowFlags.Find(Group);
			if (!ShowFlags)
			{
				continue;
			}
			
			IDetailGroup& FlagGroup = ChildBuilder.AddGroup(FName(GroupName.ToString()), GroupName);

			for (const FShowFlagData& ShowFlag : *ShowFlags)
			{
				// This show flag may be excluded from being displayed
				const TArray<FName>* ExcludedShowFlagsForGroup = ExcludedShowFlagsByGroup.Find(ShowFlag.Group);
				if (ExcludedShowFlagsForGroup && ExcludedShowFlagsForGroup->Contains(ShowFlag.ShowFlagName))
				{
					continue;
				}

				const uint32 ShowFlagIndex = ShowFlag.EngineShowFlagIndex;

				// Temporary workaround: ShowFlag.DisplayName can sometimes be blank, so the non-display name is used instead.
				const FText ShowFlagText = FText::FromName(ShowFlag.ShowFlagName);

				// Create a custom reset handler for the show flags
				FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSPLambda(this, [ShowFlagObject, ShowFlagIndex](TSharedPtr<IPropertyHandle> InChildHandle)
				{
					return !ShowFlagObject->IsShowFlagSetToDefaultValue(ShowFlagIndex);
				});
				FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSPLambda(this, [ShowFlagObject, ShowFlagIndex](TSharedPtr<IPropertyHandle> InChildHandle)
				{
					const FScopedTransaction Transaction(LOCTEXT("Transaction_ResetShowFlagValue", "Reset to Default"));
					ShowFlagObject->RevertShowFlagToDefaultValue(ShowFlagIndex);
				});
				FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);
				
				FlagGroup.AddWidgetRow()
				.PropertyHandleList({PropertyHandle})
				.OverrideResetToDefault(ResetOverride)
				.EditCondition(
					TAttribute<bool>::Create([ShowFlagObject, ShowFlagIndex]()
					{
						return ShowFlagObject->IsShowFlagOverridden(ShowFlagIndex);
					}),
					FOnBooleanValueChanged::CreateSPLambda(this, [ShowFlagObject, ShowFlagIndex](bool NewValue)
					{
						const FScopedTransaction Transaction(LOCTEXT("Transaction_EditShowFlagOverrideState", "Edit Show Flag Override State"));
						ShowFlagObject->SetShowFlagOverridden(ShowFlagIndex, NewValue);
					})
				)
				.FilterString(ShowFlagText)
				.NameContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(ShowFlagText)
						.Font(CustomizationUtils.GetRegularFont())
					]
				]
				.ValueContent()
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([ShowFlagObject, ShowFlagIndex]()
					{
						return ShowFlagObject->IsShowFlagEnabled(ShowFlagIndex)
							? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([ShowFlagObject, ShowFlagIndex](const ECheckBoxState NewState)
					{
						const FScopedTransaction Transaction(LOCTEXT("Transaction_EditShowFlagEnableState", "Edit Show Flag Enable State"));
						
						const bool bIsUsed = (NewState == ECheckBoxState::Checked);
						ShowFlagObject->SetShowFlagEnabled(ShowFlagIndex, bIsUsed);
					})
				];
			}
		}
	}
	//~ End IPropertyTypeCustomization interface
};

#undef LOCTEXT_NAMESPACE
