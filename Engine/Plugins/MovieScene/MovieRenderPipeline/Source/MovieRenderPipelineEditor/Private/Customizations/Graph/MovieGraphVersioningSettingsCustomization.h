// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "FMovieGraphVersioningSettingsCustomization"

/* Customizes how Versioning Settings are displayed in the details pane. */
class FMovieGraphVersioningSettingsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphVersioningSettingsCustomization>();
	}

protected:
	
	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		const TSharedRef<IPropertyHandle> AutoVersioningProp = InStructPropertyHandle->GetChildHandle(
			GET_MEMBER_NAME_CHECKED(FMovieGraphVersioningSettings, bAutoVersioning)).ToSharedRef();
		
		const TSharedRef<IPropertyHandle> VersionNumberProp = InStructPropertyHandle->GetChildHandle(
			GET_MEMBER_NAME_CHECKED(FMovieGraphVersioningSettings, VersionNumber)).ToSharedRef();

		HeaderRow
		.FilterString(VersionNumberProp->GetPropertyDisplayName())
		.NameContent()
		[
			// Show "Version Number" as the row name
			VersionNumberProp->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.Padding(0, 2, 8, 2)
			.HAlign(HAlign_Fill)
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda([AutoVersioningProp]()
				{
					bool bAutoVersioning = false;
					AutoVersioningProp->GetValue(bAutoVersioning);

					return bAutoVersioning;
				})

				+SWidgetSwitcher::Slot()
				[
					VersionNumberProp->CreatePropertyValueWidget()
				]

				+SWidgetSwitcher::Slot()
				[
					SNew(SEditableTextBox)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("SetByRender", "Set By Render"))
					.IsEnabled(false)
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0, 2, 8, 2)
			.AutoWidth()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				
				+ SHorizontalBox::Slot()
				.Padding(8, 0)
				.AutoWidth()
				[
					AutoVersioningProp->CreatePropertyValueWidget()
				]

				+ SHorizontalBox::Slot()
				.Padding(0)
				.AutoWidth()
				[
					AutoVersioningProp->CreatePropertyNameWidget()
				]
			]
		];
	}

	// Skip customizing children
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override { }
	//~ End IPropertyTypeCustomization interface
};

#undef LOCTEXT_NAMESPACE
