// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/JobCustomization.h"

// MovieRenderPipeline
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphCustomizationUtils.h"
#include "Graph/MovieRenderGraphEditorSettings.h"
#include "Graph/Nodes/MovieGraphPathTracerPassNode.h"
#include "MoviePipelineBasicConfig.h"
#include "MoviePipelineEditorBlueprintLibrary.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineQueue.h"
#include "Widgets/SMoviePipelineQueueEditor.h"

// Engine / PropertyEditor
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"

// Framework
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

// Slate / Widgets
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"

// Note: this static_assert needs to be done in the customization rather than where EMoviePipelineBasicDenoiserType is defined because it's
// defined in a module that cannot access EMovieGraphPathTracerDenoiserType.
static_assert(
	static_cast<uint8>(EMoviePipelineBasicDenoiserType::Spatial) == static_cast<uint8>(EMovieGraphPathTracerDenoiserType::Spatial)
	&& static_cast<uint8>(EMoviePipelineBasicDenoiserType::Temporal) == static_cast<uint8>(EMovieGraphPathTracerDenoiserType::Temporal),
	"EMoviePipelineBasicDenoiserType must match EMovieGraphPathTracerDenoiserType");

#define LOCTEXT_NAMESPACE "MoviePipelineEditor"

struct FToggleButtonDef
{
	FText Label;
	bool* Value;
	// Optional: called when the user tries to uncheck this button. Return false to prevent unchecking.
	// Use this for cross-button constraints (e.g. "at least one renderer must be active").
	TFunction<bool()> CanUncheck;
};

/**
 * Adds a row of toggle-style checkboxes to a detail category.
 * When InGetRowOverride is set (shot mode), a single override checkbox appears in the name column
 * controlling the whole row — all buttons and the row label are disabled when the override is off.
 */
static void AddToggleButtonRow(
	IDetailCategoryBuilder& InCategory,
	UMoviePipelineBasicConfig* InConfig,
	const FText& InFilterText,
	const FText& InRowLabel,
	TArrayView<const FToggleButtonDef> InButtons,
	TFunction<bool()> InGetRowOverride = {},
	TFunction<void(bool)> InSetRowOverride = {})
{
	const bool bHasRowOverride = !!InGetRowOverride;

	// When a row override is present, all buttons and the label are enabled only when the override is active.
	TAttribute<bool> RowEnabled = bHasRowOverride
		? TAttribute<bool>::CreateLambda([InGetRowOverride]() { return InGetRowOverride(); })
		: TAttribute<bool>(true);

	TSharedRef<SHorizontalBox> ButtonBox = SNew(SHorizontalBox);
	for (const FToggleButtonDef& Button : InButtons)
	{
		bool* BoolPtr = Button.Value;
		TFunction<bool()> CanUncheck = Button.CanUncheck;
		ButtonBox->AddSlot()
			.AutoWidth()
			.Padding(0, 0, 5, 0)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.IsEnabled(RowEnabled)
				.IsChecked_Lambda([BoolPtr]()
				{
					return *BoolPtr ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([InConfig, BoolPtr, CanUncheck](ECheckBoxState NewState)
				{
					if (NewState == ECheckBoxState::Unchecked && CanUncheck && !CanUncheck())
					{
						return;
					}
					InConfig->Modify();
					*BoolPtr = (NewState == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "SmallText")
					.Text(Button.Label)
				]
			];
	}

	InCategory.AddCustomRow(InFilterText)
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(-2, 0, 2, 0)
			[
				// Single override toggle for the whole row — visible in shot mode only.
				SNew(SCheckBox)
				.Visibility(bHasRowOverride ? EVisibility::Visible : EVisibility::Collapsed)
				.IsChecked_Lambda([bHasRowOverride, InGetRowOverride]()
				{
					return (bHasRowOverride && InGetRowOverride())
						? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([InConfig, bHasRowOverride, InSetRowOverride](ECheckBoxState NewState)
				{
					if (bHasRowOverride)
					{
						InConfig->Modify();
						InSetRowOverride(NewState == ECheckBoxState::Checked);
					}
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InRowLabel)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsEnabled(RowEnabled)
			]
		]
		.ValueContent()
		[ ButtonBox ];
}

/**
 * Wires an inline override toggle to a property row for shots.
 * For primary jobs (bIsShot == false) this is a no-op -- all properties are always editable.
 */
static void OptionallyAddShotInlineEditCondition(
	IDetailPropertyRow* InRow,
	UMoviePipelineBasicConfig* InConfig,
	const bool bIsShot,
	TFunction<bool()> InGetFlag,
	TFunction<void(bool)> InSetFlag)
{
	if (!bIsShot || !InRow)
	{
		return;
	}

	InRow->EditCondition(
		TAttribute<bool>::CreateLambda([InGetFlag]() -> bool
		{
			return InGetFlag();
		}),
		FOnBooleanValueChanged::CreateLambda([InConfig, InSetFlag](const bool bNew)
		{
			InConfig->Modify();
			InSetFlag(bNew);
		})
	);
}

TSharedRef<IDetailCustomization> FJobDetailsCustomization::MakeInstance()
{
	return MakeShared<FJobDetailsCustomization>();
}

void FJobDetailsCustomization::PendingDelete()
{
	// Unregister delegates. It's important to do this in PendingDelete() vs the destructor because the destructor is not called before the next
	// details panel is created (via ForceRefreshDetails()), leading to an exponential increase in the number of delegates registered.

	UPackage::PackageSavedWithContextEvent.RemoveAll(this);

	if (SelectedJob.IsValid())
	{
		SelectedJob->OnJobGraphPresetChanged.RemoveAll(this);
		SelectedJob->OnJobConfigModeChanged.RemoveAll(this);
	}

	if (SelectedShot.IsValid())
	{
		SelectedShot->OnShotGraphPresetChanged.RemoveAll(this);
	}
}

void FJobDetailsCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailBuilder = InDetailBuilder.Get();
	CustomizeDetails(*InDetailBuilder);
}

void FJobDetailsCustomization::RefreshLayout(const FString&, UPackage*, FObjectPostSaveContext) const
{
	if (DetailBuilder) { DetailBuilder->ForceRefreshDetails(); }
}

void FJobDetailsCustomization::RefreshLayout(UMoviePipelineExecutorShot*, UMovieGraphConfig*) const
{
	if (DetailBuilder) { DetailBuilder->ForceRefreshDetails(); }
}

void FJobDetailsCustomization::RefreshLayout(UMoviePipelineExecutorJob*, UMovieGraphConfig*) const
{
	if (DetailBuilder) { DetailBuilder->ForceRefreshDetails(); }
}

void FJobDetailsCustomization::RefreshLayout(UMoviePipelineExecutorJob*, EMoviePipelineConfigMode) const
{
	if (DetailBuilder) { DetailBuilder->ForceRefreshDetails(); }
}

void FJobDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	DetailBuilder = &InDetailBuilder;

	// Refresh the customization every time a save happens. Use this opportunity to update the variables in the UI. We could update the UI before
	// a save occurs, but this would be very difficult to get right when multiple subgraphs are involved.
	UPackage::PackageSavedWithContextEvent.AddSP(this, &FJobDetailsCustomization::RefreshLayout);

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	InDetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	TArray<UMoviePipelineExecutorJob*> SelectedJobs;
	TArray<UMoviePipelineExecutorShot*> SelectedShots;

	for (const TWeakObjectPtr<UObject>& SelectedObject : ObjectsBeingCustomized)
	{
		if (UMoviePipelineExecutorJob* SelectedJobTemp = Cast<UMoviePipelineExecutorJob>(SelectedObject.Get()))
		{
			SelectedJobs.Add(SelectedJobTemp);
		}
		else if (UMoviePipelineExecutorShot* SelectedShotTemp = Cast<UMoviePipelineExecutorShot>(SelectedObject.Get()))
		{
			SelectedShots.Add(SelectedShotTemp);
		}
	}

	// Hide the original assignments properties (since they present an asset picker by default) for both jobs and shots
	const TSharedRef<IPropertyHandle> JobAssignmentsProperty =
		InDetailBuilder.GetProperty(TEXT("GraphVariableAssignments"), UMoviePipelineExecutorJob::StaticClass());
	const TSharedRef<IPropertyHandle> ShotAssignmentsProperty =
		InDetailBuilder.GetProperty(TEXT("GraphVariableAssignments"), UMoviePipelineExecutorShot::StaticClass());
	const TSharedRef<IPropertyHandle> ShotPrimaryGraphAssignmentsProperty =
		InDetailBuilder.GetProperty(TEXT("PrimaryGraphVariableAssignments"), UMoviePipelineExecutorShot::StaticClass());
	InDetailBuilder.HideProperty(JobAssignmentsProperty);
	InDetailBuilder.HideProperty(ShotAssignmentsProperty);
	InDetailBuilder.HideProperty(ShotPrimaryGraphAssignmentsProperty);
	InDetailBuilder.HideProperty(
		InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMoviePipelineExecutorJob, JobName)));
	InDetailBuilder.HideProperty(
		InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMoviePipelineExecutorJob, Sequence)));

	// Only display the customized variables UI if there is one job or shot selected
	const bool bIsPrimaryJob = (SelectedJobs.Num() == 1) && SelectedShots.IsEmpty();
	const bool bIsShot = (SelectedShots.Num() == 1) && SelectedJobs.IsEmpty();
	if (!bIsPrimaryJob && !bIsShot)
	{
		return;
	}

	// Refresh the UI if the graph preset changes (so the new variable assignments are displayed)
	if (bIsShot)
	{
		SelectedShot = SelectedShots[0];
		SelectedShot->OnShotGraphPresetChanged.AddSP(this, &FJobDetailsCustomization::RefreshLayout);

		// Also listen for changes to the primary job. Changes to the primary job can trigger an update to shot variable assignments.
		if (UMoviePipelineExecutorJob* PrimaryJob = SelectedShot->GetTypedOuter<UMoviePipelineExecutorJob>())
		{
			SelectedJob = PrimaryJob;
			SelectedJob->OnJobGraphPresetChanged.AddSP(this, &FJobDetailsCustomization::RefreshLayout);
			SelectedJob->OnJobConfigModeChanged.AddSP(this, &FJobDetailsCustomization::RefreshLayout);
		}
	}
	else
	{
		SelectedJob = SelectedJobs[0];
		SelectedJob->OnJobGraphPresetChanged.AddSP(this, &FJobDetailsCustomization::RefreshLayout);
		SelectedJob->OnJobConfigModeChanged.AddSP(this, &FJobDetailsCustomization::RefreshLayout);
	}

	// Get the Basic config (if active) so we can surface its properties later.
	UMoviePipelineBasicConfig* BasicConfig = nullptr;
	if (SelectedJob.IsValid() && SelectedJob->IsUsingBasicConfiguration())
	{
		BasicConfig = SelectedJob->GetBasicConfig();
	}

	// Set up the categories for variable assignments.
	IDetailCategoryBuilder& PrimaryGraphVariablesCategory = InDetailBuilder.EditCategory(
		"PrimaryGraphVariables", LOCTEXT("PrimaryGraphVariablesCategory", "Primary Graph Variables"));
	IDetailCategoryBuilder& PrimaryGraphVariablesShotOverridesCategory = InDetailBuilder.EditCategory(
		"PrimaryGraphVariablesShotOverrides", LOCTEXT("PrimaryGraphVariablesShotOverridesCategory", "Primary Graph Variables (shot overrides)"));
	IDetailCategoryBuilder& ShotGraphVariablesCategory = InDetailBuilder.EditCategory(
		"ShotGraphVariables", LOCTEXT("ShotGraphVariablesCategory", "Shot Graph Variables"));

	// Set all categories as hidden by default. Individual categories will be made visible if variables are added under them.
	PrimaryGraphVariablesCategory.SetCategoryVisibility(false);
	PrimaryGraphVariablesShotOverridesCategory.SetCategoryVisibility(false);
	ShotGraphVariablesCategory.SetCategoryVisibility(false);

	const bool bIsGraphMode = SelectedJob.IsValid() && SelectedJob->IsUsingGraphConfiguration();
	if (bIsGraphMode)
	{
		if (bIsShot)
		{
			UE::MovieRenderPipelineEditor::Private::AddVariableAssignments(SelectedShot->GetGraphVariableAssignments(), ShotGraphVariablesCategory, DetailBuilder);
			UE::MovieRenderPipelineEditor::Private::AddVariableAssignments(SelectedShot->GetPrimaryGraphVariableAssignments(), PrimaryGraphVariablesShotOverridesCategory, DetailBuilder);
		}
		else
		{
			UE::MovieRenderPipelineEditor::Private::AddVariableAssignments(SelectedJob->GetGraphVariableAssignments(), PrimaryGraphVariablesCategory, DetailBuilder);
		}
	}

	IDetailCategoryBuilder& MovieRenderPipelineCategory = InDetailBuilder.EditCategory("Movie Render Pipeline");

	// The Console Variable Overrides category should be collapsed by default. It's considered advanced, and most people won't need to use it.
	IDetailCategoryBuilder& ConsoleVariableOverridesCategory = InDetailBuilder.EditCategory("Console Variable Overrides");
	ConsoleVariableOverridesCategory.InitiallyCollapsed(true);

	// Add Configuration mode row (only for primary jobs, not shots).
	if (bIsPrimaryJob)
	{
		AddConfigurationModeRow(MovieRenderPipelineCategory);
	}

	// In Basic mode, surface Basic config properties organized into Output and Rendering categories.
	// For primary jobs: display the job's BasicConfig (always editable).
	// For shots: display the shot's BasicShotConfig with per-property override toggles.
	// Guard on BasicConfig pointer — the enum may indicate Basic before the object is ready.
	// ActiveConfig lifetime: held as a UPROPERTY on the job/shot, which the details panel references
	// via TWeakObjectPtr. The panel is rebuilt (ForceRefreshDetails) on any mode/preset change, so
	// these lambda captures are safe as long as the owning job/shot stays alive.
	UMoviePipelineBasicConfig* ActiveConfig = nullptr;
	if (BasicConfig && bIsPrimaryJob)
	{
		ActiveConfig = BasicConfig;
	}
	else if (BasicConfig && bIsShot && SelectedShot.IsValid())
	{
		ActiveConfig = SelectedShot->GetBasicShotConfig();
	}

	if (ActiveConfig)
	{
		AddBasicConfigOutputTypeProperties(InDetailBuilder, ActiveConfig, bIsShot);
		AddBasicConfigRenderingProperties(InDetailBuilder, ActiveConfig, bIsShot);

		// Re-acquire categories created by the helpers above
		// (EditCategory returns the existing builder when called with the same ID).
		IDetailCategoryBuilder& OutputCategory = InDetailBuilder.EditCategory("Output");
		IDetailCategoryBuilder& RenderingCategory = InDetailBuilder.EditCategory("Rendering");

		// Give the categories a specific ordering for Basic mode.
		int32 SortOrder = 0;
		MovieRenderPipelineCategory.SetSortOrder(SortOrder);
		OutputCategory.SetSortOrder(++SortOrder);
		RenderingCategory.SetSortOrder(++SortOrder);
		ConsoleVariableOverridesCategory.SetSortOrder(++SortOrder);
	}
	else
	{
		// Give the categories a specific ordering for Graph/Legacy mode.
		int32 SortOrder = 0;
		MovieRenderPipelineCategory.SetSortOrder(SortOrder);
		PrimaryGraphVariablesCategory.SetSortOrder(++SortOrder);
		PrimaryGraphVariablesShotOverridesCategory.SetSortOrder(++SortOrder);
		ShotGraphVariablesCategory.SetSortOrder(++SortOrder);
		ConsoleVariableOverridesCategory.SetSortOrder(++SortOrder);
	}
}

void FJobDetailsCustomization::AddConfigurationModeRow(IDetailCategoryBuilder& InCategory)
{
	IDetailLayoutBuilder* CachedBuilder = DetailBuilder;
	TWeakObjectPtr<UMoviePipelineExecutorJob> WeakJob = SelectedJob;

	// Lambda to switch config mode on the selected job.
	// The OnJobConfigModeChanged delegate triggers a details panel refresh automatically.
	auto SwitchConfigMode = [WeakJob](const EMoviePipelineConfigMode NewMode)
	{
		FScopedTransaction Transaction(LOCTEXT("SetConfigMode_Transaction", "Set Configuration Mode"));
		if (UMoviePipelineExecutorJob* Job = WeakJob.Get())
		{
			if (!Job->SetConfigMode(NewMode))
			{
				// SetConfigMode fails for Graph when no graph is assigned; assign the default
				// from project settings. AssignDefaultGraphPresetToJob calls SetGraphPreset,
				// which internally switches ConfigMode to Graph.
				SMoviePipelineQueueEditor::AssignDefaultGraphPresetToJob(Job);
			}
			else if (NewMode == EMoviePipelineConfigMode::Preset)
			{
				UMoviePipelineEditorBlueprintLibrary::EnsureJobHasDefaultSettings(Job);
			}
		}
	};

	static const FSegmentedControlStyle PrimaryConfigModeStyle = []()
	{
		FCheckBoxStyle ButtonStyle = FAppStyle::Get().GetWidgetStyle<FSegmentedControlStyle>("SegmentedControl").ControlStyle;
		ButtonStyle
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryPress, 4.0f))
			.SetCheckedForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover);

		return FSegmentedControlStyle(FAppStyle::Get().GetWidgetStyle<FSegmentedControlStyle>("SegmentedControl"))
			.SetControlStyle(ButtonStyle)
			.SetFirstControlStyle(ButtonStyle)
			.SetLastControlStyle(ButtonStyle);
	}();

	// Configuration mode segmented control.
	const TSharedRef<SWidget> ConfigButtonBox =
		SNew(SSegmentedControl<EMoviePipelineConfigMode>)
		.Style(&PrimaryConfigModeStyle)
		.Value_Lambda([WeakJob]()
		{
			if (const UMoviePipelineExecutorJob* Job = WeakJob.Get())
			{
				return Job->GetConfigMode();
			}
			return EMoviePipelineConfigMode::Preset;
		})
		.OnValueChanged_Lambda([SwitchConfigMode](const EMoviePipelineConfigMode NewMode)
		{
			SwitchConfigMode(NewMode);
		})
		+ SSegmentedControl<EMoviePipelineConfigMode>::Slot(EMoviePipelineConfigMode::Basic)
			.Text(LOCTEXT("ConfigMode_Basic", "Basic"))
		+ SSegmentedControl<EMoviePipelineConfigMode>::Slot(EMoviePipelineConfigMode::Graph)
			.Text(LOCTEXT("ConfigMode_Graph", "Graph"))
		+ SSegmentedControl<EMoviePipelineConfigMode>::Slot(EMoviePipelineConfigMode::Preset)
			.Text(LOCTEXT("ConfigMode_Preset", "Preset"));

	// Build the mode-dependent sub-widget.
	TSharedRef<SWidget> SubWidget = SNullWidget::NullWidget;

	if (SelectedJob.IsValid() && SelectedJob->GetConfigMode() == EMoviePipelineConfigMode::Basic)
	{
		// Save... combo button with menu.
		SubWidget = SNew(SComboButton)
			.OnGetMenuContent_Lambda([WeakJob, CachedBuilder]() -> TSharedRef<SWidget>
			{
				FMenuBuilder MenuBuilder(true, nullptr);

				MenuBuilder.BeginSection(NAME_None, LOCTEXT("SaveGraphSection", "Save Movie Render Graph"));
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("SaveAsGraphConfig", "Save As Movie Render Graph Config..."),
						FText(),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([WeakJob, CachedBuilder]()
						{
							if (WeakJob.IsValid() && UMoviePipelineEditorBlueprintLibrary::SaveBasicConfigAsGraphConfig(WeakJob.Get()))
							{
								// SaveBasicConfigAsGraphConfig sets the graph preset on the job,
								// which switches the config mode to Graph.
								CachedBuilder->ForceRefreshDetails();
							}
						}))
					);
				}
				MenuBuilder.EndSection();

				MenuBuilder.BeginSection(NAME_None, LOCTEXT("SaveBasicSection", "Save Basic Config"));
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("SaveBasicAsDefault", "Save Basic Config As Default"),
						LOCTEXT("SaveBasicAsDefaultTooltip", "Save the current Basic configuration as the default for new jobs."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([WeakJob]()
						{
							if (const UMoviePipelineExecutorJob* Job = WeakJob.Get())
							{
								if (const UMoviePipelineBasicConfig* Config = Job->GetBasicConfig())
								{
									UMoviePipelineBasicConfig::SaveAsDefault(Config);
								}
							}
						}))
					);
				}
				MenuBuilder.EndSection();

				return MenuBuilder.MakeWidget();
			})
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 4, 0)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("AssetEditor.SaveAsset"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "SmallText")
					.Text(LOCTEXT("SaveBasicButton", "Save..."))
				]
			];
	}
	else if (SelectedJob.IsValid() && SelectedJob->GetConfigMode() == EMoviePipelineConfigMode::Graph)
	{
		// Graph asset picker.
		SubWidget = SNew(SObjectPropertyEntryBox)
			.AllowedClass(UMovieGraphConfig::StaticClass())
			.ObjectPath_Lambda([WeakJob]() -> FString
			{
				if (WeakJob.IsValid())
				{
					if (const UMovieGraphConfig* Graph = WeakJob->GetGraphPreset())
					{
						return Graph->GetPathName();
					}
				}
				return FString();
			})
			.OnObjectChanged_Lambda([WeakJob, CachedBuilder](const FAssetData& AssetData)
			{
				if (WeakJob.IsValid())
				{
					WeakJob->Modify();
					WeakJob->SetGraphPreset(Cast<UMovieGraphConfig>(AssetData.GetAsset()));
					CachedBuilder->ForceRefreshDetails();
				}
			})
			.AllowClear(true)
			.DisplayThumbnail(false);
	}
	else // Preset
	{
		// Preset asset picker.
		SubWidget = SNew(SObjectPropertyEntryBox)
			.AllowedClass(UMoviePipelinePrimaryConfig::StaticClass())
			.ObjectPath_Lambda([WeakJob]() -> FString
			{
				if (WeakJob.IsValid())
				{
					if (const UMoviePipelinePrimaryConfig* Preset = WeakJob->GetPresetOrigin())
					{
						return Preset->GetPathName();
					}
				}
				return FString();
			})
			.OnObjectChanged_Lambda([WeakJob, CachedBuilder](const FAssetData& AssetData)
			{
				if (WeakJob.IsValid())
				{
					WeakJob->Modify();
					WeakJob->SetPresetOrigin(Cast<UMoviePipelinePrimaryConfig>(AssetData.GetAsset()));
					CachedBuilder->ForceRefreshDetails();
				}
			})
			.AllowClear(true)
			.DisplayThumbnail(false);
	}

	// Single "Configuration" row: buttons on top, sub-widget below.
	InCategory.AddCustomRow(LOCTEXT("ConfigTypeFilter", "Configuration"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ConfigTypeLabel", "Configuration"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 2, 0, 4)
			[
				ConfigButtonBox
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SubWidget
			]
		];
}

void FJobDetailsCustomization::AddBasicConfigOutputTypeProperties(
	IDetailLayoutBuilder& InDetailBuilder,
	UMoviePipelineBasicConfig* InActiveConfig,
	const bool bIsShot)
{
	const TArray<UObject*> ActiveConfigObjects = { InActiveConfig };

	IDetailCategoryBuilder& OutputCategory = InDetailBuilder.EditCategory(
		"Output", LOCTEXT("OutputCategory", "Output"));
	OptionallyAddShotInlineEditCondition(
		OutputCategory.AddExternalObjectProperty(ActiveConfigObjects, GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, OutputDirectory)),
		InActiveConfig, bIsShot,
		[InActiveConfig]{ return !!InActiveConfig->bOverride_OutputDirectory; },
		[InActiveConfig](const bool bNewValue){ InActiveConfig->bOverride_OutputDirectory = bNewValue; });
	OptionallyAddShotInlineEditCondition(
		OutputCategory.AddExternalObjectProperty(ActiveConfigObjects, GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, FileNameFormat)),
		InActiveConfig, bIsShot,
		[InActiveConfig]{ return !!InActiveConfig->bOverride_FileNameFormat; },
		[InActiveConfig](const bool bNewValue){ InActiveConfig->bOverride_FileNameFormat = bNewValue; });
	OptionallyAddShotInlineEditCondition(
		OutputCategory.AddExternalObjectProperty(ActiveConfigObjects, GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, OutputResolution)),
		InActiveConfig, bIsShot,
		[InActiveConfig]{ return !!InActiveConfig->bOverride_OutputResolution; },
		[InActiveConfig](const bool bNewValue){ InActiveConfig->bOverride_OutputResolution = bNewValue; });

	// Custom Start/End Frame cannot be configured at the shot level
	if (!bIsShot)
	{
		OutputCategory.AddExternalObjectProperty(ActiveConfigObjects, GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, CustomStartFrame));
		OutputCategory.AddExternalObjectProperty(ActiveConfigObjects, GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, CustomEndFrame));
	}

	// Discover ALL output node classes that opt in to Basic config via GetBasicConfigShortDisplayName()
	auto IsValidBasicConfigOutputType = [](const UClass* Class) -> bool
	{
		if (Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_Hidden | CLASS_HideDropDown))
		{
			return false;
		}
		const UMovieGraphFileOutputNode* CDO = Cast<UMovieGraphFileOutputNode>(Class->GetDefaultObject());
		return CDO && !CDO->GetBasicConfigShortDisplayName().IsEmpty();
	};

	auto SortByDisplayName = [](const UClass& A, const UClass& B)
	{
		const UMovieGraphFileOutputNode* CDOA = CastChecked<UMovieGraphFileOutputNode>(A.GetDefaultObject());
		const UMovieGraphFileOutputNode* CDOB = CastChecked<UMovieGraphFileOutputNode>(B.GetDefaultObject());
		return CDOA->GetBasicConfigShortDisplayName().CompareTo(CDOB->GetBasicConfigShortDisplayName()) < 0;
	};

	TArray<UClass*> AllOutputNodeClasses;
	GetDerivedClasses(UMovieGraphFileOutputNode::StaticClass(), AllOutputNodeClasses, /*bRecursive=*/ true);
	AllOutputNodeClasses.RemoveAll([&IsValidBasicConfigOutputType](const UClass* Class) { return !IsValidBasicConfigOutputType(Class); });
	AllOutputNodeClasses.Sort(SortByDisplayName);

	// For shots: output type buttons are enabled only when the override flag is active.
	const TAttribute<bool> OutputTypeEnabled = bIsShot
		? TAttribute<bool>::CreateLambda([InActiveConfig]() { return !!InActiveConfig->bOverride_EnabledOutputTypes; })
		: TAttribute<bool>(true);

	// Create buttons for ALL output types; bind visibility to editor settings so
	// the gear menu can show/hide them without rebuilding the details panel.
	const TSharedRef<SWrapBox> OutputButtonBox = SNew(SWrapBox).UseAllottedSize(true);
	for (const UClass* OutputClass : AllOutputNodeClasses)
	{
		TSoftClassPtr<UMovieGraphFileOutputNode> SoftClassPtr(OutputClass);
		FSoftClassPath ClassPath(OutputClass);
		const UMovieGraphFileOutputNode* OutputNodeCDO = CastChecked<UMovieGraphFileOutputNode>(OutputClass->GetDefaultObject());
		FText ButtonLabel = OutputNodeCDO->GetBasicConfigShortDisplayName();

		OutputButtonBox->AddSlot()
			.Padding(0, 0, 5, 2)
			[
				SNew(SCheckBox)
				.Visibility_Lambda([ClassPath]()
				{
					const UMovieRenderGraphEditorSettings* Settings = GetDefault<UMovieRenderGraphEditorSettings>();
					return Settings->VisibleBasicConfigOutputTypes.Contains(ClassPath)
						? EVisibility::Visible : EVisibility::Collapsed;
				})
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.IsEnabled(OutputTypeEnabled)
				.IsChecked_Lambda([InActiveConfig, SoftClassPtr]()
				{
					return InActiveConfig->EnabledOutputTypes.Contains(SoftClassPtr)
						? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([InActiveConfig, SoftClassPtr](ECheckBoxState NewState)
				{
					if (NewState == ECheckBoxState::Unchecked && InActiveConfig->EnabledOutputTypes.Num() <= 1)
					{
						return;
					}
					InActiveConfig->Modify();
					if (NewState == ECheckBoxState::Checked)
					{
						InActiveConfig->EnabledOutputTypes.AddUnique(SoftClassPtr);
					}
					else
					{
						InActiveConfig->EnabledOutputTypes.Remove(SoftClassPtr);
					}
				})
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "SmallText")
					.Text(ButtonLabel)
				]
			];
	}

	// Gear menu for controlling which output types are visible and burn-in visibility.
	// All changes take effect immediately via visibility bindings.
	const TSharedRef<SComboButton> OutputTypeGearMenu = SNew(SComboButton)
		.HasDownArrow(false)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnGetMenuContent_Lambda([InActiveConfig, AllOutputNodeClasses]() -> TSharedRef<SWidget>
		{
			FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/ false, nullptr);
			UMovieRenderGraphEditorSettings* MutableSettings = GetMutableDefault<UMovieRenderGraphEditorSettings>();

			MenuBuilder.BeginSection(NAME_None, LOCTEXT("BasicOutputTypesSection", "Basic Output Types"));
			for (const UClass* OutputClass : AllOutputNodeClasses)
			{
				FSoftClassPath ClassPath(OutputClass);
				const UMovieGraphFileOutputNode* OutputNodeCDO = CastChecked<UMovieGraphFileOutputNode>(OutputClass->GetDefaultObject());
				FText Label = OutputNodeCDO->GetBasicConfigShortDisplayName();

				MenuBuilder.AddMenuEntry(
					Label,
					FText::GetEmpty(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([MutableSettings, ClassPath, InActiveConfig]()
						{
							if (MutableSettings->VisibleBasicConfigOutputTypes.Contains(ClassPath))
							{
								// Don't allow hiding the last enabled output type
								const TSoftClassPtr<UMovieGraphFileOutputNode> SoftPtr(ClassPath);
								if (InActiveConfig->EnabledOutputTypes.Contains(SoftPtr) && InActiveConfig->EnabledOutputTypes.Num() <= 1)
								{
									return;
								}
								
								MutableSettings->VisibleBasicConfigOutputTypes.Remove(ClassPath);
								
								// Remove from EnabledOutputTypes to prevent phantom graph nodes
								InActiveConfig->Modify();
								InActiveConfig->EnabledOutputTypes.Remove(SoftPtr);
							}
							else
							{
								MutableSettings->VisibleBasicConfigOutputTypes.AddUnique(ClassPath);
							}
							
							MutableSettings->SaveConfig();
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([MutableSettings, ClassPath]()
						{
							return MutableSettings->VisibleBasicConfigOutputTypes.Contains(ClassPath);
						})
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}
			MenuBuilder.EndSection();

			MenuBuilder.AddSeparator();

			MenuBuilder.BeginSection(NAME_None);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("BurnInClassMenu", "Burn In Class"),
				LOCTEXT("BurnInClassMenuTooltip", "Show the Burn In Class property in Basic Config"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([MutableSettings]()
					{
						MutableSettings->bShowBurnInForBasicConfig = !MutableSettings->bShowBurnInForBasicConfig;
						MutableSettings->SaveConfig();
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([MutableSettings]()
					{
						return MutableSettings->bShowBurnInForBasicConfig;
					})
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
			MenuBuilder.EndSection();

			return MenuBuilder.MakeWidget();
		})
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
			.DesiredSizeOverride(FVector2D(16.f, 16.f))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];

	OutputCategory.AddCustomRow(LOCTEXT("OutputTypeFilter", "Output Type"))
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(-2, 0, 2, 0)
			[
				// Override toggle — visible only in shot mode
				SNew(SCheckBox)
				.Visibility(bIsShot ? EVisibility::Visible : EVisibility::Collapsed)
				.IsChecked_Lambda([InActiveConfig]()
				{
					return InActiveConfig->bOverride_EnabledOutputTypes
						? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([InActiveConfig](const ECheckBoxState NewState)
				{
					InActiveConfig->Modify();
					InActiveConfig->bOverride_EnabledOutputTypes = (NewState == ECheckBoxState::Checked) ? 1 : 0;
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OutputTypeLabel", "Output Type"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsEnabled(OutputTypeEnabled)
			]
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				OutputButtonBox
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4, 0, 0, 0)
			[
				OutputTypeGearMenu
			]
		];

	// BurnInClass row — always created, visibility bound to editor settings
	if (IDetailPropertyRow* BurnInRow = OutputCategory.AddExternalObjectProperty(ActiveConfigObjects,
		GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, BurnInClass)))
	{
		BurnInRow->Visibility(TAttribute<EVisibility>::CreateLambda([]()
		{
			const UMovieRenderGraphEditorSettings* Settings = GetDefault<UMovieRenderGraphEditorSettings>();
			return Settings->bShowBurnInForBasicConfig ? EVisibility::Visible : EVisibility::Collapsed;
		}));
		OptionallyAddShotInlineEditCondition(BurnInRow, InActiveConfig, bIsShot,
			[InActiveConfig]{ return !!InActiveConfig->bOverride_BurnInClass; },
			[InActiveConfig](const bool bInNewValue){ InActiveConfig->bOverride_BurnInClass = bInNewValue; });
	}
}

void FJobDetailsCustomization::AddBasicConfigRenderingProperties(
	IDetailLayoutBuilder& InDetailBuilder,
	UMoviePipelineBasicConfig* InActiveConfig,
	const bool bIsShot)
{
	const TArray<UObject*> ActiveConfigObjects = { InActiveConfig };

	IDetailCategoryBuilder& RenderingCategory = InDetailBuilder.EditCategory(
		"Rendering", LOCTEXT("RenderingCategory", "Rendering"));
	RenderingCategory.SetShowAdvanced(false);

	// Renderer toggle row -- for shots, a single row-level override checkbox controls
	// both renderer flags together (renderer choice is a single setting, not two independent ones).
	const FToggleButtonDef RendererButtons[] = {
		{ LOCTEXT("DeferredToggle", "Deferred"), &InActiveConfig->bUseDeferredRenderer,
			[InActiveConfig]{ return !!InActiveConfig->bUsePathTracedRenderer; } },
		{ LOCTEXT("PathTracedToggle", "Path Traced"), &InActiveConfig->bUsePathTracedRenderer,
			[InActiveConfig]{ return !!InActiveConfig->bUseDeferredRenderer; } },
	};
	AddToggleButtonRow(RenderingCategory, InActiveConfig,
		LOCTEXT("RendererFilter", "Renderer"),
		LOCTEXT("RendererLabel", "Renderer"),
		RendererButtons,
		// Row override: enabled in shot mode, controlling both renderer flags together.
		bIsShot
			? TFunction<bool()>([InActiveConfig]{ return !!InActiveConfig->bOverride_bUseDeferredRenderer; })
			: TFunction<bool()>{},
		bIsShot
			? TFunction<void(bool)>([InActiveConfig](const bool bNewValue)
				{
					InActiveConfig->bOverride_bUseDeferredRenderer = bNewValue;
					InActiveConfig->bOverride_bUsePathTracedRenderer = bNewValue;
				})
			: TFunction<void(bool)>{});

	// Properties that apply to all renderers
	OptionallyAddShotInlineEditCondition(
		RenderingCategory.AddExternalObjectProperty(ActiveConfigObjects,
			GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, NumWarmUpFrames)),
		InActiveConfig, bIsShot,
		[InActiveConfig]{ return !!InActiveConfig->bOverride_NumWarmUpFrames; },
		[InActiveConfig](bool v){ InActiveConfig->bOverride_NumWarmUpFrames = v; });
	OptionallyAddShotInlineEditCondition(
		RenderingCategory.AddExternalObjectProperty(ActiveConfigObjects,
			GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, TemporalSampleCount)),
		InActiveConfig, bIsShot,
		[InActiveConfig]{ return !!InActiveConfig->bOverride_TemporalSampleCount; },
		[InActiveConfig](bool v){ InActiveConfig->bOverride_TemporalSampleCount = v; });

	// Always create both renderer sub-groups; bind visibility so toggling is immediate without a full panel refresh.
	const TAttribute<EVisibility> DeferredVisibility = TAttribute<EVisibility>::CreateLambda([InActiveConfig]()
	{
		return InActiveConfig->bUseDeferredRenderer ? EVisibility::Visible : EVisibility::Collapsed;
	});
	const TAttribute<EVisibility> PathTracedVisibility = TAttribute<EVisibility>::CreateLambda([InActiveConfig]()
	{
		return InActiveConfig->bUsePathTracedRenderer ? EVisibility::Visible : EVisibility::Collapsed;
	});

	{
		IDetailGroup& DeferredGroup = RenderingCategory.AddGroup("DeferredRenderer",
			LOCTEXT("DeferredRendererGroup", "Deferred Renderer"), /*bForAdvanced=*/ false, /*bStartExpanded=*/ true);
		DeferredGroup.HeaderRow()
			.Visibility(DeferredVisibility)
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("DeferredRendererGroup", "Deferred Renderer"))
			];
		{
			IDetailPropertyRow& DeferredSpatialRow = DeferredGroup.AddExternalObjectProperty(ActiveConfigObjects,
				GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, DeferredSpatialSampleCount), EPropertyLocation::Default, FAddPropertyParams());
			DeferredSpatialRow.Visibility(DeferredVisibility);
			OptionallyAddShotInlineEditCondition(&DeferredSpatialRow, InActiveConfig, bIsShot,
				[InActiveConfig]{ return !!InActiveConfig->bOverride_DeferredSpatialSampleCount; },
				[InActiveConfig](const bool bNewValue){ InActiveConfig->bOverride_DeferredSpatialSampleCount = bNewValue; });
		}
		{
			IDetailPropertyRow& DeferredAntiAliasingRow = DeferredGroup.AddExternalObjectProperty(ActiveConfigObjects,
				GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, DeferredAntiAliasingMethod), EPropertyLocation::Default, FAddPropertyParams());
			DeferredAntiAliasingRow.Visibility(DeferredVisibility);
			OptionallyAddShotInlineEditCondition(&DeferredAntiAliasingRow, InActiveConfig, bIsShot,
				[InActiveConfig]{ return !!InActiveConfig->bOverride_DeferredAntiAliasingMethod; },
				[InActiveConfig](const bool bNewValue){ InActiveConfig->bOverride_DeferredAntiAliasingMethod = bNewValue; });
		}
	}

	{
		IDetailGroup& PathTracedGroup = RenderingCategory.AddGroup("PathTracedRenderer",
			LOCTEXT("PathTracedRendererGroup", "Path Traced Renderer"), /*bForAdvanced=*/ false, /*bStartExpanded=*/ true);
		PathTracedGroup.HeaderRow()
			.Visibility(PathTracedVisibility)
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("PathTracedRendererGroup", "Path Traced Renderer"))
			];
		{
			IDetailPropertyRow& PathTracerSpatialRow = PathTracedGroup.AddExternalObjectProperty(ActiveConfigObjects,
				GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, PathTracedSpatialSampleCount), EPropertyLocation::Default, FAddPropertyParams());
			PathTracerSpatialRow.Visibility(PathTracedVisibility);
			OptionallyAddShotInlineEditCondition(&PathTracerSpatialRow, InActiveConfig, bIsShot,
				[InActiveConfig]{ return !!InActiveConfig->bOverride_PathTracedSpatialSampleCount; },
				[InActiveConfig](const bool bNewValue){ InActiveConfig->bOverride_PathTracedSpatialSampleCount = bNewValue; });
		}
		{
			IDetailPropertyRow& PathTracerDenoiserRow = PathTracedGroup.AddExternalObjectProperty(ActiveConfigObjects,
				GET_MEMBER_NAME_CHECKED(UMoviePipelineBasicConfig, PathTracedDenoiserType), EPropertyLocation::Default, FAddPropertyParams());
			PathTracerDenoiserRow.Visibility(PathTracedVisibility);
			OptionallyAddShotInlineEditCondition(&PathTracerDenoiserRow, InActiveConfig, bIsShot,
				[InActiveConfig]{ return !!InActiveConfig->bOverride_PathTracedDenoiserType; },
				[InActiveConfig](const bool bNewValue){ InActiveConfig->bOverride_PathTracedDenoiserType = bNewValue; });
		}
	}
}

#undef LOCTEXT_NAMESPACE
