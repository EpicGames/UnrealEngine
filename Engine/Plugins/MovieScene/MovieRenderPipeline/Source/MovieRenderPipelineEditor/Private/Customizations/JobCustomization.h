// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class FObjectPostSaveContext;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class UMovieGraphConfig;
class UMoviePipelineBasicConfig;
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;
class UPackage;
enum class EMoviePipelineConfigMode : uint8;

/**
 * Detail customization for jobs and shots in the queue panel.
 *
 * Surfaces the Basic configuration mode UI (segmented control, output/rendering categories,
 * shot override toggles) in addition to the existing Graph/Preset variable assignments.
 */
class FJobDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual ~FJobDetailsCustomization() override = default;

	virtual void PendingDelete() override;

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	void RefreshLayout(const FString&, UPackage*, FObjectPostSaveContext) const;
	void RefreshLayout(UMoviePipelineExecutorShot*, UMovieGraphConfig*) const;
	void RefreshLayout(UMoviePipelineExecutorJob*, UMovieGraphConfig*) const;
	void RefreshLayout(UMoviePipelineExecutorJob*, EMoviePipelineConfigMode) const;

	/** Adds the Configuration mode row (segmented control + mode-dependent sub-widget) for primary jobs. */
	void AddConfigurationModeRow(IDetailCategoryBuilder& InCategory);

	/** Adds the Output category with output type buttons, gear menu, and per-property override toggles. */
	void AddBasicConfigOutputTypeProperties(IDetailLayoutBuilder& InDetailBuilder, UMoviePipelineBasicConfig* InActiveConfig, bool bIsShot);

	/** Adds the Rendering category with renderer toggles and per-renderer sub-groups. */
	void AddBasicConfigRenderingProperties(IDetailLayoutBuilder& InDetailBuilder, UMoviePipelineBasicConfig* InActiveConfig, bool bIsShot);

	/** The details builder associated with the customization. */
	IDetailLayoutBuilder* DetailBuilder = nullptr;

	/** The primary job that's selected in the UI (can be null). */
	TWeakObjectPtr<UMoviePipelineExecutorJob> SelectedJob;

	/** The shot that's selected in the UI (can be null if a job is selected). */
	TWeakObjectPtr<UMoviePipelineExecutorShot> SelectedShot;
};
