// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IObjectNameEditSink.h"
#include "MoviePipelineQueue.h"

/** Provides an editable JobName in the details panel name area for UMoviePipelineExecutorJob. */
class FMoviePipelineJobNameEditSink : public UE::EditorWidgets::IObjectNameEditSink
{
public:
	virtual UClass* GetSupportedClass() const override
	{
		return UMoviePipelineExecutorJob::StaticClass();
	}

	virtual FText GetObjectDisplayName(UObject* Object) const override
	{
		const UMoviePipelineExecutorJob* Job = CastChecked<UMoviePipelineExecutorJob>(Object);
		return FText::FromString(Job->JobName);
	}

	virtual bool IsObjectDisplayNameReadOnly(UObject* Object) const override
	{
		return false;
	}

	virtual bool SetObjectDisplayName(UObject* Object, FString DisplayName) override
	{
		UMoviePipelineExecutorJob* Job = CastChecked<UMoviePipelineExecutorJob>(Object);
		if (Job->JobName != DisplayName)
		{
			Job->Modify();
			Job->JobName = MoveTemp(DisplayName);
			return true;
		}
		return false;
	}

	virtual FText GetObjectNameTooltip(UObject* Object) const override
	{
		return NSLOCTEXT("MoviePipelineJobNameEditSink", "JobNameTooltip", "The name of this render job");
	}
};

/** Provides a read-only OuterName + InnerName display in the details panel name area for UMoviePipelineExecutorShot. */
class FMoviePipelineShotNameEditSink : public UE::EditorWidgets::IObjectNameEditSink
{
public:
	virtual UClass* GetSupportedClass() const override
	{
		return UMoviePipelineExecutorShot::StaticClass();
	}

	virtual FText GetObjectDisplayName(UObject* Object) const override
	{
		const UMoviePipelineExecutorShot* Shot = CastChecked<UMoviePipelineExecutorShot>(Object);
		return FText::FromString(FString::Printf(TEXT("%s %s"), *Shot->OuterName, *Shot->InnerName));
	}

	virtual bool IsObjectDisplayNameReadOnly(UObject* Object) const override
	{
		return true;
	}

	virtual FText GetObjectNameTooltip(UObject* Object) const override
	{
		return NSLOCTEXT("MoviePipelineJobNameEditSink", "ShotNameTooltip", "The name of this shot");
	}
};
