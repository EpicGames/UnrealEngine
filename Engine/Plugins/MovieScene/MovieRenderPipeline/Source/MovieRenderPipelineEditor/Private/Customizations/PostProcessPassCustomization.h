// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

#include "MoviePipelineDeferredPasses.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineEditor"

/* Customizes how post process passes are displayed (for MRQ, not MRG). */
class FPostProcessPassCustomization_MRQ : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FPostProcessPassCustomization_MRQ>();
	}

protected:
	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		// Include the default header row so the array index "groups" show up (eg, "Index [0]").
		HeaderRow
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			PropertyHandle->CreatePropertyValueWidget()
		];
	}
	
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		static const TArray<FName> HiddenPropertyNames =
		{
			// The "Use Lossless Compression" property is not implemented for MRQ, only MRG
			GET_MEMBER_NAME_CHECKED(FMoviePipelinePostProcessPass, bUseLosslessCompression)
		};
		
		// Add all struct properties except the hidden ones
		uint32 NumChildren;
		if (InStructPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success)
		{
			for (uint32 Index = 0; Index < NumChildren; Index++)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = InStructPropertyHandle->GetChildHandle(Index);

				if (!HiddenPropertyNames.Contains(ChildProperty->GetProperty()->GetFName()))
				{
					StructBuilder.AddProperty(ChildProperty.ToSharedRef());
				}
			}
		}
	}
	//~ End IPropertyTypeCustomization interface
};

#undef LOCTEXT_NAMESPACE