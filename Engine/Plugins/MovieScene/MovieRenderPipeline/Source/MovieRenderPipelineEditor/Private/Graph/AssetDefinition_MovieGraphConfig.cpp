// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AssetDefinition_MovieGraphConfig.h"
#include "Graph/MovieGraphAssetToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MovieGraphConfig)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

TSoftClassPtr<UObject> UAssetDefinition_MovieGraphConfig::GetAssetClass() const
{
	return UMovieGraphConfig::StaticClass();
}

EAssetCommandResult UAssetDefinition_MovieGraphConfig::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UMovieGraphConfig* GraphToEdit : OpenArgs.LoadObjects<UMovieGraphConfig>())
	{
		TSharedRef<FMovieGraphAssetToolkit> MovieGraphEditor = MakeShared<FMovieGraphAssetToolkit>();
		MovieGraphEditor->InitMovieGraphAssetToolkit(EToolkitMode::Standalone, OpenArgs.ToolkitHost, GraphToEdit);
	}	

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
