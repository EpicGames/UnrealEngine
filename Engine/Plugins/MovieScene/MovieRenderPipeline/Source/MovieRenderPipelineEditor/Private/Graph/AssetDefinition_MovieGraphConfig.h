// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphConfig.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_MovieGraphConfig.generated.h"

UCLASS()
class UAssetDefinition_MovieGraphConfig : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MovieGraphConfig", "Movie Render Graph Config"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(80, 200, 80)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Cinematics };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
