// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieEdGraph.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphEdge.h"
#include "Graph/MovieGraphPin.h"
#include "Graph/Nodes/MovieGraphRerouteNode.h"
#include "Graph/Nodes/MovieGraphSubgraphNode.h"
#include "Graph/Nodes/MovieGraphVariableNode.h"
#include "Graph/MovieGraphNode.h"
#include "MovieEdGraphInputNode.h"
#include "MovieEdGraphOutputNode.h"
#include "MovieEdGraphRerouteNode.h"
#include "MovieEdGraphVariableNode.h"
#include "MoviePipelineEdGraphSubgraphNode.h"
#include "MovieRenderPipelineCoreModule.h"
#include "EdGraph/EdGraphPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieEdGraph)

template UMoviePipelineEdGraphNodeBase* UMoviePipelineEdGraph::CreateNodeFromRuntimeNode<UMoviePipelineEdGraphNodeInput>(UMovieGraphNode* InRuntimeNode);
template UMoviePipelineEdGraphNodeBase* UMoviePipelineEdGraph::CreateNodeFromRuntimeNode<UMoviePipelineEdGraphNodeOutput>(UMovieGraphNode* InRuntimeNode);
template UMoviePipelineEdGraphNodeBase* UMoviePipelineEdGraph::CreateNodeFromRuntimeNode<UMoviePipelineEdGraphVariableNode>(UMovieGraphNode* InRuntimeNode);
template UMoviePipelineEdGraphNodeBase* UMoviePipelineEdGraph::CreateNodeFromRuntimeNode<UMoviePipelineEdGraphSubgraphNode>(UMovieGraphNode* InRuntimeNode); 
template UMoviePipelineEdGraphNodeBase* UMoviePipelineEdGraph::CreateNodeFromRuntimeNode<UMoviePipelineEdGraphNode>(UMovieGraphNode* InRuntimeNode);

UMovieGraphConfig* UMoviePipelineEdGraph::GetPipelineGraph() const
{
	return CastChecked<UMovieGraphConfig>(GetOuter());
}

void UMoviePipelineEdGraph::InitFromRuntimeGraph(UMovieGraphConfig* InGraph)
{
	check(InGraph);

	// Runtime nodes may carry stale GraphNode pointers from a previous editor graph (e.g., when a graph was
	// programmatically modified via UMovieGraphConfig APIs after being duplicated from a saved asset, then had
	// its PipelineEdGraph reset to null). Clear any GraphNode that doesn't belong to this editor graph so fresh
	// editor nodes are created below. This ensures the editor representation always matches the runtime state.
	auto ClearStaleGraphNode = [this](UMovieGraphNode* RuntimeNode)
	{
		if (RuntimeNode && RuntimeNode->GraphNode
			&& (!IsValid(RuntimeNode->GraphNode) || RuntimeNode->GraphNode->GetOuter() != this))
		{
			RuntimeNode->GraphNode = nullptr;
		}
	};
	ClearStaleGraphNode(InGraph->GetInputNode());
	ClearStaleGraphNode(InGraph->GetOutputNode());
	for (const TObjectPtr<UMovieGraphNode>& RuntimeNode : InGraph->GetNodes())
	{
		ClearStaleGraphNode(RuntimeNode.Get());
	}

	TMap<UMovieGraphNode*, UMoviePipelineEdGraphNodeBase*> NodeLookup;

	// Input
	{
		UMovieGraphNode* InputNode = InGraph->GetInputNode();
		UMoviePipelineEdGraphNodeBase* EdInputNode = Cast<UMoviePipelineEdGraphNodeBase>(InputNode->GraphNode);
		if (!EdInputNode)
		{
			EdInputNode = CreateNodeFromRuntimeNode<UMoviePipelineEdGraphNodeInput>(InputNode);
		}
		
		NodeLookup.Add(InputNode, EdInputNode);
	}

	// Output
	{
		UMovieGraphNode* OutputNode = InGraph->GetOutputNode();
		UMoviePipelineEdGraphNodeBase* EdOutputNode = Cast<UMoviePipelineEdGraphNodeBase>(OutputNode->GraphNode);
		if (!EdOutputNode)
		{
			EdOutputNode = CreateNodeFromRuntimeNode<UMoviePipelineEdGraphNodeOutput>(OutputNode);
		}
		
		NodeLookup.Add(OutputNode, EdOutputNode);
	}

	// Create the rest of the nodes in the graph
	for (const TObjectPtr<UMovieGraphNode>& RuntimeNode : InGraph->GetNodes())
	{
		// Don't re-create the editor node if it already exists
		if (RuntimeNode->GraphNode)
		{
			NodeLookup.Add(RuntimeNode, Cast<UMoviePipelineEdGraphNodeBase>(RuntimeNode->GraphNode));
			continue;
		}
		
		if (RuntimeNode->IsA<UMovieGraphVariableNode>())
		{
			NodeLookup.Add(RuntimeNode, CreateNodeFromRuntimeNode<UMoviePipelineEdGraphVariableNode>(RuntimeNode));
		}
		else if (RuntimeNode.IsA<UMovieGraphSubgraphNode>())
		{
			NodeLookup.Add(RuntimeNode, CreateNodeFromRuntimeNode<UMoviePipelineEdGraphSubgraphNode>(RuntimeNode));
		}
		else if (RuntimeNode.IsA<UMovieGraphRerouteNode>())
		{
			NodeLookup.Add(RuntimeNode, CreateNodeFromRuntimeNode<UMoviePipelineEdGraphRerouteNode>(RuntimeNode));
		}
		else
		{
			NodeLookup.Add(RuntimeNode, CreateNodeFromRuntimeNode<UMoviePipelineEdGraphNode>(RuntimeNode));
		}
	}

	// Now that we've added an Editor Graph representation for every runtime node in the graph, link
	// the editor nodes together to match the Runtime Layout.
	for (const TPair<UMovieGraphNode*, UMoviePipelineEdGraphNodeBase*>& Pair : NodeLookup)
	{
		constexpr bool bCreateInboundLinks = false;
		constexpr bool bCreateOutboundLinks = true;
		CreateLinks(Pair.Value, bCreateInboundLinks, bCreateOutboundLinks, NodeLookup);
	}

	// NOTE: Commented-out because the runtime graph has been updated to persist an editor graph, so editor-only nodes do not need to be
	// restored anymore. If the runtime graph is updated to not persist the editor graph in the future, the below code will need to be run again.
	//
	// Restore editor-only nodes, which have no runtime node equivalent.
	// for (const TObjectPtr<UObject>& EditorOnlyNodeObject : InGraph->GetEditorOnlyNodes())
	// {
	// 	if (const UEdGraphNode* EdGraphNode = Cast<UEdGraphNode>(EditorOnlyNodeObject))
	// 	{
	// 		UEdGraphNode* NewEdGraphNode = DuplicateObject(EdGraphNode, /*Outer=*/this);
	// 		const bool bIsUserAction = false;
	// 		const bool bSelectNewNode = false;
	// 		AddNode(NewEdGraphNode, bIsUserAction, bSelectNewNode);
	// 	}
	// }

	RegisterDelegates(InGraph);

	bInitialized = true;
}

void UMoviePipelineEdGraph::RegisterDelegates(UMovieGraphConfig* InGraph)
{
	if (!InGraph->OnGraphChangedDelegate.IsBoundToObject(this))
	{
		InGraph->OnGraphChangedDelegate.AddUObject(this, &UMoviePipelineEdGraph::OnGraphConfigChanged);
	}

	if (!InGraph->OnGraphNodesDeletedDelegate.IsBoundToObject(this))
	{
		InGraph->OnGraphNodesDeletedDelegate.AddUObject(this, &UMoviePipelineEdGraph::OnGraphNodesDeleted);
	}
}

void UMoviePipelineEdGraph::CreateLinks(UMoviePipelineEdGraphNodeBase* InGraphNode, bool bCreateInboundLinks, bool bCreateOutboundLinks)
{
	check(InGraphNode);
	
	// Build runtime node to editor node map
	TMap<UMovieGraphNode*, UMoviePipelineEdGraphNodeBase*> RuntimeNodeToEdNodeMap;

	for (const TObjectPtr<UEdGraphNode>& EdGraphNode : Nodes)
	{
		if (UMoviePipelineEdGraphNodeBase* MovieEdNode = Cast<UMoviePipelineEdGraphNodeBase>(EdGraphNode))
		{
			RuntimeNodeToEdNodeMap.Add(MovieEdNode->GetRuntimeNode(), MovieEdNode);
		}
	}

	CreateLinks(InGraphNode, bCreateInboundLinks, bCreateOutboundLinks, RuntimeNodeToEdNodeMap);
}

void UMoviePipelineEdGraph::ReconstructGraph()
{
	OnGraphConfigChanged();
}

void UMoviePipelineEdGraph::CreateLinks(UMoviePipelineEdGraphNodeBase* InGraphNode, bool bCreateInboundLinks, bool bCreateOutboundLinks,
	const TMap<UMovieGraphNode*, UMoviePipelineEdGraphNodeBase*>& RuntimeNodeToEdNodeMap)
{
	check(InGraphNode);
	const UMovieGraphNode* RuntimeNode = InGraphNode->GetRuntimeNode();
	check(RuntimeNode);

	auto CreateLinks = [&](const TArray<UMovieGraphPin*>& Pins, EEdGraphPinDirection PrimaryDirection)
	{
		for (const UMovieGraphPin* Pin : Pins)
		{
			UEdGraphPin* EdPin = InGraphNode->FindPin(Pin->Properties.Label, PrimaryDirection);
			if (!EdPin)
			{
				UE_LOGF(LogMovieRenderPipeline, Error, "Invalid Pin for %ls", *Pin->Properties.Label.ToString());
				continue;
			}

			for (const UMovieGraphEdge* Edge : Pin->Edges)
			{
				if (!Edge || !Edge->IsValid())
				{
					UE_LOGF(LogMovieRenderPipeline, Error, "Invalid edge for Pin: %ls", *Pin->Properties.Label.ToString());
					continue;
				}

				UMovieGraphPin* OtherPin = PrimaryDirection == EEdGraphPinDirection::EGPD_Input ?
					Edge->InputPin : Edge->OutputPin;
				if (UMoviePipelineEdGraphNodeBase* const* ConnectedGraphNode = RuntimeNodeToEdNodeMap.Find(OtherPin->Node))
				{
					EEdGraphPinDirection SecondaryDirection = PrimaryDirection == EEdGraphPinDirection::EGPD_Input ?
						EEdGraphPinDirection::EGPD_Output : EEdGraphPinDirection::EGPD_Input;
					if (UEdGraphPin* OutEdPin = (*ConnectedGraphNode)->FindPin(OtherPin->Properties.Label, SecondaryDirection))
					{
						OutEdPin->MakeLinkTo(EdPin);
					}
					else
					{
						UE_LOGF(LogMovieRenderPipeline, Error, "Could not create link to Pin %ls from Node %ls", *Pin->Properties.Label.ToString(), *OtherPin->Node->GetFName().ToString());
						continue;
					}
				}
			}
		}
	};

	if (bCreateInboundLinks)
	{
		CreateLinks(RuntimeNode->GetInputPins(), EEdGraphPinDirection::EGPD_Input);
	}
	if (bCreateOutboundLinks)
	{
		CreateLinks(RuntimeNode->GetOutputPins(), EEdGraphPinDirection::EGPD_Output);
	}
}

void UMoviePipelineEdGraph::OnGraphConfigChanged()
{
	// TODO: Optimize this. Ideally we can target specific changes to the graph rather than rebuilding everything.
	// However, this isn't strictly necessary unless rebuilding becomes a performance bottleneck.
	for (const TObjectPtr<UEdGraphNode>& Node: Nodes)
	{
		Node->ReconstructNode();
	}
}

void UMoviePipelineEdGraph::OnGraphNodesDeleted(TArray<UMovieGraphNode*> DeletedNodes)
{
	TArray<TObjectPtr<UEdGraphNode>> EdGraphNodesToDelete;
	
	for (const TObjectPtr<UEdGraphNode>& Node : Nodes)
	{
		if (!Node)
		{
			continue;
		}

		if (const UMoviePipelineEdGraphNodeBase* GraphNode = Cast<UMoviePipelineEdGraphNodeBase>(Node.Get()))
		{
			if (DeletedNodes.Contains(GraphNode->GetRuntimeNode()))
			{
				EdGraphNodesToDelete.Add(Node);
			}
		}
	}

	for (TObjectPtr<UEdGraphNode>& EdNodeToDelete : EdGraphNodesToDelete)
	{
		RemoveNode(EdNodeToDelete);
	}
}

template <typename T>
UMoviePipelineEdGraphNodeBase* UMoviePipelineEdGraph::CreateNodeFromRuntimeNode(UMovieGraphNode* InRuntimeNode)
{
	const bool bSelectNewNode = false;
	
	FGraphNodeCreator<T> NodeCreator(*this);
	UMoviePipelineEdGraphNodeBase* NewNode = NodeCreator.CreateNode(bSelectNewNode);
	NewNode->Construct(InRuntimeNode);
	NodeCreator.Finalize();

	return NewNode;
}
