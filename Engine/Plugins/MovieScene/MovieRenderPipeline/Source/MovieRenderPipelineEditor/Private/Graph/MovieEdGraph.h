// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "MovieEdGraph.generated.h"

class UMoviePipelineEdGraphNodeBase;
class UMovieGraphNode;
class UMovieGraphConfig;

/**
* This is the editor-only graph representation of the UMovieGraphConfig. This contains
* editor only nodes (which have information about their X/Y position graphs, their widgets, etc.)
* where each node in this ed graph is tied to a node in the runtime UMovieGraphConfig.
*/
UCLASS()
class UMoviePipelineEdGraph : public UEdGraph
{
	GENERATED_BODY()

public:
	/**
	 * Initialize this Editor Graph from a Runtime Graph.
	 *
	 * This is typically called when the editor representation of the runtime graph is initially created (in order to ensure there are editor
	 * equivalents of all runtime nodes, connections, etc). However, it can also be called to ensure that all runtime objects have editor
	 * equivalents even after initial editor graph creation. This can be useful when, for example, nodes are created via the Python API (which
	 * creates runtime nodes), and editor nodes need to be created afterwards.
	 **/
	void InitFromRuntimeGraph(UMovieGraphConfig* InGraph);

	/** Register delegates that relate to the runtime graph. */
	void RegisterDelegates(UMovieGraphConfig* InGraph);

	/** Returns the runtime UMovieGraphConfig that contains this editor graph */
	class UMovieGraphConfig* GetPipelineGraph() const;

	/** Creates the links/edges between nodes in the graph */
	void CreateLinks(UMoviePipelineEdGraphNodeBase* InGraphNode, bool bCreateInboundLinks, bool bCreateOutboundLinks);

	/** Completely reconstruct the graph. This is a big hammer and should be used sparingly. */
	void ReconstructGraph();

protected:
	void CreateLinks(UMoviePipelineEdGraphNodeBase* InGraphNode, bool bCreateInboundLinks, bool bCreateOutboundLinks,
		const TMap<UMovieGraphNode*, UMoviePipelineEdGraphNodeBase*>& RuntimeNodeToEdNodeMap);

	/**
	 * Handler which deals with changes made to the runtime graph. This will most likely be replaced with more specific
	 * handlers in the future when the graph is more fully-featured.
	 */
	void OnGraphConfigChanged();

	/** Handler which deals with deleted nodes in the runtime graph. The GUIDs provided are those of the deleted runtime nodes. */
	void OnGraphNodesDeleted(TArray<UMovieGraphNode*> DeletedNodes);

protected:
	bool bInitialized = false;

private:
	/** Create a new editor node of type T from the given runtime node. */
	template <typename T>
	UMoviePipelineEdGraphNodeBase* CreateNodeFromRuntimeNode(UMovieGraphNode* InRuntimeNode);
};