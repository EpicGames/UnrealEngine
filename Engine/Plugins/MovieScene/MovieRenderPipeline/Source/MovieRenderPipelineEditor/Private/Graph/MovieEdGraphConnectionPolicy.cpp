// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieEdGraphConnectionPolicy.h"

#include "EdGraph/EdGraphPin.h"
#include "Graph/MovieEdGraph.h"
#include "Graph/MovieGraphSchema.h"

FMovieEdGraphConnectionDrawingPolicy::FMovieEdGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor,
	const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements)
	, MovieEdGraph(CastChecked<UMoviePipelineEdGraph>(InGraphObj))
{
}

void FMovieEdGraphConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	if (!MovieEdGraph)
	{
		return;
	}

	const UMovieGraphSchema* Schema = CastChecked<UMovieGraphSchema>(MovieEdGraph->GetSchema());
	
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;
	Params.WireColor = Schema->GetPinTypeColor(OutputPin->PinType);

	if (OutputPin->PinType.PinCategory == UMovieGraphSchema::PC_Branch)
	{
		Params.WireThickness = Settings->DefaultExecutionWireThickness;
	}
}