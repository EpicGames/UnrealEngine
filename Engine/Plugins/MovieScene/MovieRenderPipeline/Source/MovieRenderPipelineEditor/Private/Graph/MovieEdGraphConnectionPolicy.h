// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"

class UMoviePipelineEdGraph;
struct FConnectionParams;

/** Connection drawing policy for the render graph. */
class FMovieEdGraphConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
public:
	FMovieEdGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);
	
	//~ FMovieGraphConnectionDrawingPolicy interface 
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	//~ End FMovieGraphConnectionDrawingPolicy interface

private:
	UMoviePipelineEdGraph* MovieEdGraph;
};