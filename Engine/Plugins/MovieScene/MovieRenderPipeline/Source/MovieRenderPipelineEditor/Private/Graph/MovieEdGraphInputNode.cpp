// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieEdGraphInputNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieEdGraphInputNode)

#define LOCTEXT_NAMESPACE "MoviePipelineGraph"

FText UMoviePipelineEdGraphNodeInput::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	static const FText NodeTitle = LOCTEXT("InputsNodeTitle", "Inputs");
	return NodeTitle;
}

void UMoviePipelineEdGraphNodeInput::AllocateDefaultPins()
{
	if (RuntimeNode)
	{
		CreatePins(/*InInputPins=*/{}, RuntimeNode->GetOutputPins());
	}
}

#undef LOCTEXT_NAMESPACE
