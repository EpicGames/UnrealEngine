// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Graph/MovieEdGraphNode.h"

#include "MovieEdGraphInputNode.generated.h"


UCLASS()
class UMoviePipelineEdGraphNodeInput : public UMoviePipelineEdGraphNodeBase
{
	GENERATED_BODY()

public:
	// ~Begin UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AllocateDefaultPins() override;
	virtual bool CanUserDeleteNode() const override { return false; }
	// This is set to true so that the node is deserialized on paste, required for CanPasteHere to be called.
	virtual bool CanDuplicateNode() const override { return true; } 
	// And this is set to false so the paste doesn't actually go through, and instead we get the editor prompt notifying user.
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override { return false; }
	// ~End UEdGraphNode interface
};
