// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieEdGraphNode.h"

#include "Graph/MovieGraphPin.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphVariableNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/TransactionObjectEvent.h"
#include "MovieEdGraph.h"
#include "MovieGraphSchema.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditorActions.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Framework/Commands/GenericCommands.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieEdGraphNode)

#define LOCTEXT_NAMESPACE "MoviePipelineEdGraphNodeBase"

void UMoviePipelineEdGraphNodeBase::Construct(UMovieGraphNode* InRuntimeNode)
{
	check(InRuntimeNode);
	RuntimeNode = InRuntimeNode;
	RuntimeNode->GraphNode = this;
	
	NodePosX = InRuntimeNode->GetNodePosX();
	NodePosY = InRuntimeNode->GetNodePosY();
	
	NodeComment = InRuntimeNode->GetNodeComment();
	bCommentBubblePinned = InRuntimeNode->IsCommentBubblePinned();
	bCommentBubbleVisible = InRuntimeNode->IsCommentBubbleVisible();
	
	RegisterDelegates();
	SetEnabledState(InRuntimeNode->IsDisabled() ? ENodeEnabledState::Disabled : ENodeEnabledState::Enabled);
}

void UMoviePipelineEdGraphNodeBase::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	const TArray<FName> ChangedProperties = TransactionEvent.GetChangedProperties();

	if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UEdGraphNode, NodePosX)) ||
		ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UEdGraphNode, NodePosY)))
	{
		UpdatePosition();
	}

	if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UEdGraphNode, bCommentBubblePinned)))
	{
		UpdateCommentBubblePinned();
	}

	if (ChangedProperties.Contains(TEXT("EnabledState")))
	{
		UpdateEnableState();
	}
}

FEdGraphPinType UMoviePipelineEdGraphNodeBase::GetPinType(EMovieGraphValueType ValueType, bool bIsBranch, bool bIsWildcard, const UObject* InValueTypeObject)
{
	FEdGraphPinType EdPinType;
	EdPinType.ResetToDefaults();
	
	EdPinType.PinCategory = NAME_None;
	EdPinType.PinSubCategory = NAME_None;

	// Special case for branch pins
	if (bIsBranch)
	{
		EdPinType.PinCategory = UMovieGraphSchema::PC_Branch;
		return EdPinType;
	}

	// Special case for wildcard pins
	if (bIsWildcard)
	{
		EdPinType.PinCategory = UMovieGraphSchema::PC_Wildcard;
		return EdPinType;
	}

	switch (ValueType)
	{
	case EMovieGraphValueType::Bool:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Boolean;
		break;
	case EMovieGraphValueType::Byte:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Byte;
		break;
	case EMovieGraphValueType::Int32:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Integer;
		break;
	case EMovieGraphValueType::Int64:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Int64;
		break;
	case EMovieGraphValueType::Float:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Real;
		EdPinType.PinSubCategory = UMovieGraphSchema::PC_Float;
		break;
	case EMovieGraphValueType::Double:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Real;
		EdPinType.PinSubCategory = UMovieGraphSchema::PC_Double;
		break;
	case EMovieGraphValueType::Name:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Name;
		break;
	case EMovieGraphValueType::String:
		EdPinType.PinCategory = UMovieGraphSchema::PC_String;
		break;
	case EMovieGraphValueType::Text:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Text;
		break;
	case EMovieGraphValueType::Enum:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Enum;
		EdPinType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UObject*>(InValueTypeObject));
		break;
	case EMovieGraphValueType::Struct:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Struct;
		EdPinType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UObject*>(InValueTypeObject));
		break;
	case EMovieGraphValueType::Object:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Object;
		EdPinType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UObject*>(InValueTypeObject));
		break;
	case EMovieGraphValueType::SoftObject:
		EdPinType.PinCategory = UMovieGraphSchema::PC_SoftObject;
		EdPinType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UObject*>(InValueTypeObject));
		break;
	case EMovieGraphValueType::Class:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Class;
		EdPinType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UObject*>(InValueTypeObject));
		break;
	case EMovieGraphValueType::SoftClass:
		EdPinType.PinCategory = UMovieGraphSchema::PC_SoftClass;
		EdPinType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UObject*>(InValueTypeObject));
		break;
	default:
		EdPinType.PinCategory = UMovieGraphSchema::PC_Float;
		break;
	}
	
	return EdPinType;
}

FEdGraphPinType UMoviePipelineEdGraphNodeBase::GetPinType(const UMovieGraphPin* InPin)
{
	return GetPinType(InPin->Properties.Type, InPin->Properties.bIsBranch, InPin->Properties.bIsWildcard, InPin->Properties.TypeObject);
}

EMovieGraphValueType UMoviePipelineEdGraphNodeBase::GetValueTypeFromPinType(const FEdGraphPinType& InPinType)
{
	static const TMap<FName, EMovieGraphValueType> PinCategoryToValueType =
	{
		{UMovieGraphSchema::PC_Boolean, EMovieGraphValueType::Bool},
		{UMovieGraphSchema::PC_Byte, EMovieGraphValueType::Byte},
		{UMovieGraphSchema::PC_Integer, EMovieGraphValueType::Int32},
		{UMovieGraphSchema::PC_Int64, EMovieGraphValueType::Int64},
		{UMovieGraphSchema::PC_Float, EMovieGraphValueType::Float},
		{UMovieGraphSchema::PC_Double, EMovieGraphValueType::Double},
		{UMovieGraphSchema::PC_Name, EMovieGraphValueType::Name},
		{UMovieGraphSchema::PC_String, EMovieGraphValueType::String},
		{UMovieGraphSchema::PC_Text, EMovieGraphValueType::Text},
		{UMovieGraphSchema::PC_Enum, EMovieGraphValueType::Enum},
		{UMovieGraphSchema::PC_Struct, EMovieGraphValueType::Struct},
		{UMovieGraphSchema::PC_Object, EMovieGraphValueType::Object},
		{UMovieGraphSchema::PC_SoftObject, EMovieGraphValueType::SoftObject},
		{UMovieGraphSchema::PC_Class, EMovieGraphValueType::Class},
		{UMovieGraphSchema::PC_SoftClass, EMovieGraphValueType::SoftClass}
	};

	// Enums can be reported as bytes with a pin sub-category set to the enum
	if (Cast<UEnum>(InPinType.PinSubCategoryObject))
	{
		return EMovieGraphValueType::Enum;
	}

	// Double/float are a bit special: they're reported as a "real" w/ a float/double sub-type
	if (InPinType.PinCategory == UMovieGraphSchema::PC_Real)
	{
		if (InPinType.PinSubCategory == UMovieGraphSchema::PC_Float)
		{
			return EMovieGraphValueType::Float;
		}

		if (InPinType.PinSubCategory == UMovieGraphSchema::PC_Double)
		{
			return EMovieGraphValueType::Double;
		}
	}
	
	if (const EMovieGraphValueType* FoundValueType = PinCategoryToValueType.Find(InPinType.PinCategory))
	{
		return *FoundValueType;
	}

	UE_LOGF(LogMovieRenderPipeline, Warning, "Unable to convert pin type: category [%ls], sub-category [%ls]", *InPinType.PinCategory.ToString(), *InPinType.PinSubCategory.ToString());
	return EMovieGraphValueType::None;
}

void UMoviePipelineEdGraphNodeBase::UpdatePosition() const
{
	if (RuntimeNode)
	{
		RuntimeNode->Modify();
		RuntimeNode->SetNodePosX(NodePosX);
		RuntimeNode->SetNodePosY(NodePosY);
	}
}

void UMoviePipelineEdGraphNodeBase::UpdateCommentBubblePinned() const
{
	if (RuntimeNode)
	{
		RuntimeNode->Modify();
		RuntimeNode->SetIsCommentBubblePinned(bCommentBubblePinned);
	}
}

void UMoviePipelineEdGraphNodeBase::UpdateEnableState() const
{
	if (RuntimeNode)
	{
		RuntimeNode->Modify();
		RuntimeNode->SetDisabled(GetDesiredEnabledState() == ENodeEnabledState::Disabled);
	}
}

void UMoviePipelineEdGraphNodeBase::RegisterDelegates()
{
	if (RuntimeNode)
	{
		RuntimeNode->OnNodeChangedDelegate.AddUObject(this, &UMoviePipelineEdGraphNodeBase::OnRuntimeNodeChanged);
	}
}

void UMoviePipelineEdGraphNode::AllocateDefaultPins()
{
	if (RuntimeNode)
	{
		for(const UMovieGraphPin* InputPin : RuntimeNode->GetInputPins())
		{
			UEdGraphPin* NewPin = CreatePin(EGPD_Input, GetPinType(InputPin), InputPin->Properties.Label);
			NewPin->PinToolTip = GetPinTooltip(InputPin);
		}
		
		for(const UMovieGraphPin* OutputPin : RuntimeNode->GetOutputPins())
		{
			UEdGraphPin* NewPin = CreatePin(EGPD_Output, GetPinType(OutputPin), OutputPin->Properties.Label);
			NewPin->PinToolTip = GetPinTooltip(OutputPin);
		}
	}
}

FText UMoviePipelineEdGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (RuntimeNode)
	{
		const bool bGetDescriptive = true;
		return RuntimeNode->GetNodeTitle(bGetDescriptive);
	}

	// We don't know 100% that the null node is from an unknown plugin, but it's the only known reason why this could happen.
	const FText PluginName = OriginPluginName.IsEmpty() ? LOCTEXT("UnknownPluginName", "Unknown") : FText::FromString(OriginPluginName);
	const FText UnknownPluginNodeTitle = FText::Format(LOCTEXT("NodeTitle_UnknownPlugin", "Node from Unknown Plugin [{0}]\nSaving the graph will result in data loss for this node!\nCan be disconnected, but not deleted."), { PluginName });
	return UnknownPluginNodeTitle;
}

FText UMoviePipelineEdGraphNode::GetTooltipText() const
{
	// Return the UObject name for now for debugging purposes
	return FText::FromString(GetName());
}

void UMoviePipelineEdGraphNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);
	
	if (!Context->Node || !RuntimeNode)
	{
		return;
	}

	GetPropertyPromotionContextMenuActions(Menu, Context);
}

void UMoviePipelineEdGraphNode::GetPropertyPromotionContextMenuActions(UToolMenu* Menu, const UGraphNodeContextMenuContext* Context) const
{
	// Before fetching the overrideable properties, update dynamic properties (dynamic properties may be included in overrideable properties)
	RuntimeNode->UpdateDynamicProperties();
	
	const TArray<FMovieGraphPropertyInfo>& OverrideablePropertyInfo = RuntimeNode->GetOverrideablePropertyInfo();
	
	FToolMenuSection& PinActionsSection = Menu->FindOrAddSection("EdGraphSchemaPinActions");
	if (const UEdGraphPin* SelectedPin = Context->Pin)
	{
		// Find the property info associated with the selected pin
		const FMovieGraphPropertyInfo* PropertyInfo = OverrideablePropertyInfo.FindByPredicate([SelectedPin](const FMovieGraphPropertyInfo& PropInfo)
		{
			return PropInfo.Name == SelectedPin->GetFName();
		});

		// Allow promotion of the property to a variable if the property info could be found. Follow the behavior of blueprints, which allows promotion
		// even if there is an existing connection to the pin.
		if (PropertyInfo)
		{
			const FMovieGraphPropertyInfo& TargetProperty = *PropertyInfo;
			
			PinActionsSection.AddMenuEntry(
				SelectedPin->GetFName(),
				LOCTEXT("PromotePropertyToVariable_Label", "Promote to Variable"),
				LOCTEXT("PromotePropertyToVariable_Tooltip", "Promote this property to a new variable and connect the variable to this pin."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateWeakLambda(this, [this, TargetProperty]()
					{
						PromotePropertyToVariable(TargetProperty);
					}),
					FCanExecuteAction())
			);
		}
	}
	
	FToolMenuSection& ExposeAsPinSection = Menu->AddSection("MoviePipelineGraphExposeAsPin", LOCTEXT("ExposeAsPin", "Expose Property as Pin"));
	for (const FMovieGraphPropertyInfo& PropertyInfo : OverrideablePropertyInfo)
	{
		// If a property is permanently exposed on the node, don't allow it to be toggled off
		if (PropertyInfo.bIsPermanentlyExposed)
		{
			continue;
		}
		
		ExposeAsPinSection.AddMenuEntry(
			PropertyInfo.Name,
			PropertyInfo.ContextMenuName.IsEmpty() ? FText::FromName(PropertyInfo.Name) : PropertyInfo.ContextMenuName,
			LOCTEXT("PromotePropertyToPin", "Promote this property to a pin on this node."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(this, &UMoviePipelineEdGraphNode::TogglePromotePropertyToPin, PropertyInfo.Name),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([this, PropertyInfo]()
				{
					const TArray<FMovieGraphPropertyInfo>& ExposedProperties = RuntimeNode->GetExposedProperties();
					return ExposedProperties.Contains(PropertyInfo) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})),
			EUserInterfaceActionType::ToggleButton
		);
	}

	if (OverrideablePropertyInfo.IsEmpty())
	{
		ExposeAsPinSection.AddMenuEntry(
			"NoPropertiesAvailable",
			FText::FromString("No properties available"),
			LOCTEXT("PromotePropertyToPin_NoneAvailable", "No properties are available to promote."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateLambda([]() { return false; }))
		);
	}

	// Show legacy/deprecated properties as pins so they can be unchecked (to remove the pin), as they no longer show up in the OverrideablePropertyInfo
	// Unfortunately, the only way we know a legacy property is if there's already a pin exposed for it, but it's no longer in the OverrideablePropertyInfo
	// list. Once unchecked, the pin cannot be re-added anymore.
	TArray<FMovieGraphPropertyInfo> DeprecatedProperties;
	for (const FMovieGraphPropertyInfo& ExposedProperty : RuntimeNode->GetExposedProperties())
	{
		// For each of our actually exposed properties, check them against the properties that are overrideable.
		// If we find a match, then we know that the exposed property is still overrideable, thus not a deprecated one.
		bool bIsRealProperty = OverrideablePropertyInfo.ContainsByPredicate([ExposedProperty](const FMovieGraphPropertyInfo& OverrideableProperty)
			{
				return ExposedProperty == OverrideableProperty;
			});

		if (!bIsRealProperty)
		{
			// This adds a copy but the only thing we need from it below are the name and context menu name.
			DeprecatedProperties.Add(ExposedProperty);
		}
	}

	if (DeprecatedProperties.Num() > 0)
	{
		FToolMenuSection& DeprecatedPropertiesSection = Menu->AddSection("DeprecatedProperties", LOCTEXT("DeprecatedProperties", "Deprecated Properties"));
		for (const FMovieGraphPropertyInfo& PropertyInfo : DeprecatedProperties)
		{
			DeprecatedPropertiesSection.AddMenuEntry(
				PropertyInfo.Name,
				PropertyInfo.ContextMenuName.IsEmpty() ? FText::FromName(PropertyInfo.Name) : PropertyInfo.ContextMenuName,
				LOCTEXT("UnPromoteLegacyProperty", "Remove this deprecated pin (which cannot be re-added once removed due to being deprecated)"),
				FSlateIcon(),
				FUIAction(
					// This can still call Toggle because toggle will see that it is a exposed property and allow untoggling it, even if
					// it's no longer a overrideable property (because it's deprecated).
					FExecuteAction::CreateUObject(this, &UMoviePipelineEdGraphNode::TogglePromotePropertyToPin, PropertyInfo.Name),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([this, PropertyInfo]()
						{
							const TArray<FMovieGraphPropertyInfo>& ExposedProperties = RuntimeNode->GetExposedProperties();
							return ExposedProperties.Contains(PropertyInfo) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})),
				EUserInterfaceActionType::ToggleButton
			);

		}
	}

}

void UMoviePipelineEdGraphNode::PromotePropertyToVariable(const FMovieGraphPropertyInfo& TargetProperty) const
{
	FScopedTransaction ScopedTransaction(LOCTEXT("PromotePropertyToVariable_Transaction", "Promote Property to Variable"));
	
	const FName PromotedVariableName = TargetProperty.PromotionName.IsNone() ? TargetProperty.Name : TargetProperty.PromotionName;
	
	// Note: AddVariable() will take care of determining a unique name if there is already a variable with the property's name
	if (UMovieGraphVariable* NewGraphVariable = RuntimeNode->GetGraph()->AddVariable(PromotedVariableName))
	{
		// Set the new variable's type to match the property that is being promoted
		UObject* ValueTypeObject = const_cast<UObject*>(TargetProperty.ValueTypeObject.Get());
		NewGraphVariable->SetValueType(TargetProperty.ValueType, ValueTypeObject);

		// When promoting, set the variable's default value. The node can optionally provide a custom
		// default value (eg, for properties that have special behavior when not overridden). Otherwise,
		// use the property's current value so there's no behavior change in the graph after the promotion.
		{
			FString TargetPropertyValue;
			if (!RuntimeNode->GetCustomPromotionDefaultValue(TargetProperty, TargetPropertyValue))
			{
				if (TargetProperty.bIsDynamicProperty)
				{
					RuntimeNode->GetDynamicPropertyValue(TargetProperty.Name, TargetPropertyValue);
				}
				else if (const FProperty* TargetFProperty = FindFProperty<FProperty>(RuntimeNode->GetClass(), TargetProperty.Name))
				{
					TargetFProperty->ExportTextItem_InContainer(TargetPropertyValue, RuntimeNode, nullptr, RuntimeNode, PPF_None);
				}
			}

			NewGraphVariable->SetValueSerializedString(TargetPropertyValue);
		}

		// When creating the new action, since it's only being used to create a node, the category, display name, and tooltip can just be empty
		const TSharedPtr<FMovieGraphSchemaAction_NewVariableNode> NewAction = MakeShared<FMovieGraphSchemaAction_NewVariableNode>(
			FText::GetEmpty(), FText::GetEmpty(), NewGraphVariable->GetGuid(), FText::GetEmpty());
		NewAction->NodeClass = UMovieGraphVariableNode::StaticClass();

		// Put the new node in a roughly ok-ish position relative to this node
		const FVector2f NewLocation(NodePosX - 200.0f, NodePosY);

		// Note: Providing FromPin will trigger the action to connect the new node and this node
		UEdGraphPin* FromPin = FindPin(TargetProperty.Name, EGPD_Input);
		NewAction->PerformAction(GetGraph(), FromPin, NewLocation);
	}
}

void UMoviePipelineEdGraphNode::TogglePromotePropertyToPin(const FName PropertyName) const
{
	FScopedTransaction ScopedTransaction(LOCTEXT("PromotePropertyToPin_Transaction", "Promote Property to Pin"));
	
	RuntimeNode->TogglePromotePropertyToPin(PropertyName);
}

bool UMoviePipelineEdGraphNodeBase::ShouldCreatePin(const UMovieGraphPin* InPin) const
{
	return true;
}

void UMoviePipelineEdGraphNodeBase::CreatePins(const TArray<UMovieGraphPin*>& InInputPins, const TArray<UMovieGraphPin*>& InOutputPins)
{
	bool bHasAdvancedPin = false;

	for (const UMovieGraphPin* InputPin : InInputPins)
	{
		if (!ShouldCreatePin(InputPin))
		{
			continue;
		}

		UEdGraphPin* Pin = CreatePin(EGPD_Input, GetPinType(InputPin), InputPin->Properties.Label);
		Pin->PinToolTip = GetPinTooltip(InputPin);
		// Pin->bAdvancedView = InputPin->Properties.bAdvancedPin;
		bHasAdvancedPin |= Pin->bAdvancedView;
	}

	for (const UMovieGraphPin* OutputPin : InOutputPins)
	{
		if (!ShouldCreatePin(OutputPin))
		{
			continue;
		}

		UEdGraphPin* Pin = CreatePin(EGPD_Output, GetPinType(OutputPin), OutputPin->Properties.Label);
		Pin->PinToolTip = GetPinTooltip(OutputPin);
		// Pin->bAdvancedView = OutputPin->Properties.bAdvancedPin;
		bHasAdvancedPin |= Pin->bAdvancedView;
	}

	if (bHasAdvancedPin && AdvancedPinDisplay == ENodeAdvancedPins::NoPins)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
	}
	else if (!bHasAdvancedPin)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::NoPins;
	}
}

FString UMoviePipelineEdGraphNodeBase::GetPinTooltip(const UMovieGraphPin* InPin) const
{
	const EMovieGraphValueType PinType = InPin->Properties.Type;
	const FText TypeObjectText = InPin->Properties.TypeObject ? FText::FromString(InPin->Properties.TypeObject.Get()->GetName()) : FText::GetEmpty();

	FText TypeText;
	if (InPin->Properties.bIsBranch)
	{
		TypeText = LOCTEXT("PinTypeTooltip_Branch", "Branch");
	}
	else if (InPin->Properties.bIsWildcard)
	{
		TypeText = LOCTEXT("PinTypeTooltip_Wildcard", "Any");
	}
	else if (InPin->Properties.Type == EMovieGraphValueType::Float)
	{
		// Floats and doubles are compatible with each other in MRG, so to the user, make them both appear as "float" (and give the extra hint about
		// precision for the people who really care)
		TypeText = LOCTEXT("PinTypeTooltip_Float", "Float (single-precision)");
	}
	else if (InPin->Properties.Type == EMovieGraphValueType::Double)
	{
		TypeText = LOCTEXT("PinTypeTooltip_Double", "Float (double-precision)");
	}
	else
	{
		TypeText = StaticEnum<EMovieGraphValueType>()->GetDisplayNameTextByValue(static_cast<int64>(PinType));
	}

	const FText PinTooltipFormat = LOCTEXT("PinTypeTooltip_NoValueTypeObject", "Type: {ValueType}");
	const FText PinTooltipFormatWithTypeObject = LOCTEXT("PinTypeTooltip_WithValueTypeObject", "Type: {ValueType} ({ValueTypeObject})");

	FFormatNamedArguments NamedArgs;
	NamedArgs.Add(TEXT("ValueType"), TypeText);
	NamedArgs.Add(TEXT("ValueTypeObject"), TypeObjectText);

	const FText PinTooltip = InPin->Properties.TypeObject
		? FText::Format(PinTooltipFormatWithTypeObject, NamedArgs)
		: FText::Format(PinTooltipFormat, NamedArgs);

	return PinTooltip.ToString();
}

void UMoviePipelineEdGraphNodeBase::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (RuntimeNode == nullptr || FromPin == nullptr)
	{
		return;
	}

	const bool bFromPinIsInput = FromPin->Direction == EEdGraphPinDirection::EGPD_Input;
	const TArray<UMovieGraphPin*>& OtherPinsList = bFromPinIsInput ? RuntimeNode->GetOutputPins() : RuntimeNode->GetInputPins();

	// Try to connect to the first compatible pin
	bool bDidAutoconnect = false;
	for (const UMovieGraphPin* OtherPin : OtherPinsList)
	{
		check(OtherPin);

		const FName& OtherPinName = OtherPin->Properties.Label;
		UEdGraphPin* ToPin = FindPinChecked(OtherPinName, bFromPinIsInput ? EGPD_Output : EGPD_Input);
		if (ToPin && GetSchema()->TryCreateConnection(FromPin, ToPin))
		{
			// The pin (or owning node) may have been re-generated after the connection was made (eg, this happens with Reroute nodes)
			ToPin = FindPinChecked(OtherPinName, bFromPinIsInput ? EGPD_Output : EGPD_Input);
			
			// Connection succeeded. Notify our other node that their connections changed.
			if (ToPin->GetOwningNode())
			{
				ToPin->GetOwningNode()->NodeConnectionListChanged();
			}
			bDidAutoconnect = true;
			break;
		}
	}

	// Notify ourself of the connection list changing too.
	if (bDidAutoconnect)
	{
		NodeConnectionListChanged();
	}
}

FLinearColor UMoviePipelineEdGraphNodeBase::GetNodeTitleColor() const
{
	if (RuntimeNode)
	{
		return RuntimeNode->GetNodeTitleColor();
	}

	return FLinearColor::Black;
}

FSlateIcon UMoviePipelineEdGraphNodeBase::GetIconAndTint(FLinearColor& OutColor) const
{
	if (RuntimeNode)
	{
		return RuntimeNode->GetIconAndTint(OutColor);
	}

	OutColor = FLinearColor::Yellow;
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Alert");
}

bool UMoviePipelineEdGraphNodeBase::ShowPaletteIconOnNode() const
{
	// Reveals the icon set by GetIconAndTint() in the top-left corner of the node
	return true;
}

void UMoviePipelineEdGraphNodeBase::OnUpdateCommentText(const FString& NewComment)
{
	Super::OnUpdateCommentText(NewComment);

	if (RuntimeNode && (RuntimeNode->GetNodeComment() != NewComment))
	{
		RuntimeNode->SetNodeComment(NewComment);
	}
}

void UMoviePipelineEdGraphNodeBase::OnCommentBubbleToggled(bool bInCommentBubbleVisible)
{
	Super::OnCommentBubbleToggled(bInCommentBubbleVisible);

	if (RuntimeNode && (RuntimeNode->IsCommentBubbleVisible() != bInCommentBubbleVisible))
	{
		RuntimeNode->SetIsCommentBubbleVisible(bInCommentBubbleVisible);
	}
}

void UMoviePipelineEdGraphNodeBase::OnRuntimeNodeChanged(const UMovieGraphNode* InChangedNode)
{
	if (InChangedNode == GetRuntimeNode())
	{
		// During Undo/Redo ReconstructNode gets called twice. When the runtime UObject gets
		// its properties restored, we hear the delegate broadcast and this function gets run,
		// and then the editor objects are restored (and are rebuilt after undo/redo). This
		// creates a problem where when redoing, it restores the editor nodes to a temporary
		// mid-transaction state, which then causes a crash.
		// To avoid this we skip calling ReconstructNode during undo/redo, knowing that it will be
		// reconstructed later alongside the whole graph.
		if (!GIsTransacting)
		{
			ReconstructNode();
		}
	}
}

void UMoviePipelineEdGraphNodeBase::PostLoad()
{
	Super::PostLoad();

	// This might be a node that came from a plugin that's not loaded; if that's the case, skip PostLoad().
	if (!IsValid(RuntimeNode))
	{
		// This should really be a warning, but currently you cannot delete an invalid node. So even if a user bypasses/disconnects an invalid node,
		// they'll see this warning every time they render, which isn't ideal. There will be plenty of other warnings/errors that show up if they
		// do not bypass the node.
		UE_LOGF(LogMovieRenderPipeline, Log, "Found an invalid node during PostLoad(). It's probably from a plugin which is not currently loaded.");
		return;
	}

	// If the plugin this node came from can be determined, cache it. If the plugin is unloaded at some point, we can display this cached plugin name
	// in the UI as a hint.
	const FString NodePackageName = FPackageName::GetShortName(RuntimeNode->GetClass()->GetPackage());
	if (const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().GetModuleOwnerPlugin(FName(NodePackageName)))
	{
		OriginPluginName = Plugin->GetFriendlyName();
	}

	// Some older nodes did not have the pin type properly set on the editor pin (specifically the value type object).
	for (UEdGraphPin* Pin : GetAllPins())
	{
		const UMovieGraphPin* RuntimePin = Pin->Direction == EGPD_Input
			? RuntimeNode->GetInputPin(Pin->PinName)
			: RuntimeNode->GetOutputPin(Pin->PinName);
		
		if (RuntimePin)
		{
			UObject* NonConstValueTypeObject = const_cast<UObject*>(RuntimePin->Properties.TypeObject.Get());
			Pin->PinType.PinSubCategoryObject = MakeWeakObjectPtr(NonConstValueTypeObject);
		}
	}

	RegisterDelegates();	
}

void UMoviePipelineEdGraphNodeBase::ReconstructNode()
{
	// Don't reconstruct the node during copy-paste. If we allow reconstruction,
	// then the editor graph reconstructs connections to previous nodes that were
	// not included in the copy/paste. This does not affect connections within the copy/pasted nodes.
	if (bDisableReconstructNode)
	{
		return;
	}

	// Also don't reconstruct if the runtime node is nullptr. This is most likely because the node is from an unloaded plugin.
	if (!GetRuntimeNode())
	{
		return;
	}

	UMoviePipelineEdGraph* Graph = CastChecked<UMoviePipelineEdGraph>(GetGraph());
	Graph->Modify();

	ReconstructPins();

	// Reconstruct connections
	const bool bCreateInbound = true;
	const bool bCreateOutbound = true;
	Graph->CreateLinks(this, bCreateInbound, bCreateOutbound);

	Graph->NotifyGraphChanged();
}

void UMoviePipelineEdGraphNodeBase::ReconstructPins()
{
	Modify();
	
	// Store copy of old pins
	TArray<UEdGraphPin*> OldPins = MoveTemp(Pins);
	Pins.Reset();
	
	// Generate new pins
	CreatePins(RuntimeNode->GetInputPins(), RuntimeNode->GetOutputPins());
	
	// Transfer persistent data from old to new pins
	for (UEdGraphPin* OldPin : OldPins)
	{
		for (UEdGraphPin* NewPin : Pins)
		{
			if ((OldPin->PinName == NewPin->PinName) && (OldPin->Direction == NewPin->Direction))
			{
				// Remove invalid entries
				OldPin->LinkedTo.Remove(nullptr);

				NewPin->MovePersistentDataFromOldPin(*OldPin);
				break;
			}
		}
	}
	
	// Remove old pins
	for (UEdGraphPin* OldPin : OldPins)
	{
		OldPin->BreakAllPinLinks();
		OldPin->SubPins.Remove(nullptr);
		DestroyPin(OldPin);
	}

	GetGraph()->NotifyGraphChanged();
}

void UMoviePipelineEdGraphNodeBase::PrepareForCopying()
{
	if (RuntimeNode)
	{
		// Temporarily take ownership of the model's node, so that it is not deleted when copying.
		// This is restored in PostCopy
		RuntimeNode->Rename(nullptr, this, REN_DontCreateRedirectors | REN_DoNotDirty);
	}

	const UMoviePipelineEdGraph* MovieGraphEditorGraph = CastChecked<UMoviePipelineEdGraph>(GetGraph());
	const UMovieGraphConfig* RuntimeGraph = MovieGraphEditorGraph->GetPipelineGraph();

	// Track where this node came from for copy/paste purposes
	OriginGraph = RuntimeGraph->GetPathName();
}

void UMoviePipelineEdGraphNodeBase::PostCopy()
{
	if (RuntimeNode)
	{
		// We briefly took ownership of the runtime node to create the copy/paste buffer,
		// restore the ownership back to the owning graph.
		UMoviePipelineEdGraph* MovieGraphEditorGraph = CastChecked<UMoviePipelineEdGraph>(GetGraph());
		UMovieGraphConfig* RuntimeGraph = MovieGraphEditorGraph->GetPipelineGraph();
		check(RuntimeGraph);
		RuntimeNode->Rename(nullptr, RuntimeGraph, REN_DontCreateRedirectors | REN_DoNotDirty);
	}
}

void UMoviePipelineEdGraphNodeBase::PostPasteNode()
{
	bDisableReconstructNode = true;
}

void UMoviePipelineEdGraphNodeBase::PostPaste()
{
	if (RuntimeNode)
	{
		// The editor nodes preserved the connections between nodes when copying/pasting
		// but we intentionally don't preserve the edges of the runtime graph when copying
		// (as the ownership isn't always clear given both input/output edges, which node owns
		// the edge, the one inside the copied graph? Or the one outside it?), so instead 
		// we just rebuild the runtime edge connections based on the editor graph connectivity. 
		RebuildRuntimeEdgesFromPins();

		// Ensure we're listening to the delegate for this pasted node, because we may have skipped ::Construct
		RuntimeNode->OnNodeChangedDelegate.AddUObject(this, &UMoviePipelineEdGraphNodeBase::OnRuntimeNodeChanged);
		RuntimeNode->SetNodePosX(NodePosX);
		RuntimeNode->SetNodePosY(NodePosY);
	}

	bDisableReconstructNode = false;
}

void UMoviePipelineEdGraphNodeBase::RebuildRuntimeEdgesFromPins()
{
	check(RuntimeNode);
	
	for (UEdGraphPin* Pin : Pins)
	{
		// For each of our output pins, find the other editor node it's connected to, then
		// translate that to runtime components and reconnect the runtime components. We only do
		// the output side because it creates a two-way connection, and we're not worried about
		// the nodes outside the copy/pasted nodes, as we won't have reconstructed the connection to them
		// (so the resulting pasted nodes have no connection outside their "island" of copy/pasted nodes)
		if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
		{
			for (UEdGraphPin* LinkedToPin : Pin->LinkedTo)
			{
				UEdGraphNode* ConnectedEdGraphNode = LinkedToPin->GetOwningNode();
				UMoviePipelineEdGraphNodeBase* ConnectedMovieGraphNode = CastChecked<UMoviePipelineEdGraphNodeBase>(ConnectedEdGraphNode);
				
				if (UMovieGraphNode* ConnectedRuntimeNode = ConnectedMovieGraphNode->GetRuntimeNode())
				{
					UMovieGraphConfig* Graph = RuntimeNode->GetGraph();
					check(Graph);

					Graph->AddLabeledEdge(RuntimeNode, Pin->PinName, ConnectedRuntimeNode, LinkedToPin->PinName);
				}
			}
		}
	}

}

void UMoviePipelineEdGraphNodeBase::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->Node)
	{
		return;
	}

	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaNodeActions", LOCTEXT("NodeActionsHeader", "Node Actions"));

		// The runtime node must be valid in order to be manipulated. If it's nullptr (likely from an unloaded plugin) then it cannot be changed.
		if (const UMoviePipelineEdGraphNodeBase* GraphNode = Cast<const UMoviePipelineEdGraphNodeBase>(Context->Node))
		{
			if (GraphNode->GetRuntimeNode())
			{
				Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
				Section.AddMenuEntry(FGenericCommands::Get().Delete);
				Section.AddMenuEntry(FGenericCommands::Get().Cut);
				Section.AddMenuEntry(FGenericCommands::Get().Copy);
				Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
			}
		}

		Section.AddMenuEntry(FGraphEditorCommands::Get().EnableNodes);
		Section.AddMenuEntry(FGraphEditorCommands::Get().DisableNodes);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaOrganization", LOCTEXT("OrganizationHeader", "Organization"));
		Section.AddSubMenu("Alignment", LOCTEXT("AlignmentHeader", "Alignment"), FText(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* AlignmentMenu)
			{
				{
					FToolMenuSection& SubSection = AlignmentMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
				}

				{
					FToolMenuSection& SubSection = AlignmentMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
				}
			}));
	}

	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaCommentGroup", LOCTEXT("CommentGroupHeader", "Comment Group"));
		Section.AddMenuEntry(FGraphEditorCommands::Get().CreateComment,
			LOCTEXT("MultiCommentDesc", "Create Comment from Selection"),
			LOCTEXT("CommentToolTip", "Create a resizable comment box around selection."));
	}
}

#undef LOCTEXT_NAMESPACE
