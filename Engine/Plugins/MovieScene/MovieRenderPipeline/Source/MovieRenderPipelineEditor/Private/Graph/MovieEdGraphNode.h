// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "EdGraph/EdGraphNode.h"
#include "Graph/MovieGraphNode.h"

#include "MovieEdGraphNode.generated.h"

class UMovieGraphNode;

UCLASS(Abstract)
class UMoviePipelineEdGraphNodeBase : public UEdGraphNode
{
	GENERATED_BODY()
public:
	UMovieGraphNode* GetRuntimeNode() const { return RuntimeNode; }

	void Construct(UMovieGraphNode* InRuntimeNode);

	void OnRuntimeNodeChanged(const UMovieGraphNode* InChangedNode);

	//~ Begin UEdGraphNode Interface
	virtual void PostLoad() override;
	virtual void ReconstructNode() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual bool ShowPaletteIconOnNode() const override;
	virtual void OnUpdateCommentText(const FString& NewComment) override;
	virtual void OnCommentBubbleToggled(bool bInCommentBubbleVisible) override;
	virtual void PrepareForCopying() override;
	virtual void PostPasteNode() override;
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	//~ End UEdGraphNode Interface

	static FEdGraphPinType GetPinType(EMovieGraphValueType ValueType, bool bIsBranch, bool bIsWildcard, const UObject* InValueTypeObject = nullptr);
	static FEdGraphPinType GetPinType(const UMovieGraphPin* InPin);
	static EMovieGraphValueType GetValueTypeFromPinType(const FEdGraphPinType& InPinType);

	// Called after PrepareForCopying to restore changes made during preparation. 
	virtual void PostCopy();
	virtual void PostPaste();

	//~ Begin UObject interface
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	//~ End UObject interface

protected:
	virtual bool ShouldCreatePin(const UMovieGraphPin* InPin) const;
	void RebuildRuntimeEdgesFromPins();

	void CreatePins(const TArray<UMovieGraphPin*>& InInputPins, const TArray<UMovieGraphPin*>& InOutputPins);
	
	/** Gets the tooltip for the given pin. */
	virtual FString GetPinTooltip(const UMovieGraphPin* InPin) const;

	/** Recreate the pins on this node, discarding all existing pins. */
	void ReconstructPins();

	/** Update the position of the underlying runtime node to match the editor node. */
	void UpdatePosition() const;

	/**
	 * Update the comment bubble pin state on the underlying runtime node to match the editor node. Note that
	 * comment bubble visibility state and comment text are handled by UEdGraphNode overrides.
	 */
	void UpdateCommentBubblePinned() const;

	/**
	 * Update the enable state of the underlying runtime node to match the editor node.
	 */
	void UpdateEnableState() const;

	/** Registers any delegates needed by the node. */
	void RegisterDelegates();

protected:
	/** The runtime node that this editor node represents. */
	UPROPERTY()
	TObjectPtr<UMovieGraphNode> RuntimeNode;

	/**
	 * Cached friendly name of the plugin that this node originated from. In the event that the origin plugin is unloaded, we can provide the name of
	 * the plugin as a hint. The runtime node will be null in this scenario, so specifying the name of the plugin could be very helpful.
	 */
	UPROPERTY()
	FString OriginPluginName;

	/** Should we early out in ReconstructNode(), skipping restoring the connections to other nodes. Set during Copy/Paste. */
	bool bDisableReconstructNode = false;

	/** The graph that this node originated from during a copy/paste operation. This value should never be used outside of copy/paste logic. */
	UPROPERTY(SkipSerialization)
	FSoftObjectPath OriginGraph;
};

UCLASS()
class UMoviePipelineEdGraphNode : public UMoviePipelineEdGraphNodeBase
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	//~ End UEdGraphNode Interface

protected:
	/** Toggle the promotion of the property with the given name to a pin. */
	void TogglePromotePropertyToPin(const FName PropertyName) const;

private:
	void GetPropertyPromotionContextMenuActions(UToolMenu* Menu, const UGraphNodeContextMenuContext* Context) const;

	/**
	 * Promote the property identified by TargetProperty to a variable. This means 1) creating a new variable w/ the type associated with the
	 * target property, 2) creating a new variable node linked to this new variable, and 3) connecting the variable node to TargetProperty's pin.
	 */
	void PromotePropertyToVariable(const FMovieGraphPropertyInfo& TargetProperty) const;
};
