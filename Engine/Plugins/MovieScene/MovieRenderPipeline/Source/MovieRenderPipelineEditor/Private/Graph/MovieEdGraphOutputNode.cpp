// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieEdGraphOutputNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieEdGraphOutputNode)

#define LOCTEXT_NAMESPACE "MoviePipelineGraph"

FText UMoviePipelineEdGraphNodeOutput::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	static const FText NodeTitle = LOCTEXT("OutputsNodeTitle", "Outputs");
	return NodeTitle;
}

void UMoviePipelineEdGraphNodeOutput::AllocateDefaultPins()
{
	if (RuntimeNode)
	{
		CreatePins(RuntimeNode->GetInputPins(), /*InOutputPins=*/{});
	}
}
#undef LOCTEXT_NAMESPACE
