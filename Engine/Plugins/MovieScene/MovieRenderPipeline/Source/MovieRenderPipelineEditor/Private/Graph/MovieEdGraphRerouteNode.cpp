// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieEdGraphRerouteNode.h"

#include "EdGraph/EdGraphPin.h"
#include "SGraphNodeKnot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieEdGraphRerouteNode)

#define LOCTEXT_NAMESPACE "MoviePipelineGraph"

FText UMoviePipelineEdGraphRerouteNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	static const FText RerouteNodeTitle = LOCTEXT("RerouteNodeTitle", "Reroute");
	return RerouteNodeTitle;
}

UEdGraphPin* UMoviePipelineEdGraphRerouteNode::GetPassThroughPin(const UEdGraphPin* FromPin) const
{
	if (FromPin && Pins.Contains(FromPin))
	{
		return FromPin == Pins[0] ? Pins[1] : Pins[0];
	}

	return nullptr;
}

bool UMoviePipelineEdGraphRerouteNode::ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const
{
	OutInputPinIndex = 0;
	OutOutputPinIndex = 1;

	return true;
}

bool UMoviePipelineEdGraphRerouteNode::CanSplitPin(const UEdGraphPin* Pin) const
{
	return false;
}

TSharedPtr<SGraphNode> UMoviePipelineEdGraphRerouteNode::CreateVisualWidget()
{
	return SNew(SGraphNodeKnot, this);
}

#undef LOCTEXT_NAMESPACE
