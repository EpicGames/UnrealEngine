// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieEdGraphNode.h"

#include "MovieEdGraphRerouteNode.generated.h"

/** A node which reroutes connections in order to organize the graph cleanly. */
UCLASS()
class UMoviePipelineEdGraphRerouteNode : public UMoviePipelineEdGraphNode
{
	GENERATED_BODY()

public:
	// ~Begin UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* FromPin) const override;
	virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override;
	virtual bool CanSplitPin(const UEdGraphPin* Pin) const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	// ~End UEdGraphNode interface
};