// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieEdGraphVariableNode.h"

#include "MovieEdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphVariableNode.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieEdGraphVariableNode)

#define LOCTEXT_NAMESPACE "MoviePipelineGraph"

FText UMoviePipelineEdGraphVariableNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	static const FText VariableNodeTitle = LOCTEXT("GetVariableNodeTitle", "Get Variable");
	static const FText GlobalVariableNodeTitle = LOCTEXT("GetGlobalVariableNodeTitle", "Get Global Variable");

	const UMovieGraphVariableNode* VariableNode = Cast<UMovieGraphVariableNode>(RuntimeNode);

	return (VariableNode && VariableNode->IsGlobalVariable()) ? GlobalVariableNodeTitle : VariableNodeTitle;
}

void UMoviePipelineEdGraphVariableNode::AllocateDefaultPins()
{
	if (const UMovieGraphVariableNode* VariableNode = Cast<UMovieGraphVariableNode>(RuntimeNode))
	{
		const TArray<UMovieGraphPin*>& OutputPins = RuntimeNode->GetOutputPins();
		if (!OutputPins.IsEmpty())
		{
			UEdGraphPin* NewPin = CreatePin(EGPD_Output, GetPinType(OutputPins[0]), FName(VariableNode->GetVariable()->GetMemberName()));
			NewPin->PinToolTip = GetPinTooltip(OutputPins[0]);
		}
	}
}

bool UMoviePipelineEdGraphVariableNode::CanPasteHere(const UEdGraph* TargetGraph) const
{
	const UMoviePipelineEdGraph* MovieEdGraph = Cast<UMoviePipelineEdGraph>(TargetGraph);
	if (!MovieEdGraph)
	{
		return false;
	}
	
	const UMovieGraphConfig* DestinationMovieGraph = MovieEdGraph->GetPipelineGraph();
	if (!DestinationMovieGraph)
	{
		return false;
	}
	
	// Only allow pasting the variable node into the graph that it originated from
	if (DestinationMovieGraph->GetPathName() != OriginGraph.ToString())
	{
		FNotificationInfo Info(LOCTEXT("VariableNodePasteWarning", "Variable nodes cannot be pasted between graphs."));
		Info.ExpireDuration = 3.0f;
		Info.bUseLargeFont = false;
		Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
		const TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_None);
		}
		
		return false;
	}

	return true;
}

FText UMoviePipelineEdGraphVariableNode::GetTooltipText() const
{
	const FString VariableDescription = GetVariableDescription();
	if (!VariableDescription.IsEmpty())
	{
		return FText::FromString(VariableDescription);
	}
	
	return Super::GetTooltipText();
}

FString UMoviePipelineEdGraphVariableNode::GetPinTooltip(const UMovieGraphPin* InPin) const
{
	FString Tooltip = Super::GetPinTooltip(InPin);

	// Add the variable description to the tooltip if available. There's only one pin on the variable node, so we don't need to check to see which
	// pin is being hovered over.
	const FString VariableDescription = GetVariableDescription();
	if (!VariableDescription.IsEmpty())
	{
		Tooltip += "\n\n" + VariableDescription;
	}

	return Tooltip;
}

FString UMoviePipelineEdGraphVariableNode::GetVariableDescription() const
{
	if (const UMovieGraphVariableNode* VariableNode = Cast<UMovieGraphVariableNode>(RuntimeNode))
	{
		if (const UMovieGraphVariable* VariableMember = VariableNode->GetVariable())
		{
			if (!VariableMember->Description.IsEmpty())
			{
				return VariableMember->Description;
			}
		}
	}

	return FString();
}

#undef LOCTEXT_NAMESPACE
