// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieEdGraphNode.h"

#include "MovieEdGraphVariableNode.generated.h"

/** An editor node which represents an accessor for a variable defined on the graph. */
UCLASS()
class UMoviePipelineEdGraphVariableNode : public UMoviePipelineEdGraphNodeBase
{
	GENERATED_BODY()

public:
	// ~Begin UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AllocateDefaultPins() override;
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	virtual FText GetTooltipText() const override;
	// ~End UEdGraphNode interface

protected:
	// ~Begin UMoviePipelineEdGraphNodeBase interface
	virtual FString GetPinTooltip(const UMovieGraphPin* InPin) const override;
	// ~End UMoviePipelineEdGraphNodeBase interface

private:
	/** Get the description (if any) for the variable this node represents. */
	FString GetVariableDescription() const;
};