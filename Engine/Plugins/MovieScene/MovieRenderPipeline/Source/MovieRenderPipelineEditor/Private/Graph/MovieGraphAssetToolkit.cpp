// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphAssetToolkit.h"

#include "Customizations/Graph/MovieGraphApplyCVarPresetNodeCustomization.h"
#include "Customizations/Graph/MovieGraphCollectionsCustomization.h"
#include "Customizations/Graph/MovieGraphExecuteScriptNodeCustomization.h"
#include "Customizations/Graph/MovieGraphLightModifierCustomization.h"
#include "Customizations/Graph/MovieGraphMultiLayerEXRNodeCustomization.h"
#include "Customizations/Graph/MovieGraphMaterialParameterCollectionModifierNodeCustomization.h"
#include "Customizations/Graph/MovieGraphMemberCustomization.h"
#include "Customizations/Graph/MovieGraphMetadataAttributeCustomization.h"
#include "Customizations/Graph/MovieGraphModifiersCustomization.h"
#include "Customizations/Graph/MovieGraphNodeCustomization.h"
#include "Customizations/Graph/MovieGraphPathTracedRendererNodeCustomization.h"
#include "Customizations/Graph/MovieGraphPostProcessPassCustomization.h"
#include "Customizations/Graph/MovieGraphSelectNodeCustomization.h"
#include "Customizations/Graph/MovieGraphSetCVarValueNodeCustomization.h"
#include "Customizations/Graph/MovieGraphShowFlagsCustomization.h"
#include "Customizations/Graph/MovieGraphVersioningSettingsCustomization.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphApplyCVarPresetNode.h"
#include "Graph/Nodes/MovieGraphCollectionNode.h"
#include "Graph/Nodes/MovieGraphExecuteScriptNode.h"
#include "Graph/Nodes/MovieGraphLightModifierNode.h"
#include "Graph/Nodes/MovieGraphMaterialParameterCollectionModifierNode.h"
#include "Graph/Nodes/MovieGraphModifierNode.h"
#include "Graph/Nodes/MovieGraphPathTracerPassNode.h"
#include "Graph/Nodes/MovieGraphSelectNode.h"
#include "Graph/Nodes/MovieGraphSetMetadataAttributesNode.h"
#include "Graph/Nodes/MovieGraphSetCVarValueNode.h"
#include "Graph/Renderers/MovieGraphShowFlags.h"
#include "MovieEdGraphNode.h"
#include "MovieGraphImageSequenceOutputNode.h"
#include "MovieGraphSchema.h"
#include "MoviePipelineCommands.h"
#include "MovieRenderPipelineSettings.h"
#include "SMovieGraphActiveRenderSettingsTabContent.h"
#include "SMovieGraphMembersTabContent.h"

#include "Framework/Commands/GenericCommands.h"
#include "GraphEditor.h"
#include "IDetailRootObjectCustomization.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorModule.h"
#include "Selection.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Graph/SMovieGraphConfigPanel.h"

#define LOCTEXT_NAMESPACE "MovieGraphAssetToolkit"

const FName FMovieGraphAssetToolkit::AppIdentifier(TEXT("MovieGraphAssetEditorApp"));
const FName FMovieGraphAssetToolkit::GraphTabId(TEXT("MovieGraphAssetToolkit"));
const FName FMovieGraphAssetToolkit::DetailsTabId(TEXT("MovieGraphAssetToolkitDetails"));
const FName FMovieGraphAssetToolkit::MembersTabId(TEXT("MovieGraphAssetToolkitMembers"));
const FName FMovieGraphAssetToolkit::ActiveRenderSettingsTabId(TEXT("MovieGraphAssetToolkitActiveRenderSettings"));

void SMovieGraphSyncCollectionToOutlinerButton::Construct(const FArguments& InArgs)
{
	SelectedNodesAttribute = InArgs._SelectedNodes;

	ChildSlot
	[
		SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		.ContentPadding(0.f)
		.ToolTipText(LOCTEXT("PreviewCollectionButton_Tooltip", "Evaluate the collection and select the matched actors in the Outliner."))
		.Visibility_Lambda([this]()
		{
			const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = SelectedNodesAttribute.Get();
			if ((SelectedObjects.Num() == 1) && SelectedObjects[0].IsValid() && SelectedObjects[0]->IsA<UMovieGraphCollectionNode>())
			{
				return EVisibility::Visible;
			}

			return EVisibility::Hidden;
		})
		.OnClicked_Lambda([this]()
		{
			EvaluateCollectionAndSelect();

			return FReply::Handled();
		})
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("FoliageEditMode.SelectAll"))
		]
	];
}

void SMovieGraphSyncCollectionToOutlinerButton::EvaluateCollectionAndSelect() const
{
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = SelectedNodesAttribute.Get();
	if ((SelectedObjects.Num() != 1) || !SelectedObjects[0].IsValid())
	{
		return;
	}
			
	if (const UMovieGraphCollectionNode* CollectionNode = Cast<UMovieGraphCollectionNode>(SelectedObjects[0]))
	{
		// Evaluate the collection based on the current editor world
		const FMovieGraphEvaluationResult EvaluationResult =
			CollectionNode->Collection->EvaluateActorsAndComponents(GEditor->GetEditorWorldContext().World());

		// Select all actors matched by the collection
		{
			GEditor->GetSelectedActors()->Modify();
			GEditor->GetSelectedActors()->BeginBatchSelectOperation();
			GEditor->GetSelectedActors()->DeselectAll();
			
			for (AActor* Actor : EvaluationResult.MatchingActors)
			{
				constexpr bool bShouldSelect = true;
				constexpr bool bNotifyAfterSelect = false;
				constexpr bool bSelectEvenIfHidden = true;
				GEditor->SelectActor(Actor, bShouldSelect, bNotifyAfterSelect, bSelectEvenIfHidden);
			}

			constexpr bool bNotify = false;
			GEditor->GetSelectedActors()->EndBatchSelectOperation(bNotify);
		}
	}
}

/** Header customization for when multiple objects are displayed in the details panel. */
class FMovieGraphDetailsRootObjectCustomization final : public IDetailRootObjectCustomization
{
public:
	explicit FMovieGraphDetailsRootObjectCustomization(const TSharedRef<IDetailsView>& InDetailsView)
		: DetailsView(InDetailsView)
	{
		
	}

	// IDetailRootObjectCustomization interface
	virtual TSharedPtr<SWidget> CustomizeObjectHeader(const FDetailsObjectSet& InRootObjectSet, const TSharedPtr<ITableRow>& InTableRow) override
	{
		FText DisplayName = InRootObjectSet.CommonBaseClass->GetDisplayNameText();
		if (const UMovieGraphNode* GraphNode = Cast<const UMovieGraphNode>(InRootObjectSet.RootObjects[0]))
		{
			DisplayName = GraphNode->GetNodeTitle();
		}
		
		return
			SNew(SBox)
			.Padding(5.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(DisplayName)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
			];
	}

	virtual bool AreObjectsVisible(const FDetailsObjectSet& InRootObjectSet) const override
	{
		return true;
	}
	
	virtual bool ShouldDisplayHeader(const FDetailsObjectSet& InRootObjectSet) const override
	{
		// Only display the header if multiple objects are selected
		if (DetailsView.IsValid())
		{
			return DetailsView.Pin()->GetSelectedObjects().Num() > 1;
		}
		
		return true;
	}
	// End IDetailRootObjectCustomization interface

private:
	TWeakPtr<IDetailsView> DetailsView;
};

FMovieGraphAssetToolkit::FMovieGraphAssetToolkit()
	: bIsInternalSelectionChange(false)
{
}

void FMovieGraphAssetToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(
		LOCTEXT("WorkspaceMenu_MovieGraphAssetToolkit", "Render Graph Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(GraphTabId, FOnSpawnTab::CreateSP(this, &FMovieGraphAssetToolkit::SpawnTab_RenderGraphEditor))
		.SetDisplayName(LOCTEXT("RenderGraphTab", "Render Graph"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(DetailsTabId, FOnSpawnTab::CreateSP(this, &FMovieGraphAssetToolkit::SpawnTab_RenderGraphDetails))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(MembersTabId, FOnSpawnTab::CreateSP(this, &FMovieGraphAssetToolkit::SpawnTab_RenderGraphMembers))
		.SetDisplayName(LOCTEXT("MembersTab", "Members"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(ActiveRenderSettingsTabId, FOnSpawnTab::CreateSP(this, &FMovieGraphAssetToolkit::SpawnTab_RenderGraphActiveRenderSettings))
		.SetDisplayName(LOCTEXT("ActiveRenderSettingsTab", "Active Render Settings"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Debug"));
}

void FMovieGraphAssetToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	
	InTabManager->UnregisterTabSpawner(GraphTabId);
	InTabManager->UnregisterTabSpawner(DetailsTabId);
	InTabManager->UnregisterTabSpawner(MembersTabId);
	InTabManager->UnregisterTabSpawner(ActiveRenderSettingsTabId);
}

void FMovieGraphAssetToolkit::InitMovieGraphAssetToolkit(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UMovieGraphConfig* InitGraph)
{
	InitialGraph = InitGraph;
	
	// Note: Changes to the layout should include a increment to the layout's ID, i.e.
	// MoviePipelineRenderGraphEditor[X] -> MoviePipelineRenderGraphEditor[X+1]. Otherwise, layouts may be messed up
	// without a full reset to layout defaults inside the editor.
	const FName LayoutString = FName("MoviePipelineRenderGraphEditor2");

	// Override the Default Layout provided by FBaseAssetToolkit to hide the viewport and details panel tabs.
	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout(FName(LayoutString))
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(1.f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->AddTab(MembersTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->SetHideTabWell(true)
					->AddTab(GraphTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
					FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(DetailsTabId, ETabState::OpenedTab)
					)
					->Split
					(
					FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(ActiveRenderSettingsTabId, ETabState::OpenedTab)
					)
				)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	InitAssetEditor(Mode, InitToolkitHost, AppIdentifier, Layout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InitGraph);

	BindGraphCommands();
	ExtendToolkitMenu();
}

TSharedRef<SDockTab> FMovieGraphAssetToolkit::SpawnTab_RenderGraphEditor(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		.TabColorScale(GetTabColorScale())
		.Label(LOCTEXT("GraphTab_Title", "Graph"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				GetDefaultGraphEditWarning()
			]
			
			+ SVerticalBox::Slot()
			[
				SAssignNew(MovieGraphWidget, SMoviePipelineGraphPanel)
				.Graph(InitialGraph)
				.OnGraphSelectionChanged_Lambda([this](const TArray<UObject*>& NewSelection) -> void
				{
					if (bIsInternalSelectionChange)
					{
						return;
					}
					
					// Reset selection in the Members panel
					if (MembersTabContent.IsValid())
					{
						TGuardValue<bool> GuardSelectionChange(bIsInternalSelectionChange, true);
						MembersTabContent->ClearSelection();
					}

					if (SelectedGraphObjectsDetailsWidget.IsValid())
					{
						SelectedGraphObjectsDetailsWidget->SetObjects(NewSelection);
					}
				})
			]
		];

	return NewDockTab;
}

TSharedRef<SDockTab> FMovieGraphAssetToolkit::SpawnTab_RenderGraphMembers(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		.TabColorScale(GetTabColorScale())
		.Label(LOCTEXT("MembersTab_Title", "Members"))
		[
			SAssignNew(MembersTabContent, SMovieGraphMembersTabContent)
			.Editor(SharedThis(this))
			.Graph(InitialGraph)
			.OnActionSelected_Lambda([this](const TArray<TSharedPtr<FEdGraphSchemaAction>>& Selection, ESelectInfo::Type Type)
			{
				if (bIsInternalSelectionChange)
				{
					return;
				}
				
				// Reset selection in the graph
				if (MovieGraphWidget.IsValid())
				{
					TGuardValue<bool> GuardSelectionChange(bIsInternalSelectionChange, true);
					MovieGraphWidget->ClearGraphSelection();
				}
				
				TArray<UObject*> SelectedObjects;
				for (const TSharedPtr<FEdGraphSchemaAction>& SelectedAction : Selection)
				{
					const FMovieGraphSchemaAction* GraphAction = static_cast<FMovieGraphSchemaAction*>(SelectedAction.Get());
					if (GraphAction->ActionTarget)
					{
						SelectedObjects.Add(GraphAction->ActionTarget);
					}
				}

				if (SelectedGraphObjectsDetailsWidget.IsValid())
				{
					SelectedGraphObjectsDetailsWidget->SetObjects(SelectedObjects);
				}
			})
		];

	return NewDockTab;
}

TSharedRef<SDockTab> FMovieGraphAssetToolkit::SpawnTab_RenderGraphDetails(const FSpawnTabArgs& Args)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bCustomNameAreaLocation = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.ViewIdentifier = "MovieGraphSettings";
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bShowSectionSelector = true;

	RegisterDetailsViewSections();

	SelectedGraphObjectsDetailsWidget = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	SelectedGraphObjectsDetailsWidget->SetRootObjectCustomizationInstance(
		MakeShared<FMovieGraphDetailsRootObjectCustomization>(SelectedGraphObjectsDetailsWidget.ToSharedRef()));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphMember::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphMemberCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphNodeCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphSelectNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphSelectNodeCustomization::MakeInstance));
	
	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphSetCVarValueNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphSetCVarValueNodeCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphApplyCVarPresetNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphApplyCVarPresetNodeCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphMaterialParameterCollectionModifierNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphMaterialParameterCollectionModifierNodeCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyTypeLayout(
		UMovieGraphShowFlags::StaticClass()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieGraphShowFlagsCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyTypeLayout(
		FMovieGraphMetadataAttribute::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieGraphMetadataAttributeCustomization::MakeInstance));
	
	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyTypeLayout(
		FMovieGraphVersioningSettings::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieGraphVersioningSettingsCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphImageSequenceOutputNode_EXR::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphMultiLayerEXRNodeCustomization::MakeInstance));
	
	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphCollectionNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphCollectionsCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphModifierNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphModifiersCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphLightModifierNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphLightModifierCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyTypeLayout(
		FMoviePipelinePostProcessPass::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieGraphPostProcessPassCustomization::MakeInstance));
	
	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphPathTracerRenderPassNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphPathTracedRendererNodeCustomization::MakeInstance));

	SelectedGraphObjectsDetailsWidget->RegisterInstancedCustomPropertyLayout(
		UMovieGraphExecuteScriptNode::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphExecuteScriptNodeCustomization::MakeInstance));
	
	TSharedRef<SWidget> CustomContent = SAssignNew(NameAreaCustomContent, SHorizontalBox)
	+ SHorizontalBox::Slot()
	[
		SNew(SMovieGraphSyncCollectionToOutlinerButton)
		.SelectedNodes_Lambda([this]()
		{
			return SelectedGraphObjectsDetailsWidget->GetSelectedObjects();
		})
	];
	
	SelectedGraphObjectsDetailsWidget->SetNameAreaCustomContent(CustomContent);

	return SNew(SDockTab)
		.TabColorScale(GetTabColorScale())
		.Label(LOCTEXT("DetailsTab_Title", "Details"))
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.Padding(10.f, 4.f, 0.f, 0.f)
			.AutoHeight()
			[
				SelectedGraphObjectsDetailsWidget->GetNameAreaWidget().ToSharedRef()
			]

			+ SVerticalBox::Slot()
			[
				SelectedGraphObjectsDetailsWidget.ToSharedRef()
			]
		];
}

TSharedRef<SDockTab> FMovieGraphAssetToolkit::SpawnTab_RenderGraphActiveRenderSettings(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.TabColorScale(GetTabColorScale())
		.Label(LOCTEXT("ActiveRenderSettings_Title", "Active Render Settings"))
		[
			SAssignNew(ActiveRenderSettingsTabContent, SMovieGraphActiveRenderSettingsTabContent)
			.Graph(InitialGraph)
		];
}

void FMovieGraphAssetToolkit::RegisterDetailsViewSections()
{
	static const FName PropertyEditor("PropertyEditor");
	static const FName CategoryMetadataKey("Category");
	static const TArray<FString> CategoriesToExclude = { FString(TEXT("Tags")) };
	
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

	// Do a 1:1 mapping of node categories to section names
	for (const UClass* NodeClass : UMovieGraphSchema::GetNodeClasses())
	{
		for (TFieldIterator<FProperty> PropertyIt(NodeClass); PropertyIt; ++PropertyIt)
		{
			const FProperty* NodeProperty = *PropertyIt;
			const FString* CategoryMetadata = NodeProperty->FindMetaData(CategoryMetadataKey);

			if (CategoryMetadata && !CategoryMetadata->IsEmpty() && !CategoriesToExclude.Contains(*CategoryMetadata))
			{
				// Only top-level categories can be added as sections.
				if (CategoryMetadata->Contains(TEXT("|")))
				{
					continue;
				}

				const FName SectionName = FName(*CategoryMetadata);
				const FText LocalizedCategory = FObjectEditorUtils::GetCategoryText(NodeProperty);
				
				const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(NodeClass->GetFName(), SectionName, LocalizedCategory);
				Section->AddCategory(SectionName);
			}
		}
	}
}

void FMovieGraphAssetToolkit::BindGraphCommands()
{
	ToolkitCommands->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FMovieGraphAssetToolkit::DeleteSelectedMembers),
		FCanExecuteAction::CreateSP(this, &FMovieGraphAssetToolkit::CanDeleteSelectedMembers));

	ToolkitCommands->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FMovieGraphAssetToolkit::DuplicateSelectedMembers),
		FCanExecuteAction::CreateSP(this, &FMovieGraphAssetToolkit::CanDuplicateSelectedMembers),
		FIsActionChecked::CreateLambda([]() { return false; }),
		FCanExecuteAction::CreateSP(this, &FMovieGraphAssetToolkit::IsDuplicateVisible));

	ToolkitCommands->MapAction(FMoviePipelineCommands::Get().ZoomToWindow,
			FExecuteAction::CreateSP(this, &FMovieGraphAssetToolkit::OnZoomToWindow),
			FCanExecuteAction::CreateSP(this, &FMovieGraphAssetToolkit::CanZoomToWindow));
		
	ToolkitCommands->MapAction(FMoviePipelineCommands::Get().ZoomToSelection,
		FExecuteAction::CreateSP(this, &FMovieGraphAssetToolkit::OnZoomToSelection),
		FCanExecuteAction::CreateSP(this, &FMovieGraphAssetToolkit::CanZoomToSelection));
}

void FMovieGraphAssetToolkit::ExtendToolkitMenu() const
{
	UToolMenus* ToolMenus = UToolMenus::Get();	
	if (UToolMenu* MainMenu = ToolMenus->ExtendMenu(GetToolMenuName()))
	{
		FToolMenuSection& Section = MainMenu->FindOrAddSection(NAME_None);
		if (!Section.FindEntry("View"))
		{
			Section.AddSubMenu(
				"View",
				LOCTEXT("ViewMenu", "View"),
				LOCTEXT("ViewMenu_ToolTip", "Open the View menu"),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					FToolMenuSection& ZoomSection = InMenu->AddSection("ViewZoom", LOCTEXT("ViewMenuZoomHeading", "Zoom"));
					ZoomSection.AddMenuEntry(FMoviePipelineCommands::Get().ZoomToWindow);
					ZoomSection.AddMenuEntry(FMoviePipelineCommands::Get().ZoomToSelection);
				})
			).InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);
		}
	}
}

void FMovieGraphAssetToolkit::DeleteSelectedMembers()
{
	if (MembersTabContent.IsValid())
	{
		MembersTabContent->DeleteSelectedMembers();
	}
}

bool FMovieGraphAssetToolkit::CanDeleteSelectedMembers()
{
	if (MembersTabContent.IsValid())
	{
		return MembersTabContent->CanDeleteSelectedMembers();
	}

	return false;
}

void FMovieGraphAssetToolkit::DuplicateSelectedMembers()
{
	if (MembersTabContent.IsValid())
	{
		MembersTabContent->DuplicateSelectedMembers();
	}
}

bool FMovieGraphAssetToolkit::CanDuplicateSelectedMembers()
{
	if (MembersTabContent.IsValid())
	{
		return MembersTabContent->CanDuplicateSelectedMembers();
	}

	return false;
}

bool FMovieGraphAssetToolkit::IsDuplicateVisible()
{
	return CanDuplicateSelectedMembers();
}

void FMovieGraphAssetToolkit::OnZoomToWindow() const
{
	if (MovieGraphWidget.IsValid())
	{
		if (const TSharedPtr<SGraphEditor> GraphEditor = MovieGraphWidget->GetGraphEditor().Pin())
		{
			constexpr bool bOnlySelection = false;
			GraphEditor->ZoomToFit(bOnlySelection);
		}
	}
}

bool FMovieGraphAssetToolkit::CanZoomToWindow() const
{
	return MovieGraphWidget.IsValid() && MovieGraphWidget->GetGraphEditor().IsValid();
}

void FMovieGraphAssetToolkit::OnZoomToSelection() const
{
	if (MovieGraphWidget.IsValid())
	{
		if (const TSharedPtr<SGraphEditor> GraphEditor = MovieGraphWidget->GetGraphEditor().Pin())
		{
			constexpr bool bOnlySelection = true;
			GraphEditor->ZoomToFit(bOnlySelection);
		}
	}
}

bool FMovieGraphAssetToolkit::CanZoomToSelection() const
{
	return MovieGraphWidget.IsValid() && MovieGraphWidget->GetGraphEditor().IsValid();
}

void FMovieGraphAssetToolkit::PersistEditorOnlyNodes() const
{
	if (InitialGraph && InitialGraph->PipelineEdGraph)
	{
		TArray<TObjectPtr<const UObject>> EditorOnlyNodes;
		for (const TObjectPtr<UEdGraphNode>& GraphEdNode : InitialGraph->PipelineEdGraph->Nodes)
		{
			// Treat any non-MRQ nodes as editor-only nodes
			check(GraphEdNode);
			if (!GraphEdNode->IsA<UMoviePipelineEdGraphNodeBase>())
			{
				EditorOnlyNodes.Add(GraphEdNode);
			}
		}

		InitialGraph->SetEditorOnlyNodes(EditorOnlyNodes);
	}
}

TSharedRef<SWidget> FMovieGraphAssetToolkit::GetDefaultGraphEditWarning() const
{
	// Determine if the default graph is being edited
	bool bIsDefaultGraphBeingEdited = false;
	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	const TSoftObjectPtr<UMovieGraphConfig> ProjectDefaultGraph = ProjectSettings->DefaultGraph;
	if (const UMovieGraphConfig* DefaultGraph = ProjectDefaultGraph.LoadSynchronous())
	{
		bIsDefaultGraphBeingEdited = (DefaultGraph == InitialGraph.Get());
	}
	
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Warning"))
		.BorderBackgroundColor_Lambda([]()
		{
			FLinearColor WarningColor = FAppStyle::GetSlateColor("Colors.Warning").GetSpecifiedColor();
			WarningColor.A = 0.3f;
			return WarningColor;
		})
		.Padding(5.f)
		.Visibility(bIsDefaultGraphBeingEdited ? EVisibility::Visible : EVisibility::Collapsed)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 0, 5.f, 0)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.WarningWithColor"))
			]

			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("GraphTab_EditingDefaultGraphWarning", "The default graph asset is being edited. 'Save As' to save a new graph asset."))
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
			]
		];
}

FName FMovieGraphAssetToolkit::GetToolkitFName() const
{
	return FName("MovieGraphEditor");
}

FText FMovieGraphAssetToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("MovieGraphAppLabel", "Movie Graph Editor");
}

FString FMovieGraphAssetToolkit::GetWorldCentricTabPrefix() const
{
	return TEXT("MovieGraphEditor");
}

FLinearColor FMovieGraphAssetToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

void FMovieGraphAssetToolkit::SaveAsset_Execute()
{
	// Editor-only nodes are copied to the underlying runtime graph on save/close
	PersistEditorOnlyNodes();
	
	// Perform the default save process
	// TODO: This will fail silently on a transient graph and won't trigger a Save As
	FAssetEditorToolkit::SaveAsset_Execute();

	// TODO: Any custom save logic here
}

void FMovieGraphAssetToolkit::OnAssetsSavedAs(const TArray<UObject*>& SavedObjects)
{
	FAssetEditorToolkit::OnAssetsSavedAs(SavedObjects);

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

	// The default behavior for SaveAs in the toolkit doesn't properly re-open the assets that were saved during a SaveAs, it only closes the assets
	// that were the source of the SaveAs. After a SaveAs, the graph potentially goes through a complete data change, and re-opening is the most
	// reliable way to make sure that the graph and editor are properly in sync. Without this, connections, delegates, etc can get badly out-of-sync
	// after a SaveAs. Generally a crash won't result, but the graph will be in a nearly unusable state.
	for (UObject* SavedObject : SavedObjects)
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(SavedObject);
		AssetEditorSubsystem->NotifyAssetClosed(SavedObject, this);
	}

	AssetEditorSubsystem->OpenEditorForAssets_Advanced(SavedObjects, ToolkitMode, ToolkitHost.Pin());
}

void FMovieGraphAssetToolkit::OnClose()
{
	// Editor-only nodes are copied to the underlying runtime graph on save/close
	PersistEditorOnlyNodes();
	
	FAssetEditorToolkit::OnClose();
}

#undef LOCTEXT_NAMESPACE
