// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"

/** Evaluates a collection and syncs the matched actors to the Outliner selection. */
class SMovieGraphSyncCollectionToOutlinerButton final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMovieGraphSyncCollectionToOutlinerButton) {}
		/** The nodes selected in the graph. */
		SLATE_ATTRIBUTE(TArray<TWeakObjectPtr<UObject>>, SelectedNodes)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Evaluates the collection and selects all matches in the Outliner. */
	void EvaluateCollectionAndSelect() const;

private:
	/** The nodes which are currently selected in the graph. */
	TAttribute<TArray<TWeakObjectPtr<UObject>>> SelectedNodesAttribute;
};

class FMovieGraphAssetToolkit :  public FAssetEditorToolkit
{
public:
	FMovieGraphAssetToolkit();
	virtual ~FMovieGraphAssetToolkit() override = default;

	//~ Begin FAssetEditorToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual void SaveAsset_Execute() override;
	virtual void OnAssetsSavedAs(const TArray<UObject*>& SavedObjects) override;
	virtual void OnClose() override;
	//~ End FAssetEditorToolkit interface

	//~ Begin IToolkit interface
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit interface

	void InitMovieGraphAssetToolkit(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, class UMovieGraphConfig* InitGraph);

private:
	TSharedRef<SDockTab> SpawnTab_RenderGraphEditor(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_RenderGraphMembers(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_RenderGraphDetails(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_RenderGraphActiveRenderSettings(const FSpawnTabArgs& Args);

	/** Register sections used by the graph's details panel. */
	void RegisterDetailsViewSections();

	/** Register commands used by the editor. */
	void BindGraphCommands();

	/** Adds additional commands to the toolkit's menu. */
	void ExtendToolkitMenu() const;

	/** Deletes the member(s) which are currently selected. */
	void DeleteSelectedMembers();

	/** Determines if the selected member(s) can be deleted. */
	bool CanDeleteSelectedMembers();

	/** Duplicates the member(s) which are currently selected. */
	void DuplicateSelectedMembers();

	/** Determines if the selected member(s) can be duplicated. */
	bool CanDuplicateSelectedMembers();

	/** Determines if the Duplicate action is visible for the selected members. */
	bool IsDuplicateVisible();

	/** Frames the entire graph so it is visible. */
	void OnZoomToWindow() const;

	/** Determines if the entire graph can be framed. */
	bool CanZoomToWindow() const;

	/** Frames the selected node(s) so they are visible. */
	void OnZoomToSelection() const;

	/** Determines if the selected node(s) can be framed. */
	bool CanZoomToSelection() const;

	/** Copies editor-only nodes to the underlying runtime graph so they can be restored later. */
	void PersistEditorOnlyNodes() const;

	/** Generate the widget that displays a warning banner if the default graph is being edited. */
	TSharedRef<SWidget> GetDefaultGraphEditWarning() const;

private:
	/** The details panel for the selected object(s) in the graph */
	TSharedPtr<IDetailsView> SelectedGraphObjectsDetailsWidget;

	/** Custom widgetry shown above the filter box in the details panel. */
	TSharedPtr<SWidget> NameAreaCustomContent;

	/** The widget that contains the main node graph */
	TSharedPtr<class SMoviePipelineGraphPanel> MovieGraphWidget;

	/** The widget which encapsulates all of the content in the Members tab */
	TSharedPtr<class SMovieGraphMembersTabContent> MembersTabContent;

	/** The widget which encapsulates all of the content in the Active Render Settings tab */
	TSharedPtr<class SMovieGraphActiveRenderSettingsTabContent> ActiveRenderSettingsTabContent;
	
	/** The graph that the editor was initialized with */
	TObjectPtr<UMovieGraphConfig> InitialGraph;

	/** Flag which is set when a selection change is not triggered by the user */
	bool bIsInternalSelectionChange;

	static const FName AppIdentifier;
	static const FName GraphTabId;
	static const FName DetailsTabId;
	static const FName MembersTabId;
	static const FName ActiveRenderSettingsTabId;
};
