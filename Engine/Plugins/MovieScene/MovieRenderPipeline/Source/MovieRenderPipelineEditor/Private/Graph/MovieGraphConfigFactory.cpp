// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphConfigFactory.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphSubgraphNode.h"
#include "HAL/IConsoleManager.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieRenderPipelineSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphConfigFactory)

UMovieGraphConfigFactory::UMovieGraphConfigFactory()
{
	bCreateNew = true;
	bEditAfterNew = false;
	SupportedClass = UMovieGraphConfig::StaticClass();
}

UObject* UMovieGraphConfigFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// If requesting a graph w/ a subgraph, skip the normal graph creation which looks for an existing graph asset to use as a template
	if (InitialSubgraphAsset)
	{
		UMovieGraphConfig* NewGraph = NewObject<UMovieGraphConfig>(InParent, Class, Name, Flags);
		AddSubgraphNodeToGraph(NewGraph);
		return NewGraph;
	}
	
	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	const TSoftObjectPtr<UMovieGraphConfig> ProjectDefaultGraph = ProjectSettings->DefaultGraph;

	// Duplicate the default graph asset provided in Project Settings
	if (const UMovieGraphConfig* DefaultGraph = ProjectDefaultGraph.LoadSynchronous())
	{
		return DuplicateObject<UMovieGraphConfig>(DefaultGraph, InParent, Name);
	}

	// If the default couldn't be loaded, try loading the default supplied by MRQ. Note that this could be the same as
	// the default above if it wasn't changed by the user; failure to load this indicates a larger issue.
	const FSoftObjectPath DefaultGraphPath(UMovieRenderPipelineProjectSettings::GetDefaultGraphPath());
	if (const UMovieGraphConfig* DefaultGraph = Cast<UMovieGraphConfig>(DefaultGraphPath.TryLoad()))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Could not load the default graph [%ls] specified in Project Settings. Falling back to MRQ-supplied default.", *DefaultGraphPath.GetAssetPathString());
		
		return DuplicateObject<UMovieGraphConfig>(DefaultGraph, InParent, Name);
	}

	// This should never happen, but create an empty graph as a last resort (which is most likely just an Input node and an Output node).
	UE_LOGF(LogMovieRenderPipeline, Warning, "Could not load the default graph [%ls] supplied by MRQ. Falling back to an empty graph.", *DefaultGraphPath.GetAssetPathString());
	
	return NewObject<UMovieGraphConfig>(InParent, Class, Name, Flags);
}

void UMovieGraphConfigFactory::AddSubgraphNodeToGraph(UMovieGraphConfig* InTargetGraph) const
{
	auto ExposeSubgraphInputOutputOnTargetGraph = [InTargetGraph](
		const UMovieGraphInterfaceBase* InSubgraphInputOrOutput, const bool bIsInput, UMovieGraphSubgraphNode* InSubgraphNode)
	{
		// Add a new member to the target graph if this is not the Globals pin (which already exists)
		if (InSubgraphInputOrOutput->GetMemberName() != UMovieGraphNode::GlobalsPinName)
		{
			UMovieGraphInterfaceBase* NewMember = bIsInput
				? static_cast<UMovieGraphInterfaceBase*>(InTargetGraph->AddInput())
				: static_cast<UMovieGraphInterfaceBase*>(InTargetGraph->AddOutput());
			NewMember->SetMemberName(InSubgraphInputOrOutput->GetMemberName());
		}
		
		UMovieGraphNode* FromNode = bIsInput ? InTargetGraph->GetInputNode() : InSubgraphNode;
		UMovieGraphNode* ToNode = bIsInput ? InSubgraphNode : InTargetGraph->GetOutputNode();
		const FName FromPinLabel = *InSubgraphInputOrOutput->GetMemberName();
		const FName ToPinLabel = *InSubgraphInputOrOutput->GetMemberName();

		// Connect up the new member to the subgraph node
		InTargetGraph->AddLabeledEdge(FromNode, FromPinLabel, ToNode, ToPinLabel);
	};
	
	UMovieGraphSubgraphNode* NewSubgraphNode = InTargetGraph->ConstructRuntimeNode<UMovieGraphSubgraphNode>();
	NewSubgraphNode->SetSubGraphAsset(InitialSubgraphAsset);

	// Roughly center the subgraph node between the Inputs and Outputs nodes.
	const int32 InputNodePosX = InTargetGraph->GetInputNode()->GetNodePosX();
	const int32 OutputNodePosX = InTargetGraph->GetOutputNode()->GetNodePosX();
	NewSubgraphNode->SetNodePosX(InputNodePosX + ((OutputNodePosX - InputNodePosX) / 2.f));
	NewSubgraphNode->SetNodePosY(InTargetGraph->GetInputNode()->GetNodePosY());

	// Expose the subgraph's inputs on the target graph and connect
	for (const UMovieGraphInput* SubgraphInput : InitialSubgraphAsset->GetInputs())
	{
		constexpr bool bIsInput = true;
		ExposeSubgraphInputOutputOnTargetGraph(SubgraphInput, bIsInput, NewSubgraphNode);
	}

	// Expose the subgraph's outputs on the target graph and connect
	for (const UMovieGraphOutput* SubgraphOutput : InitialSubgraphAsset->GetOutputs())
	{
		constexpr bool bIsInput = false;
		ExposeSubgraphInputOutputOnTargetGraph(SubgraphOutput, bIsInput, NewSubgraphNode);
	}
}
