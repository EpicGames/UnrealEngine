// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphPinFactory.h"

#include "KismetPins/SGraphPinExec.h"
#include "MovieGraphSchema.h"

TSharedPtr<SGraphPin> FMovieGraphPanelPinFactory::CreatePin(UEdGraphPin* InPin) const
{
	if (InPin->GetSchema()->IsA<UMovieGraphSchema>() && InPin->PinType.PinCategory == UMovieGraphSchema::PC_Branch)
	{
		return SNew(SGraphPinExec, InPin);
	}

	return nullptr;
}