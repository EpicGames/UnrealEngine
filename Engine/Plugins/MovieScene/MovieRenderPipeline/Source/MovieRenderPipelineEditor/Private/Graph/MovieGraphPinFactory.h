// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraphUtilities.h"

/** Pin factory for the render graph. */
class FMovieGraphPanelPinFactory : public FGraphPanelPinFactory
{
public:
	//~ FGraphPanelPinFactory interface
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override;
	//~ End FGraphPanelPinFactory interface
};