// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphQuickRender.h"

#include "Camera/CameraActor.h"
#include "CineCameraActor.h"
#include "Editor.h"
#include "Engine/LevelStreaming.h"
#include "EngineUtils.h"
#include "HAL/FileManager.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "LevelUtils.h"
#include "Misc/MessageDialog.h"
#include "MoviePipelineEditorBlueprintLibrary.h"
#include "MoviePipelinePIEExecutor.h"
#include "MoviePipelinePostRenderFileDisplayProcessor.h"
#include "MoviePipelineQueueSubsystem.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieRenderPipelineSettings.h"
#include "MovieScene.h"
#include "MovieSceneSequencePlayer.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "ObjectEditorUtils.h"
#include "Scalability.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Sections/MovieSceneLevelVisibilitySection.h"
#include "SequencerUtilities.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieSceneLevelVisibilityTrack.h"
#include "Tracks/MovieSceneSubTrack.h"

#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "Graph/MovieRenderGraphEditorSettings.h"
#include "Graph/Nodes/MovieGraphApplyViewportLookNode.h"
#include "Graph/Nodes/MovieGraphCollectionNode.h"
#include "Graph/Nodes/MovieGraphDeferredPassNode.h"
#include "Graph/Nodes/MovieGraphGlobalGameOverrides.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "Graph/Nodes/MovieGraphModifierNode.h"
#include "Graph/Nodes/MovieGraphVideoOutputNode.h"

#if WITH_EDITOR
#include "Editor/Transactor.h"
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#include "PackageHelperFunctions.h"
#include "Selection.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif

#if WITH_OCIO
#include "IOpenColorIOModule.h"
#include "OpenColorIODisplayManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphQuickRender)

#define LOCTEXT_NAMESPACE "FMovieGraphQuickRender"

namespace
{
	/**
	 * Writes a target level sequence to disk and opens it in Sequencer. Generally this is useful for debugging transient level sequences generated
	 * by Quick Render. InAssetPath should be of the format /Game/Temp/<AssetName>.
	 */
	void DEBUG_SaveAndOpenSequence(const ULevelSequence* InTargetLevelSequence, const FString& InAssetPath)
	{
		if (!IsValid(InTargetLevelSequence))
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "[%s]: Need a valid, non-null level sequence to save/open.", __FUNCTION__);
			return;
		}

		const FString PackageFilename = FPackageName::LongPackageNameToFilename(*InAssetPath, FPackageName::GetAssetPackageExtension());

		// Make the new package which will contain the (duplicated) level sequence.
		UPackage* NewPackage = CreatePackage(*InAssetPath);
		if (!NewPackage)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "[%s]: Could not create new package.", __FUNCTION__);
			return;
		}

		// Duplicate the target level sequence into the new package.
		ULevelSequence* NewLevelSequence = CastChecked<ULevelSequence>(StaticDuplicateObject(InTargetLevelSequence, NewPackage));
		if (!NewLevelSequence)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "[%s]: Unable to duplicate target sequence.", __FUNCTION__);
			return;
		}

		// To ensure that SavePackageHelper() writes to disk, the new level sequence cannot have RF_Transient
		NewLevelSequence->ClearFlags(RF_Transient);
		NewLevelSequence->SetFlags(RF_Standalone | RF_Public);
		NewLevelSequence->MarkPackageDirty();

		// Save the level sequence to disk and open in Sequencer.
		SavePackageHelper(NewPackage, *PackageFilename, RF_Standalone, GWarn, SAVE_None);
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewLevelSequence);
	}

	/**
	 * Gets the override level sequence from the mode settings (set via scripting), or the active level sequence if no override was set. If no
	 * override was set, and there's no active level sequence, a warning dialog box will be shown if `bWarn` is true.
	 */
	ULevelSequence* GetCurrentLevelSequenceOrWarn(const TObjectPtr<const UMovieGraphQuickRenderModeSettings>& InModeSettings, EMovieGraphQuickRenderMode ActiveQuickRenderMode, const bool bWarn = true)
	{
		// Scripting may override the level sequence to use
		ULevelSequence* LevelSequenceOverride = InModeSettings->LevelSequenceOverride.LoadSynchronous();
		if (IsValid(LevelSequenceOverride))
		{
			return LevelSequenceOverride;
		}
		
		ULevelSequence* CurrentLevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
		if (!CurrentLevelSequence && bWarn)
		{
			const FText ModeName = StaticEnum<EMovieGraphQuickRenderMode>()->GetDisplayNameTextByValue(static_cast<int64>(ActiveQuickRenderMode));
			const FText MessageText = LOCTEXT("NoActiveSequenceWarning", "Quick Render needs a level sequence opened in Sequencer in order to function in the current mode [{0}]. Open a level sequence, then try again.");
		
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(MessageText, ModeName));

			return nullptr;
		}

		return CurrentLevelSequence;
	}

	/**
	 * Gets the world-space transform of the given camera, taking into account parent actors in the hierarchy as well as relative transforms
	 * on the camera component.
	 */
	FTransform GetWorldSpaceCameraTransform(const AActor* InActor)
	{
		if (!InActor)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Invalid camera detected in Quick Render. The render may not appear correct.");
			return FTransform();
		}

		// The "camera" being passed in could be a pilot cam, which may not be an actual camera. Thus it won't have a camera component on it, so don't
		// treat the lack of a camera component as an error.
		const ACameraActor* CameraActor = Cast<ACameraActor>(InActor);
		if (!CameraActor)
		{
			return InActor->GetTransform();
		}

		const UCameraComponent* CameraComponent = CameraActor->GetCameraComponent();
		if (!CameraComponent)
		{
			// Use the camera's transform as a fallback
			return InActor->GetTransform();
		}

		// The component's transform will most likely be set to relative (vs. world). Calling GetComponentTransform() will take
		// into account not only the parent actors and the camera actor itself, but the component's transform as well. When the new camera
		// is spawned, it will inherit the camera component's transform/offset. Therefore, to prevent the camera component's transform
		// from being applied twice, we need to get the camera actor's position *without* the component's transform baked in (hence the
		// Inverse() call).
		const FTransform ComponentRelTransform = CameraComponent->GetRelativeTransform();
		const FTransform& ComponentWorldTransform = CameraComponent->GetComponentTransform();
		const FTransform FinalTransform = ComponentRelTransform.Inverse() * ComponentWorldTransform;

		return FinalTransform;
	}

	/**
	 * Adds a Transform track for the given camera (with the associated binding). Generally adding a Transform track is not necessary -- the camera
	 * is spawned in the PIE world, and the transform of the source camera transfers over correctly. However, if the camera has *parent* actors which
	 * also have transforms, then those will not be taken into account when the camera is spawned. To ensure that the entire hierarchy of transforms
	 * is accounted for, bake in the world transform of the camera into a Transform track.
	 */
	void AddCameraTransformTrack(UMovieScene* InMovieScene, const FGuid& InCameraBindingGuid, const ACameraActor* InCameraToAdd)
	{
		UMovieScene3DTransformTrack* TransformTrack = InMovieScene->AddTrack<UMovieScene3DTransformTrack>(InCameraBindingGuid);
		if (!TransformTrack)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Unable to add Transform track for one or more cameras being used in Quick Render. The render may not appear correct.");
			return;
		}

		UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->CreateNewSection());
		if (!TransformSection)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Unable to add transforms for one or more cameras being used in Quick Render. The render may not appear correct.");
			return;
		}

		const FTransform FinalTransform = GetWorldSpaceCameraTransform(InCameraToAdd);

		const UE::Math::TVector<double> Location = FinalTransform.GetLocation();
		const UE::Math::TVector<double> Rotation = FinalTransform.GetRotation().Euler();
		const UE::Math::TVector<double> Scale = FinalTransform.GetScale3D();

		TransformTrack->AddSection(*TransformSection);
		TransformSection->SetRange(TRange<FFrameNumber>::All());
		TransformSection->SetMask(FMovieSceneTransformMask(EMovieSceneTransformChannel::All));

		TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
		DoubleChannels[0]->SetDefault(Location.X);
		DoubleChannels[1]->SetDefault(Location.Y);
		DoubleChannels[2]->SetDefault(Location.Z);
		DoubleChannels[3]->SetDefault(Rotation.X);
		DoubleChannels[4]->SetDefault(Rotation.Y);
		DoubleChannels[5]->SetDefault(Rotation.Z);
		DoubleChannels[6]->SetDefault(Scale.X);
		DoubleChannels[7]->SetDefault(Scale.Y);
		DoubleChannels[8]->SetDefault(Scale.Z);
	}
}

/**
 * Wraps the scope that this is created in with a dummy transaction. When going out of scope, the dummy transaction will be canceled, effectively
 * blocking any transactions that occurred while in scope.
 */
class FMovieGraphTransactionBlocker
{
public:
	FMovieGraphTransactionBlocker()
	{
#if WITH_EDITOR
		TransactionId = GEditor->BeginTransaction(FText::GetEmpty());
#endif
	}

	~FMovieGraphTransactionBlocker()
	{
#if WITH_EDITOR
		GEditor->CancelTransaction(TransactionId);
#endif
	}

private:
	int32 TransactionId = 0;
};

void UMovieGraphQuickRenderSubsystem::BeginQuickRender(const EMovieGraphQuickRenderMode InQuickRenderMode, const UMovieGraphQuickRenderModeSettings* InQuickRenderSettings)
{
	UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);

	if (Subsystem->IsRendering())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Unable to Quick Render: Rendering already in progress.");
		return;
	}

	QuickRenderMode = InQuickRenderMode;

	// Store a reference to the settings used, so the executor callbacks can rely on them still existing.
	QuickRenderModeSettings = InQuickRenderSettings;

	// Find the editor world
	UWorld* EditorWorld = GetWorldOfType(EWorldType::Editor);
	if (!EditorWorld)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Unable to Quick Render: Could not find a currently active editor world.");
		return;
	}
	
	// Cache out anything necessary before PIE starts
	CachePrePieData(EditorWorld);

	// Set up the rendering and "utility" level sequences. These are both required to do a render successfully.
	if (!SetUpAllLevelSequences())
	{
		return;
	}

	// Don't do additional work if the selected mode doesn't have what it needs to do a render. AreModePrerequisitesMet() will output the appropriate warnings.
	if (!AreModePrerequisitesMet())
	{
		return;
	}

	// Generate a new queue, fully populated with a job to render from, the graph assigned to that job, and job-level variable overrides applied.
	TObjectPtr<UMoviePipelineExecutorJob> NewJob = nullptr;
	if (!GenerateAndPopulateQueue(TemporaryQueue, NewJob, QuickRenderModeSettings))
	{
		return;
	}
	
	// Evaluate the graph. Walking the graph to determine which nodes are present in each branch (especially taking into consideration
	// subgraphs) is difficult here, so evaluation is the best way to deal with this.
	//
	// Ideally we would not have to evaluate the graph here, and we could get the evaluated graph from elsewhere within the pipeline.
	// However, that would require exposing the inner workings of the pipeline to an outside API, which was deemed a bigger negative
	// than taking the (very small) performance hit of doing a graph evaluation here.
	FString OutError;
	FMovieGraphTraversalContext Context;
	Context.Job = NewJob;
	Context.RootGraph = TemporaryGraph.Get();
	TemporaryEvaluatedGraph = TemporaryGraph->CreateFlattenedGraph(Context, OutError);
	if (!TemporaryEvaluatedGraph)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Unable to Quick Render: Could not create an evaluated graph. Reason: %ls", *OutError);
		return;
	}

	// Update the graph as needed in order to properly apply the settings specified by the user. This will perform a graph evaluation, so the job
	// should be fully initialized at this point.
	ApplyQuickRenderUpdatesToDuplicateGraph(EditorWorld);

	// Create the executor. Note that it's allowed to use unsaved levels (many animators work off of unsaved levels). Because Quick Render is
	// creating a temporary queue and rendering locally, using an unsaved level is safe.
	// Always use the PIE executor for Quick Render, rather than the executor specified in the project settings for normal MRG.
	TemporaryExecutor = NewObject<UMoviePipelinePIEExecutor>(GetTransientPackage());
	TemporaryExecutor->SetAllowUsingUnsavedLevels(true);
	TemporaryExecutor->SetPostRenderAction(QuickRenderModeSettings->PostRenderBehavior);

	// Do any needed setup in the PIE world before rendering starts
	TemporaryExecutor->OnIndividualJobStarted().AddLambda([this, EditorWorld](UMoviePipelineExecutorJob* InJob)
	{
		PerformPreRenderSetup(EditorWorld);

		// DEBUG: If the rendering level sequence needs to be inspected in Sequencer for debugging, uncomment this line.
		// DEBUG_SaveAndOpenSequence(RenderingLevelSequence.Get(), TEXT("/Game/Temp/TempRenderingLevelSequence"));
	});

	// Clean up
	TemporaryExecutor->OnExecutorFinished().AddLambda([this](UMoviePipelineExecutorBase* InExecutor, bool bSuccess)
	{
		RestorePreRenderState();
		PerformPostRenderCleanup();
	});

	FMoviePipelineTelemetry::SendQuickRenderRequestedTelemetry(InQuickRenderMode);

	// Do the render
	Subsystem->RenderQueueInstanceWithExecutorInstance(TemporaryQueue.Get(), TemporaryExecutor.Get());
}

void UMovieGraphQuickRenderSubsystem::PlayLastRender()
{
	FMoviePipelinePostRenderFileDisplayProcessor::OpenPreviousFiles(GetDefault<UMovieRenderGraphEditorSettings>()->PostRenderSettings);
}

bool UMovieGraphQuickRenderSubsystem::CanPlayLastRender()
{
	return FMoviePipelinePostRenderFileDisplayProcessor::CanOpenPreviousFiles();
}

void UMovieGraphQuickRenderSubsystem::OpenOutputDirectory(const UMovieGraphQuickRenderModeSettings* InQuickRenderSettings)
{
	if (!InQuickRenderSettings)
	{
		return;
	}

	// In order to properly resolve the output directory, we have to set up a valid job.
	TObjectPtr<UMoviePipelineExecutorJob> TempJob = nullptr;
	TObjectPtr<UMoviePipelineQueue> TempQueue = nullptr;
	if (!GenerateAndPopulateQueue(TempQueue, TempJob, InQuickRenderSettings))
	{
		return;
	}

	const FString ResolvedOutputDirectory = UMoviePipelineEditorBlueprintLibrary::ResolveOutputDirectoryFromJob(TempJob);

	if (!ResolvedOutputDirectory.IsEmpty())
	{
		// The directory might not exist yet. Create it (if needed) so ExploreFolder() can open it.
		IFileManager::Get().MakeDirectory(*ResolvedOutputDirectory, true);

		FPlatformProcess::ExploreFolder(*ResolvedOutputDirectory);
	}
}

UMovieGraphConfig* UMovieGraphQuickRenderSubsystem::GetQuickRenderGraph(UMovieGraphConfig* InUserSpecifiedGraph, TMap<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>>& OutOriginalToDupeMap)
{
	UMovieGraphConfig* QuickRenderGraph = InUserSpecifiedGraph;
	if (!QuickRenderGraph)
	{
		if (const UMovieRenderPipelineProjectSettings* MoviePipelineProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>())
		{
			QuickRenderGraph = MoviePipelineProjectSettings->DefaultQuickRenderGraph.LoadSynchronous();
			if (!QuickRenderGraph)
			{
				UE_LOGF(LogMovieRenderPipeline, Error, "Unable to Quick Render: The default graph specified in project settings could not be loaded.");
				return nullptr;
			}
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Unable to Quick Render: Could not get the movie pipeline project settings.");
			return nullptr;
		}
	}

	if (!QuickRenderGraph)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Unable to Quick Render: No valid graph could be loaded.");
		return nullptr;
	}

	// Duplicate the graph so changes can be made to it without affecting the graph asset being used
	UMovieGraphConfig* DuplicateGraph = MoviePipeline::DuplicateConfigRecursive(QuickRenderGraph, OutOriginalToDupeMap);
	TemporaryGraph = DuplicateGraph;

	return TemporaryGraph.Get();
}

bool UMovieGraphQuickRenderSubsystem::GenerateAndPopulateQueue(TObjectPtr<UMoviePipelineQueue>& OutQueue, TObjectPtr<UMoviePipelineExecutorJob>& OutJob, TObjectPtr<const UMovieGraphQuickRenderModeSettings> InQuickRenderSettings)
{
	if (!InQuickRenderSettings)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Quick Render: Invalid mode settings provided.");
		return false;
	}
	
	UWorld* EditorWorld = GetWorldOfType(EWorldType::Editor);
	if (!EditorWorld)
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Quick Render: Could not find an editor world.");
		return false;
	}
	
	// Allocate a new queue that will be used temporarily
	OutQueue = NewObject<UMoviePipelineQueue>(GetTransientPackage());

	// Add a temp job to the queue, and give it the editor's level and the current level sequence being edited in Sequencer
	OutJob = OutQueue->AllocateNewJob(UMoviePipelineExecutorJob::StaticClass());
	OutJob->SetSequence(FSoftObjectPath(RenderingLevelSequence.Get()));
	OutJob->Map = FSoftObjectPath(EditorWorld);
	OutJob->JobName = FString(TEXT("QuickRender"));

	// Update the job to use the quick render graph (either the default, or the one specified in the settings)
	const UMovieGraphConfig* QuickRenderGraph = GetQuickRenderGraph(InQuickRenderSettings->GraphPreset.LoadSynchronous(), OriginalGraphToDupeMap);
	if (!QuickRenderGraph)
	{
		// GetQuickRenderGraph() will output an error to the log if a graph was not returned
		return false;
	}
	OutJob->SetGraphPreset(QuickRenderGraph);

	// Apply any job-level variable overrides, if they were specified
	for (const TObjectPtr<UMovieJobVariableAssignmentContainer>& IncomingVariableAssignment : InQuickRenderSettings->GraphVariableAssignments)
	{
		// Map the setting's graph to the duplicate graph
		const UMovieGraphConfig* AssignmentGraph = IncomingVariableAssignment->GetGraphConfig().LoadSynchronous();
		const TObjectPtr<UMovieGraphConfig>* DuplicateGraph = OriginalGraphToDupeMap.Find(AssignmentGraph);
		if (!DuplicateGraph)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Quick Render: Unable to properly set variable overrides for graph [%ls].", *AssignmentGraph->GetName());
			continue;
		}

		UMovieJobVariableAssignmentContainer* NewVariableAssignments = OutJob->GetOrCreateJobVariableAssignmentsForGraph(*DuplicateGraph);

		for (const UMovieGraphVariable* GraphVariable : (*DuplicateGraph)->GetVariables())
		{
			bool bIsVariableAssignmentEnabled = false;
			IncomingVariableAssignment->GetVariableAssignmentEnableState(GraphVariable, bIsVariableAssignmentEnabled);
			
			// Copying values by serialized string isn't ideal, but it's the easiest way to transfer values between the variable assignments
			NewVariableAssignments->SetValueSerializedString(GraphVariable, IncomingVariableAssignment->GetValueSerializedString(GraphVariable));
			NewVariableAssignments->SetVariableAssignmentEnableState(GraphVariable, bIsVariableAssignmentEnabled);
		}
	}

	return true;
}

void UMovieGraphQuickRenderSubsystem::PerformPostRenderCleanup()
{
	// The temp objects are part of the transient package, so manually release them when the render is finished
	TemporaryQueue = nullptr;
	TemporaryGraph = nullptr;
	TemporaryExecutor = nullptr;
	TemporaryEvaluatedGraph = nullptr;
	OriginalGraphToDupeMap.Empty();
	RenderingLevelSequence = nullptr;
	UtilityLevelSequence = nullptr;
	QuickRenderModeSettings = nullptr;
}

void UMovieGraphQuickRenderSubsystem::ApplyQuickRenderUpdatesToDuplicateGraph(const UWorld* InEditorWorld)
{
	check(IsValid(TemporaryGraph));

	// Apply changes to Global Output Settings
	{
		UMovieGraphGlobalOutputSettingNode* GlobalOutputSettings = Cast<UMovieGraphGlobalOutputSettingNode>(InjectNodeIntoBranch(UMovieGraphGlobalOutputSettingNode::StaticClass(), UMovieGraphNode::GlobalsPinName));

		auto SetStartFrame = [&GlobalOutputSettings](const int32 StartFrame) -> void
		{
			GlobalOutputSettings->bOverride_CustomPlaybackRangeStart = true;
			GlobalOutputSettings->CustomPlaybackRangeStart.Type = EMovieGraphSequenceRangeType::Custom;
			GlobalOutputSettings->CustomPlaybackRangeStart.Value = StartFrame;
		};

		auto SetEndFrame = [&GlobalOutputSettings](const int32 EndFrame) -> void
		{
			GlobalOutputSettings->bOverride_CustomPlaybackRangeEnd = true;
			GlobalOutputSettings->CustomPlaybackRangeEnd.Type = EMovieGraphSequenceRangeType::Custom;
			GlobalOutputSettings->CustomPlaybackRangeEnd.Value = EndFrame;
		};

		const TRange<FFrameNumber> PlaybackRange = GetPlaybackRange();
		if (!PlaybackRange.IsEmpty())
		{
			const int32 StartFrame = PlaybackRange.GetLowerBoundValue().Value;
			const int32 EndFrame = PlaybackRange.GetUpperBoundValue().Value;
			
			SetStartFrame(StartFrame);
			SetEndFrame(EndFrame);
			
			UE_LOGF(LogMovieRenderPipeline, Display, "Quick Render Start Frame: %d, End Frame: %d", StartFrame, EndFrame);
		}
	}

	// Inject an Apply Viewport Look node if any viewport look flags were specified. Logic outside of Quick Render will for look this node in order to
	// determine how to apply some of its settings
	const EMovieGraphQuickRenderViewportLookFlags LookFlags = static_cast<EMovieGraphQuickRenderViewportLookFlags>(QuickRenderModeSettings->ViewportLookFlags);
	if (LookFlags != EMovieGraphQuickRenderViewportLookFlags::None)
	{
		UMovieGraphApplyViewportLookNode* ApplyViewportLookNode = Cast<UMovieGraphApplyViewportLookNode>(InjectNodeIntoBranch(UMovieGraphApplyViewportLookNode::StaticClass(), UMovieGraphNode::GlobalsPinName));
		check(ApplyViewportLookNode);

#define APPLY_VIEWPORT_LOOK_FLAG(EnumName, FlagName) \
		if (IsViewportLookFlagActive(EnumName)) \
		{ \
			ApplyViewportLookNode->bOverride_b##FlagName = true; \
			ApplyViewportLookNode->b##FlagName = true; \
		} \

		// Transfer the flags from the render settings to the Apply Viewport Look node
		APPLY_VIEWPORT_LOOK_FLAG(EMovieGraphQuickRenderViewportLookFlags::Ocio, Ocio);
		APPLY_VIEWPORT_LOOK_FLAG(EMovieGraphQuickRenderViewportLookFlags::ShowFlags, ShowFlags);
		APPLY_VIEWPORT_LOOK_FLAG(EMovieGraphQuickRenderViewportLookFlags::ViewMode, ViewMode);
		APPLY_VIEWPORT_LOOK_FLAG(EMovieGraphQuickRenderViewportLookFlags::Visibility, Visibility);
#undef APPLY_VIEWPORT_LOOK_FLAG
	}

	// Apply viewport/editor actor visibility
	if (IsViewportLookFlagActive(EMovieGraphQuickRenderViewportLookFlags::Visibility))
	{
		ApplyQuickRenderUpdatesToDuplicateGraph_ApplyEditorVisibility(InEditorWorld);
	}

	// Apply editor-only actor visibility. Normally editor-only actors won't be processed by the MRG modifier system.
	if (IsViewportLookFlagActive(EMovieGraphQuickRenderViewportLookFlags::EditorOnlyActors))
	{
		ApplyQuickRenderUpdatesToDuplicateGraph_ApplyEditorOnlyActorVisibility(InEditorWorld);
	}

	// Apply OCIO if it is activated on the viewport and enabled in the viewport look flags
	if (IsViewportLookFlagActive(EMovieGraphQuickRenderViewportLookFlags::Ocio))
	{
#if WITH_OCIO
		if (const FEditorViewportClient* ViewportClient = UMovieGraphApplyViewportLookNode::GetViewportClient())
		{
			if (const FOpenColorIODisplayConfiguration* OcioConfiguration = IOpenColorIOModule::Get().GetDisplayManager().GetDisplayConfiguration(ViewportClient))
			{
				if (OcioConfiguration->bIsEnabled)
				{
					ApplyQuickRenderUpdatesToDuplicateGraph_ApplyOcio(OcioConfiguration);
				}
			}
		}
#endif	// WITH_OCIO
	}

	// Apply viewport scalability settings if the graph didn't specify an explicit scalability setting
	ApplyQuickRenderUpdatesToDuplicateGraph_Scalability();
}

void UMovieGraphQuickRenderSubsystem::ApplyQuickRenderUpdatesToDuplicateGraph_ApplyEditorVisibility(const UWorld* InEditorWorld)
{
	TArray<FMovieGraphActorQueryEntry> VisibleActors;
	TArray<FMovieGraphActorQueryEntry> HiddenActors;
	
	// Determine the actors that are hidden and visible
	for (FActorIterator It(InEditorWorld); It; ++It)
	{
		AActor* Actor = *It;

		if (!Actor)
		{
			continue;
		}

		// Don't process ABrush actors (actors derived from ABrush are probably ok). The builder brush is problematic because of how it decides to
		// render itself. There's no API we can call to accurately determine if it's going to show up in the viewport, and they're hidden in game
		// by default, so just skip them so they don't show up in the render (needing them to show up in renders should be rare).
		if (Actor->GetClass() == ABrush::StaticClass())
		{
			continue;
		}

		if (Actor->IsHiddenEd())
		{
			FMovieGraphActorQueryEntry& HiddenActorEntry = HiddenActors.AddDefaulted_GetRef();
			HiddenActorEntry.ActorToMatch = Actor;
		}
		else
		{
			FMovieGraphActorQueryEntry& VisibleActorEntry = VisibleActors.AddDefaulted_GetRef();
			VisibleActorEntry.ActorToMatch = Actor;
		}
	}

	UMovieGraphConditionGroupQuery_Actor* VisibleActorQuery =
		AddNewCollectionWithVisibilityModifier<UMovieGraphConditionGroupQuery_Actor>(TEXT("VISIBLE_ACTORS"), false);
	VisibleActorQuery->ActorsAndComponentsToMatch = VisibleActors;
	
	UMovieGraphConditionGroupQuery_Actor* HiddenActorQuery =
		AddNewCollectionWithVisibilityModifier<UMovieGraphConditionGroupQuery_Actor>(TEXT("HIDDEN_ACTORS"), true);
	HiddenActorQuery->ActorsAndComponentsToMatch = HiddenActors;
}

void UMovieGraphQuickRenderSubsystem::ApplyQuickRenderUpdatesToDuplicateGraph_ApplyEditorOnlyActorVisibility(const UWorld* InEditorWorld)
{
	TArray<FMovieGraphActorQueryEntry> VisibleActors;
	
	// Determine the editor-only actors that should be made visible
	for (FActorIterator It(InEditorWorld); It; ++It)
	{
		AActor* Actor = *It;

		if (!Actor)
		{
			continue;
		}

		bool bHasEditorOnlyComponent = false;
		
		for (const UActorComponent* ActorComponent : Actor->GetComponents())
		{
			if (ActorComponent->IsEditorOnly())
			{
				bHasEditorOnlyComponent = true;
				break;
			}
		}

		if (Actor->IsEditorOnly() || Actor->bIsEditorPreviewActor || bHasEditorOnlyComponent)
		{
			FMovieGraphActorQueryEntry& VisibleActorEntry = VisibleActors.AddDefaulted_GetRef();
			VisibleActorEntry.ActorToMatch = Actor;
		}

		// The control rig actor class name is referenced by name here in order to prevent a dependency.
		static const FName ControlRigShapeActorName(TEXT("ControlRigShapeActor"));

		// The Control Rig manipulation gizmos are important to show in the editor-only actor mode, but they will not be copied into the PIE world
		// because they're marked as Transient. To work around this, we can temporarily mark them as RF_NonPIEDuplicateTransient which will inform
		// the serialization process that it's OK to copy these actors to PIE.
		if (Actor->HasAnyFlags(RF_Transient) && (Actor->GetClass()->GetName() == ControlRigShapeActorName))
		{
			// Since we need to un-set this flag after the render is done, only apply RF_NonPIEDuplicateTransient if it's not already applied (ie,
			// we don't want to remove RF_NonPIEDuplicateTransient after the render finishes if Quick Render didn't originally apply it).
			if (!Actor->HasAnyFlags(RF_NonPIEDuplicateTransient))
			{
				CachedPrePieData.TemporarilyNonTransientActors.Add(MakeWeakObjectPtr(Actor));
				Actor->SetFlags(RF_NonPIEDuplicateTransient);
			}

			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (Component->HasAnyFlags(RF_Transient))
				{
					if (!Component->HasAnyFlags(RF_NonPIEDuplicateTransient))
					{
						CachedPrePieData.TemporarilyNonTransientComponents.Add(MakeWeakObjectPtr(Component));
						Component->SetFlags(RF_NonPIEDuplicateTransient);
					}
				}
			}
		}
	}

	constexpr bool bModifierShouldHide = false;
	constexpr bool bProcessEditorOnlyActors = true;
	UMovieGraphConditionGroupQuery_Actor* VisibleActorQuery =
		AddNewCollectionWithVisibilityModifier<UMovieGraphConditionGroupQuery_Actor>(TEXT("VISIBLE_EDITORONLY_ACTORS"), bModifierShouldHide, bProcessEditorOnlyActors);
	VisibleActorQuery->ActorsAndComponentsToMatch = VisibleActors;
}

void UMovieGraphQuickRenderSubsystem::ApplyQuickRenderUpdatesToDuplicateGraph_ApplyOcio(const FOpenColorIODisplayConfiguration* InOcioConfiguration)
{
	// For each branch, apply OCIO configuration.
	for (const FName& BranchName : TemporaryEvaluatedGraph->GetBranchNames())
	{
		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = false;
		const TArray<UMovieGraphFileOutputNode*> FileOutputNodes =
			TemporaryEvaluatedGraph->GetSettingsForBranch<UMovieGraphFileOutputNode>(BranchName, bIncludeCDOs, bExactMatch);
		const TArray<UMovieGraphImagePassBaseNode*> ImagePassNodes =
			TemporaryEvaluatedGraph->GetSettingsForBranch<UMovieGraphImagePassBaseNode>(BranchName, bIncludeCDOs, bExactMatch);

		// Update all file output nodes to have the viewport's OCIO configuration
		for (UMovieGraphFileOutputNode* FileOutputNode : FileOutputNodes)
		{
			static const FName OcioOverrideName(TEXT("bOverride_OCIOConfiguration"));
			static const FName OcioConfigName(TEXT("OCIOConfiguration"));

			// The evaluated graph contains a node of this type, so insert a new node into the non-evaluated graph downstream to
			// override the OCIO settings.
			if (UMovieGraphNode* NewFileOutputNode = InjectNodeIntoBranch(FileOutputNode->GetClass(), BranchName))
			{
				// This is hacky, but OCIO properties are not uniformly inherited across image/video nodes. The properties are, however,
				// named the same.
				const FBoolProperty* OverrideProp = FindFProperty<FBoolProperty>(NewFileOutputNode->GetClass(), OcioOverrideName);
				const FStructProperty* OcioProp = FindFProperty<FStructProperty>(NewFileOutputNode->GetClass(), OcioConfigName);
				
				if (OverrideProp && OcioProp)
				{
					OverrideProp->SetPropertyValue_InContainer(NewFileOutputNode, true);
					OcioProp->SetValue_InContainer(NewFileOutputNode, InOcioConfiguration);
				}
			}
		}

		// Also enable OCIO on the renderer nodes
		for (UMovieGraphImagePassBaseNode* ImagePassNode : ImagePassNodes)
		{
			static const FName AllowOcioName(TEXT("bAllowOCIO"));
			static const FName DisableToneCurveOverrideName(TEXT("bOverride_bDisableToneCurve"));
			static const FName DisableToneCurveName(TEXT("bDisableToneCurve"));

			const FBoolProperty* AllowOcioProp = FindFProperty<FBoolProperty>(ImagePassNode->GetClass(), AllowOcioName);
			if (!AllowOcioProp)
			{
				// This node doesn't support OCIO
				continue;
			}

			// Skip nodes that have "Allow OCIO" explicitly turned off
			bool AllowOcioValue;
			AllowOcioProp->GetValue_InContainer(ImagePassNode, &AllowOcioValue);
			if (!AllowOcioValue)
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "Quick Render was set to apply viewport OCIO, but the renderer node [%ls] has Allow OCIO turned off. OCIO will not be applied in this node's render.", *ImagePassNode->GetName());
				continue;
			}

			// Insert a new node of this type into the non-evaluated graph downstream to override the OCIO settings.
			if (UMovieGraphNode* NewNode = InjectNodeIntoBranch(ImagePassNode->GetClass(), BranchName))
			{
				// Like the file output nodes, OCIO properties are not uniformly inherited across renderer nodes. The properties are, however,
				// named the same.
				UMovieGraphImagePassBaseNode* NewRendererNode = Cast<UMovieGraphImagePassBaseNode>(NewNode);

				const FBoolProperty* DisableToneCurveOverrideProp = FindFProperty<FBoolProperty>(NewRendererNode->GetClass(), DisableToneCurveOverrideName);
				const FBoolProperty* DisableToneCurveProp = FindFProperty<FBoolProperty>(NewRendererNode->GetClass(), DisableToneCurveName);

				// At this point "Allow OCIO" is turned on, so we just need to make sure the tone curve is disabled.
				if (DisableToneCurveOverrideProp && DisableToneCurveProp)
				{
					DisableToneCurveOverrideProp->SetPropertyValue_InContainer(NewRendererNode, true);
					DisableToneCurveProp->SetPropertyValue_InContainer(NewRendererNode, true);
				}
			}
		}
	}
}

void UMovieGraphQuickRenderSubsystem::ApplyQuickRenderUpdatesToDuplicateGraph_Scalability()
{
	constexpr bool bIncludeCDOs = false;
	constexpr bool bExactMatch = true;
	const UMovieGraphGlobalGameOverridesNode* GameOverridesNode =
		TemporaryEvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalGameOverridesNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);

	// If the user specified an explicit scalability level to use, there's nothing to do here. Otherwise, we'll apply the value that the viewport
	// is using.
	if (GameOverridesNode && GameOverridesNode->bOverride_ScalabilityQualityLevel)
	{
		return;
	}
	
	if (UMovieGraphGlobalGameOverridesNode* NewGameOverridesNode =
			Cast<UMovieGraphGlobalGameOverridesNode>(InjectNodeIntoBranch(UMovieGraphGlobalGameOverridesNode::StaticClass(), UMovieGraphNode::GlobalsPinName)))
	{
		const Scalability::FQualityLevels QualityLevels = Scalability::GetQualityLevels();
		const int32 MinQualityLevel = QualityLevels.GetMinQualityLevel();
		const int32 QualityLevel = QualityLevels.GetSingleQualityLevel();

		int32 QualityLevelToUse;

		// Use the overall value set for scalability if it's not custom (-1). If that's custom, fall back to the minimum quality level. If that's
		// custom, then default to High.
		if (QualityLevel != -1)
		{
			QualityLevelToUse = QualityLevel;
		}
		else if (MinQualityLevel != -1)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Scalability settings are not all set to the same level, which is not supported by Quick Render. Using the minimum from all settings, which is [%ls].",
				*Scalability::GetScalabilityNameFromQualityLevel(MinQualityLevel).ToString());

			QualityLevelToUse = MinQualityLevel;
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "One or more scalability settings are set to custom, which is not supported by Quick Render. Defaulting to High scalability");

			QualityLevelToUse = 2;	// High
		}

		NewGameOverridesNode->bOverride_ScalabilityQualityLevel = true;
		NewGameOverridesNode->ScalabilityQualityLevel = static_cast<EMovieGraphScalabilityQualityLevel>(QualityLevelToUse);
	}
}

template <typename QueryType>
QueryType* UMovieGraphQuickRenderSubsystem::AddNewCollectionWithVisibilityModifier(const FString& InOperationName, const bool bModifierShouldHide, const bool bProcessEditorOnlyActors)
{
	const UMovieGraphCollectionNode* NewCollectionNode = Cast<UMovieGraphCollectionNode>(InjectNodeIntoBranch(UMovieGraphCollectionNode::StaticClass(), UMovieGraphNode::GlobalsPinName));

	const FString NewCollectionName(TEXT("__AUTOGEN_COLLECTION_") + InOperationName);
	NewCollectionNode->Collection->SetCollectionName(NewCollectionName);

	QueryType* NewConditionGroupQuery = Cast<QueryType>(NewCollectionNode->Collection->AddConditionGroup()->AddQuery(QueryType::StaticClass()));

	UMovieGraphModifierNode* NewModifierNode = Cast<UMovieGraphModifierNode>(InjectNodeIntoBranch(UMovieGraphModifierNode::StaticClass(), UMovieGraphNode::GlobalsPinName));
	NewModifierNode->ModifierName = TEXT("__AUTOGEN_MODIFIER_") + InOperationName;
	NewModifierNode->AddCollection(FName(NewCollectionName));

	UMovieGraphRenderPropertyModifier* RenderPropertyModifier = Cast<UMovieGraphRenderPropertyModifier>(NewModifierNode->GetModifier(UMovieGraphRenderPropertyModifier::StaticClass()));

	RenderPropertyModifier->bOverride_bIsHidden = true;
	RenderPropertyModifier->bIsHidden = bModifierShouldHide;

	if (bProcessEditorOnlyActors)
	{
		RenderPropertyModifier->bOverride_bProcessEditorOnlyActors = true;
		RenderPropertyModifier->bProcessEditorOnlyActors = true;
	}

	// The caller can modify the returned condition group as needed
	return NewConditionGroupQuery;
}

UMovieGraphNode* UMovieGraphQuickRenderSubsystem::InjectNodeIntoBranch(const TSubclassOf<UMovieGraphNode> NodeType, const FName& InBranchName) const
{
	check(IsValid(TemporaryGraph));

	UMovieGraphNode* OutputNode = TemporaryGraph->GetOutputNode();
	if (!OutputNode->GetInputPin(InBranchName))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Quick Render: Could not find branch [%ls] to inject a node override into. The render may look different than expected.", *InBranchName.ToString());
		return nullptr;
	}

	// InsertBefore handles both cases: if a node is already connected to the branch pin it is reconnected
	// upstream of the new node, and if the branch is empty the new node is simply connected to the pin.
	return TemporaryGraph->InsertBefore(OutputNode, NodeType, InBranchName);
}

bool UMovieGraphQuickRenderSubsystem::AreModePrerequisitesMet() const
{
	if (QuickRenderMode == EMovieGraphQuickRenderMode::SelectedCameras)
	{
		TArray<ACameraActor*> SelectedCameras;
		GEditor->GetSelectedActors()->GetSelectedObjects<ACameraActor>(SelectedCameras);

		if (SelectedCameras.IsEmpty())
		{
			FMessageDialog::Open(
				EAppMsgType::Ok,
				LOCTEXT("NoCamerasSelectedWarning", "The 'Selected Camera(s)' mode needs at least one camera selected. Select some cameras in the Outliner, then try again."));
			
			return false;
		}
	}

	return true;
}

void UMovieGraphQuickRenderSubsystem::CachePrePieData(UWorld* InEditorWorld)
{
	// Prevent all existing level sequences in the world from auto-playing. We need the level sequence from Quick Render to be in control. 
	for (auto It = TActorIterator<ALevelSequenceActor>(InEditorWorld); It; ++It)
	{
		const ALevelSequenceActor* LevelSequenceActor = *It;
		if (LevelSequenceActor->PlaybackSettings.bAutoPlay)
		{
			(*It)->PlaybackSettings.bAutoPlay = false;
			CachedPrePieData.ModifiedLevelSequenceActors.Add(MakeWeakObjectPtr(*It));
		}
	}

	// Cache out the selected camera actors before PIE starts. They'll be deselected once the PIE window shows up.
	TArray<ACameraActor*> CameraActors;
	GEditor->GetSelectedActors()->GetSelectedObjects<ACameraActor>(CameraActors);
	CopyToWeakArray(CachedPrePieData.SelectedCameras, CameraActors);

	// If the viewport is locked to an actor (likely a camera), cache it here. It may temporarily change while PIE is being started, so it needs to
	// be determined before PIE starts up.
	if (const FLevelEditorViewportClient* ViewportClient = UMovieGraphApplyViewportLookNode::GetViewportClient())
	{
		const FLevelViewportActorLock& ActorLock = ViewportClient->IsLockedToCinematic() ? ViewportClient->GetCinematicActorLock() : ViewportClient->GetActorLock();
		if (ActorLock.HasValidLockedActor())
		{
			// Duplicate the pilot cam because it may be spawned by Sequencer. If that's the case, it'll be null by the time PIE starts up again.
			CachedPrePieData.ViewportActorLock.Reset(DuplicateObject<AActor>(ActorLock.LockedActor.Get(), this));

			// Cache the pilot cam's current world position. Needs to be cached out now because Sequencer will potentially change it
			// once PIE starts up.
			CachedPrePieData.bUseViewportActorLockWorldTransform = true;
			CachedPrePieData.ViewportActorLockWorldTransform = GetWorldSpaceCameraTransform(ActorLock.LockedActor.Get());
		}

		// Duplicate the pilot cam component if there is one. As soon as PIE starts, the component's values will change to reflect the PIE world, so
		// we duplicate it in order to be able to reference the pre-PIE property values.
		if (const UCameraComponent* PilotCamComponent = ViewportClient->GetCameraComponentForView())
		{
			CachedPrePieData.ViewportActorLockCameraComponent.Reset(DuplicateObject<UCameraComponent>(PilotCamComponent, this));
		}

		// Cache the viewport camera properties now so we don't rely on the viewport being active when PIE starts later on.
		CachedPrePieData.ViewportCamera.bIsValid = true;
		CachedPrePieData.ViewportCamera.Location = ViewportClient->GetViewLocation();
		CachedPrePieData.ViewportCamera.Rotation = ViewportClient->GetViewRotation();
		CachedPrePieData.ViewportCamera.bIsPerspective = ViewportClient->IsPerspective();
		CachedPrePieData.ViewportCamera.FOV = ViewportClient->ViewFOV;
		CachedPrePieData.ViewportCamera.AspectRatio = ViewportClient->AspectRatio;
		CachedPrePieData.ViewportCamera.OrthoWidth = ViewportClient->Viewport
			? ViewportClient->Viewport->GetSizeXY().X * ViewportClient->GetOrthoUnitsPerPixel(ViewportClient->Viewport)
			: 0.f;

		// For axis-aligned ortho viewports, the rotation obtained from GetViewRotation() does not produce the correct
		// result. Pre-compute the correct rotation here so the cached data is ready to use directly.
		if (!CachedPrePieData.ViewportCamera.bIsPerspective)
		{
			switch (ViewportClient->ViewportType)
			{
			case LVT_OrthoTop:    CachedPrePieData.ViewportCamera.Rotation = FRotator(-90.f, -180.f, 0.f); break;
			case LVT_OrthoBottom: CachedPrePieData.ViewportCamera.Rotation = FRotator(90.f, 0.f, 0.f); break;
			case LVT_OrthoLeft:   CachedPrePieData.ViewportCamera.Rotation = FRotator(0.f, -90.f, 0.f); break;
			case LVT_OrthoRight:  CachedPrePieData.ViewportCamera.Rotation = FRotator(0.f, 90.f, 0.f); break;
			case LVT_OrthoBack:   CachedPrePieData.ViewportCamera.Rotation = FRotator(0.f, 0.f, 0.f); break;
			case LVT_OrthoFront:  CachedPrePieData.ViewportCamera.Rotation = FRotator(0.f, 180.f, 0.f); break;
			default:              CachedPrePieData.ViewportCamera.Rotation = FRotator(0.f, 0.f, 0.f); break;
			}
		}
	}
}

void UMovieGraphQuickRenderSubsystem::RestorePreRenderState()
{
	// Restore the modified level sequences to the way they were before the render.
	for (const TWeakObjectPtr<ALevelSequenceActor>& LevelSequenceActor : CachedPrePieData.ModifiedLevelSequenceActors)
	{
		if (LevelSequenceActor.IsValid())
		{
			LevelSequenceActor->PlaybackSettings.bAutoPlay = true;
		}
	}
	
	CachedPrePieData.SelectedCameras.Empty();
	
	// Remove the Utility subsequence that was added to the Rendering sequence
	UMovieSceneTrack* TrackToRemove = nullptr;
	for (UMovieSceneSection* Section : RenderingLevelSequence->GetMovieScene()->GetAllSections())
	{
		if (const UMovieSceneSubSection* SubsequenceSection = Cast<UMovieSceneSubSection>(Section))
		{
			if (SubsequenceSection->GetSequence() == UtilityLevelSequence.Get())
			{
				TrackToRemove = Section->GetTypedOuter<UMovieSceneTrack>();
				break;
			}
		}
	}

	if (TrackToRemove)
	{
		RenderingLevelSequence->GetMovieScene()->RemoveTrack(*TrackToRemove);
	}

	// Strip any transient flags off of relevant components and actors
	for (const TWeakObjectPtr<UActorComponent>& SceneComponent : CachedPrePieData.TemporarilyNonTransientComponents)
	{
		if (SceneComponent.IsValid())
		{
			SceneComponent->ClearFlags(RF_NonPIEDuplicateTransient);
		}
	}

	for (const TWeakObjectPtr<AActor>& Actor : CachedPrePieData.TemporarilyNonTransientActors)
	{
		if (Actor.IsValid())
		{
			Actor->ClearFlags(RF_NonPIEDuplicateTransient);
		}
	}

	CachedPrePieData.TemporarilyNonTransientComponents.Empty();
	CachedPrePieData.TemporarilyNonTransientActors.Empty();
	CachedPrePieData.ViewportActorLock.Reset();
	CachedPrePieData.ViewportActorLockCameraComponent.Reset();
	CachedPrePieData.bUseViewportActorLockWorldTransform = false;
	CachedPrePieData.ViewportCamera.bIsValid = false;
}

ULevelSequence* UMovieGraphQuickRenderSubsystem::SetUpRenderingLevelSequence() const
{
	auto CreateTemporaryLevelSequence = []() -> ULevelSequence*
	{
		ULevelSequence* TemporaryLevelSequence = NewObject<ULevelSequence>(GetTransientPackage(), TEXT("QuickRenderSequence"), RF_Transient);
		TemporaryLevelSequence->Initialize();

		return TemporaryLevelSequence;
	};
	
	// For the Current Viewport mode, if a level sequence is active, use that to drive the render (even though this mode generates stills, the level
	// sequence can impact transforms, etc which need to be captured). If no level sequence is active, use a temporary level sequence.
	if (QuickRenderMode == EMovieGraphQuickRenderMode::CurrentViewport)
	{
		constexpr bool bWarn = false;
		ULevelSequence* CurrentLevelSequence = GetCurrentLevelSequenceOrWarn(QuickRenderModeSettings, QuickRenderMode, bWarn);
		if (IsValid(CurrentLevelSequence))
		{
			return CurrentLevelSequence;
		}

		return CreateTemporaryLevelSequence();
	}

	// For the Selected Cameras mode, a temporary level sequence will always be rendered from. This temporary sequence will ultimately have the
	// utility level sequence added to it as a subsequence, and that will control the camera cuts as well as include the active level sequence (if
	// there is one). This mode needs more control over the active level sequence if it exists, which is why it is not used as the rendering level
	// sequence directly (see comments in SetUpUtilityLevelSequence() for more details).
	if (QuickRenderMode == EMovieGraphQuickRenderMode::SelectedCameras)
	{
		return CreateTemporaryLevelSequence();
	}

	// Otherwise, use the level sequence that's specified via scripting, or currently active in Sequencer.
	ULevelSequence* CurrentLevelSequence = GetCurrentLevelSequenceOrWarn(QuickRenderModeSettings, QuickRenderMode);
	if (!CurrentLevelSequence)
	{
		return nullptr;
	}
	
	return CurrentLevelSequence;
}

ULevelSequence* UMovieGraphQuickRenderSubsystem::SetUpUtilityLevelSequence() const
{
	ULevelSequence* UtilitySequence = NewObject<ULevelSequence>(GetTransientPackage(), TEXT("QuickRenderUtilitySequence"), RF_Transient);
	UtilitySequence->Initialize();

	// For the Selected Cameras mode, we need to add the level sequence that's active (if there is one) as a subsequence. This needs to be done once
	// PER camera. This is strange, but it's done in order to accomodate the current playhead of the active level sequence. If the user is on frame
	// 100 for example, EACH camera needs to render frame 100. The camera cuts generated for this mode work such that there's one camera cut per frame
	// in the utility sequence, so without adding the level sequence as a subsequence once PER camera, you'd wind up with camera 1 rendering frame
	// 100, camera 2 rendering frame 101, and so on. The level sequence section that's added per camera is set up to start at the current playhead, so
	// the effect is that each camera renders the active sequence at the current playhead. It's also given a very negative bias so the camera cuts
	// added to the utility sequence have priority over any that may exist in the active level sequence.
	//
	// An alternative to this is using the "Render all cameras" mode, but that has the potential to use significantly more memory. If rendering many
	// cameras at once, the approach taken here the potential to be much more efficient.
	if (QuickRenderMode == EMovieGraphQuickRenderMode::SelectedCameras)
	{
		constexpr bool bWarn = false;
		ULevelSequence* CurrentLevelSequence = GetCurrentLevelSequenceOrWarn(QuickRenderModeSettings, QuickRenderMode, bWarn);
		
		if (CurrentLevelSequence)
		{
			UMovieScene* MovieScene = UtilitySequence->GetMovieScene();
			const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
			const FFrameRate DestinationRate = MovieScene->GetTickResolution();

			// The playhead position in the root level sequence
			const FMovieSceneSequencePlaybackParams GlobalPosition = ULevelSequenceEditorBlueprintLibrary::GetGlobalPosition();

			// Add the a section for the active level sequence for each camera
			for (int32 CameraIndex = 0; CameraIndex < CachedPrePieData.SelectedCameras.Num(); ++CameraIndex)
			{
				UMovieSceneSubTrack* NewSubsequenceTrack = CastChecked<UMovieSceneSubTrack>(MovieScene->AddTrack(UMovieSceneSubTrack::StaticClass()));
				UMovieSceneSubSection* SequenceSubsection = NewSubsequenceTrack->AddSequence(CurrentLevelSequence, 0, 0);
				SequenceSubsection->Parameters.StartFrameOffset = FFrameRate::TransformTime(GlobalPosition.Frame.FrameNumber, DisplayRate, DestinationRate).GetFrame();
				SequenceSubsection->Parameters.HierarchicalBias = -100000;
				SequenceSubsection->SetRange(TRange<FFrameNumber>(
					FFrameRate::TransformTime(FFrameTime(CameraIndex), DisplayRate, DestinationRate).GetFrame(),
					FFrameRate::TransformTime(FFrameTime(CameraIndex + 1), DisplayRate, DestinationRate).GetFrame()));
			}
		}
	}

	return UtilitySequence;
}

bool UMovieGraphQuickRenderSubsystem::SetUpAllLevelSequences()
{
	// Get/create the level sequence that should be rendered from (varies depending on the Quick Render mode).
	RenderingLevelSequence = SetUpRenderingLevelSequence();

	// Also set up the utility level sequence (which is used, for example, for overriding level visibility)
	UtilityLevelSequence = SetUpUtilityLevelSequence();

	return IsValid(RenderingLevelSequence) && IsValid(UtilityLevelSequence);
}

UWorld* UMovieGraphQuickRenderSubsystem::GetWorldOfType(const EWorldType::Type WorldType) const
{
	UWorld* World = nullptr;
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == WorldType)
		{
			World = Context.World();
			break;
		}
	}

	return World;
}

TRange<FFrameNumber> UMovieGraphQuickRenderSubsystem::GetPlaybackRange()
{
	FFrameNumber StartFrame;
	FFrameNumber EndFrame;

	// If we're in Current Viewport, the playback range is always one frame long (only a single frame is rendered).
	if (QuickRenderMode == EMovieGraphQuickRenderMode::CurrentViewport)
	{
		// If rendering from the active level sequence, the playback position should be where the current playhead is.
		if (RenderingLevelSequence == ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence())
		{
			const FMovieSceneSequencePlaybackParams GlobalPosition = ULevelSequenceEditorBlueprintLibrary::GetGlobalPosition();
			
			return TRange<FFrameNumber>::Inclusive(GlobalPosition.Frame.FrameNumber, GlobalPosition.Frame.FrameNumber.Value + 1);
		}
		
		return TRange<FFrameNumber>::Inclusive(0, 1);
	}

	// The Selected Cameras mode renders one frame for each camera.
	if (QuickRenderMode == EMovieGraphQuickRenderMode::SelectedCameras)
	{
		return TRange<FFrameNumber>::Inclusive(0, CachedPrePieData.SelectedCameras.Num());
	}

	if (QuickRenderModeSettings->FrameRangeType == EMovieGraphQuickRenderFrameRangeType::SelectionRange)
	{
		const int32 SelectionStartFrame = ULevelSequenceEditorBlueprintLibrary::GetSelectionRangeStart();
		const int32 SelectionEndFrame = ULevelSequenceEditorBlueprintLibrary::GetSelectionRangeEnd();

		if ((SelectionStartFrame != 0) || (SelectionEndFrame != 0))
		{
			StartFrame = ConvertSubSequenceFrameToRootFrame(SelectionStartFrame);
			EndFrame = ConvertSubSequenceFrameToRootFrame(SelectionEndFrame);
		}
	}
	else if (QuickRenderModeSettings->FrameRangeType == EMovieGraphQuickRenderFrameRangeType::Custom)
	{
		StartFrame = ConvertSubSequenceFrameToRootFrame(QuickRenderModeSettings->CustomStartFrame);
		EndFrame = ConvertSubSequenceFrameToRootFrame(QuickRenderModeSettings->CustomEndFrame);
	}
	else
	{
		// The frame range has not been overridden, so provide an empty range to indicate that the level sequence's normal
		// playback range should be used.
		return TRange<FFrameNumber>::Empty();
	}

	return TRange<FFrameNumber>::Inclusive(StartFrame, EndFrame);
}

FFrameNumber UMovieGraphQuickRenderSubsystem::ConvertSubSequenceFrameToRootFrame(const int32 FrameNum) const
{
	// The "current" level sequence is the root-most level sequence; the "focused" level sequence is the one that's currently visible in Sequencer
	// (and the level sequence that is having its frame number mapped back to the root)
	const ULevelSequence* CurrentLevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	const ULevelSequence* FocusedLevelSequence = ULevelSequenceEditorBlueprintLibrary::GetFocusedLevelSequence();

	// There's nothing to convert if Sequencer is currently viewing the root-level sequence
	if (CurrentLevelSequence == FocusedLevelSequence)
	{
		return FrameNum;
	}

	// The hierarchy does not include the root, has the outermost subsequence at index 0, and the innermost subsequence at the end
	const TArray<UMovieSceneSubSection*> SubSequenceHierarchy = ULevelSequenceEditorBlueprintLibrary::GetSubSequenceHierarchy();
	if (SubSequenceHierarchy.IsEmpty())
	{
		// Shouldn't happen if the current and focused level sequences are different
		return FrameNum;
	}

	// Map the frame number into tick resolution; the OuterToInnerTransform() appears to need the frame time to be in this format
	const UMovieSceneSubSection* LastSubSection = SubSequenceHierarchy.Last();
	const FFrameRate SubSectionDisplayRate = LastSubSection->GetSequence()->GetMovieScene()->GetDisplayRate();
	const FFrameRate SubSectionTickResolution = LastSubSection->GetSequence()->GetMovieScene()->GetTickResolution();
	FFrameTime RootFrameTime = ConvertFrameTime(FrameNum, SubSectionDisplayRate, SubSectionTickResolution);

	// Walk the subsequence hierarchy in reverse, applying inverse transforms to get back to the outermost (root) frame time 
	for (int32 Index = SubSequenceHierarchy.Num() - 1; Index >= 0; --Index)
	{
		const UMovieSceneSubSection* SubSection = SubSequenceHierarchy[Index];
		if (!SubSection)
		{
			// GetSubSequenceHierarchy() can return nullptr for sections in some cases
			UE_LOGF(LogMovieRenderPipeline, Error, "Quick Render: Found an invalid subsequence; rendered frame ranges may be incorrect.");
			continue;
		}

		if (TOptional<FFrameTime> TransformedTime = SubSection->OuterToInnerTransform().Inverse().TryTransformTime(RootFrameTime))
		{
			RootFrameTime = TransformedTime.GetValue();
		}
	}

	// Map the frame number back to the display rate
	const FFrameRate RootDisplayRate = CurrentLevelSequence->GetMovieScene()->GetDisplayRate();
	const FFrameRate RootTickResolution = CurrentLevelSequence->GetMovieScene()->GetTickResolution();
	RootFrameTime = ConvertFrameTime(RootFrameTime, RootTickResolution, RootDisplayRate);

	return RootFrameTime.GetFrame();
}

void UMovieGraphQuickRenderSubsystem::PerformPreRenderSetup(UWorld* InEditorWorld)
{
	// Adds the specified camera to the Camera Cut track in the given level sequence.
	// If there's already a camera cut section within the Camera Cut track, a new one will be added immediately after.
	// The added section is only one frame long unless OptionalFrameRangeOverride is provided.
	auto AddCameraToLevelSequence = [](ACameraActor* InCameraToAdd, const ULevelSequence* InDestLevelSequence, const TRange<FFrameNumber> OptionalFrameRangeOverride = TRange<FFrameNumber>::Empty())
	{
		FMovieGraphTransactionBlocker TransactionBlocker;
		
		UMovieScene* MovieScene = InDestLevelSequence->GetMovieScene();

		// Add a Camera Cut track if one hasn't already been added
		UMovieSceneCameraCutTrack* CameraCutTrack = Cast<UMovieSceneCameraCutTrack>(MovieScene->GetCameraCutTrack());
		if (!CameraCutTrack)
		{
			CameraCutTrack = CastChecked<UMovieSceneCameraCutTrack>(MovieScene->AddCameraCutTrack(UMovieSceneCameraCutTrack::StaticClass()));
		}

		const int32 NumCameraCutSections = CameraCutTrack->GetAllSections().Num();

		// Determine the playback range for the new camera cut. Each camera cut section has a playback range of one frame.
		const FFrameNumber StartFrame = MovieScene->GetPlaybackRange().GetUpperBound().GetValue();
		const FFrameNumber EndFrame = FFrameRate::TransformTime(FFrameTime(NumCameraCutSections + 1), MovieScene->GetDisplayRate(), MovieScene->GetTickResolution()).GetFrame();

		// The entire playback range for the movie scene always starts a 0 and ends at the last camera cut section's end frame, unless there's an override.
		const bool bHasFrameRangeOverride = !OptionalFrameRangeOverride.IsEmpty();
		const TRange<FFrameNumber> PlaybackRange = bHasFrameRangeOverride
			? OptionalFrameRangeOverride
			: TRange<FFrameNumber>::Inclusive(FFrameNumber(0), EndFrame);
		MovieScene->SetPlaybackRange(PlaybackRange);
		
		UE::Sequencer::FCreateBindingParams CreateBindingParams;
		CreateBindingParams.bSpawnable = true;
		CreateBindingParams.bAllowCustomBinding = true;
		CreateBindingParams.BindingNameOverride = InCameraToAdd->GetActorLabel();

		// Create the binding for the camera
		const FGuid CameraBinding = FSequencerUtilities::CreateOrReplaceBinding(nullptr, MovieScene->GetTypedOuter<UMovieSceneSequence>(), InCameraToAdd, CreateBindingParams);

		const TRange<FFrameNumber> SectionRange = bHasFrameRangeOverride
			? OptionalFrameRangeOverride
			: TRange<FFrameNumber>::Inclusive(StartFrame, EndFrame);

		// Add the new camera to the camera cut track.
		UMovieSceneCameraCutSection* CameraCutSection = CameraCutTrack->AddNewCameraCut(UE::MovieScene::FRelativeObjectBindingID(CameraBinding), StartFrame);
		CameraCutSection->SetRange(SectionRange);

		// The camera may be parented to other actors within a hierarchy, so we bake in the world transform of the camera.
		AddCameraTransformTrack(MovieScene, CameraBinding, InCameraToAdd);
	};
	
	// Spawn a new camera which mimics the viewport's camera 
	auto DuplicateViewportCamera = [this]() -> ACameraActor*
	{
		// Get the PIE world, this is where we need to target the changes (not the editor world). Otherwise we'll have to deal with deleting actors
		// after the render finishes, which isn't ideal (and this will also create entries in the undo stack, also not ideal).
		UWorld* PieWorld = GetWorldOfType(EWorldType::PIE);
		
		FActorSpawnParameters CameraSpawnParams;
		CameraSpawnParams.ObjectFlags |= RF_Transient;
		CameraSpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

		// If the viewport is piloting a camera (and not another type of non-camera actor), we should just duplicate it. We can't just copy over
		// the basic transform + FOV, etc because this may be a cine camera with many properties set on it (like auto-exposure).
		// Note that this supports "cinematic locks" (like cameras piloted via Sequencer) and normal piloted cameras.
		if (CachedPrePieData.ViewportActorLock.IsValid() && CachedPrePieData.ViewportActorLockCameraComponent.IsValid())
		{
			if (ACameraActor* ViewportPilotActor = Cast<ACameraActor>(CachedPrePieData.ViewportActorLock.Get()))
			{
				// Use the current pilot cam as the template
				CameraSpawnParams.Template = ViewportPilotActor;

				ACameraActor* SpawnedCamera = Cast<ACameraActor>(PieWorld->SpawnActor(ViewportPilotActor->GetClass(), nullptr, CameraSpawnParams));
				if (SpawnedCamera)
				{
					// Copy over cine camera components to the spawned camera if the viewport was piloted by a cine cam
					if (ACineCameraActor* SpawnedCineCam = Cast<ACineCameraActor>(SpawnedCamera))
					{
						UCineCameraComponent* CachedCineCamComponent = Cast<UCineCameraComponent>(CachedPrePieData.ViewportActorLockCameraComponent.Get());
						UCineCameraComponent* DestCineCamComponent = SpawnedCineCam->GetCineCameraComponent();

						// Copy over the cine cam properties to the destination component. Unfortunately UEngine::CopyPropertiesForUnrelatedObjects()
						// does not appear to work here for some reason, but luckily the component has its settings grouped together for easy copying.
						if (CachedCineCamComponent && DestCineCamComponent)
						{
							DestCineCamComponent->SetFilmback(CachedCineCamComponent->Filmback);
							DestCineCamComponent->SetLensSettings(CachedCineCamComponent->LensSettings);
							DestCineCamComponent->SetFocusSettings(CachedCineCamComponent->FocusSettings);
							DestCineCamComponent->SetCropSettings(CachedCineCamComponent->CropSettings);
							DestCineCamComponent->SetCurrentFocalLength(CachedCineCamComponent->CurrentFocalLength);
							DestCineCamComponent->SetCurrentAperture(CachedCineCamComponent->CurrentAperture);
							DestCineCamComponent->CurrentFocusDistance = CachedCineCamComponent->CurrentFocusDistance;
							DestCineCamComponent->ExposureMethod = CachedCineCamComponent->ExposureMethod;
						}
					}

					// Ensure the spawned camera has the correct world-space transform
					const FTransform CameraTransform = CachedPrePieData.bUseViewportActorLockWorldTransform
						? CachedPrePieData.ViewportActorLockWorldTransform
						: GetWorldSpaceCameraTransform(ViewportPilotActor);
					SpawnedCamera->SetActorTransform(CameraTransform);
				}

				return SpawnedCamera;
			}
		}
		
		// Add a new camera that mimics the viewport's camera to the level sequence. Use a regular camera actor here rather than a cine camera. Using
		// a cine camera massively complicates things and creates a situation where it's difficult to match what the viewport looks like (eg, exposure).
		ACameraActor* NewCamera = Cast<ACameraActor>(PieWorld->SpawnActor<ACameraActor>(CameraSpawnParams));

		// Update the camera to look like the viewport's camera. Use the viewport camera properties that were cached in
		// CachePrePieData() rather than querying the viewport client now — by the time this lambda runs (after PIE
		// starts), the viewport tab may no longer be active/visible, which would cause GetViewportClient() to return
		// null and produce a render from the wrong camera.
		if (CachedPrePieData.ViewportCamera.bIsValid)
		{
			// Start from the cached viewport camera position/rotation (ortho rotation is already corrected at cache time)
			FVector CameraPosition = CachedPrePieData.ViewportCamera.Location;
			FRotator CameraRotation = CachedPrePieData.ViewportCamera.Rotation;

			// If the viewport is being piloted, then we should use the properties of that to render from instead of the viewport's
			// non-piloted camera properties. Note that this only covers non-camera pilot actors; camera pilot actors are covered above.
			if (CachedPrePieData.bUseViewportActorLockWorldTransform)
			{
				CameraPosition = CachedPrePieData.ViewportActorLockWorldTransform.GetTranslation();
				CameraRotation = CachedPrePieData.ViewportActorLockWorldTransform.GetRotation().Rotator();
			}

			NewCamera->SetActorLocation(CameraPosition);
			NewCamera->SetActorRotation(CameraRotation);

			// Note: At extreme zoom levels, the ortho clipping planes don't match up exactly with the viewport. This is something to improve in the future.
			UCameraComponent* CameraComponent = NewCamera->GetCameraComponent();
			CameraComponent->SetProjectionMode(CachedPrePieData.ViewportCamera.bIsPerspective ? ECameraProjectionMode::Perspective : ECameraProjectionMode::Orthographic);
			CameraComponent->SetAspectRatio(CachedPrePieData.ViewportCamera.AspectRatio);
			CameraComponent->SetFieldOfView(CachedPrePieData.ViewportCamera.FOV);
			CameraComponent->SetOrthoWidth(CachedPrePieData.ViewportCamera.OrthoWidth);
			CameraComponent->SetAutoCalculateOrthoPlanes(true);
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Quick Render: Could not determine the active viewport to render from, so the camera used will be incorrect.");
		}

		return NewCamera;
	};
	
	// Adds a Level Visibility track to the Utility subsequence to ensure that the visibility state of levels in the editor is reflected
	// in the render/PIE. Blueprint streamable levels, for example, may be shown in the editor, but will not typically show up in the
	// render unless made visible by code or in Sequencer.
	auto AddLevelVisibilityTracks = [this, InEditorWorld]() -> void
	{
		TArray<FName> VisibleLevelNames;
		TArray<FName> HiddenLevelNames;

		// Determine which levels are currently visible/hidden
		for (const ULevelStreaming* Level : InEditorWorld->GetStreamingLevels())
		{
			if (!Level)
			{
				continue;
			}
	
			const FName LevelName = FPackageName::GetShortFName(Level->GetWorldAssetPackageFName());
			if (FLevelUtils::IsStreamingLevelVisibleInEditor(Level))
			{
				VisibleLevelNames.Add(LevelName);
			}
			else
			{
				HiddenLevelNames.Add(LevelName);
			}
		}

		UMovieScene* UtilityMovieScene = UtilityLevelSequence->GetMovieScene();

		// Add a track to show visible levels
		UMovieSceneLevelVisibilityTrack* LevelVisibilityTrack_Visible = CastChecked<UMovieSceneLevelVisibilityTrack>(UtilityMovieScene->AddTrack(UMovieSceneLevelVisibilityTrack::StaticClass()));
		UMovieSceneLevelVisibilitySection* LevelVisibilitySection_Visible = CastChecked<UMovieSceneLevelVisibilitySection>(LevelVisibilityTrack_Visible->CreateNewSection());
		LevelVisibilityTrack_Visible->AddSection(*LevelVisibilitySection_Visible);
		LevelVisibilitySection_Visible->SetVisibility(ELevelVisibility::Visible);
		LevelVisibilitySection_Visible->SetRange(UtilityMovieScene->GetPlaybackRange());
		LevelVisibilitySection_Visible->SetLevelNames(VisibleLevelNames);

		// Add a track to hide hidden levels
		UMovieSceneLevelVisibilityTrack* LevelVisibilityTrack_Hidden = CastChecked<UMovieSceneLevelVisibilityTrack>(UtilityMovieScene->AddTrack(UMovieSceneLevelVisibilityTrack::StaticClass()));
		UMovieSceneLevelVisibilitySection* LevelVisibilitySection_Hidden = CastChecked<UMovieSceneLevelVisibilitySection>(LevelVisibilityTrack_Hidden->CreateNewSection());
		LevelVisibilityTrack_Hidden->AddSection(*LevelVisibilitySection_Hidden);
		LevelVisibilitySection_Hidden->SetVisibility(ELevelVisibility::Hidden);
		LevelVisibilitySection_Hidden->SetRange(UtilityMovieScene->GetPlaybackRange());
		LevelVisibilitySection_Hidden->SetLevelNames(HiddenLevelNames);
	};

	UMovieScene* RenderingMovieScene = RenderingLevelSequence->GetMovieScene();

	// Always add the Utility sequence. It's used for various things in different modes. Set the hierarchical bias of the new subsequence section so
	// its effects take precedence over the level sequence it's being added to. Use a very high number; the value used here is arbitrary, but should
	// be high enough to do the job. Note that the subsequence section is added with 0 duration, its length will be determined later.
	UMovieSceneSubTrack* NewSubsequenceTrack = CastChecked<UMovieSceneSubTrack>(RenderingMovieScene->AddTrack(UMovieSceneSubTrack::StaticClass()));
	UMovieSceneSubSection* SequenceSubsection = NewSubsequenceTrack->AddSequence(UtilityLevelSequence.Get(), 0, 0);
	SequenceSubsection->Parameters.HierarchicalBias = 1000000;

	// Track the frame range that should be used for rendering. By default this is the rendering sequence's playback range, but for many modes which
	// generate a dynamic level sequence for rendering (and the rendering sequence starts as empty), this will need to be adjusted.
	TRange<FFrameNumber> RenderingFrameRange = RenderingMovieScene->GetPlaybackRange();

	// Do mode-specific setup
	if (QuickRenderMode == EMovieGraphQuickRenderMode::CurrentViewport)
	{
		// Adding a camera will tweak the length of the utility sequence's playback range
		ACameraActor* NewCamera = DuplicateViewportCamera();
		AddCameraToLevelSequence(NewCamera, UtilityLevelSequence.Get(), RenderingMovieScene->GetPlaybackRange());

		RenderingFrameRange = UtilityLevelSequence->GetMovieScene()->GetPlaybackRange();
	}
	else if (QuickRenderMode == EMovieGraphQuickRenderMode::UseViewportCameraInSequence)
	{
		// Add the viewport camera to the utility sequence to override any other cameras in use
		ACameraActor* NewCamera = DuplicateViewportCamera();
		AddCameraToLevelSequence(NewCamera, UtilityLevelSequence.Get(), RenderingMovieScene->GetPlaybackRange());
	}
	else if (QuickRenderMode == EMovieGraphQuickRenderMode::SelectedCameras)
	{
		for (const TWeakObjectPtr<ACameraActor>& SelectedCamera : CachedPrePieData.SelectedCameras)
		{
			if (SelectedCamera.IsValid())
			{
				// Adding a camera will tweak the length of the utility sequence's playback range
				AddCameraToLevelSequence(SelectedCamera.Get(), UtilityLevelSequence.Get());

				RenderingFrameRange = UtilityLevelSequence->GetMovieScene()->GetPlaybackRange();
			}
		}
	}

	// Now that cameras have been added and the length of the render has been established, make sure the playback ranges are correct
	RenderingLevelSequence->GetMovieScene()->SetPlaybackRange(RenderingFrameRange);
	UtilityLevelSequence->GetMovieScene()->SetPlaybackRange(RenderingFrameRange);
	SequenceSubsection->SetRange(RenderingFrameRange);

	// Add the Level Visibility tracks to the Utility sequence if matching the editor visibility was requested
	const EMovieGraphQuickRenderViewportLookFlags LookFlags = static_cast<EMovieGraphQuickRenderViewportLookFlags>(QuickRenderModeSettings->ViewportLookFlags);
	if (IsViewportLookFlagActive(EMovieGraphQuickRenderViewportLookFlags::Visibility))
	{
		AddLevelVisibilityTracks();
	}
}

bool UMovieGraphQuickRenderSubsystem::IsViewportLookFlagActive(const EMovieGraphQuickRenderViewportLookFlags ViewportLookFlag) const
{
	if (!IsValid(QuickRenderModeSettings))
	{
		return false;
	}
	
	const EMovieGraphQuickRenderViewportLookFlags LookFlags = static_cast<EMovieGraphQuickRenderViewportLookFlags>(QuickRenderModeSettings->ViewportLookFlags);
	
	return QuickRenderModeSettings->bOverride_ViewportLookFlags && EnumHasAnyFlags(LookFlags, ViewportLookFlag);
}

#undef LOCTEXT_NAMESPACE
