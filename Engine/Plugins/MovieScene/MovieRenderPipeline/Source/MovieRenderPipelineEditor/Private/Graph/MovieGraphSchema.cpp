// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphSchema.h"

#include "EdGraphSchema_K2.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphInputNode.h"
#include "Graph/Nodes/MovieGraphOutputNode.h"
#include "Graph/Nodes/MovieGraphRenderLayerNode.h"
#include "Graph/Nodes/MovieGraphRerouteNode.h"
#include "Graph/Nodes/MovieGraphSubgraphNode.h"
#include "Graph/Nodes/MovieGraphVariableNode.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/MovieEdGraph.h"
#include "Graph/MovieEdGraphConnectionPolicy.h"
#include "Graph/MovieEdGraphNode.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "ScopedTransaction.h"
#include "GraphEditor.h"
#include "MovieEdGraphInputNode.h"
#include "MovieEdGraphOutputNode.h"
#include "MovieEdGraphRerouteNode.h"
#include "MovieEdGraphVariableNode.h"
#include "MoviePipelineEdGraphSubgraphNode.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphSchema)

TArray<UClass*> UMovieGraphSchema::MoviePipelineNodeClasses;

#define LOCTEXT_NAMESPACE "MoviePipelineGraphSchema"

const FName UMovieGraphSchema::PC_Branch(TEXT("branch"));	// The branch looks like an Exec pin, but isn't the same thing, so we don't use the BP Exec type
const FName UMovieGraphSchema::PC_Wildcard(UEdGraphSchema_K2::PC_Wildcard);
const FName UMovieGraphSchema::PC_Boolean(UEdGraphSchema_K2::PC_Boolean);
const FName UMovieGraphSchema::PC_Byte(UEdGraphSchema_K2::PC_Byte);
const FName UMovieGraphSchema::PC_Integer(UEdGraphSchema_K2::PC_Int);
const FName UMovieGraphSchema::PC_Int64(UEdGraphSchema_K2::PC_Int64);
const FName UMovieGraphSchema::PC_Real(UEdGraphSchema_K2::PC_Real);
const FName UMovieGraphSchema::PC_Float(UEdGraphSchema_K2::PC_Float);
const FName UMovieGraphSchema::PC_Double(UEdGraphSchema_K2::PC_Double);
const FName UMovieGraphSchema::PC_Name(UEdGraphSchema_K2::PC_Name);
const FName UMovieGraphSchema::PC_String(UEdGraphSchema_K2::PC_String);
const FName UMovieGraphSchema::PC_Text(UEdGraphSchema_K2::PC_Text);
const FName UMovieGraphSchema::PC_Enum(UEdGraphSchema_K2::PC_Enum);
const FName UMovieGraphSchema::PC_Struct(UEdGraphSchema_K2::PC_Struct);
const FName UMovieGraphSchema::PC_Object(UEdGraphSchema_K2::PC_Object);
const FName UMovieGraphSchema::PC_SoftObject(UEdGraphSchema_K2::PC_SoftObject);
const FName UMovieGraphSchema::PC_Class(UEdGraphSchema_K2::PC_Class);
const FName UMovieGraphSchema::PC_SoftClass(UEdGraphSchema_K2::PC_SoftClass);

const FText FMovieGraphSchemaAction::UserVariablesCategory = LOCTEXT("UserVariablesCategory", "User Variables");
const FText FMovieGraphSchemaAction::GlobalVariablesCategory = LOCTEXT("GlobalVariablesCategory", "Global Variables");

namespace UE::MovieGraph::Private
{
	UMovieGraphNode* GetGraphNodeFromEdPin(const UEdGraphPin* InPin)
	{
		if (!InPin)
		{
			return nullptr;
		}
		
		const UMoviePipelineEdGraphNodeBase* EdGraphNode = CastChecked<UMoviePipelineEdGraphNodeBase>(InPin->GetOwningNode());

		UMovieGraphNode* RuntimeNode = EdGraphNode->GetRuntimeNode();
		if (!IsValid(RuntimeNode))
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Cannot find the runtime node associated with the editor node (the node may be from a plugin that's not currently loaded).");
			return nullptr;
		}

		return RuntimeNode;
	}

	UMovieGraphPin* GetGraphPinFromEdPin(const UEdGraphPin* InPin)
	{
		const UMovieGraphNode* GraphNode = GetGraphNodeFromEdPin(InPin);
		if (!IsValid(GraphNode))
		{
			return nullptr;
		}
		
		UMovieGraphPin* GraphPin = (InPin->Direction == EGPD_Input) ? GraphNode->GetInputPin(InPin->PinName) : GraphNode->GetOutputPin(InPin->PinName);
		check(GraphPin);

		return GraphPin;
	}
	
	UMovieGraphConfig* GetGraphFromEdPin(const UEdGraphPin* InPin)
	{
		const UMovieGraphNode* RuntimeNode = GetGraphNodeFromEdPin(InPin);
		if (!IsValid(RuntimeNode))
		{
			return nullptr;
		}

		UMovieGraphConfig* RuntimeGraph = RuntimeNode->GetGraph();
		check(RuntimeGraph);

		return RuntimeGraph;
	}
}

void UMovieGraphSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	/*UMovieGraphConfig* RuntimeGraph = CastChecked<UMoviePipelineEdGraph>(&Graph)->GetPipelineGraph();
	const FScopedTransaction Transaction(LOCTEXT("GraphEditor_NewNode", "Create Pipeline Graph Node."));
	RuntimeGraph->Modify();
	const bool bSelectNewNode = false;

	// Input Node
	{
		UMovieGraphNode* RuntimeNode = RuntimeGraph->ConstructRuntimeNode<UMovieGraphInputNode>();

		// Now create the editor graph node
		FGraphNodeCreator<UMoviePipelineEdGraphNodeInput> NodeCreator(Graph);
		UMoviePipelineEdGraphNodeBase* GraphNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode);
		GraphNode->SetRuntimeNode(RuntimeNode);
		NodeCreator.Finalize();
	}

	// Output Node
	{
		UMovieGraphNode* RuntimeNode = RuntimeGraph->ConstructRuntimeNode<UMovieGraphOutputNode>();

		// Now create the editor graph node
		FGraphNodeCreator<UMoviePipelineEdGraphNodeOutput> NodeCreator(Graph);
		UMoviePipelineEdGraphNodeBase* GraphNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode);
		GraphNode->SetRuntimeNode(RuntimeNode);
		NodeCreator.Finalize();
	}*/
}

bool UMovieGraphSchema::SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const
{
	// No maps, sets, or arrays
	return ContainerType == EPinContainerType::None;
}

bool UMovieGraphSchema::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	// The graph doesn't support editing default values for pins yet
	return true;
}

bool UMovieGraphSchema::SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const
{
	bool bIsSupported = false;

	if ((InSourcePinDirection == EGPD_Input) && Cast<UMoviePipelineEdGraphNodeInput>(InTargetNode))
	{
		bIsSupported = true;
		OutErrorMessage = LOCTEXT("AddPinToInputNode", "Add Pin to Input Node");
	}
	else if ((InSourcePinDirection == EGPD_Output) && Cast<UMoviePipelineEdGraphNodeOutput>(InTargetNode))
	{
		bIsSupported = true;
		OutErrorMessage = LOCTEXT("AddPinToOutputNode", "Add Pin to Output Node");
	}
	
	return bIsSupported;
}

UEdGraphPin* UMovieGraphSchema::DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const
{
	UEdGraphPin* NewEdPin = nullptr;
	
	const UMoviePipelineEdGraphNodeBase* EdNode = Cast<UMoviePipelineEdGraphNodeBase>(InTargetNode);
	if (!EdNode)
	{
		return nullptr;
	}

	const UMovieGraphNode* RuntimeNode = EdNode->GetRuntimeNode();
	if (!RuntimeNode)
	{
		return nullptr;
	}

	if (UMovieGraphConfig* GraphConfig = RuntimeNode->GetGraph())
	{
		FText NewMemberName;
		if (InSourcePinName == NAME_None)
		{
			// If the source of the connection is a Render Layer node, then name the new member based on the layer name. Otherwise, give the new
			// member a generic name.
			if (const UMovieGraphRenderLayerNode* RenderLayerNode = Cast<UMovieGraphRenderLayerNode>(UE::MovieGraph::Private::GetGraphNodeFromEdPin(PinBeingDropped)))
			{
				NewMemberName = FText::FromString(RenderLayerNode->LayerName);
			}
			else
			{
				NewMemberName = (InSourcePinDirection == EGPD_Input) ? LOCTEXT("NewInputName", "NewInput") : LOCTEXT("NewOutputName", "NewOutput");
			}
		}
		else
		{
			NewMemberName = FText::FromName(InSourcePinName);
		}

		UMovieGraphInterfaceBase* NewMember;
		if (InSourcePinDirection == EGPD_Input)
		{
			NewMember = GraphConfig->AddInput(NewMemberName);
		}
		else
		{
			NewMember = GraphConfig->AddOutput(NewMemberName);
		}
		
		if (NewMember)
		{
			NewMember->bIsBranch = InSourcePinType.PinCategory == PC_Branch;

			if (!NewMember->bIsBranch)
			{
				NewMember->SetValueType(UMoviePipelineEdGraphNode::GetValueTypeFromPinType(InSourcePinType), InSourcePinType.PinSubCategoryObject.Get());
			}

			// Return the last pin on the node (which was just added above)
			NewEdPin = EdNode->GetPinAt(EdNode->GetAllPins().Num() - 1);
		}
	}

	return NewEdPin; 
}

void UMovieGraphSchema::SetPinBeingDroppedOnNode(UEdGraphPin* InSourcePin) const
{
	PinBeingDropped = InSourcePin;
}

bool UMovieGraphSchema::SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const
{
	if (!Graph || !Node)
	{
		return false;
	}

	const UMoviePipelineEdGraphNodeBase* EdGraphNode = CastChecked<UMoviePipelineEdGraphNodeBase>(Node);
	UMovieGraphNode* RuntimeNode = EdGraphNode->GetRuntimeNode();
	check(RuntimeNode);

	UMovieGraphConfig* OwningGraph = RuntimeNode->GetGraph();
	check(OwningGraph);

	OwningGraph->RemoveNode(RuntimeNode);

	return true;
}

void UMovieGraphSchema::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	const FScopedTransaction Transaction(LOCTEXT("CreateRerouteNodeOnWire", "Create Reroute Node"));

	// This is a temporary action for creating the reroute node, so the category, display name, tooltip, and keywords can just be empty.
	constexpr int32 Grouping = 0;
	const TSharedPtr<FMovieGraphSchemaAction> RerouteNodeAction =
		MakeShared<FMovieGraphSchemaAction_NewNode>(FText::GetEmpty(), FText::GetEmpty(), FText::GetEmpty(), Grouping, FText::GetEmpty()); 
	RerouteNodeAction->NodeClass = UMovieGraphRerouteNode::StaticClass();

	// The node should be centered on where the mouse is clicked. The node's position is based on its top-left corner, so offset by
	// half of its width/height to center it on the mouse. Note that the spacer size is duplicated from inside SGraphNodeKnot.
	const FVector2f NodeSpacerSize(42.0f, 24.0f);
	const FVector2f NodePosition = GraphPosition - (NodeSpacerSize * 0.5f);
	
	UEdGraph* OwningGraph = PinA->GetOwningNode()->GetGraph();
	if (ensure(OwningGraph))
	{
		// Break the existing connection (ie, the connection that was clicked on).
		PinA->BreakLinkTo(PinB);

		// Create the reroute node. Use nullptr as the FromPin -- we'll manually perform connections via TryCreateConnection() so the
		// pin type propagates properly and the runtime graph is informed of the change.
		UEdGraphPin* FromPin = nullptr;
		const UEdGraphNode* NewRerouteNode = RerouteNodeAction->PerformAction(OwningGraph, FromPin, NodePosition);

		// Connect the reroute node's right (output) pin back to the previously-connected downstream node.
		if (UEdGraphPin* OutputPin = NewRerouteNode->FindPin(NAME_None, EGPD_Output))
		{
			TryCreateConnection(OutputPin, PinB);
		}

		// Connect the reroute node's left (input) pin back to the previously-connected upstream node.
		if (UEdGraphPin* InputPin = NewRerouteNode->FindPin(NAME_None, EGPD_Input))
		{
			TryCreateConnection(PinA, InputPin);
		}
	}
}

bool UMovieGraphSchema::IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const
{
	return CurrentCacheRefreshID != InVisualizationCacheID;
}

int32 UMovieGraphSchema::GetCurrentVisualizationCacheID() const
{
	return CurrentCacheRefreshID;
}

void UMovieGraphSchema::ForceVisualizationCacheClear() const
{
	++CurrentCacheRefreshID;
}

void UMovieGraphSchema::InitMoviePipelineNodeClasses()
{
	if (MoviePipelineNodeClasses.Num() > 0)
	{
		return;
	}

	for (TObjectIterator<UClass> It; It; ++It)
	{
		if (It->IsChildOf(UMovieGraphNode::StaticClass())
			&& !It->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Hidden | CLASS_HideDropDown))
		{
			MoviePipelineNodeClasses.Add(*It);
		}
	}

	MoviePipelineNodeClasses.Sort();
}

const TArray<UClass*>& UMovieGraphSchema::GetNodeClasses()
{
	if (MoviePipelineNodeClasses.IsEmpty())
	{
		InitMoviePipelineNodeClasses();
	}
	
	return MoviePipelineNodeClasses;
}

bool UMovieGraphSchema::IsConnectionToBranchAllowed(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin, FText& OutError) const
{
	const UMovieGraphPin* ToPin = UE::MovieGraph::Private::GetGraphPinFromEdPin(InputPin);
	const UMovieGraphPin* FromPin = UE::MovieGraph::Private::GetGraphPinFromEdPin(OutputPin);
	
	return FromPin->IsConnectionToBranchAllowed(ToPin, OutError);
}

void UMovieGraphSchema::AddExtraMenuActions(FGraphActionMenuBuilder& ActionMenuBuilder) const
{
	// Comment action. Only add if there's no FromPin (ie, no connection is currently being built).
	if (!ActionMenuBuilder.FromPin)
	{
		ActionMenuBuilder.AddAction(CreateCommentMenuAction());
	}
}

TSharedRef<FMovieGraphSchemaAction_NewComment> UMovieGraphSchema::CreateCommentMenuAction() const
{
	const FText CommentMenuDesc = LOCTEXT("AddComment", "Add Comment");
	const FText CommentCategory;
	const FText CommentDescription = LOCTEXT("AddCommentTooltip", "Create a resizable comment box.");

	const TSharedRef<FMovieGraphSchemaAction_NewComment> NewCommentAction = MakeShared<FMovieGraphSchemaAction_NewComment>(CommentCategory, CommentMenuDesc, CommentDescription, 0);
	return NewCommentAction;
}

void UMovieGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	InitMoviePipelineNodeClasses();

	const UMoviePipelineEdGraph* Graph = Cast<UMoviePipelineEdGraph>(ContextMenuBuilder.CurrentGraph);
	if (!Graph)
	{
		return;
	}

	const UMovieGraphConfig* RuntimeGraph = Graph->GetPipelineGraph();
	if (!RuntimeGraph)
	{
		return;
	}

	for (UClass* PipelineNodeClass : MoviePipelineNodeClasses)
	{
		const UMovieGraphNode* PipelineNode = PipelineNodeClass->GetDefaultObject<UMovieGraphNode>();
		if (PipelineNodeClass == UMovieGraphVariableNode::StaticClass())
		{
			// Add variable actions separately
			continue;
		}
		
		if (PipelineNodeClass == UMovieGraphInputNode::StaticClass() ||
			PipelineNodeClass == UMovieGraphOutputNode::StaticClass())
		{
			// Can't place Input and Output nodes manually.
			continue;
		}

		// Determine if this node type can be created in the branch that FromPin is in. FromPin is non-null if the node is being created and connected
		// to an existing pin in one step (ie, the user is currently creating a connection).
		bool bCanAppearInMenu = true;
		if (ContextMenuBuilder.FromPin)
		{
			const UMovieGraphConfig* GraphConfig = UE::MovieGraph::Private::GetGraphFromEdPin(ContextMenuBuilder.FromPin);
			const UMovieGraphPin* FromGraphPin = UE::MovieGraph::Private::GetGraphPinFromEdPin(ContextMenuBuilder.FromPin);
			UMovieGraphNode* FromGraphNode = UE::MovieGraph::Private::GetGraphNodeFromEdPin(ContextMenuBuilder.FromPin);
			
			if (GraphConfig && FromGraphPin && FromGraphNode)
			{
				// Get the branch name that FromPin is on (there should only be one branch name found in this scenario)
				constexpr bool bStopAtSubgraph = true;
				TArray<FString> FromBranchNames = (ContextMenuBuilder.FromPin->Direction == EGPD_Input)
					? GraphConfig->GetDownstreamBranchNames(FromGraphNode, FromGraphPin, bStopAtSubgraph)
					: GraphConfig->GetUpstreamBranchNames(FromGraphNode, FromGraphPin, bStopAtSubgraph);

				// Determine if a specific node class can be created on this branch given its branch restriction
				bool bBranchRestrictionIsOk = true;
				if (PipelineNode->GetBranchRestriction() == EMovieGraphBranchRestriction::Globals)
				{
					bBranchRestrictionIsOk = FromBranchNames.Contains(UMovieGraphNode::GlobalsPinNameString);
				}
				else if (PipelineNode->GetBranchRestriction() == EMovieGraphBranchRestriction::RenderLayer)
				{
					bBranchRestrictionIsOk = !FromBranchNames.Contains(UMovieGraphNode::GlobalsPinNameString);
				}
				else
				{
					// The branch restriction is "Any", so the node creation should be ok
				}

				// Determine if the node can be shown in the menu. An exception to the above rules is the Reroute node -- this can always
				// be created in any context.
				if (PipelineNodeClass == UMovieGraphRerouteNode::StaticClass())
				{
					bCanAppearInMenu = true;
				}
				else
				{
					bCanAppearInMenu = bBranchRestrictionIsOk && (ContextMenuBuilder.FromPin->PinType.PinCategory == PC_Branch);
				}
			}
		}
		
		if (bCanAppearInMenu)
		{
			const FText Name = PipelineNode->GetNodeTitle();
			const FText Category = PipelineNode->GetMenuCategory();
			const FText Tooltip = LOCTEXT("CreateNode_Tooltip", "Create a node of this type.");
			constexpr int32 Grouping = 0;
			const FText Keywords = PipelineNode->GetKeywords();
			
			TSharedPtr<FMovieGraphSchemaAction> NewAction = MakeShared<FMovieGraphSchemaAction_NewNode>(Category, Name, Tooltip, Grouping, Keywords); 
			NewAction->NodeClass = PipelineNodeClass;

			ContextMenuBuilder.AddAction(NewAction);
		}
	}

	// Create an accessor node action for each variable the graph has
	constexpr bool bIncludeGlobal = true;
	for (const UMovieGraphVariable* Variable : RuntimeGraph->GetVariables(bIncludeGlobal))
	{
		const FText Name = FText::Format(LOCTEXT("CreateVariable_Name", "Get {0}"), FText::FromString(Variable->GetMemberName()));
		const FText Category = Variable->IsGlobal() ? LOCTEXT("CreateGlobalVariable_Category", "Global Variables") : LOCTEXT("CreateVariable_Category", "Variables");
		const FText Tooltip = LOCTEXT("CreateVariable_Tooltip", "Create an accessor node for this variable.");
		
		TSharedPtr<FMovieGraphSchemaAction> NewAction = MakeShared<FMovieGraphSchemaAction_NewVariableNode>(Category, Name, Variable->GetGuid(), Tooltip);
		NewAction->NodeClass = UMovieGraphVariableNode::StaticClass();
		
		// Determine if this node can be created and connected to FromPin
		bool bCanAppearInMenu = true;
		if (ContextMenuBuilder.FromPin)
		{
			if (const UMovieGraphPin* FromPin = UE::MovieGraph::Private::GetGraphPinFromEdPin(ContextMenuBuilder.FromPin))
			{
				// Variable type and pin type must match
				bCanAppearInMenu = (FromPin->Properties.Type == Variable->GetValueType()) && (FromPin->Properties.TypeObject == Variable->GetValueTypeObject());
			}
		}

		if (bCanAppearInMenu)
		{
			ContextMenuBuilder.AddAction(NewAction);
		}
	}

	AddExtraMenuActions(ContextMenuBuilder);
}

const FPinConnectionResponse UMovieGraphSchema::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	const UMovieGraphPin* FromPin = UE::MovieGraph::Private::GetGraphPinFromEdPin(PinA);
	const UMovieGraphPin* ToPin = UE::MovieGraph::Private::GetGraphPinFromEdPin(PinB);
	
	if (!IsValid(FromPin) || !IsValid(ToPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("InvalidNodeError", "The to/from pin/node is invalid!"));
	}
	
	return FromPin->CanCreateConnection_PinConnectionResponse(ToPin);
}

bool UMovieGraphSchema::TryCreateConnection(UEdGraphPin* InA, UEdGraphPin* InB) const
{
	check(InA && InB);
	const UEdGraphPin* A = (InA->Direction == EGPD_Output) ? InA : InB;
	const UEdGraphPin* B = (InA->Direction == EGPD_Input) ? InA : InB;

	const UMoviePipelineEdGraphNodeBase* EdGraphNodeA = CastChecked<UMoviePipelineEdGraphNodeBase>(A->GetOwningNode());
	const UMoviePipelineEdGraphNodeBase* EdGraphNodeB = CastChecked<UMoviePipelineEdGraphNodeBase>(B->GetOwningNode());

	UMovieGraphNode* RuntimeNodeA = EdGraphNodeA->GetRuntimeNode();
	UMovieGraphNode* RuntimeNodeB = EdGraphNodeB->GetRuntimeNode();

	// If the node associated with either of the pins is invalid, the node is probably from a plugin that isn't loaded. If this is the case, bail
	// on creating the connection.
	if (!IsValid(RuntimeNodeA) || !IsValid(RuntimeNodeB))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Cannot create a connection to/from a node which is not currently valid (it may be from a plugin that's not currently loaded).");
		return false;
	}

	UMovieGraphConfig* RuntimeGraph = RuntimeNodeA->GetGraph();
	check(RuntimeGraph);

	// See if the native UEdGraph connection goes through.
	// If the connection was made, try to propagate the change to our runtime graph.
	const bool bModified = Super::TryCreateConnection(InA, InB);
	if (bModified)
	{
		const bool bReconstructNodeB = RuntimeGraph->AddLabeledEdge(RuntimeNodeA, A->PinName, RuntimeNodeB, B->PinName);
		//if (bReconstructNodeB)
		//{
		//	RuntimeNodeB->ReconstructNode();
		//}
	}

	return bModified;
}

void UMovieGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
	const FScopedTransaction Transaction(LOCTEXT("MoviePipelineGraphEditor_BreakPinLinks", "Break Pin Links"));
	Super::BreakPinLinks(TargetPin, bSendsNodeNotification);

	UEdGraphNode* GraphNode = TargetPin.GetOwningNode();
	UMoviePipelineEdGraphNodeBase* MoviePipelineEdGraphNode = CastChecked<UMoviePipelineEdGraphNodeBase>(GraphNode);

	// The node may be invalid if it's from a plugin that isn't currently loaded. Skip breaking the connection, although that's not ideal because the
	// user is probably trying to get rid of the broken node.
	UMovieGraphNode* RuntimeNode = MoviePipelineEdGraphNode->GetRuntimeNode();
	if (!IsValid(RuntimeNode))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Cannot remove connection from node which is not currently valid (it may be from a plugin that's not currently loaded).");
		return;
	}

	UMovieGraphConfig* RuntimeGraph = RuntimeNode->GetGraph();
	check(RuntimeGraph);

	if (TargetPin.Direction == EEdGraphPinDirection::EGPD_Input)
	{
		RuntimeGraph->RemoveInboundEdges(RuntimeNode, TargetPin.PinName);
	}
	else if (TargetPin.Direction == EEdGraphPinDirection::EGPD_Output)
	{
		RuntimeGraph->RemoveOutboundEdges(RuntimeNode, TargetPin.PinName);
	}
}

void UMovieGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction(LOCTEXT("MoviePipelineGraphEditor_BreakSinglePinLinks", "Break Single Pin Link"));
	Super::BreakSinglePinLink(SourcePin, TargetPin);

	UEdGraphNode* SourceGraphNode = SourcePin->GetOwningNode();
	UEdGraphNode* TargetGraphNode = TargetPin->GetOwningNode();

	UMoviePipelineEdGraphNodeBase* SourcePipelineGraphNode = CastChecked<UMoviePipelineEdGraphNodeBase>(SourceGraphNode);
	UMoviePipelineEdGraphNodeBase* TargetPipelineGraphNode = CastChecked<UMoviePipelineEdGraphNodeBase>(TargetGraphNode);

	UMovieGraphNode* SourceRuntimeNode = SourcePipelineGraphNode->GetRuntimeNode();
	UMovieGraphNode* TargetRuntimeNode = TargetPipelineGraphNode->GetRuntimeNode();

	if (!IsValid(SourceRuntimeNode) || !IsValid(TargetRuntimeNode))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Cannot remove connection from node which is not currently valid (it may be from a plugin that's not currently loaded).");
		return;
	}

	UMovieGraphConfig* RuntimeGraph = SourceRuntimeNode->GetGraph();
	check(RuntimeGraph);

	RuntimeGraph->RemoveLabeledEdge(SourceRuntimeNode, SourcePin->PinName, TargetRuntimeNode, TargetPin->PinName);
}

FLinearColor UMovieGraphSchema::GetTypeColor(const FName& InPinCategory, const FName& InPinSubCategory)
{
	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();

	if (InPinCategory == PC_Branch)
	{
		return Settings->ExecutionPinTypeColor;
	}

	if (InPinCategory == PC_Boolean)
	{
		return Settings->BooleanPinTypeColor;
	}
	
	if (InPinCategory == PC_Byte)
	{
		return Settings->BytePinTypeColor;
	}

	if (InPinCategory == PC_Integer)
	{
		return Settings->IntPinTypeColor;
	}

	if (InPinCategory == PC_Int64)
	{
		return Settings->Int64PinTypeColor;
	}

	if (InPinCategory == PC_Float)
	{
		return Settings->FloatPinTypeColor;
	}

	if (InPinCategory == PC_Double)
	{
		return Settings->FloatPinTypeColor;
	}

	// Use the same pin color for floats and doubles. These types can be used interchangeably within the graph, and it's confusing to have them
	// be different colors (because it implies that they cannot be used together).
	if (InPinCategory == PC_Real)
	{
		if (InPinSubCategory == PC_Float)
		{
			return Settings->FloatPinTypeColor;
		}

		if (InPinSubCategory == PC_Double)
		{
			return Settings->FloatPinTypeColor;
		}
	}

	if (InPinCategory == PC_Name)
	{
		return Settings->NamePinTypeColor;
	}

	if (InPinCategory == PC_String)
	{
		return Settings->StringPinTypeColor;
	}

	if (InPinCategory == PC_Text)
	{
		return Settings->TextPinTypeColor;
	}

	if (InPinCategory == PC_Enum)
	{
		return Settings->BytePinTypeColor;
	}

	if (InPinCategory == PC_Struct)
	{
		return Settings->StructPinTypeColor;
	}
	
	if (InPinCategory == PC_Object)
	{
		return Settings->ObjectPinTypeColor;
	}

	if (InPinCategory == PC_SoftObject)
	{
		return Settings->SoftObjectPinTypeColor;
	}
	
	if (InPinCategory == PC_Class)
    {
    	return Settings->ClassPinTypeColor;
    }

	if (InPinCategory == PC_SoftClass)
	{
		return Settings->SoftClassPinTypeColor;
	}
	
	return Settings->DefaultPinTypeColor;
}

FLinearColor UMovieGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	// GetTypeColor only considers pin category/subcategory names, so struct types like Vector/Rotator/Transform
	// all resolve to the generic struct color. Check PinSubCategoryObject for well-known struct types first.
	// These types are probably never used in MRG, but this helps UI consistency.
	if (PinType.PinCategory == PC_Struct && PinType.PinSubCategoryObject.IsValid())
	{
		const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();

		if (PinType.PinSubCategoryObject == TBaseStructure<FVector>::Get()
			|| PinType.PinSubCategoryObject == TVariantStructure<FVector3f>::Get())
		{
			return Settings->VectorPinTypeColor;
		}

		if (PinType.PinSubCategoryObject == TBaseStructure<FRotator>::Get())
		{
			return Settings->RotatorPinTypeColor;
		}

		if (PinType.PinSubCategoryObject == TBaseStructure<FTransform>::Get())
		{
			return Settings->TransformPinTypeColor;
		}
	}

	return GetTypeColor(PinType.PinCategory, PinType.PinSubCategory);
}

FConnectionDrawingPolicy* UMovieGraphSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID,
	float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements,
	UEdGraph* InGraphObj) const
{
	return new FMovieEdGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}

void FMovieGraphSchemaAction::MovePersistentItemToCategory(const FText& NewCategoryName)
{
	if (const TObjectPtr<UMovieGraphVariable> TargetVariable = Cast<UMovieGraphVariable>(ActionTarget))
	{
		FString NewCategory = NewCategoryName.ToString();
		
		// If moving to the root, the category will be User Variables
		if (NewCategory == UserVariablesCategory.ToString())
		{
			NewCategory = FString();
		}

		const FScopedTransaction Transaction(LOCTEXT("GraphEditor_SetVariableCategory", "Set Variable Category"));
		
		// Remove the "User Variables" prefix. Variables themselves do not store that part of the category.
		const FString UserVariablesRootPrefix = FString::Format(TEXT("{0}|"), {UserVariablesCategory.ToString()});
		NewCategory = NewCategory.StartsWith(UserVariablesRootPrefix) ? NewCategory.RightChop(UserVariablesRootPrefix.Len()) : NewCategory;
		TargetVariable->SetCategory(NewCategory);
	}
}

bool FMovieGraphSchemaAction::ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction)
{
	const TSharedRef<FMovieGraphSchemaAction> GraphAction = StaticCastSharedRef<FMovieGraphSchemaAction>(OtherAction);

	const TObjectPtr<UMovieGraphVariable> BeforeVariable = Cast<UMovieGraphVariable>(GraphAction->ActionTarget);
	if (!BeforeVariable)
	{
		return false;
	}

	const TObjectPtr<UMovieGraphVariable> TargetVariable = Cast<UMovieGraphVariable>(ActionTarget);
	if (!TargetVariable)
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("GraphEditor_MoveVariable", "Move Variable"));

	BeforeVariable->GetOwningGraph()->MoveVariableBefore(TargetVariable, BeforeVariable);

	return true;
}

FMovieGraphSchemaAction_NewNode::FMovieGraphSchemaAction_NewNode(FText InNodeCategory, FText InDisplayName, FText InToolTip, int32 InGrouping, FText InKeywords)
	: FMovieGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InDisplayName), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords))
{
	
}

UEdGraphNode* FMovieGraphSchemaAction_NewNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UMovieGraphConfig* RuntimeGraph = CastChecked<UMoviePipelineEdGraph>(ParentGraph)->GetPipelineGraph();
	const FScopedTransaction Transaction(LOCTEXT("GraphEditor_NewNode", "Create Pipeline Graph Node."));
	RuntimeGraph->Modify();
	ParentGraph->Modify();

	UMovieGraphNode* RuntimeNode = RuntimeGraph->ConstructRuntimeNode<UMovieGraphNode>(NodeClass);

	// Now create the editor graph node
	FGraphNodeCreator<UMoviePipelineEdGraphNode> NodeCreator(*ParentGraph);

	// Define the ed graph node type here if it differs from UMoviePipelineEdGraphNode
	// If other ed node class types are needed here,
	// we should let ed nodes declare their equivalent runtime node,
	// and use that mapping to determine the applicable ed node type rather than hard-coding.
	TSubclassOf<UMoviePipelineEdGraphNode> InvokableEdGraphNodeClass = UMoviePipelineEdGraphNode::StaticClass();
	if (RuntimeNode->IsA(UMovieGraphSubgraphNode::StaticClass()))
	{
		InvokableEdGraphNodeClass = UMoviePipelineEdGraphSubgraphNode::StaticClass();
	}
	else if (RuntimeNode->IsA(UMovieGraphRerouteNode::StaticClass()))
	{
		InvokableEdGraphNodeClass = UMoviePipelineEdGraphRerouteNode::StaticClass();
	}
	
	UMoviePipelineEdGraphNode* GraphNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode, InvokableEdGraphNodeClass);
	GraphNode->Construct(RuntimeNode);
	GraphNode->NodePosX = Location.X;
	GraphNode->NodePosY = Location.Y;


	// Finalize generates a guid, calls a post-place callback, and allocates default pins if needed
	NodeCreator.Finalize();

	if (FromPin)
	{
		GraphNode->AutowireNewNode(FromPin);
	}
	return GraphNode;
}

FMovieGraphSchemaAction_NewVariableNode::FMovieGraphSchemaAction_NewVariableNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableGuid, FText InToolTip)
	: FMovieGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InDisplayName), MoveTemp(InToolTip), 0)
	, VariableGuid(InVariableGuid)
{
	
}

UEdGraphNode* FMovieGraphSchemaAction_NewVariableNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UMovieGraphConfig* RuntimeGraph = CastChecked<UMoviePipelineEdGraph>(ParentGraph)->GetPipelineGraph();
	const FScopedTransaction Transaction(LOCTEXT("GraphEditor_NewVariableNode", "Add New Variable Accessor Node"));
	RuntimeGraph->Modify();
	ParentGraph->Modify();

	UMovieGraphNode* RuntimeNode = RuntimeGraph->ConstructRuntimeNode<UMovieGraphNode>(NodeClass);
	if (UMovieGraphVariableNode* VariableNode = Cast<UMovieGraphVariableNode>(RuntimeNode))
	{
		VariableNode->SetVariable(RuntimeGraph->GetVariableByGuid(VariableGuid));
	}

	// Now create the variable node
	FGraphNodeCreator<UMoviePipelineEdGraphVariableNode> NodeCreator(*ParentGraph);
	UMoviePipelineEdGraphVariableNode* GraphNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode);
	GraphNode->Construct(RuntimeNode);
	GraphNode->NodePosX = Location.X;
	GraphNode->NodePosY = Location.Y;
	
	// Finalize generates a guid, calls a post-place callback, and allocates default pins if needed
	NodeCreator.Finalize();

	if (FromPin)
	{
		GraphNode->AutowireNewNode(FromPin);
	}

	return GraphNode;
}

UEdGraphNode* FMovieGraphSchemaAction_NewComment::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UEdGraphNode_Comment* CommentTemplate = NewObject<UEdGraphNode_Comment>();

	const TSharedPtr<SGraphEditor> GraphEditorPtr = SGraphEditor::FindGraphEditorForGraph(ParentGraph);

	FVector2f SpawnLocation = Location;
	FSlateRect Bounds;
	if (GraphEditorPtr && GraphEditorPtr->GetBoundsForSelectedNodes(Bounds, 50.0f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}
	
	const FScopedTransaction Transaction(LOCTEXT("GraphEditor_NewCommentNode", "Add New Comment Node"));
	ParentGraph->Modify();
	
	return FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(ParentGraph, CommentTemplate, SpawnLocation, bSelectNewNode);;
}

#undef LOCTEXT_NAMESPACE // "MoviePipelineGraphSchema"
