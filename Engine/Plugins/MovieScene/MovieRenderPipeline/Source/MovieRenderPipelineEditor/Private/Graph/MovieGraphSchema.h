// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "EdGraph/EdGraphSchema.h"
#include "Templates/SubclassOf.h"
#include "MovieGraphSchema.generated.h"

class UMovieGraphNode;


UCLASS()
class UMovieGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	//~ Begin EdGraphSchema Interface
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool TryCreateConnection(UEdGraphPin* InA, UEdGraphPin* InB) const override;
	static FLinearColor GetTypeColor(const FName& InPinCategory, const FName& InPinSubCategory);
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	virtual bool SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const override;
	virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	virtual bool SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const override;
	virtual UEdGraphPin* DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const override;
	virtual void SetPinBeingDroppedOnNode(UEdGraphPin* InSourcePin) const override;
	virtual bool SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const override;
	virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override;
	virtual bool IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const override;
	virtual int32 GetCurrentVisualizationCacheID() const override;
	virtual void ForceVisualizationCacheClear() const override;
	//~ End EdGraphSchema Interface
	
	static void InitMoviePipelineNodeClasses();

	/**
	 * Gets all node classes that the graph is aware of and are usable within the graph (including those from plugins).
	 * Should only be called after all plugins that contain nodes have been loaded.
	 */
	static const TArray<UClass*>& GetNodeClasses();

private:
	/**
	 * Determines if the connection between InputPin and OutputPin follows branch restriction rules. OutError is populated
	 * with an error if the connection should be rejected and the function will return false.
	 */
	bool IsConnectionToBranchAllowed(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin, FText& OutError) const;

	/**
	 * Adds extra menu actions to the context/palette menu.
	 */
	void AddExtraMenuActions(FGraphActionMenuBuilder& ActionMenuBuilder) const;

	/**
	 * Returns a menu action for creating a new comment in the graph.
	 * Note that this is not the same as adding a new comment to the graph via hotkey.
	 */
	TSharedRef<FMovieGraphSchemaAction_NewComment> CreateCommentMenuAction() const;

public:
	// Allowed "PinCategory" values for use on EdGraphPin
	static const FName PC_Branch;
	static const FName PC_Wildcard;
	static const FName PC_Boolean;
	static const FName PC_Byte;
	static const FName PC_Integer;
	static const FName PC_Int64;
	static const FName PC_Real;
	static const FName PC_Float;
	static const FName PC_Double;
	static const FName PC_Name;
	static const FName PC_String;
	static const FName PC_Text;
	static const FName PC_Enum;
	static const FName PC_Struct;
	static const FName PC_Object;
	static const FName PC_SoftObject;
	static const FName PC_Class;
	static const FName PC_SoftClass;
	
private:
	static TArray<UClass*> MoviePipelineNodeClasses;

	/** Counter incremented when cached visualization data (tooltips, titles, etc.) should be refreshed. */
	mutable int32 CurrentCacheRefreshID = 0;

	/** The pin that is being dropped on another node during a drag-drop operation. Controlled by SupportsDropPinOnNode(). */
	mutable UEdGraphPin* PinBeingDropped = nullptr;
};

/** Base class for schema actions in the graph. */
USTRUCT()
struct FMovieGraphSchemaAction : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;

	virtual ~FMovieGraphSchemaAction() override = default;
	
	static FName StaticGetTypeId()
	{
		static FName Type("FMovieGraphSchemaAction");
		return Type;
	}

	static const FText UserVariablesCategory;
	static const FText GlobalVariablesCategory;

	virtual void MovePersistentItemToCategory(const FText& NewCategoryName) override;
	virtual bool ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction) override;

	/** The object the action relates to. */
	UPROPERTY()
	TObjectPtr<UObject> ActionTarget = nullptr;

	UPROPERTY()
	TSubclassOf<UMovieGraphNode> NodeClass;
};

/** Schema action for creating a new node in the graph. */
USTRUCT()
struct FMovieGraphSchemaAction_NewNode : public FMovieGraphSchemaAction
{
	GENERATED_BODY()

	FMovieGraphSchemaAction_NewNode()
		: FMovieGraphSchemaAction()
	{}

	FMovieGraphSchemaAction_NewNode(FText InNodeCategory, FText InDisplayName, FText InToolTip, int32 InGrouping, FText InKeywords);

	virtual ~FMovieGraphSchemaAction_NewNode() override = default;
	
	static FName StaticGetTypeId()
	{
		static FName Type("FMovieGraphSchemaAction_NewNode");
		return Type;
	}
	
	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Schema action for creating a new variable node in the graph. */
USTRUCT()
struct FMovieGraphSchemaAction_NewVariableNode : public FMovieGraphSchemaAction
{
	GENERATED_BODY()
	
	static FName StaticGetTypeId()
	{
		static FName Type("FMovieGraphSchemaAction_NewVariableNode");
		return Type;
	}

	FMovieGraphSchemaAction_NewVariableNode()
		: FMovieGraphSchemaAction()
	{}

	FMovieGraphSchemaAction_NewVariableNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableGuid, FText InToolTip);

	virtual ~FMovieGraphSchemaAction_NewVariableNode() override = default;
	
	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

private:
	/** GUID of the runtime variable this action relates to. */
	UPROPERTY()
	FGuid VariableGuid;
};

/** Schema action for creating a new comment node in the graph. */
USTRUCT()
struct FMovieGraphSchemaAction_NewComment : public FMovieGraphSchemaAction
{
	GENERATED_BODY()

	static FName StaticGetTypeId()
	{
		static FName Type("FMovieGraphSchemaAction_NewComment");
		return Type;
	}

	// Inherit the base class's constructors for context menu/palette
	using FMovieGraphSchemaAction::FMovieGraphSchemaAction;

	FMovieGraphSchemaAction_NewComment()
		: FMovieGraphSchemaAction()
	{}

	//~ Begin FEdGraphSchemaAction Interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};