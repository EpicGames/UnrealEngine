// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineEdGraphSubgraphNode.h"

#include "Graph/Nodes/MovieGraphSubgraphNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineEdGraphSubgraphNode)

UObject* UMoviePipelineEdGraphSubgraphNode::GetJumpTargetForDoubleClick() const
{
	if (const UMovieGraphSubgraphNode* SubgraphNode = Cast<UMovieGraphSubgraphNode>(RuntimeNode))
	{
		if (UMovieGraphConfig* SubgraphAsset =  SubgraphNode->GetSubgraphAsset())
		{
			return SubgraphAsset;
		}
	}
	
	return Super::GetJumpTargetForDoubleClick();
}
