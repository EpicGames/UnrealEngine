// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieEdGraphNode.h"

#include "MoviePipelineEdGraphSubgraphNode.generated.h"


UCLASS()
class UMoviePipelineEdGraphSubgraphNode : public UMoviePipelineEdGraphNode
{
	GENERATED_BODY()

public:
	// ~Begin UEdGraphNode interface
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	// ~End UEdGraphNode interface
};
