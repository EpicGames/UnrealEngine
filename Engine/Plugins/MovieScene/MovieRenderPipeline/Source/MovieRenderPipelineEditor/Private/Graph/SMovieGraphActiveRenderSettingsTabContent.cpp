// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMovieGraphActiveRenderSettingsTabContent.h"

#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/Nodes/MovieGraphRenderPassNode.h"
#include "IContentBrowserSingleton.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineQueueSubsystem.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineActiveRenderSettingsTabContent"

const FName SMovieGraphActiveRenderSettingsTreeItem::ColumnID_Name(TEXT("Name"));
const FName SMovieGraphActiveRenderSettingsTreeItem::ColumnID_Value(TEXT("Value"));

const FName FActiveRenderSettingsTreeElement::RootName_Globals(TEXT("Globals"));
const FName FActiveRenderSettingsTreeElement::RootName_Branches(TEXT("Branches"));

FActiveRenderSettingsTreeElement::FActiveRenderSettingsTreeElement(const FName& InName, const EElementType InType)
	: Name(InName)
	, Type(InType)
{
}

FString FActiveRenderSettingsTreeElement::GetValue() const
{
	FString ValueString;
	
	if (!SettingsProperty || !SettingsNode)
	{
		return ValueString;
	}

	// If the property is a dynamic property, then get its value from the node
	for (const FPropertyBagPropertyDesc& PropertyDesc : SettingsNode->GetDynamicPropertyDescriptions())
	{
		if (SettingsProperty->GetFName() == PropertyDesc.Name)
		{
			SettingsNode->GetDynamicPropertyValue(PropertyDesc.Name, ValueString);
			return ValueString;
		}
	}

	// If the property implements IMovieGraphTraversableObject, get the value via GetMergedProperties().
	// Value will be formatted as a newline-delimited list of "PropertyName = PropertyValue" strings.
	const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(SettingsProperty);
	if (ObjectProperty && ObjectProperty->PropertyClass->ImplementsInterface(UMovieGraphTraversableObject::StaticClass()))
	{
		const IMovieGraphTraversableObject* MergeableProperty = Cast<IMovieGraphTraversableObject>(
			ObjectProperty->GetObjectPropertyValue(ObjectProperty->ContainerPtrToValuePtr<void>(SettingsNode)));
		
		TArray<FString> MergedValues;
		Algo::Transform(MergeableProperty->GetMergedProperties(), MergedValues, [](const TPair<FString, FString>& Pair)
		{
			return FString::Format(TEXT("{0} = {1}"), {Pair.Key, Pair.Value});
		});
		
		return FString::Join(MergedValues, TEXT("\n"));
	}

	// Otherwise, ask the property for its value directly
	SettingsProperty->ExportTextItem_InContainer(ValueString, SettingsNode, nullptr, nullptr, PPF_None);
	return ValueString;
}

bool FActiveRenderSettingsTreeElement::IsBranchRenderable() const
{
	if (Type != EElementType::NamedBranch)
	{
		return false;
	}

	// The branch is renderable if there's a render pass node in the branch's children
	for (const TSharedPtr<FActiveRenderSettingsTreeElement>& Child : GetChildren())
	{
		if (Child.IsValid() && Child->SettingsNode && Child->SettingsNode->IsA<UMovieGraphRenderPassNode>())
		{
			return true;
		}
	}

	return false;
}

const TArray<TSharedPtr<FActiveRenderSettingsTreeElement>>& FActiveRenderSettingsTreeElement::GetChildren() const
{
	static const FName HideInActiveRenderSettingsMetaDataKey(TEXT("HideInActiveRenderSettings"));
	
	auto MakeElementFromNode = [this](const TObjectPtr<UMovieGraphNode>& Node) -> TSharedPtr<FActiveRenderSettingsTreeElement>
	{
		// The element name should include the node's instance name (if the instance name isn't empty)
		FString ElementName = Node->GetNodeTitle().ToString();
		if (const UMovieGraphSettingNode* SettingNode = Cast<UMovieGraphSettingNode>(Node))
		{
			const FString InstanceName = SettingNode->GetNodeInstanceName();
			if (!InstanceName.IsEmpty())
			{
				ElementName = FString::Format(TEXT("{0} - {1}"), {ElementName, InstanceName});
			}
		}
		
		TSharedPtr<FActiveRenderSettingsTreeElement> Element =
			MakeShared<FActiveRenderSettingsTreeElement>(FName(*ElementName), EElementType::Node);
		Element->SettingsNode = Node;
		Element->FlattenedGraph = FlattenedGraph;
		Element->ParentElement = AsWeak();

		return MoveTemp(Element);
	};

	// Returned the cached children if they are available
	if (!ChildrenCache.IsEmpty())
	{
		return ChildrenCache;
	}
	
	if (!FlattenedGraph.IsValid())
	{
		// This should be an empty array
		return ChildrenCache;
	}

	// For the root "Globals" element: all nodes which were found in the Globals branch should be children
	if ((Name == RootName_Globals) && (Type == EElementType::Root))
	{
		if (const FMovieGraphEvaluatedBranchConfig* BranchConfig = FlattenedGraph->BranchConfigMapping.Find(RootName_Globals))
		{
			for (const TObjectPtr<UMovieGraphNode>& Node : BranchConfig->GetNodes())
			{
				ChildrenCache.Add(MakeElementFromNode(Node));
			}
		}

		return ChildrenCache;
	}

	// For the root "Branches" element: all branches which were found should be children
	if ((Name == RootName_Branches) && (Type == EElementType::Root))
	{
		for (const auto& Pair : FlattenedGraph->BranchConfigMapping)
		{
			if (Pair.Key != RootName_Globals)
			{
				TSharedPtr<FActiveRenderSettingsTreeElement> Element =
					MakeShared<FActiveRenderSettingsTreeElement>(FName(Pair.Key), EElementType::NamedBranch);
				Element->FlattenedGraph = FlattenedGraph;
				Element->ParentElement = AsWeak();
				
				ChildrenCache.Add(MoveTemp(Element));
			}
		}

		return ChildrenCache;
	}

	// For named branch elements: all the nodes under the branch should be children
	if (Type == EElementType::NamedBranch)
	{
		const FMovieGraphEvaluatedBranchConfig* BranchConfig = FlattenedGraph->BranchConfigMapping.Find(Name);
		if (!BranchConfig)
		{
			return ChildrenCache;
		}

		for (const TObjectPtr<UMovieGraphNode>& Node : BranchConfig->GetNodes())
		{
			ChildrenCache.Add(MakeElementFromNode(Node));
		}

		return ChildrenCache;
	}

	// For node elements: all overrideable properties on the node should be children
	if (Type == EElementType::Node)
	{
		for (const FProperty* Property : SettingsNode->GetAllOverrideableProperties())
		{
			// Some properties opt-out of being shown in the Active Render Settings editor
			if (Property->HasMetaData(HideInActiveRenderSettingsMetaDataKey))
			{
				continue;
			}
			
			TSharedPtr<FActiveRenderSettingsTreeElement> Element =
				MakeShared<FActiveRenderSettingsTreeElement>(FName(Property->GetDisplayNameText().ToString()), EElementType::Property);
			Element->SettingsNode = SettingsNode;
			Element->SettingsProperty = Property;
			Element->FlattenedGraph = FlattenedGraph;
			Element->ParentElement = AsWeak();
			
			ChildrenCache.Add(MoveTemp(Element));
		}
	}
	
	return ChildrenCache;
}

void FActiveRenderSettingsTreeElement::ClearCachedChildren() const
{
	ChildrenCache.Empty();
}

uint32 FActiveRenderSettingsTreeElement::GetHash() const
{
	if (ElementHash != 0)
	{
		return ElementHash;
	}

	ElementHash = GetTypeHash(Name);
	if (ParentElement.IsValid())
	{
		ElementHash = HashCombine(ElementHash, ParentElement.Pin()->GetHash());
	}

	return ElementHash;
}

void SMovieGraphActiveRenderSettingsTreeItem::Construct(
	const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const TSharedPtr<FActiveRenderSettingsTreeElement>& InTreeElement)
{
	WeakTreeElement = InTreeElement;

	SMultiColumnTableRow<TSharedPtr<FActiveRenderSettingsTreeElement>>::Construct(
		SMultiColumnTableRow<TSharedPtr<FActiveRenderSettingsTreeElement>>::FArguments()
			.Padding(0)	
		, InOwnerTable);
}

TSharedRef<SWidget> SMovieGraphActiveRenderSettingsTreeItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	const TSharedPtr<FActiveRenderSettingsTreeElement> TreeElement = WeakTreeElement.Pin();
	if (!TreeElement.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	
	if (ColumnName == ColumnID_Name)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
				.IndentAmount(16)
				.ShouldDrawWires(false)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 2.f, 0)
			[
				SNew(SImage)
				.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Texture2D").GetIcon())
				.ToolTipText(LOCTEXT("RenderableBranchTooltip", "This branch outputs rendered images."))
				.Visibility_Lambda([TreeElement]()
				{
					return TreeElement->IsBranchRenderable() ? EVisibility::Visible : EVisibility::Collapsed;
				})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(FText::FromName(TreeElement->Name))
				.Font_Lambda([TreeElement]()
				{
					switch (TreeElement->Type)
					{
					case FActiveRenderSettingsTreeElement::EElementType::Root:
						return FCoreStyle::GetDefaultFontStyle("Bold", 12);
					case FActiveRenderSettingsTreeElement::EElementType::NamedBranch:
						return FCoreStyle::GetDefaultFontStyle("Bold", 10);
					case FActiveRenderSettingsTreeElement::EElementType::Property:
						return FCoreStyle::GetDefaultFontStyle("Italic", 9);
					default:
						return FCoreStyle::GetDefaultFontStyle("Regular", 9);
					}
				})
				.ColorAndOpacity_Lambda([TreeElement]()
				{
					return (TreeElement->Type == FActiveRenderSettingsTreeElement::EElementType::Property)
						? FSlateColor::UseSubduedForeground()
						: FSlateColor::UseForeground();
				})
			];
	}

	if (ColumnName == ColumnID_Value)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(FText::FromString(TreeElement->GetValue()))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			];
	}

	return SNullWidget::NullWidget;
}

void SMovieGraphActiveRenderSettingsTabContent::Construct(const FArguments& InArgs)
{
	CurrentGraph = InArgs._Graph;

	const UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);

	// Initialize with the current queue (if there is one), and pick the first job by default (if a job exists)
	if (UMoviePipelineQueue* Queue = Subsystem->GetQueue())
	{
		// Note: The queue from the queue subsystem is probably transient
		TraversalQueue = Queue;

		const TArray<UMoviePipelineExecutorJob*>& Jobs = TraversalQueue->GetJobs();
		if (!Jobs.IsEmpty())
		{
			TraversalJob = Jobs[0];
		}
	}

	// Add the two default root elements, "Globals" and "Branches", which are always visible. These elements will
	// generate the children which should be displayed under them in the tree.
	RootElements.Add(MakeShared<FActiveRenderSettingsTreeElement>(
		FActiveRenderSettingsTreeElement::RootName_Globals, FActiveRenderSettingsTreeElement::EElementType::Root));
	RootElements.Add(MakeShared<FActiveRenderSettingsTreeElement>(
		FActiveRenderSettingsTreeElement::RootName_Branches, FActiveRenderSettingsTreeElement::EElementType::Root));
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPositiveActionButton)
				.Text(LOCTEXT("Button_EvaluateGraph", "Evaluate Graph"))
				.Icon(FAppStyle::GetBrush("Icons.Refresh"))
				.OnClicked(this, &SMovieGraphActiveRenderSettingsTabContent::OnEvaluateGraphClicked)
			]

			// +SHorizontalBox::Slot()
			// .Padding(5, 0, 0, 0)
			// .AutoWidth()
			// [
			// 	SNew(SPositiveActionButton)
			// 	.Text(LOCTEXT("Button_EvaluationContext", "Evaluation Context..."))
			// 	.Icon(FAppStyle::GetBrush("EditorPreferences.TabIcon"))
			// 	.OnGetMenuContent(this, &SMovieGraphActiveRenderSettingsTabContent::GenerateEvaluationContextMenu)
			// ]
		]

		+ SVerticalBox::Slot()
		.Padding(0, 10, 10, 0)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility_Lambda([this]()
			{
				return TraversalError.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
			})

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(10)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.ErrorWithColor"))
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				.Text_Lambda([this]()
				{
					return FText::FromString(TraversalError);
				})
			]
		]

		+ SVerticalBox::Slot()
		[
			SAssignNew(TreeView, STreeView<TSharedPtr<FActiveRenderSettingsTreeElement>>)
			.Visibility_Lambda([this]()
			{
				return TraversalError.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.TreeItemsSource(&RootElements)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(this, &SMovieGraphActiveRenderSettingsTabContent::GenerateTreeRow)
			.OnGetChildren(this, &SMovieGraphActiveRenderSettingsTabContent::GetChildrenForTree)
			.OnExpansionChanged(this, &SMovieGraphActiveRenderSettingsTabContent::OnExpansionChanged)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+ SHeaderRow::Column(SMovieGraphActiveRenderSettingsTreeItem::ColumnID_Name)
				.DefaultLabel(LOCTEXT("NameColumnLabel", "Name"))
				.SortMode(EColumnSortMode::None)
				.FillWidth(0.75f)

				+ SHeaderRow::Column(SMovieGraphActiveRenderSettingsTreeItem::ColumnID_Value)
				.DefaultLabel(LOCTEXT("ValueColumnLabel", "Value"))
				.SortMode(EColumnSortMode::None)
				.FillWidth(0.25f)
			)
			.Visibility_Lambda([this]()
			{
				return FlattenedGraph.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
			})
		]

		+ SVerticalBox::Slot()
		.Padding(5.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NeedsEvaluationWarning", "Render settings have not been evaluated yet."))
			.Font(FCoreStyle::GetDefaultFontStyle("Italic", 10))
			.Visibility_Lambda([this]()
			{
				// If there's a traversal error, only show the error and not this warning
				if (!TraversalError.IsEmpty())
				{
					return EVisibility::Collapsed;					
				}
				
				return FlattenedGraph.IsValid() ? EVisibility::Collapsed : EVisibility::Visible;
			})
		]
	];
}

void SMovieGraphActiveRenderSettingsTabContent::TraverseGraph()
{
	if (!CurrentGraph.IsValid())
	{
		return;
	}

	// Clear out the resolved cached children under the root nodes
	for (const TSharedPtr<FActiveRenderSettingsTreeElement>& RootElement : RootElements)
	{
		RootElement->ClearCachedChildren();
	}

	FMovieGraphTraversalContext Context;
	Context.Job = TraversalJob.Get();
	// Context.Shot = ?	// TODO: The shot should be exposed in the UI at some point

	// Traverse the graph, and update the root elements
	FlattenedGraph = TStrongObjectPtr(CurrentGraph->CreateFlattenedGraph(Context, TraversalError));
	if (!FlattenedGraph.IsValid())
	{
		// TraversalError was set, which will be picked up by the UI and displayed instead of the tree
		return;
	}
	
	for (const TSharedPtr<FActiveRenderSettingsTreeElement>& RootElement : RootElements)
	{
		RootElement->FlattenedGraph = MakeWeakObjectPtr(FlattenedGraph.Get());
	}
	
	TreeView->RequestTreeRefresh();

	// Restore tree expansion state after the refresh
	for (const TSharedPtr<FActiveRenderSettingsTreeElement>& RootElement : RootElements)
	{
		RestoreExpansionStateRecursive(RootElement);
	}
}

FReply SMovieGraphActiveRenderSettingsTabContent::OnEvaluateGraphClicked()
{
	TraverseGraph();
	return FReply::Handled();
}

TSharedRef<SWidget> SMovieGraphActiveRenderSettingsTabContent::GenerateEvaluationContextMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("JobContext_MenuSection", "Job Context"));
	{
		const TSharedRef<SWidget> JobPicker = SNew(SBox)
		.WidthOverride(400)
		.Padding(10)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(0.3f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("JobContext_QueueLabel", "Queue"))
				]

				+SHorizontalBox::Slot()
				.FillWidth(0.7f)
				[
					SNew(SPositiveActionButton)
					.Text(this, &SMovieGraphActiveRenderSettingsTabContent::GetQueueButtonText)
					.OnGetMenuContent(this, &SMovieGraphActiveRenderSettingsTabContent::MakeQueueButtonMenuContents)
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(0.3f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("JobContext_JobLabel", "Job"))
				]

				+SHorizontalBox::Slot()
				.FillWidth(0.7f)
				[
					SNew(SComboBox<TSharedPtr<FString>>)
					.IsEnabled_Lambda([this]() { return TraversalQueue != nullptr; })
					.OptionsSource(&AvailableJobsInQueue)
					.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
					{
						return SNew(STextBlock)
							.Text(FText::FromString(*Item));
					})
					.OnSelectionChanged(this, &SMovieGraphActiveRenderSettingsTabContent::HandleJobSelected)
					[
						SNew(STextBlock)
						.MinDesiredWidth(50.f)
						.Text_Lambda([this]()
						{
							if (TraversalJob.IsValid())
							{
								return FText::FromString(TraversalJob->JobName);
							}

							return LOCTEXT("JobContext_NoJobSelected", "No job selected");
						})
					]
				]
			]
		];

		MenuBuilder.AddWidget(JobPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SMovieGraphActiveRenderSettingsTabContent::OnExpansionChanged(TSharedPtr<FActiveRenderSettingsTreeElement> InElement, bool bInExpanded)
{
	if (bInExpanded)
	{
		ExpandedElements.Add(InElement->GetHash());
	}
	else
	{
		ExpandedElements.Remove(InElement->GetHash());
	}
}

void SMovieGraphActiveRenderSettingsTabContent::RestoreExpansionStateRecursive(const TSharedPtr<FActiveRenderSettingsTreeElement>& InElement)
{
	TreeView->SetItemExpansion(InElement, ExpandedElements.Contains(InElement->GetHash()));
	
	for (const TSharedPtr<FActiveRenderSettingsTreeElement>& ChildElement : InElement->GetChildren())
	{
		RestoreExpansionStateRecursive(ChildElement);
	}
}

TSharedRef<ITableRow> SMovieGraphActiveRenderSettingsTabContent::GenerateTreeRow(
	TSharedPtr<FActiveRenderSettingsTreeElement> InTreeElement, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMovieGraphActiveRenderSettingsTreeItem, OwnerTable, InTreeElement)
		.Visibility_Lambda([InTreeElement]()
		{
			// If the root element has no children, hide it
			if (InTreeElement->Type == FActiveRenderSettingsTreeElement::EElementType::Root)
			{
				if (InTreeElement->GetChildren().IsEmpty())
				{
					return EVisibility::Hidden;
				}
			}

			return EVisibility::Visible;
		});
}

void SMovieGraphActiveRenderSettingsTabContent::GetChildrenForTree(
	TSharedPtr<FActiveRenderSettingsTreeElement> InItem, TArray<TSharedPtr<FActiveRenderSettingsTreeElement>>& OutChildren)
{
	OutChildren.Append(InItem->GetChildren());
}

FText SMovieGraphActiveRenderSettingsTabContent::GetQueueButtonText() const
{
	if (!TraversalQueue.IsValid())
	{
		return LOCTEXT("JobContext_PickAQueue", "Pick a queue...");
	}
	
	// If the queue has an origin queue, use the origin's name. A queue will have a defined origin
	// if the queue is coming from the queue subsystem (since the subsystem usually points to a transient
	// queue which is being displayed in the queue UI).
	if (const UMoviePipelineQueue* QueueOrigin = TraversalQueue->GetQueueOrigin())
	{
		return FText::FromString(QueueOrigin->GetName());
	}

	// If the queue doesn't have an origin, this queue was probably picked here in the graph. If it has a
	// non-transient package, it has been saved at some point, so we can use the queue name.
	const UPackage* Package = TraversalQueue->GetPackage();
	if (Package && !Package->HasAnyPackageFlags(PKG_TransientFlags))
	{
		return FText::FromString(TraversalQueue->GetName());
	}

	// This is most likely an unsaved transient queue from the queue UI
	return LOCTEXT("JobContext_UnnamedQueue", "Unnamed queue");
}

TSharedRef<SWidget> SMovieGraphActiveRenderSettingsTabContent::MakeQueueButtonMenuContents()
{
	FMenuBuilder MenuBuilder(true, nullptr);
    IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

    FAssetPickerConfig AssetPickerConfig;
    {
		// If a transient queue (with a queue origin) is active, select the origin queue. The origin queue will be saved
		// so it will properly show up as selected in the asset picker (whereas a transient queue will not).
		AssetPickerConfig.InitialAssetSelection = (TraversalQueue.IsValid() && TraversalQueue->GetQueueOrigin())
			? FAssetData(TraversalQueue->GetQueueOrigin())
			: FAssetData(TraversalQueue.Get());
		
    	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
    	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
    	AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
    	AssetPickerConfig.bAllowNullSelection = false;
    	AssetPickerConfig.bShowBottomToolbar = true;
    	AssetPickerConfig.bAutohideSearchBar = false;
    	AssetPickerConfig.bAllowDragging = false;
    	AssetPickerConfig.bCanShowClasses = false;
    	AssetPickerConfig.bShowPathInColumnView = false;
    	AssetPickerConfig.bShowTypeInColumnView = false;
    	AssetPickerConfig.bSortByPathInColumnView = false;
    	AssetPickerConfig.SaveSettingsName = TEXT("MovieRenderGraphTraversalContextQueue");
    	AssetPickerConfig.AssetShowWarningText = LOCTEXT("JobContext_NoQueuesWarning", "No Queues Found");
    	AssetPickerConfig.Filter.ClassPaths.Add(UMoviePipelineQueue::StaticClass()->GetClassPathName());
    	AssetPickerConfig.OnAssetSelected.BindLambda([this](const FAssetData& InAsset)
    	{
    		TraversalQueue = Cast<UMoviePipelineQueue>(InAsset.GetAsset());
    		if (TraversalQueue.IsValid())
    		{
    			const UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
    			check(Subsystem);

    			// If the selected queue matches the current queue being edited in the queue UI,
    			// use the transient queue from the queue UI. This will let the user edit the queue
    			// without saving, and have those updates reflected in the traversal.
    			UMoviePipelineQueue* CurrentQueue = Subsystem->GetQueue();
    			if (CurrentQueue && (CurrentQueue->GetQueueOrigin() == TraversalQueue))
    			{
    				TraversalQueue = CurrentQueue;
    			}

    			// Reset job selection state
    			AvailableJobsInQueue.Empty();
    			TraversalJob = nullptr;
    			Algo::Transform(TraversalQueue->GetJobs(), AvailableJobsInQueue, [](const UMoviePipelineExecutorJob* Job)
    			{
    				return MakeShared<FString>(Job->JobName);
    			});
    		}

    		if (QueuePickerWidget.IsValid())
    		{
    			FSlateApplication::Get().DismissMenuByWidget(QueuePickerWidget.ToSharedRef());
    		}
    	});
    }

    SAssignNew(QueuePickerWidget, SBox)
    	.WidthOverride(300.f)
    	.HeightOverride(300.f)
    	[
    		ContentBrowser.CreateAssetPicker(AssetPickerConfig)
    	];

    MenuBuilder.AddWidget(QueuePickerWidget.ToSharedRef(), FText(), true, false);

    return MenuBuilder.MakeWidget();
}

void SMovieGraphActiveRenderSettingsTabContent::HandleJobSelected(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
{
	// Find the associated job object
	for (UMoviePipelineExecutorJob* Job : TraversalQueue->GetJobs())
	{
		if (Job && (Job->JobName == *Item))
		{
			TraversalJob = Job;
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
