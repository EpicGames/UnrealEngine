// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphConfig.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

class UMoviePipelineExecutorJob;
class UMoviePipelineQueue;
class UMovieGraphNode;

/**
 * Represents the data of one element displayed in the tree, which is one of the types in EElementType. Provides a way
 * of getting the children under the element.
 */
class FActiveRenderSettingsTreeElement : public TSharedFromThis<FActiveRenderSettingsTreeElement>
{
public:
	/** The type of this element. */
	enum class EElementType : uint8
	{
		Root,			///< A root element; does not represent any actual data within the graph
		NamedBranch,	///< Represents a named branch (output) within the graph
		Node,			///< Represents a settings node within the graph
		Property		///< Represents a property under an element with type EElementType::Node
	};
	
	static const FName RootName_Globals;
	static const FName RootName_Branches;
	
	FActiveRenderSettingsTreeElement(const FName& InName, const EElementType InType);

	/** If this element represents a property, returns the string representation of the value. Otherwise returns an empty string. */
	FString GetValue() const;

	/** Determines if this element is a branch, and the branch has renderable output. */
	bool IsBranchRenderable() const;

	/**
	 * Gets the child elements nested under this element. Returns a cached result if available, otherwise calculates the
	 * children and returns the result. Call ClearCachedChildren() if the cache needs to be cleared (eg, when the tree
	 * is being refreshed).
	 */
	const TArray<TSharedPtr<FActiveRenderSettingsTreeElement>>& GetChildren() const;

	/** Clears the cached result of GetChildren(). */
	void ClearCachedChildren() const;

	/** Gets the hash that uniquely identifies this element in the tree. */
	uint32 GetHash() const;

public:
	/** The name of the element as it appears in the tree. */
	FName Name;

	/** The type of item in the tree this element represents. */
	EElementType Type = EElementType::Root;

	/**
	 * If this element represents a node in the graph, this is the pointer to that node. If the element represents a
	 * property, this is the pointer to the owning node.
	 */
	TObjectPtr<UMovieGraphNode> SettingsNode = nullptr;

	/** If this element represents a property on a node, this is the pointer to that property. */
	const FProperty* SettingsProperty = nullptr;

	/** The result of the graph traversal. */
	TWeakObjectPtr<UMovieGraphEvaluatedConfig> FlattenedGraph = nullptr;

	/** This element's parent element in the tree.  */
	TWeakPtr<const FActiveRenderSettingsTreeElement> ParentElement = nullptr;

private:
	/** The (cached) child elements nested under this element. */
	mutable TArray<TSharedPtr<FActiveRenderSettingsTreeElement>> ChildrenCache;

	/** The (cached) hash of this element. */
	mutable uint32 ElementHash = 0;
};

/**
 * The widget that is responsible for generating the per-column content for each element in the tree.
 */
class SMovieGraphActiveRenderSettingsTreeItem : public SMultiColumnTableRow<TSharedPtr<FActiveRenderSettingsTreeElement>>
{
	SLATE_BEGIN_ARGS(SMovieGraphActiveRenderSettingsTreeItem) {}
	SLATE_END_ARGS()

	static const FName ColumnID_Name;
	static const FName ColumnID_Value;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable,
		const TSharedPtr<FActiveRenderSettingsTreeElement>& InTreeElement);

	//~ Begin SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	//~ End SMultiColumnTableRow overrides

private:
	/** The element that this tree item is displaying. */
	TWeakPtr<FActiveRenderSettingsTreeElement> WeakTreeElement;
};

/**
 * Contents of the "Active Render Settings" tab in the graph asset editor.
 */
class SMovieGraphActiveRenderSettingsTabContent : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMovieGraphActiveRenderSettingsTabContent)
		: _Graph(nullptr)
	
		{}
		
		/** The graph that is currently displayed. */
		SLATE_ARGUMENT(class UMovieGraphConfig*, Graph)
	
	SLATE_END_ARGS();
	
	void Construct(const FArguments& InArgs);

private:
	/** Traverses the graph. Populates the FlattenedGraph member and refreshes the UI view. */
	void TraverseGraph();

	/** Handles the button click for evaluating the graph. */
	FReply OnEvaluateGraphClicked();

	/** Generates the menu for the Evaluation Context button. */
	TSharedRef<SWidget> GenerateEvaluationContextMenu();

	/** Handles the expansion state change of an element in the tree. */
	void OnExpansionChanged(TSharedPtr<FActiveRenderSettingsTreeElement> InElement, bool bInExpanded);

	/** Recursively restores the cached expansion state of the tree, starting at the specified element. */
	void RestoreExpansionStateRecursive(const TSharedPtr<FActiveRenderSettingsTreeElement>& InElement);

	/** Generates a row in the tree widget based on the provided element. */
	TSharedRef<ITableRow> GenerateTreeRow(TSharedPtr<FActiveRenderSettingsTreeElement> InTreeElement, const TSharedRef<STableViewBase>& OwnerTable);

	/** Gets the child elements in the tree for the provided element. */
	void GetChildrenForTree(TSharedPtr<FActiveRenderSettingsTreeElement> InItem, TArray<TSharedPtr<FActiveRenderSettingsTreeElement>>& OutChildren);

	/** Gets the text that is displayed on the queue picker button (in the evaluation context menu). */
	FText GetQueueButtonText() const;

	/** Generates the menu contents for the queue picker button. Populates the QueuePickerWidget member. */
	TSharedRef<SWidget> MakeQueueButtonMenuContents();

	/** Handles a new job selection in the evaluation context menu. */
	void HandleJobSelected(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo);

private:
	/** The runtime graph that this UI gets data from. */
	TWeakObjectPtr<UMovieGraphConfig> CurrentGraph = nullptr;

	/** The main tree widget displayed in the tab. */
	TSharedPtr<STreeView<TSharedPtr<FActiveRenderSettingsTreeElement>>> TreeView = nullptr;

	/** The hashes of elements that are currently expanded in the tree. */
	TSet<uint32> ExpandedElements;

	/** The last result of a graph traversal. */
	TStrongObjectPtr<UMovieGraphEvaluatedConfig> FlattenedGraph = nullptr;

	/** The root-most elements in the tree, which are always present. */
	TArray<TSharedPtr<FActiveRenderSettingsTreeElement>> RootElements;

	/** The widget that houses the queue asset picker. */
	TSharedPtr<SWidget> QueuePickerWidget = nullptr;

	/** The queue that is selected in the evaluation context settings. */
	TWeakObjectPtr<UMoviePipelineQueue> TraversalQueue = nullptr;

	/** The job that is selected in the evaluation context settings. */
	TWeakObjectPtr<UMoviePipelineExecutorJob> TraversalJob = nullptr;

	/** The jobs that are available to be selected in the evaluation context settings. */
	TArray<TSharedPtr<FString>> AvailableJobsInQueue;

	/** An error that was set during traversal. This will show up in the UI and prevent the tree from being shown. */
	FString TraversalError;
};
