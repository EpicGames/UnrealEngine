// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMovieGraphMembersTabContent.h"

#include "EdGraph/EdGraphSchema.h"
#include "Framework/Commands/GenericCommands.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphSchema.h"
#include "Graph/Nodes/MovieGraphVariableNode.h"
#include "MovieEdGraphNode.h"
#include "ScopedTransaction.h"
#include "SGraphActionMenu.h"
#include "SGraphPalette.h"
#include "Toolkits/AssetEditorToolkit.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineMembersTabContent"

namespace UE::MovieGraph::Private
{
	/** Gets a graph member from a FEdGraphSchemaAction. */
	UMovieGraphMember* GetMemberFromAction(FEdGraphSchemaAction* SchemaAction)
	{
		if (!SchemaAction)
		{
			return nullptr;
		}

		const FMovieGraphSchemaAction* SelectedGraphAction = static_cast<FMovieGraphSchemaAction*>(SchemaAction);

		// Currently the action menu only includes UMovieGraphMember targets
		if (UMovieGraphMember* SelectedMember = Cast<UMovieGraphMember>(SelectedGraphAction->ActionTarget))
		{
			return SelectedMember;
		}

		return nullptr;
	}

	/** Gets all selected graph members of the specified MemberType within the action menu. */
	template<typename MemberType>
	TArray<MemberType*> GetAllSelectedMembers(const TSharedPtr<SGraphActionMenu> InActionMenu)
	{
		TArray<MemberType*> SelectedMembers;
		
		if (!InActionMenu.IsValid())
		{
			return SelectedMembers;
		}

		TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
		InActionMenu->GetSelectedActions(SelectedActions);
	
		for (TSharedPtr<FEdGraphSchemaAction> SelectedAction : SelectedActions)
		{
			if (UMovieGraphMember* GraphMember = GetMemberFromAction(SelectedAction.Get()))
			{
				if (MemberType* Member = Cast<MemberType>(GraphMember))
				{
					SelectedMembers.Add(Member);
				}
			}
		}

		return SelectedMembers;
	}

	static bool GetIconAndColorFromDataType(const UMovieGraphMember* InGraphMember, const FSlateBrush*& OutPrimaryBrush, FSlateColor& OutIconColor, const FSlateBrush*& OutSecondaryBrush, FSlateColor& OutSecondaryColor)
	{
		if (!InGraphMember)
		{
			return false;
		}
		
		const UMovieGraphInterfaceBase* InterfaceBase = Cast<UMovieGraphInterfaceBase>(InGraphMember);
		const bool bIsBranch = InterfaceBase ? InterfaceBase->bIsBranch : false;
		constexpr bool bIsWildcard = false;
		const FEdGraphPinType PinType = UMoviePipelineEdGraphNodeBase::GetPinType(InGraphMember->GetValueType(), bIsBranch, bIsWildcard, InGraphMember->GetValueTypeObject());

		OutPrimaryBrush = FAppStyle::GetBrush("Kismet.AllClasses.VariableIcon");
		OutIconColor = GetDefault<UMovieGraphSchema>()->GetPinTypeColor(PinType);
		OutSecondaryBrush = nullptr;
		OutSecondaryColor = FSlateColor(FLinearColor::Transparent);
		
		return true;
	}
}

const TArray<FText> SMovieGraphMembersTabContent::ActionMenuSectionNames
{
	LOCTEXT("ActionMenuSectionName_Invalid", "INVALID"),
	LOCTEXT("ActionMenuSectionName_Inputs", "Inputs"),
	LOCTEXT("ActionMenuSectionName_Outputs", "Outputs"),
	LOCTEXT("ActionMenuSectionName_Variables", "Variables")
};

void SMovieGraphMembersTabContent::Construct(const FArguments& InArgs)
{
	EditorToolkit = InArgs._Editor;
	CurrentGraph = InArgs._Graph;
	OnActionSelected = InArgs._OnActionSelected;

	// Update the UI whenever the graph adds/updates variables. In this case it's not known which variable is added/updated, so just pass nullptr.
	UMovieGraphMember* UpdatedMember = nullptr;
	CurrentGraph->OnGraphVariablesChangedDelegate.AddSP(this, &SMovieGraphMembersTabContent::RefreshMemberActions, UpdatedMember);

	// Also update the UI when inputs/outputs are added to the graph.
	CurrentGraph->OnGraphInputAddedDelegate.AddSPLambda(this, [this](UMovieGraphInput* InInput)
	{
		RefreshMemberActions(InInput);
	});

	CurrentGraph->OnGraphOutputAddedDelegate.AddSPLambda(this, [this](UMovieGraphOutput* InOutput)
	{
		RefreshMemberActions(InOutput);
	});
	
	ChildSlot
	[
		SAssignNew(ActionMenu, SGraphActionMenu)
		.OnActionSelected(OnActionSelected)
		.AutoExpandActionMenu(true)
		.AlphaSortItems(false)
		.OnActionDragged(this, &SMovieGraphMembersTabContent::OnActionDragged)
		.OnCategoryDragged(this, &SMovieGraphMembersTabContent::OnCategoryDragged)
		.OnCreateWidgetForAction(this, &SMovieGraphMembersTabContent::CreateActionWidget)
		.OnCollectStaticSections(this, &SMovieGraphMembersTabContent::CollectStaticSections)
		.OnContextMenuOpening(this, &SMovieGraphMembersTabContent::OnContextMenuOpening)
		.OnGetSectionTitle(this, &SMovieGraphMembersTabContent::GetSectionTitle)
		.OnGetSectionWidget(this, &SMovieGraphMembersTabContent::GetSectionWidget)
		.UseSectionStyling(true)
		.OnCollectAllActions(this, &SMovieGraphMembersTabContent::CollectAllActions)
		.OnActionMatchesName(this, &SMovieGraphMembersTabContent::ActionMatchesName)
	];
}

FReply SMovieGraphMembersTabContent::OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent)
{
	const TSharedPtr<FEdGraphSchemaAction> Action(!InActions.IsEmpty() ? InActions[0] : nullptr);
	if (!Action.IsValid())
	{
		return FReply::Unhandled();
	}
	
	const FMovieGraphSchemaAction* GraphSchemaAction = static_cast<FMovieGraphSchemaAction*>(Action.Get());
	const TObjectPtr<UObject> ActionTarget = GraphSchemaAction->ActionTarget;

	if (UMovieGraphVariable* VariableMember = Cast<UMovieGraphVariable>(ActionTarget.Get()))
	{
		return FReply::Handled().BeginDragDrop(FMovieGraphDragAction_Variable::New(Action, VariableMember));
	}
	
	return FReply::Unhandled();
}

FReply SMovieGraphMembersTabContent::OnCategoryDragged(const FText& InCategory, const FPointerEvent& MouseEvent)
{
	const TSharedRef<FMovieGraphDragAction_Category> DragOperation = FMovieGraphDragAction_Category::New(InCategory, CurrentGraph);
	return FReply::Handled().BeginDragDrop(DragOperation);
}

TSharedRef<SWidget> SMovieGraphMembersTabContent::CreateActionWidget(FCreateWidgetForActionData* InCreateData) const
{
	const uint32 SectionID = InCreateData->Action->GetSectionID();

	// For variables, inputs, and outputs, show an icon w/ the color representing the member's type (in addition to the member name)
	if (SectionID == static_cast<uint32>(EActionSection::Variables)
		|| SectionID == static_cast<uint32>(EActionSection::Inputs)
		|| SectionID == static_cast<uint32>(EActionSection::Outputs))
	{
		const FMovieGraphSchemaAction* GraphAction = static_cast<FMovieGraphSchemaAction*>(InCreateData->Action.Get());
		const UMovieGraphMember* Member = Cast<UMovieGraphMember>(GraphAction->ActionTarget);

		const FSlateBrush* PrimaryBrush = FAppStyle::GetBrush("Kismet.AllClasses.VariableIcon");
		FSlateColor IconColor = FSlateColor(FLinearColor::White);
		const FSlateBrush* SecondaryBrush = nullptr;
		FSlateColor SecondaryColor;
		UE::MovieGraph::Private::GetIconAndColorFromDataType(Member, PrimaryBrush, IconColor, SecondaryBrush, SecondaryColor);

		const TSharedPtr<SHorizontalBox> RowBox = SNew(SHorizontalBox);
		RowBox->AddSlot()
			.AutoWidth()
			.Padding(0, 0, 5, 0)
			[
				SNew(SImage)
				.Image(PrimaryBrush)
				.ColorAndOpacity(IconColor)
			];
			
		RowBox->AddSlot()
			.FillWidth(1.f)
			[
				SNew(SGraphPaletteItem, InCreateData)
			];

		return RowBox->AsShared();
	}
	
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(InCreateData->Action->GetMenuDescription())
		];
}

void SMovieGraphMembersTabContent::ClearSelection() const
{
	if (ActionMenu.IsValid())
	{
		ActionMenu->SelectItemByName(NAME_None);
	}
}

void SMovieGraphMembersTabContent::DeleteSelectedMembers()
{
	if (!CurrentGraph)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("DeleteGraphMembers", "Delete Graph Member(s)"));
	
	for (UMovieGraphMember* Member : UE::MovieGraph::Private::GetAllSelectedMembers<UMovieGraphMember>(ActionMenu))
	{
		MemberChangedHandles.Remove(Member);
		CurrentGraph->DeleteMember(Member);
	}

	RefreshMemberActions();
}

bool SMovieGraphMembersTabContent::CanDeleteSelectedMembers() const
{
	// Don't allow deletion if the member was explicitly marked as non-deletable
	for (const UMovieGraphMember* Member : UE::MovieGraph::Private::GetAllSelectedMembers<UMovieGraphMember>(ActionMenu))
	{
		if (!Member->IsDeletable())
		{
			return false;
		}
	}

	return true;
}

void SMovieGraphMembersTabContent::DuplicateSelectedMembers()
{
	if (!CurrentGraph)
	{
		return;
	}
	
	FScopedTransaction Transaction(LOCTEXT("DuplicateGraphMembers", "Duplicate Graph Member(s)"));

	for (UMovieGraphVariable* GraphVariable : UE::MovieGraph::Private::GetAllSelectedMembers<UMovieGraphVariable>(ActionMenu))
	{
		if (GraphVariable && !GraphVariable->IsGlobal())
		{
			CurrentGraph->DuplicateVariable(GraphVariable);
		}
	}

	RefreshMemberActions();
}

bool SMovieGraphMembersTabContent::CanDuplicateSelectedMembers() const
{
	// Only allow duplication of non-global variables for now
	for (const UMovieGraphVariable* GraphVariable : UE::MovieGraph::Private::GetAllSelectedMembers<UMovieGraphVariable>(ActionMenu))
	{
		if (GraphVariable && !GraphVariable->IsGlobal())
		{
			return true;
		}
	}

	return false;
}

void SMovieGraphMembersTabContent::PostUndo(bool bSuccess)
{
	// Normally the UI relies on delegates to determine when to refresh. However, undo/redo do not fire those delegates, so refresh whenever there
	// is an undo/redo.
	RefreshMemberActions();
}

void SMovieGraphMembersTabContent::PostRedo(bool bSuccess)
{
	RefreshMemberActions();
}

void SMovieGraphMembersTabContent::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	static const FText EmptyCategory = FText::GetEmpty();
	
	if (!CurrentGraph)
	{
		return;
	}

	FGraphActionMenuBuilder ActionMenuBuilder;

	// Creates a new action in the action menu under a specific section w/ the provided action target
	auto AddToActionMenu = [&ActionMenuBuilder, this](UMovieGraphMember* ActionTarget, const EActionSection Section, const FText& Category) -> void
	{
		const FText MemberActionDesc = FText::FromString(ActionTarget->GetMemberName());
		const FText MemberActionTooltip = FText::FromString(ActionTarget->Description);
		const FText MemberActionKeywords;
		const int32 MemberActionSectionID = static_cast<int32>(Section);
		const TSharedPtr<FMovieGraphSchemaAction> MemberAction(new FMovieGraphSchemaAction(Category, MemberActionDesc, MemberActionTooltip, 0, MemberActionKeywords, MemberActionSectionID));
		MemberAction->ActionTarget = ActionTarget;
		ActionMenuBuilder.AddAction(MemberAction);

		// Update actions when a member is updated (renamed, etc). Only subscribe to the delegate once.
		if (!MemberChangedHandles.Contains(ActionTarget))
		{
			FDelegateHandle MemberChangedDelegate;
			if (UMovieGraphInput* InputMember = Cast<UMovieGraphInput>(ActionTarget))
			{
				MemberChangedDelegate = InputMember->OnMovieGraphInputChangedDelegate.AddSP(this, &SMovieGraphMembersTabContent::RefreshMemberActions);
			}
			else if (UMovieGraphOutput* OutputMember = Cast<UMovieGraphOutput>(ActionTarget))
			{
				MemberChangedDelegate = OutputMember->OnMovieGraphOutputChangedDelegate.AddSP(this, &SMovieGraphMembersTabContent::RefreshMemberActions);
			}
			else if (UMovieGraphVariable* VariableMember = Cast<UMovieGraphVariable>(ActionTarget))
			{
				MemberChangedDelegate = VariableMember->OnMovieGraphVariableChangedDelegate.AddSP(this, &SMovieGraphMembersTabContent::RefreshMemberActions);
			}
			else
			{
				checkf(false, TEXT("Found an unsupported member type when adding it to the action menu."));
				return;
			}
			
			MemberChangedHandles.Add(ActionTarget, MemberChangedDelegate);
		}
	};

	for (UMovieGraphInput* Input : CurrentGraph->GetInputs())
	{
		if (Input && Input->IsDeletable())
		{
			AddToActionMenu(Input, EActionSection::Inputs, EmptyCategory);
		}
	}

	for (UMovieGraphOutput* Output : CurrentGraph->GetOutputs())
	{
		if (Output && Output->IsDeletable())
		{
			AddToActionMenu(Output, EActionSection::Outputs, EmptyCategory);
		}
	}

	const bool bIncludeGlobal = true;
	const TArray<UMovieGraphVariable*> AllVariables = CurrentGraph->GetVariables(bIncludeGlobal);

	// Add non-global variables first
	for (UMovieGraphVariable* Variable : AllVariables)
	{
		if (Variable && !Variable->IsGlobal())
		{
			// If the user didn't specify a custom category, just use the default. If one was specified, concat the default category and the user's category
			// so they display in a hierarchy.
			FText VariableCategory = FMovieGraphSchemaAction::UserVariablesCategory;
			if (!Variable->GetCategory().IsEmpty())
			{
				VariableCategory = FText::Format(INVTEXT("{0}|{1}"), FMovieGraphSchemaAction::UserVariablesCategory, FText::FromString(Variable->GetCategory()));
			}
			
			AddToActionMenu(Variable, EActionSection::Variables, VariableCategory);
		}
	}

	// Add global variables after user-declared variables
	for (UMovieGraphVariable* Variable : AllVariables)
	{
		if (Variable && Variable->IsGlobal())
		{
			AddToActionMenu(Variable, EActionSection::Variables, FMovieGraphSchemaAction::GlobalVariablesCategory);
		}
	}
	
	OutAllActions.Append(ActionMenuBuilder);
}

bool SMovieGraphMembersTabContent::ActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const
{
	if (const UMovieGraphMember* Member = UE::MovieGraph::Private::GetMemberFromAction(InAction))
	{
		return InName == Member->GetMemberName();
	}

	return false;
}

void SMovieGraphMembersTabContent::CollectStaticSections(TArray<int32>& StaticSectionIDs)
{
	// Start at index 1 to skip the invalid section
	for (int32 Index = 1; Index < static_cast<int32>(EActionSection::COUNT); ++Index)
	{
		StaticSectionIDs.Add(Index);
	}
}

FText SMovieGraphMembersTabContent::GetSectionTitle(int32 InSectionID)
{
	if (ensure(ActionMenuSectionNames.IsValidIndex(InSectionID)))
	{
		return ActionMenuSectionNames[InSectionID];
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> SMovieGraphMembersTabContent::GetSectionWidget(TSharedRef<SWidget> RowWidget, int32 InSectionID)
{
	FText ButtonTooltip;

	switch (static_cast<EActionSection>(InSectionID))
	{
	case EActionSection::Inputs:
		ButtonTooltip = LOCTEXT("ButtonTooltip_Inputs", "Add Input");
		break;

	case EActionSection::Outputs:
		ButtonTooltip = LOCTEXT("ButtonTooltip_Outputs", "Add Output");
		break;

	case EActionSection::Variables:
		ButtonTooltip = LOCTEXT("ButtonTooltip_Variables", "Add Variable");
		break;

	default:
		return SNullWidget::NullWidget;
	}
	
	return
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked(this, &SMovieGraphMembersTabContent::OnAddButtonClickedOnSection, InSectionID)
		.ContentPadding(FMargin(1, 0))
		.ToolTipText(ButtonTooltip)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}

TSharedPtr<SWidget> SMovieGraphMembersTabContent::OnContextMenuOpening()
{
	if (!ActionMenu.IsValid())
	{
		return nullptr;
	}

	TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
	ActionMenu->GetSelectedActions(SelectedActions);
	if (SelectedActions.IsEmpty())
	{
		return nullptr;
	}

	TSharedPtr< FAssetEditorToolkit> PinnedToolkit = EditorToolkit.Pin();
	if(!PinnedToolkit.IsValid())
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(true, PinnedToolkit->GetToolkitCommands());
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
	
	return MenuBuilder.MakeWidget();
}

FReply SMovieGraphMembersTabContent::OnAddButtonClickedOnSection(const int32 InSectionID)
{
	const EActionSection Section = static_cast<EActionSection>(InSectionID);

	if (Section == EActionSection::Inputs)
	{
		FScopedTransaction Transaction(LOCTEXT("AddNewInput", "Add New Input"));
		CurrentGraph->AddInput();
	}
	else if (Section == EActionSection::Outputs)
	{
		FScopedTransaction Transaction(LOCTEXT("AddNewOutput", "Add New Output"));
		CurrentGraph->AddOutput();
	}
	else if (Section == EActionSection::Variables)
	{
		FScopedTransaction Transaction(LOCTEXT("AddNewVariable", "Add New Variable"));
		CurrentGraph->AddVariable();
	}

	return FReply::Handled();
}

void SMovieGraphMembersTabContent::RefreshMemberActions(UMovieGraphMember* UpdatedMember)
{
	// Currently the entire action menu is refreshed rather than a specific action being targeted

	// Cache the currently selected member, so it can be referenced after the refresh
	const UMovieGraphMember* SelectedMember = nullptr;
	TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
	ActionMenu->GetSelectedActions(SelectedActions);
	if (!SelectedActions.IsEmpty())
	{
		if (const UMovieGraphMember* Member = UE::MovieGraph::Private::GetMemberFromAction(SelectedActions[0].Get()))
		{
			SelectedMember = Member;
		}
	}

	// Do the refresh
	const bool bPreserveExpansion = true;
	ActionMenu->RefreshAllActions(bPreserveExpansion);

	// Re-select the updated member if it was previously selected. The action menu performs a reselect after a refresh
	// automatically based on action name, but the member may have been renamed (so an explicit re-select is needed).
	if (SelectedMember && (SelectedMember == UpdatedMember))
	{
		// Members can potentially have the same name (in different sections), so when re-selecting, make sure the right section is used.
		int32 SectionId = static_cast<int32>(EActionSection::Variables);
		if (SelectedMember->IsA<UMovieGraphInput>())
		{
			SectionId = static_cast<int32>(EActionSection::Inputs);
		}
		else if (SelectedMember->IsA<UMovieGraphOutput>())
		{
			SectionId = static_cast<int32>(EActionSection::Outputs);
		}
		
		ActionMenu->SelectItemByName(FName(SelectedMember->GetMemberName()), ESelectInfo::Direct, SectionId);
	}
}

TSharedRef<FMovieGraphDragAction_Variable> FMovieGraphDragAction_Variable::New(
	TSharedPtr<FEdGraphSchemaAction> InAction, UMovieGraphVariable* InVariable)
{
	TSharedRef<FMovieGraphDragAction_Variable> DragAction = MakeShared<FMovieGraphDragAction_Variable>();
	DragAction->SourceAction = InAction;
	DragAction->WeakVariable = InVariable;
	DragAction->Construct();
	
	return DragAction;
}

void FMovieGraphDragAction_Variable::HoverTargetChanged()
{
	const FSlateBrush* ErrorBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
	const FSlateBrush* OkBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
	
	if (SourceAction.IsValid())
	{
		// Moving the action to a category
		if (!HoveredCategoryName.IsEmpty())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("DisplayName"), SourceAction->GetMenuDescription());
			Args.Add(TEXT("HoveredCategoryName"), HoveredCategoryName);
			
			if (HoveredCategoryName.EqualTo(SourceAction->GetCategory()))
			{
				const FText ErrorMessage = LOCTEXT("MoveVariableToCategory_Error", "Cannot move variable to '{HoveredCategoryName}' because it is already in that category.");
				SetSimpleFeedbackMessage(ErrorBrush, FLinearColor::White, FText::Format(ErrorMessage, Args));
			}
			else
			{
				const FText Message = LOCTEXT("MoveVariableToCategory_OK", "Move '{DisplayName}' to category '{HoveredCategoryName}'.");
				SetSimpleFeedbackMessage(OkBrush, FLinearColor::White, FText::Format(Message, Args));
			}

			return;
		}

		// Moving the action before another action
		if (HoveredAction.IsValid())
		{
			const TSharedPtr<FEdGraphSchemaAction> HoveredActionPtr = HoveredAction.Pin();
			FFormatNamedArguments Args;
			Args.Add(TEXT("DraggedDisplayName"), SourceAction->GetMenuDescription());
			Args.Add(TEXT("DropTargetDisplayName"), HoveredActionPtr->GetMenuDescription());

			if (HoveredAction == SourceAction)
			{
				const FText ErrorMessage = LOCTEXT("MoveVariable_SameVariableError", "Cannot move variable '{DraggedDisplayName}' before itself.");
				SetSimpleFeedbackMessage(ErrorBrush, FLinearColor::White, FText::Format(ErrorMessage, Args));
			}
			else
			{
				const FText Message = LOCTEXT("MoveVariable_OK", "Move '{DraggedDisplayName}' before '{DropTargetDisplayName}'.");
				SetSimpleFeedbackMessage(OkBrush, FLinearColor::White, FText::Format(Message, Args));
			}

			return;
		}
	}
	
	FGraphSchemaActionDragDropAction::HoverTargetChanged();
}

FReply FMovieGraphDragAction_Variable::DroppedOnAction(TSharedRef<FEdGraphSchemaAction> Action)
{
	// The drop can only target another variable action
	const TSharedRef<FMovieGraphSchemaAction> GraphAction = StaticCastSharedRef<FMovieGraphSchemaAction>(Action);
	if (!GraphAction->ActionTarget->IsA<UMovieGraphVariable>())
	{
		return FReply::Unhandled();
	}

	FScopedTransaction Transaction(LOCTEXT("MoveGraphVariable", "Move Graph Variable"));

	SourceAction->ReorderToBeforeAction(GraphAction);

	return FReply::Handled();
}

FReply FMovieGraphDragAction_Variable::DroppedOnCategory(FText Category)
{
	FScopedTransaction Transaction(LOCTEXT("MoveGraphVariableToCategory", "Move Graph Variable to Category"));
	
	SourceAction->MovePersistentItemToCategory(Category);

	return FReply::Handled();
}

FReply FMovieGraphDragAction_Variable::DroppedOnPanel(
	const TSharedRef<SWidget>& InPanel, const FVector2f& InScreenPosition, const FVector2f& InGraphPosition, UEdGraph& InGraph)
{
	const UMovieGraphVariable* VariableMember = WeakVariable.Get();
	if (!VariableMember)
	{
		return FReply::Unhandled();
	}

	// When creating the new action, since it's only being used to create a node, the category, display name, and tooltip can just be empty
	const TSharedPtr<FMovieGraphSchemaAction> NewAction = MakeShared<FMovieGraphSchemaAction_NewVariableNode>(
		FText::GetEmpty(), FText::GetEmpty(), VariableMember->GetGuid(), FText::GetEmpty());
	NewAction->NodeClass = UMovieGraphVariableNode::StaticClass();
	
	constexpr UEdGraphPin* FromPin = nullptr;
	constexpr bool bSelectNewNode = true;
	NewAction->PerformAction(&InGraph, FromPin, FDeprecateSlateVector2D(InGraphPosition), bSelectNewNode);

	return FReply::Handled();
}

void FMovieGraphDragAction_Variable::GetDefaultStatusSymbol(
	const FSlateBrush*& OutPrimaryBrush, FSlateColor& OutIconColor, const FSlateBrush*& OutSecondaryBrush, FSlateColor& OutSecondaryColor) const
{
	const UMovieGraphVariable* VariableMember = WeakVariable.Get();
	if (!VariableMember ||
		!UE::MovieGraph::Private::GetIconAndColorFromDataType(VariableMember, OutPrimaryBrush, OutIconColor, OutSecondaryBrush, OutSecondaryColor))
	{
		return FGraphSchemaActionDragDropAction::GetDefaultStatusSymbol(OutPrimaryBrush, OutIconColor, OutSecondaryBrush, OutSecondaryColor);
	}
}

TSharedRef<FMovieGraphDragAction_Category> FMovieGraphDragAction_Category::New(const FText& InCategory, UMovieGraphConfig* InGraph)
{
	TSharedRef<FMovieGraphDragAction_Category> Operation = MakeShared<FMovieGraphDragAction_Category>();
	Operation->Construct();
	Operation->DraggedCategory = InCategory;
	Operation->GraphConfig = InGraph;
	
	return Operation;
}

void FMovieGraphDragAction_Category::HoverTargetChanged()
{
	const FSlateBrush* ErrorBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
	const FSlateBrush* OkBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

	// Get the name of the category without all of the "|" separators
	TArray<FString> CategoryHierarchy;
	DraggedCategory.ToString().ParseIntoArray(CategoryHierarchy, TEXT("|"));

	if (CategoryHierarchy.IsEmpty())
	{
		FGraphSchemaActionDragDropAction::HoverTargetChanged();
		return;
	}
	
	const FString DraggedCategoryName = CategoryHierarchy.Last();
	
	FFormatNamedArguments Args;
	Args.Add(TEXT("CategoryName"), FText::FromString(DraggedCategoryName));
	Args.Add(TEXT("HoveredCategoryName"), HoveredCategoryName);

	// Moving the category to another category
	if (!HoveredCategoryName.IsEmpty())
	{
		if (HoveredCategoryName.EqualTo(DraggedCategory))
		{
			const FText ErrorMessage = LOCTEXT("MoveCategoryToCategory_SelfError", "Cannot move category '{CategoryName}' before itself.");
			SetSimpleFeedbackMessage(ErrorBrush, FLinearColor::White, FText::Format(ErrorMessage, Args));
		}
		else
		{
			const FText Message = LOCTEXT("MoveCategoryToCategory_OK", "Move category '{CategoryName}' before category '{HoveredCategoryName}'.");
			SetSimpleFeedbackMessage(OkBrush, FLinearColor::White, FText::Format(Message, Args));
		}

		// Unfortunately the DraggedCategory is set to the display name and there's no easy way to change this. Because of that, we can't warn about
		// dragging a child category into its immediate parent, or a parent category into a child category. That will have to be handled when the
		// drop actually happens.

		return;
	}

	// Invalid drop of a category on a non-category item
	if (HoveredAction.IsValid())
	{
		const FText ErrorMessage = LOCTEXT("MoveCategoryToCategory_InvalidActionError", "Can only insert category '{CategoryName}' before another category.");
		SetSimpleFeedbackMessage(ErrorBrush, FLinearColor::White, FText::Format(ErrorMessage, Args));

		return;
	}
	
	FGraphSchemaActionDragDropAction::HoverTargetChanged();
}

FReply FMovieGraphDragAction_Category::DroppedOnCategory(FText Category)
{
	static const FString UserVariablesPrefix = FString::Format(TEXT("{0}|"), {FMovieGraphSchemaAction::UserVariablesCategory.ToString()});;
	
	if (GraphConfig)
	{
		const FString DraggedCategoryString = DraggedCategory.ToString();
		const FString CategoryString = Category.ToString();

		// Remove the "User Variables|" prefix. "User Variables" is the top-level category the variables are displayed under; variables should not
		// have this stored as part of their category.
		const FString SourceCategory = DraggedCategoryString.StartsWith(UserVariablesPrefix)
			? DraggedCategoryString.RightChop(UserVariablesPrefix.Len())
			: DraggedCategoryString;
		const FString DestinationCategory = CategoryString.StartsWith(UserVariablesPrefix)
			? CategoryString.RightChop(UserVariablesPrefix.Len())
			: CategoryString;

		FScopedTransaction Transaction(LOCTEXT("MoveGraphVariableCategory", "Move Graph Variable Category"));
		
		// SourceAction will be invalid for a category drag/drop, so go directly to the graph
		GraphConfig->MoveCategoryBefore(SourceCategory, DestinationCategory);
		
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE