// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "GraphEditorDragDropAction.h"
#include "SGraphActionMenu.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SCompoundWidget.h"

class FAssetEditorToolkit;
class SGraphActionMenu;
class UMovieGraphVariable;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;

/**
 * Contents of the "Members" tab in the graph asset editor.
 */
class SMovieGraphMembersTabContent : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
public:
	DECLARE_DELEGATE_TwoParams(FOnActionSelected, const TArray<TSharedPtr<FEdGraphSchemaAction>>&, ESelectInfo::Type);
	
	SLATE_BEGIN_ARGS(SMovieGraphMembersTabContent)
		: _Graph(nullptr)
		, _Editor(nullptr)
	
		{}

		/** An event which is triggered when an action is selected. */
		SLATE_EVENT(FOnActionSelected, OnActionSelected)
		
		/** The graph that is currently displayed. */
		SLATE_ARGUMENT(class UMovieGraphConfig*, Graph)

		/** The editor that is displaying this widget. */
		SLATE_ARGUMENT(TSharedPtr<FAssetEditorToolkit>, Editor)
	
	SLATE_END_ARGS();
	
	void Construct(const FArguments& InArgs);

	/** Handler for when an action in the action menu is dragged. */
	FReply OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent);

	/** Handler for when a category in the action menu is dragged. */
	FReply OnCategoryDragged(const FText& InCategory, const FPointerEvent& MouseEvent);

	/** Creates the widget for an action within the action menu. */
	TSharedRef<SWidget> CreateActionWidget(FCreateWidgetForActionData* CreateWidgetForActionData) const;

	/** Resets the selected members in the UI. */
	void ClearSelection() const;

	/** Deletes the member(s) which are currently selected from the graph and the UI. */
	void DeleteSelectedMembers();

	/** Determines if all selected member(s) can be deleted. */
	bool CanDeleteSelectedMembers() const;

	/** Duplicates the member(s) which are currently selected in the UI. */
	void DuplicateSelectedMembers();

	/** Determines if at least one selected member can be duplicated. */
	bool CanDuplicateSelectedMembers() const;

	//~ Begin FSelfRegisteringEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient Interface

private:
	/** The section identifier in the action widget. */
	enum class EActionSection : uint8
	{
		Invalid,
		
		Inputs,
		Outputs,
		Variables,

		COUNT
	};

	/** The names of the sections in the action widget. */
	static const TArray<FText> ActionMenuSectionNames;
	
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
	void CollectStaticSections(TArray<int32>& StaticSectionIDs);
	FText GetSectionTitle(int32 InSectionID);
	TSharedRef<SWidget> GetSectionWidget(TSharedRef<SWidget> RowWidget, int32 InSectionID);

	/** Returns true if the action matches the specified name, else false. */
	bool ActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const;

	/** Handler which deals with populating the context menu. */
	TSharedPtr<SWidget> OnContextMenuOpening();

	/** Handler which deals with clicking the add button for an action section. */
	FReply OnAddButtonClickedOnSection(const int32 InSectionID);

	/** Refresh/regenerate the action menu when the given member is updated. */
	void RefreshMemberActions(class UMovieGraphMember* UpdatedMember = nullptr);

private:
	/** The editor that this widget is associated with. */
	TWeakPtr<FAssetEditorToolkit> EditorToolkit;

	/** The action menu displayed in the UI which allows for creation/manipulation of graph members (eg, variables). */
	TSharedPtr<SGraphActionMenu> ActionMenu;

	/** The runtime graph that this UI gets/sets data on. */
	TObjectPtr<UMovieGraphConfig> CurrentGraph;
	
	/** Delegate to call when an action is selected */
	FOnActionSelected OnActionSelected;

	/** Handles to delegates handling member changes. */
	TMap<TWeakObjectPtr<UMovieGraphMember>, FDelegateHandle> MemberChangedHandles;
};

/* Drag-and-drop action which handles variable members. */
class FMovieGraphDragAction_Variable : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FMovieGraphDragAction_Variable, FGraphSchemaActionDragDropAction)

	static TSharedRef<FMovieGraphDragAction_Variable> New(
		TSharedPtr<FEdGraphSchemaAction> InAction, UMovieGraphVariable* InVariable);

	virtual void HoverTargetChanged() override;
	virtual FReply DroppedOnAction(TSharedRef<FEdGraphSchemaAction> Action) override;
	virtual FReply DroppedOnCategory(FText Category) override;
	virtual FReply DroppedOnPanel(
		const TSharedRef<SWidget>& InPanel, const FVector2f& InScreenPosition, const FVector2f& InGraphPosition, UEdGraph& InGraph) override;

protected:
	virtual void GetDefaultStatusSymbol(
		const FSlateBrush*& OutPrimaryBrush, FSlateColor& OutIconColor, FSlateBrush const*& OutSecondaryBrush, FSlateColor& OutSecondaryColor) const override;

private:
	/** The variable member that the drag-and-drop is associated with. */
	TWeakObjectPtr<UMovieGraphVariable> WeakVariable;
};

/* Drag-and-drop action which handles variable categories. */
class FMovieGraphDragAction_Category : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FMovieGraphDragAction_Category, FGraphSchemaActionDragDropAction)

	static TSharedRef<FMovieGraphDragAction_Category> New(const FText& InCategory, UMovieGraphConfig* InGraph);

	virtual void HoverTargetChanged() override;
	virtual FReply DroppedOnCategory(FText Category) override;

public:
	/** The category that is being dragged. */
	FText DraggedCategory;

	/** The graph associated with the drag/drop action. */
	UMovieGraphConfig* GraphConfig = nullptr;
};