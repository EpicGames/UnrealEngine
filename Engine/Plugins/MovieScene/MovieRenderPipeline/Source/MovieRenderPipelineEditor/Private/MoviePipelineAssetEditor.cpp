// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineAssetEditor.h"
#include "MoviePipelineConfigAssetEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineAssetEditor)

TSharedPtr<FBaseAssetToolkit> UMoviePipelineAssetEditor::CreateToolkit()
{
	return MakeShared<FMoviePipelineConfigAssetEditor>(this);
}

void UMoviePipelineAssetEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
	InObjectsToEdit.Add(ObjectToEdit);
}

void UMoviePipelineAssetEditor::SetObjectToEdit(UObject* InObject)
{
	ObjectToEdit = InObject;
}
