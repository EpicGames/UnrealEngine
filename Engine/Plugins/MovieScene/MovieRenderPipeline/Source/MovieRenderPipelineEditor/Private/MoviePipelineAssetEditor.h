// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"

#include "MoviePipelineAssetEditor.generated.h"


UCLASS()
class UMoviePipelineAssetEditor : public UAssetEditor
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;
	virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit) override;
	
	void SetObjectToEdit(UObject* InObject);
private:
	UPROPERTY(Transient)
	TObjectPtr<UObject> ObjectToEdit;
};