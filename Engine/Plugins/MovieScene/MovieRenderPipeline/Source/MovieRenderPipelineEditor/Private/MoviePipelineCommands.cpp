// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineCommands.h"
#include "MovieRenderPipelineStyle.h"

#define LOCTEXT_NAMESPACE "MoviePipelineCommands"

FMoviePipelineCommands::FMoviePipelineCommands()
	: TCommands<FMoviePipelineCommands>("MovieRenderPipeline", LOCTEXT("MoviePipelineCommandsLabel", "Movie Pipeline"), NAME_None, FMovieRenderPipelineStyle::StyleName)
{
}

void FMoviePipelineCommands::RegisterCommands()
{
	UI_COMMAND(ResetStatus, "ResetStatus", "Reset Status", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::R));

	// Graph commands
	UI_COMMAND(ZoomToWindow, "Zoom to Graph Extents", "Fit the current view to the entire graph", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ZoomToSelection, "Zoom to Selection", "Fit the current view to the selection", EUserInterfaceActionType::Button, FInputChord(EKeys::Home));
}

#undef LOCTEXT_NAMESPACE
