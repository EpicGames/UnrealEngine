// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

class FMoviePipelineCommands : public TCommands<FMoviePipelineCommands>
{
public:
	FMoviePipelineCommands();

	TSharedPtr<FUICommandInfo> ResetStatus;

	// Graph commands
	TSharedPtr<FUICommandInfo> ZoomToWindow;
	TSharedPtr<FUICommandInfo> ZoomToSelection;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
