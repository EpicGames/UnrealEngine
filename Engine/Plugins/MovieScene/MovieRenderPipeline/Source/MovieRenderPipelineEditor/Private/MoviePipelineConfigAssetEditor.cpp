// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineConfigAssetEditor.h"
#include "MovieRenderPipelineEditorModule.h"
#include "MoviePipelineConfigBase.h"
#include "Widgets/SMoviePipelineConfigPanel.h"
#include "MoviePipelinePrimaryConfig.h"
#include "Widgets/Docking/SDockTab.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "EditorFontGlyphs.h"

#define LOCTEXT_NAMESPACE "MoviePipelineConfigAssetEditor"

const FName FMoviePipelineConfigAssetEditor::ContentTabId(TEXT("PipelineAssetEditor_Config"));

FMoviePipelineConfigAssetEditor::FMoviePipelineConfigAssetEditor(UAssetEditor* InOwningAssetEditor)
	: FBaseAssetToolkit(InOwningAssetEditor)
{
	LayoutAppendix = TEXT("MoviePipelineConfigAssetEditor");
	FString LayoutString = TEXT("Standalone_Test_Layout_") + LayoutAppendix;

	// Override the Default Layout provided by FBaseAssetToolkit to hide the viewport and details panel tabs.
	StandaloneDefaultLayout = FTabManager::NewLayout(FName(LayoutString))
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(1.f)
				->SetHideTabWell(true)
				->AddTab(ContentTabId, ETabState::OpenedTab)
			)
		);
}

void FMoviePipelineConfigAssetEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_PipelineAssetEditor", "Movie Pipeline Asset Editor"));

	FBaseAssetToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(ContentTabId, FOnSpawnTab::CreateSP(this, &FMoviePipelineConfigAssetEditor::SpawnTab_ConfigEditor))
		.SetDisplayName(LOCTEXT("ConfigTab", "Config"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

void FMoviePipelineConfigAssetEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(ContentTabId);
}

TSharedRef<SDockTab> FMoviePipelineConfigAssetEditor::SpawnTab_ConfigEditor(const FSpawnTabArgs& Args)
{
	UMoviePipelineConfigBase* Config = Cast<UMoviePipelineConfigBase>(GetEditingObject());

	ConfigEditorPanel =
		SNew(SMoviePipelineConfigPanel, Config->GetClass())
		.Job(nullptr)
		.Shot(nullptr)
		.BasePreset(nullptr)
		.BaseConfig(nullptr)
		.AssetToEdit(Config);

	// Extend the tool menu specific to this tool
	UToolMenus* ToolMenus = UToolMenus::Get();
	FName ParentName;
	FName ToolMenuToolbarName = GetToolMenuToolbarName(ParentName);

	{
		UToolMenu* ToolbarBuilder = ToolMenus->RegisterMenu(ToolMenuToolbarName, ParentName, EMultiBoxType::ToolBar);
		{
			FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
			{
				FToolMenuSection& Section = ToolbarBuilder->AddSection("Settings", TAttribute<FText>(), InsertAfterAssetSection);
				Section.AddEntry(FToolMenuEntry::InitWidget(FName("Settings"), ConfigEditorPanel->MakeSettingsWidget(), LOCTEXT("AddSettings_Label", "Settings")));
			}
		}
	}

	// Tabs and Windows have different backgrounds, so we copy the window style to
	// make the 'void' area look similar.
	const FSlateBrush* BackgroundBrush = &FCoreStyle::Get().GetWidgetStyle<FWindowStyle>("Window").ChildBackgroundBrush;

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		.TabColorScale(GetTabColorScale())
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(BackgroundBrush)
			]
			+ SOverlay::Slot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					ConfigEditorPanel.ToSharedRef()
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(4.f)
					.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
					.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
					.Content()
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0, 0, 4, 0)
						[
							SNew(STextBlock)
							.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
							.Text(FEditorFontGlyphs::Exclamation_Triangle)
							.ColorAndOpacity(FLinearColor::Yellow)
						]
						+SHorizontalBox::Slot()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PipelineLimitedEditing", "Editing capabilities are limited in the asset editor. Edit this preset through the Movie Render Queue UI for full editing capabilities (validation, format texts, etc.)"))
						]
					]
				]
			]
		];

	return NewDockTab;
}

FName FMoviePipelineConfigAssetEditor::GetToolkitFName() const
{
	return FName("MoviePipelineConfigAssetEditor");
}

#undef LOCTEXT_NAMESPACE
