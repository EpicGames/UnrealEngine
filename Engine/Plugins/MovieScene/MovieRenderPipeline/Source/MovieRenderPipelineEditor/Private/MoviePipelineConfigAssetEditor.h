// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Tools/BaseAssetToolkit.h"

class UMoviePipelineConfigBase;

class FMoviePipelineConfigAssetEditor :  public FBaseAssetToolkit
{
public:

	FMoviePipelineConfigAssetEditor(UAssetEditor* InOwningAssetEditor);
	virtual ~FMoviePipelineConfigAssetEditor() = default;

	/** FBaseAssetToolkit interface */
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual FName GetToolkitFName() const override;

private:
	TSharedRef<SDockTab> SpawnTab_ConfigEditor(const FSpawnTabArgs& Args);

private:
	static const FName ContentTabId;
	TSharedPtr<class SMoviePipelineConfigPanel> ConfigEditorPanel;
};
