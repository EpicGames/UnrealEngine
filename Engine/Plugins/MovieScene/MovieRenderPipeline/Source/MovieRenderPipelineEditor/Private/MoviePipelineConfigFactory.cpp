// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineConfigFactory.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineShotConfig.h"
#include "AssetTypeCategories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineConfigFactory)

UMoviePipelinePrimaryConfigFactory::UMoviePipelinePrimaryConfigFactory()
{
	bCreateNew = true;
	bEditAfterNew = false;
	SupportedClass = UMoviePipelinePrimaryConfig::StaticClass();
}

UObject* UMoviePipelinePrimaryConfigFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UMoviePipelinePrimaryConfig>(InParent, Class, Name, Flags);
}

uint32 UMoviePipelinePrimaryConfigFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Type::None; 
}

UMoviePipelineShotConfigFactory::UMoviePipelineShotConfigFactory()
{
	bCreateNew = true;
	bEditAfterNew = false;
	SupportedClass = UMoviePipelineShotConfig::StaticClass();
}

UObject* UMoviePipelineShotConfigFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UMoviePipelineShotConfig>(InParent, Class, Name, Flags);
}

uint32 UMoviePipelineShotConfigFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Type::None; 
}
