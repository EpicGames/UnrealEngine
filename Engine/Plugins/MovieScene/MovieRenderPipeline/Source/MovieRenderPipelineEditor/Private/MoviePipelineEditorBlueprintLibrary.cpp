// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineEditorBlueprintLibrary.h"

#include "ContentBrowserModule.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "FileHelpers.h"
#include "IContentBrowserSingleton.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "LevelSequence.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineQueueSubsystem.h"
#include "MovieRenderPipelineSettings.h"
#include "Misc/MessageDialog.h"
#include "PackageHelperFunctions.h"
#include "FileHelpers.h"
#include "Misc/FileHelper.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Graph/MovieRenderGraphEditorSettings.h"
#include "ScopedTransaction.h"
#include "SequencerUtilities.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelineOutputSetting.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "MoviePipelineBasicConfig.h"
#include "Styling/AppStyle.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineEditorBlueprintLibrary)

#define LOCTEXT_NAMESPACE "MoviePipelineEditorBlueprintLibrary"

bool UMoviePipelineEditorBlueprintLibrary::ExportConfigToAsset(const UMoviePipelinePrimaryConfig* InConfig, const FString& InPackagePath, const FString& InFileName, const bool bInSaveAsset, UMoviePipelinePrimaryConfig*& OutAsset, FText& OutErrorReason)
{
	if(!InConfig)
	{
		OutErrorReason = LOCTEXT("CantExportNullConfigToPackage", "Can't export a null configuration to a package.");
		return false;
	}
	
	FString FixedAssetName = ObjectTools::SanitizeObjectName(InFileName);
	FString NewPackageName = FPackageName::GetLongPackagePath(InPackagePath) + TEXT("/") + FixedAssetName;

	if (!FPackageName::IsValidLongPackageName(NewPackageName, false, &OutErrorReason))
	{
		return false;
	}

	UPackage* NewPackage = CreatePackage(*NewPackageName);
	NewPackage->MarkAsFullyLoaded();
	NewPackage->AddToRoot();
	
	// Duplicate the provided config into this package.
	UMoviePipelinePrimaryConfig* NewConfig = Cast<UMoviePipelinePrimaryConfig>(StaticDuplicateObject(InConfig, NewPackage, FName(*InFileName), RF_NoFlags));
	NewConfig->SetFlags(RF_Public | RF_Transactional | RF_Standalone);
	NewConfig->MarkPackageDirty();

	// Mark it so it shows up in the Content Browser immediately
	FAssetRegistryModule::AssetCreated(NewConfig);

	// If they want to save, ask them to save (and add to version control)
	if (bInSaveAsset)
	{
		TArray<UPackage*> Packages;
		Packages.Add(NewConfig->GetOutermost());

		return UEditorLoadingAndSavingUtils::SavePackages(Packages, true);
	}

	return true;
}

bool UMoviePipelineEditorBlueprintLibrary::IsMapValidForRemoteRender(const TArray<UMoviePipelineExecutorJob*>& InJobs)
{
	for (const UMoviePipelineExecutorJob* Job : InJobs)
	{
		FString PackageName = Job->Map.GetLongPackageName();
		if (!FPackageName::IsValidLongPackageName(PackageName))
		{
			return false;
		}
	}
	return true;
}

void UMoviePipelineEditorBlueprintLibrary::WarnUserOfUnsavedMap()
{
	FText FailureReason = LOCTEXT("UnsavedMapFailureDialog", "One or more jobs in the queue have an unsaved map as their target map. These unsaved maps cannot be loaded by an external process, and the render has been aborted.");
	FMessageDialog::Open(EAppMsgType::Ok, FailureReason);
}

UMoviePipelineQueue* UMoviePipelineEditorBlueprintLibrary::SaveQueueToManifestFile(UMoviePipelineQueue* InPipelineQueue, FString& OutManifestFilePath)
{
	FString InFileName = TEXT("QueueManifest");
	FString InPackagePath = TEXT("/Engine/MovieRenderPipeline/Editor/Transient");

	FString FixedAssetName = ObjectTools::SanitizeObjectName(InFileName);
	FString NewPackageName = FPackageName::GetLongPackagePath(InPackagePath) + TEXT("/") + FixedAssetName;

	// If there's already a package with this name, rename it so that the newly created one can always get a fixed name.
	// The fixed name is important because in the new process it'll start the unique name count over.
	if (UPackage* OldPackage = FindObject<UPackage>(nullptr, *NewPackageName))
	{
		FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), UPackage::StaticClass(), "DEAD_NewProcessExecutor_SerializedPackage");
		OldPackage->Rename(*UniqueName.ToString());
		OldPackage->SetFlags(RF_Transient);
	}

	UPackage* NewPackage = CreatePackage(*NewPackageName);

	// Duplicate the Queue into this package as we don't want to just rename the existing that belongs to the editor subsystem.
	UMoviePipelineQueue* DuplicatedQueue = CastChecked<UMoviePipelineQueue>(StaticDuplicateObject(InPipelineQueue, NewPackage));
	DuplicatedQueue->SetFlags(RF_Public | RF_Transactional | RF_Standalone);

	// Save the package to disk.
	FString ManifestFileName = TEXT("MovieRenderPipeline/QueueManifest") + FPackageName::GetTextAssetPackageExtension();
	OutManifestFilePath = FPaths::ProjectSavedDir() / ManifestFileName;

	// Fully load the package before trying to save.
	LoadPackage(NewPackage, *NewPackageName, LOAD_None);

	{
		UEditorLoadingSavingSettings* SaveSettings = GetMutableDefault<UEditorLoadingSavingSettings>();
		uint32 bSCCAutoAddNewFiles = SaveSettings->bSCCAutoAddNewFiles;
		SaveSettings->bSCCAutoAddNewFiles = 0;
		bool bSuccess = SavePackageHelper(NewPackage, *OutManifestFilePath);
		SaveSettings->bSCCAutoAddNewFiles = bSCCAutoAddNewFiles;
		
		if(!bSuccess)
		{
			return nullptr;
		}
	}

	NewPackage->SetFlags(RF_Transient);
	NewPackage->ClearFlags(RF_Standalone);
	DuplicatedQueue->SetFlags(RF_Transient);
	DuplicatedQueue->ClearFlags(RF_Public | RF_Transactional | RF_Standalone);

	return DuplicatedQueue;
}

FString UMoviePipelineEditorBlueprintLibrary::ConvertManifestFileToString(const FString& InManifestFilePath)
{
	// Due to API limitations we can't convert package -> text directly and instead need to re-load it, escape it, and then put it onto the command line :-)
	FString OutString;
	FFileHelper::LoadFileToString(OutString, *InManifestFilePath);

	return OutString;
}

UMoviePipelineExecutorJob* UMoviePipelineEditorBlueprintLibrary::CreateJobFromSequence(UMoviePipelineQueue* InPipelineQueue, const ULevelSequence* InSequence)
{
	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();

	InPipelineQueue->Modify();

	UMoviePipelineExecutorJob* NewJob = InPipelineQueue->AllocateNewJob(ProjectSettings->DefaultExecutorJob.TryLoadClass<UMoviePipelineExecutorJob>());
	if (!ensureAlwaysMsgf(NewJob, TEXT("Failed to allocate new job! Check the DefaultExecutorJob is not null in Project Settings!")))
	{
		return nullptr;
	}

	NewJob->Modify();

	TArray<FString> AssociatedMaps = FSequencerUtilities::GetAssociatedLevelSequenceMapPackages(InSequence);
	FSoftObjectPath CurrentWorld;

	UWorld* EditorWorld = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;

	// We'll assume they went to render from the current world if it's been saved.
	if (EditorWorld && (!EditorWorld->GetOutermost()->GetPathName().StartsWith(TEXT("/Temp/Untitled")) || AssociatedMaps.Num() == 0))
	{
		CurrentWorld = FSoftObjectPath(EditorWorld);
	}
	else if (AssociatedMaps.Num() > 0)
	{
		// So associated maps are only packages and not assets, but FSoftObjectPath needs assets.
		// We know that they are map packages, and map packages should be /Game/Foo.Foo, so we can
		// just do some string manipulation here as there isn't a generic way to go from Package->Object.
		FString MapPackage = AssociatedMaps[0];
		MapPackage = FString::Printf(TEXT("%s.%s"), *MapPackage, *FPackageName::GetShortName(MapPackage));

		CurrentWorld = FSoftObjectPath(MapPackage);
	}

	// Job author is intentionally left blank so that it doesn't get saved into queues. It will
	// be resolved into a username when the Movie Pipeline starts if it is blank.
	FSoftObjectPath Sequence(InSequence);
	NewJob->Map = CurrentWorld;
	NewJob->SetSequence(Sequence);
	NewJob->JobName = NewJob->Sequence.GetAssetName();

	return NewJob;
}

void UMoviePipelineEditorBlueprintLibrary::ApplyDefaultConfigurationTypeToJob(UMoviePipelineExecutorJob* InJob)
{
	if (!InJob)
	{
		return;
	}

	const UMovieRenderGraphEditorSettings* EditorSettings = GetDefault<UMovieRenderGraphEditorSettings>();
	switch (EditorSettings->DefaultConfigType)
	{
	case EMoviePipelineDefaultConfigType::Basic:
		InJob->SetupBasicConfiguration();
		break;
	case EMoviePipelineDefaultConfigType::MovieRenderGraph:
		AssignDefaultGraphPresetToJob(InJob, /*bCreateTransaction=*/false);
		break;
	case EMoviePipelineDefaultConfigType::LegacyPreset:
	{
		const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
		if (ProjectSettings->LastPresetOrigin.IsValid())
		{
			InJob->SetPresetOrigin(ProjectSettings->LastPresetOrigin.Get());
		}
		EnsureJobHasDefaultSettings(InJob);
		break;
	}
	default:
		checkNoEntry();
		break;
	}
}

void UMoviePipelineEditorBlueprintLibrary::AssignDefaultGraphPresetToJob(UMoviePipelineExecutorJob* InJob, const bool bCreateTransaction)
{
	if (!InJob)
	{
		return;
	}

	TOptional<FScopedTransaction> Transaction;
	if (bCreateTransaction)
	{
		Transaction.Emplace(LOCTEXT("SetJobGraphConfig_Transaction", "Set Graph Config on Job"));
	}
	InJob->Modify();

	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	const TSoftObjectPtr<UMovieGraphConfig> ProjectDefaultGraph = ProjectSettings->DefaultGraph;
	if (const UMovieGraphConfig* DefaultGraph = ProjectDefaultGraph.LoadSynchronous())
	{
		InJob->SetGraphPreset(DefaultGraph);
	}
	else
	{
		FNotificationInfo Info(LOCTEXT("UpdateJobGraphConfig_InvalidGraphNotification", "Unable to Set Graph Config on Job"));
		Info.SubText = LOCTEXT("UpdateJobGraphConfig_InvalidGraphNotificationSubtext", "The Graph Asset specified in Project Settings (Movie Render Pipeline > Default Graph) could not be loaded.");
		Info.Image = FAppStyle::GetBrush(TEXT("Icons.Warning"));
		Info.ExpireDuration = 5.0f;

		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

void UMoviePipelineEditorBlueprintLibrary::EnsureJobHasDefaultSettings(UMoviePipelineExecutorJob* NewJob, bool bForce)
{
	if (!NewJob || !NewJob->GetConfiguration())
	{
		return;
	}

	// Skip if defaults have already been applied to this job, unless the caller is explicitly
	// forcing a re-apply (typically after resetting the Configuration to a fresh empty state).
	// This prevents settings the user has deliberately removed from being resurrected on
	// subsequent Basic/Graph -> Preset switches.
	if (!bForce && NewJob->HasInitializedLegacyDefaults())
	{
		return;
	}

	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	for (FSoftClassPath SettingClassPath : ProjectSettings->DefaultClasses)
	{
		TSubclassOf<UMoviePipelineSetting> SettingClass = SettingClassPath.TryLoadClass<UMoviePipelineSetting>();
		if (!SettingClass)
		{
			continue;
		}

		if (SettingClass->HasAnyClassFlags(CLASS_Abstract))
		{
			continue;
		}

		const bool bIncludeDisabledSettings = true;
		UMoviePipelineSetting* ExistingSetting = NewJob->GetConfiguration()->FindSettingByClass(SettingClass, bIncludeDisabledSettings);
		if (!ExistingSetting)
		{
			NewJob->GetConfiguration()->FindOrAddSettingByClass(SettingClass);
		}
	}

	NewJob->MarkLegacyPresetDefaultsAsInitialized();
}

bool UMoviePipelineEditorBlueprintLibrary::GetDisplayOutputPathFromJob(UMoviePipelineExecutorJob* InJob, FString& OutOutputPath)
{
	if (ensureMsgf(InJob, TEXT("%hs: Could not get format string because provided UMoviePipelineExecutorJob was invalid."), __FUNCTION__))
	{
		if (InJob->IsUsingGraphConfiguration())
		{
			if (InJob->GetGraphPreset())
			{
				FString OutputDirectory;
				InJob->GetGraphPreset()->GetOutputDirectory(OutputDirectory);
				OutOutputPath = OutputDirectory;
				return true;
			}
		}

		if (InJob->GetConfiguration())
		{
			const UMoviePipelineOutputSetting* OutputSetting = InJob->GetConfiguration()->FindSetting<UMoviePipelineOutputSetting>();
			check(OutputSetting);

			OutOutputPath = OutputSetting->OutputDirectory.Path;
			return true;
		}
	}

	return false;
}

FString UMoviePipelineEditorBlueprintLibrary::ResolveOutputDirectoryFromJob(UMoviePipelineExecutorJob* InJob)
{
	FText DiscardedError;
	return TryResolveOutputDirectoryFromJob(InJob, DiscardedError);
}

FString UMoviePipelineEditorBlueprintLibrary::TryResolveOutputDirectoryFromJob(UMoviePipelineExecutorJob* InJob, FText& OutError)
{
	OutError = FText::GetEmpty();

	if (!InJob)
	{
		OutError = LOCTEXT("ResolveOutputDir_NullJob", "No job was provided.");
		return FString();
	}

	UMovieGraphEvaluatedConfig* EvaluatedGraph = nullptr;
	FString FormatString;
	
	// Set up as many parameters as we can to try and resolve most of the string.
	// If using the graph, don't rely on GetDisplayOutputPathFromJob() because it may not take into account variables and other more complicated ways
	// the output path can be set. It's good for display purposes because it runs fast, but since ResolveOutputDirectoryFromJob() is not called
	// continuously from the UI, we can take a slower approach here and ensure that the fully correct path is fetched. This needs the evaluated graph.
	if (InJob->IsUsingGraphConfiguration())
	{
		UMovieGraphConfig* GraphPreset = InJob->GetGraphPreset();
		if (!GraphPreset)
		{
			OutError = LOCTEXT("ResolveOutputDir_MissingGraphPreset", "The job is set to use a Movie Render Graph but no graph asset is assigned.");
			return FString();
		}

		FString OutTraversalError;
		FMovieGraphTraversalContext TraversalContext;
		TraversalContext.Job = InJob;

		EvaluatedGraph = GraphPreset->CreateFlattenedGraph(TraversalContext, OutTraversalError);
		if (!EvaluatedGraph)
		{
			// Traversal failed (e.g. an invalid connection in the graph, a node from an unloaded plugin exists, etc).
			OutError = FText::Format(LOCTEXT("ResolveOutputDir_TraversalFailed", "Could not evaluate the Movie Render Graph for this job: {0}"),
				FText::FromString(OutTraversalError));
			return FString();
		}

		constexpr bool bIncludeCDOs = true;
		constexpr bool bExactMatch = true;
		UMovieGraphGlobalOutputSettingNode* OutputSettings = EvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);
		if (!OutputSettings)
		{
			OutError = LOCTEXT("ResolveOutputDir_MissingOutputSettings", "Could not find the Global Output Settings node in the evaluated graph.");
			return FString();
		}
		FormatString = OutputSettings->OutputDirectory.Path;
	}
	else
	{
		GetDisplayOutputPathFromJob(InJob, FormatString);
	}

	FString OutResolvedPath;

	FPaths::NormalizeFilename(FormatString);

	if (InJob->IsUsingGraphConfiguration())
	{
		const bool bGetNextVersion = false;
		FMovieGraphFilenameResolveParams Params;
		Params.Job = InJob;
		Params.InitializationTime = FDateTime::UtcNow();
		Params.InitializationTimeOffset = FDateTime::Now() - FDateTime::UtcNow();
		Params.Version = UMovieGraphBlueprintLibrary::ResolveVersionNumber(Params, bGetNextVersion);
		Params.RenderDataIdentifier.RootBranchName = UMovieGraphNode::GlobalsPinName;
		Params.EvaluatedConfig = EvaluatedGraph;

		FMovieGraphResolveArgs Dummy;
		OutResolvedPath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FormatString, Params, Dummy);
	}
	else
	{
		// By having it swap {camera_name} and {shot_name} with an unresolvable tag, it will
		// stay in the resolved path and can be removed using the code below.
		static const FString DummyTag = TEXT("{dontresolvethis}");
		const bool bGetNextVersion = false;
		FMoviePipelineFilenameResolveParams Params;
		Params.Job = InJob;
		Params.ShotNameOverride = DummyTag;
		Params.CameraNameOverride = DummyTag;
		Params.InitializationTime = FDateTime::UtcNow();
		Params.InitializationTimeOffset = FDateTime::Now() - FDateTime::UtcNow();
		Params.InitializationVersion = UMoviePipelineBlueprintLibrary::ResolveVersionNumber(Params, bGetNextVersion);

		FMoviePipelineFormatArgs Dummy;
		UMoviePipelineBlueprintLibrary::ResolveFilenameFormatArguments(FormatString, Params, OutResolvedPath, Dummy);
	}

	// Drop the extension if it exists
	FString Extension = FPaths::GetExtension(OutResolvedPath);
	if (Extension.Len() > 0 && OutResolvedPath.EndsWith(Extension))
	{
		OutResolvedPath.RemoveFromEnd(Extension);
	}

	if (FPaths::IsRelative(OutResolvedPath))
	{
		OutResolvedPath = FPaths::ConvertRelativePathToFull(OutResolvedPath);
	}

	// In the event that they used a {format_string} we couldn't resolve (such as shot name), then
	// we'll trim off anything after the format string.
	int32 FormatStringToken;
	if (OutResolvedPath.FindChar(TEXT('{'), FormatStringToken))
	{
		// Just as a last bit of saftey, we'll trim anything between the { and the preceeding /. This is
		// in case they did something like Render_{Date}, we wouldn't want to make a folder named Render_.
		// We search backwards from where we found the first { brace, so that will get us the last usable slash.
		int32 LastSlashIndex = OutResolvedPath.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd, FormatStringToken);
		if (LastSlashIndex != INDEX_NONE)
		{
			OutResolvedPath.LeftInline(LastSlashIndex + 1);
		}
	}

	return OutResolvedPath;
}

bool UMoviePipelineEditorBlueprintLibrary::SaveBasicConfigAsGraphConfig(UMoviePipelineExecutorJob* InJob)
{
	if (!InJob || !InJob->IsUsingBasicConfiguration())
	{
		return false;
	}

	UMoviePipelineBasicConfig* BasicConfig = InJob->GetBasicConfig();
	if (!BasicConfig)
	{
		return false;
	}

	UMovieGraphConfig* GeneratedGraph = UMoviePipelineBasicConfig::GenerateGraph(BasicConfig, GetTransientPackage());
	if (!GeneratedGraph)
	{
		return false;
	}

	// Show Save As dialog for the user to choose asset path/name
	FSaveAssetDialogConfig SaveDialogConfig;
	SaveDialogConfig.DialogTitleOverride = LOCTEXT("SaveAsGraph_DialogTitle", "Save As Movie Render Graph Config");
	SaveDialogConfig.DefaultPath = TEXT("/Game/");
	SaveDialogConfig.DefaultAssetName = TEXT("NewMovieRenderGraphConfig");
	SaveDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveDialogConfig);
	if (SaveObjectPath.IsEmpty())
	{
		return false; // User canceled
	}

	// Create package, duplicate generated graph into it, save
	const FString NewPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
	const FString NewAssetName = FPaths::GetBaseFilename(NewPackageName);

	UPackage* NewPackage = CreatePackage(*NewPackageName);
	NewPackage->MarkAsFullyLoaded();

	// The generated graph was built by programmatically modifying a duplicate of the default graph via
	// UMovieGraphConfig APIs (InsertBefore, AddLabeledEdge, etc.). These APIs only update the runtime
	// representation; the duplicated PipelineEdGraph still reflects the original default graph's connections.
	// Clear it so the editor rebuilds a fresh editor graph from the authoritative runtime data when opened.
	// See UMoviePipelineEdGraph::InitFromRuntimeGraph() for the stale GraphNode cleanup that complements this.
	if (GeneratedGraph->PipelineEdGraph)
	{
		GeneratedGraph->PipelineEdGraph->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
		GeneratedGraph->PipelineEdGraph = nullptr;
	}

	UMovieGraphConfig* SavedGraph = Cast<UMovieGraphConfig>(
		StaticDuplicateObject(GeneratedGraph, NewPackage, FName(*NewAssetName)));
	if (!SavedGraph)
	{
		return false;
	}
	SavedGraph->SetFlags(RF_Public | RF_Transactional | RF_Standalone);
	SavedGraph->MarkPackageDirty();

	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(NewPackage);
	if (!UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, true))
	{
		return false;
	}

	FAssetRegistryModule::AssetCreated(SavedGraph);

	// Switch the job to Graph mode using the saved asset
	FScopedTransaction Transaction(LOCTEXT("SaveBasicAsGraph_Transaction", "Save Basic Config as Graph Config"));
	InJob->Modify();
	InJob->SetGraphPreset(SavedGraph);

	return true;
}

#undef LOCTEXT_NAMESPACE // "MoviePipelineEditorBlueprintLibrary"
