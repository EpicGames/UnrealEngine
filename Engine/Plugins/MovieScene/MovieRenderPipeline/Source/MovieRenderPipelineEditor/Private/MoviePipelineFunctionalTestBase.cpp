// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineFunctionalTestBase.h"
#include "MoviePipeline.h"
#include "MoviePipelineQueue.h"
#include "Tests/AutomationCommon.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Misc/Paths.h"
#include "JsonObjectConverter.h"
#include "Misc/FileHelper.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineEditorBlueprintLibrary.h"
#include "ImageComparer.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Interfaces/IScreenShotManager.h"
#include "Interfaces/IScreenShotToolsModule.h"
#include "AutomationBlueprintFunctionLibrary.h"
#include "AutomationWorkerMessages.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineFunctionalTestBase)

AMoviePipelineFunctionalTestBase::AMoviePipelineFunctionalTestBase()
{
	ImageToleranceLevel = EImageTolerancePreset::IgnoreLess;
	bPerformDiff = true;
}


void AMoviePipelineFunctionalTestBase::PrepareTest() 
{
	if (!QueuePreset)
	{
		FinishTest(EFunctionalTestResult::Failed, TEXT("No Queue Preset asset specified, nothing to test!"));
	}

	if (QueuePreset->GetJobs().Num() == 0)
	{
		FinishTest(EFunctionalTestResult::Failed, TEXT("Queue Preset has no jobs, nothing to test!"));
	}

	if (QueuePreset->GetJobs().Num() > 1)
	{
		FinishTest(EFunctionalTestResult::Failed, TEXT("Only one job per queue currently supported!"));
	}
}

bool AMoviePipelineFunctionalTestBase::IsReady_Implementation()
{ 
	return Super::IsReady_Implementation();
}

void AMoviePipelineFunctionalTestBase::StartTest()
{
	ActiveQueue = NewObject<UMoviePipelineQueue>(GetWorld());
	ActiveQueue->CopyFrom(QueuePreset);

	// PrepareTest() guarantees exactly one job in the queue.
	UMoviePipelineExecutorJob* Job = ActiveQueue->GetJobs()[0];

	// PIE will already be running at this point so we want to instantiate an instance of
	// the Movie Pipeline in the current world and just run it. This doesn't test the UI/PIE 
	// portion of the system but that is more stable than the actual featureset.
	if (Job->IsUsingGraphConfiguration())
	{
		ActiveMoviePipeline = NewObject<UMovieGraphPipeline>(GetWorld());
	}
	else
	{
		ActiveMoviePipeline = NewObject<UMoviePipeline>(GetWorld());
	}

	ActiveMoviePipeline->OnMoviePipelineWorkFinished().AddUObject(this, &AMoviePipelineFunctionalTestBase::OnMoviePipelineFinished);
	ActiveMoviePipeline->OnMoviePipelineShotWorkFinished().AddUObject(this, &AMoviePipelineFunctionalTestBase::OnJobShotFinished);

	// Ensure we've initialized any transient settings (ie: the game overrides setting that is automatically added),
	// otherwise it won't get called.
	if (UMoviePipeline* PipelineAsLegacy = Cast<UMoviePipeline>(ActiveMoviePipeline))
	{
		Job->GetConfiguration()->InitializeTransientSettings();
		PipelineAsLegacy->Initialize(Job);
	}
	else if (UMovieGraphPipeline* PipelineAsGraph = Cast<UMovieGraphPipeline>(ActiveMoviePipeline))
	{
		PipelineAsGraph->Initialize(Job, FMovieGraphInitConfig());
	}
}

void AMoviePipelineFunctionalTestBase::OnJobShotFinished(FMoviePipelineOutputData InOutputData)
{
	if (ActiveMoviePipeline)
	{
		ActiveMoviePipeline->OnMoviePipelineShotWorkFinished().RemoveAll(this);
	}
}

void AMoviePipelineFunctionalTestBase::OnMoviePipelineFinished(FMoviePipelineOutputData InOutputData)
{
	if (ActiveMoviePipeline)
	{
		ActiveMoviePipeline->OnMoviePipelineWorkFinished().RemoveAll(this);
	}

	if (!InOutputData.bSuccess)
	{
		FinishTest(EFunctionalTestResult::Failed, TEXT("MoviePipeline encountered an internal error. See log for details."));
	}
	else
	{
		CompareRenderOutputToGroundTruth(InOutputData);
	}
}

bool SaveOutputToGroundTruth(const FString& GroundTruthDirectory, FMoviePipelineOutputData OutputData)
{
	FString GroundTruthFilepath = GroundTruthDirectory / "GroundTruth.json";

	// We need to rewrite the Output Data to be relative to our new directory, and then copy all
	// of the files from the old location to the new location. We want to keep these relative to
	// the original output directory from MRQ, so that we can make tests that ensure sub-folder
	// structures get generated correctly.
	FString OriginalRootOutputDirectory = UMoviePipelineEditorBlueprintLibrary::ResolveOutputDirectoryFromJob(OutputData.Job);

	// Gather the generated file paths for presets and graphs
	TArray<FString*> GeneratedFilePaths;
	if (OutputData.Job->IsUsingGraphConfiguration())
	{
		for (FMovieGraphRenderOutputData& Shot : OutputData.GraphData)
		{
			for (TPair<FMovieGraphRenderDataIdentifier, FMovieGraphRenderLayerOutputData>& RenderLayerData : Shot.RenderLayerData)
			{
				for (FString& FilePath : RenderLayerData.Value.FilePaths)
				{
					GeneratedFilePaths.Add(&FilePath);
				}
			}
		}
	}
	else
	{
		for (FMoviePipelineShotOutputData& Shot : OutputData.ShotData)
		{
			for (TPair<FMoviePipelinePassIdentifier, FMoviePipelineRenderPassOutputData>& Pair : Shot.RenderPassData)
			{
				for (FString& FilePath : Pair.Value.FilePaths)
				{
					GeneratedFilePaths.Add(&FilePath);
				}
			}
		}
	}

	// With all generated file paths known, rewrite them to be relative to the automation directory
	for (FString* FilePath : GeneratedFilePaths)
	{
		FString RelativePath = *FilePath;
		if (!FPaths::MakePathRelativeTo(/*Out*/ RelativePath, *OriginalRootOutputDirectory))
		{
			UE_LOGF(LogTemp, Error, "Could not generate ground truth, unable to generate relative paths.");
			return false;
		}

		FString NewPath = GroundTruthDirectory / RelativePath;
		FString AbsoluteNewPath = FPaths::ConvertRelativePathToFull(NewPath);
		if (IFileManager::Get().Copy(*AbsoluteNewPath, GetData(*FilePath)) != ECopyResult::COPY_OK)
		{
			UE_LOGF(LogTemp, Error, "Could not generate ground truth, unable to copy existing image to Automation directory.");
			return false;
		}

		// Now rewrite the FilePath in the struct to the new location so that when we save the json below it has the right path.
		FString RelativeToAutomationDir = NewPath;
		if (!FPaths::MakePathRelativeTo(/*InOut*/ RelativeToAutomationDir, *GroundTruthDirectory))
		{
			UE_LOGF(LogTemp, Error, "Could not generate ground truth, unable to generate relative paths.");
			return false;
		}

		*FilePath = RelativeToAutomationDir;
	}

	// Now that we've copied all of the images across we can serialize the struct to a json string
	FString SerializedJson;
	FJsonObjectConverter::UStructToJsonObjectString(OutputData, SerializedJson);

	// And finally write it to disk.
	FFileHelper::SaveStringToFile(SerializedJson, *GroundTruthFilepath);

	return true;
}

void AMoviePipelineFunctionalTestBase::CompareRenderOutputToGroundTruth(FMoviePipelineOutputData InOutputData)
{
	const bool bIsUsingGraph = InOutputData.Job->IsUsingGraphConfiguration();
	
	// Grab the (expected) resolution from the configuration (graph or preset)
	FString ErrorMsg;
	FIntPoint OutputResolution = GetExpectedOutputResolution(InOutputData, ErrorMsg);
	if (!ErrorMsg.IsEmpty())
	{
		FinishTest(EFunctionalTestResult::Failed, *ErrorMsg);
		return;
	}

	// Build screenshot data for this test. This contains a lot of metadata about RHI, Platform, etc that we'll use to generate the output folder name.
	FAutomationScreenshotData Data = UAutomationBlueprintFunctionLibrary::BuildScreenshotData(GetWorld(), TestLabel, OutputResolution.X, OutputResolution.Y);

	// Convert the Screenshot Data into Metadata
	const FImageTolerance ImageTolerance = GetImageToleranceForPreset(ImageToleranceLevel, CustomImageTolerance);

	FAutomationScreenshotMetadata MetaData(Data);
	MetaData.bHasComparisonRules = true;
	MetaData.ToleranceRed = ImageTolerance.Red;
	MetaData.ToleranceGreen = ImageTolerance.Green;
	MetaData.ToleranceBlue = ImageTolerance.Blue;
	MetaData.ToleranceAlpha = ImageTolerance.Alpha;
	MetaData.ToleranceMinBrightness = ImageTolerance.MinBrightness;
	MetaData.ToleranceMaxBrightness = ImageTolerance.MaxBrightness;
	MetaData.bIgnoreAntiAliasing = ImageTolerance.IgnoreAntiAliasing;
	MetaData.bIgnoreColors = ImageTolerance.IgnoreColors;
	MetaData.MaximumLocalError = ImageTolerance.MaximumLocalError;
	MetaData.MaximumGlobalError = ImageTolerance.MaximumGlobalError;

	IScreenShotToolsModule& ScreenShotModule = FModuleManager::LoadModuleChecked<IScreenShotToolsModule>("ScreenShotComparisonTools");
	IScreenShotManagerPtr ScreenshotManager = ScreenShotModule.GetScreenShotManager();

	// Now we know where to look for our ground truth data.
	TArray<FString> GroundTruthFilenames = ScreenshotManager->FindApprovedFiles(MetaData, TEXT("GroundTruth.json"));
	if (GroundTruthFilenames.Num() == 0)
	{
		FString IdealReportDirectory = ScreenshotManager->GetIdealApprovedFolderForImage(MetaData);
		UE_LOGF(LogTemp, Error, "Failed to find a GroundTruth file at %ls, generating one now. Rerun the test after verifying them!", *IdealReportDirectory);
		SaveOutputToGroundTruth(IdealReportDirectory, InOutputData);
		FinishTest(EFunctionalTestResult::Failed, TEXT("Generated ground truth, run test again after verifying the ground truth is correct."));
		return;
	}
	FString GroundTruthFilename = GroundTruthFilenames[0];
	UE_LOGF(LogTemp, Log, "GroundTruth file located at %ls", *GroundTruthFilename);
	FString ReportDirectory = FPaths::GetPath(GroundTruthFilename);

	// The ground truth file exists, so we can load it and turn it back into a FMoviePipelineOutputData struct.
	FString LoadedGroundTruthJsonStr;
	FFileHelper::LoadFileToString(LoadedGroundTruthJsonStr, *GroundTruthFilename);

	FMoviePipelineOutputData GroundTruthData;
	FJsonObjectConverter::JsonObjectStringToUStruct<FMoviePipelineOutputData>(LoadedGroundTruthJsonStr, &GroundTruthData);

	// Do some basic checks on our new data to ensure it output the expected number of files/shots/etc, before doing
	// the computationally expensive image comparisons.
	int32 NumShotsNew = bIsUsingGraph ? InOutputData.GraphData.Num() : InOutputData.ShotData.Num();
	int32 NumShotsGT = bIsUsingGraph ? GroundTruthData.GraphData.Num() : GroundTruthData.ShotData.Num();
	if (NumShotsNew != NumShotsGT)
	{
		FinishTest(EFunctionalTestResult::Failed, FString::Printf(TEXT("Mismatched number of shots between GroudTruth and New. Expected %d got %d."), NumShotsGT, NumShotsNew));
		return;
	}
		
	TMap<FString, FString> OldToNewImagesToCompare;
	
	for (int32 ShotIndex = 0; ShotIndex < NumShotsGT; ShotIndex++)
	{
		// And then repeat this for each render pass in the shot
		int32 NumRenderPassesNew = bIsUsingGraph ? InOutputData.GraphData[ShotIndex].RenderLayerData.Num() : InOutputData.ShotData[ShotIndex].RenderPassData.Num();
		int32 NumRenderPassesGT = bIsUsingGraph ? GroundTruthData.GraphData[ShotIndex].RenderLayerData.Num() : GroundTruthData.ShotData[ShotIndex].RenderPassData.Num();
		if (NumRenderPassesNew != NumRenderPassesGT)
		{
			FinishTest(EFunctionalTestResult::Failed, FString::Printf(TEXT("Mismatched number of shots between GroudTruth and New. Expected %d got %d."), NumRenderPassesGT, NumRenderPassesNew));
			return;
		}

		// Update the map that links a ground truth image to its equivalent test-generated image.
		if (bIsUsingGraph)
		{
			UpdateGroundTruthToTestImageMap(GroundTruthData.GraphData[ShotIndex].RenderLayerData, InOutputData.GraphData[ShotIndex].RenderLayerData, ReportDirectory, OldToNewImagesToCompare);
		}
		else
		{
			UpdateGroundTruthToTestImageMap(GroundTruthData.ShotData[ShotIndex].RenderPassData, InOutputData.ShotData[ShotIndex].RenderPassData, ReportDirectory, OldToNewImagesToCompare);
		}
	}
		
	if (bPerformDiff)
	{
		// Time for the computationally expensive part, doing image comparisons!
		TSharedPtr<FImageComparisonResult> ComparisonResult = ScreenshotManager->CompareImageSequence(OldToNewImagesToCompare, MetaData);
		if (ComparisonResult.IsValid() && !ComparisonResult->AreSimilar())
		{
			ScreenshotManager->NotifyAutomationTestFrameworkOfImageComparison(*ComparisonResult);
			FinishTest(EFunctionalTestResult::Failed, TEXT("Frames failed comparison tolerance!"));
			return;
		}
	}

	UE_LOGF(LogTemp, Display, "All image sequences from %ls are similar to the Ground Truth.", *MetaData.ScreenShotName);

	FinishTest(EFunctionalTestResult::Succeeded, TEXT(""));
}

FIntPoint AMoviePipelineFunctionalTestBase::GetExpectedOutputResolution(const FMoviePipelineOutputData& InOutputData, FString& OutErrorMsg) const
{
	OutErrorMsg.Empty();
	
	const bool bIsUsingGraph = InOutputData.Job->IsUsingGraphConfiguration();
	
	// Grab the (expected) resolution from the configuration (graph or preset)
	FIntPoint OutputResolution(0, 0);
	if (bIsUsingGraph)
	{
		FMovieGraphTraversalContext TraversalContext;
		TraversalContext.Job = InOutputData.Job;

		FString OutEvaluationError;
		UMovieGraphEvaluatedConfig* EvaluatedGraph = InOutputData.Job->GetGraphPreset()->CreateFlattenedGraph(TraversalContext, OutEvaluationError);
		if (!EvaluatedGraph)
		{
			OutErrorMsg = FString::Printf(TEXT("Unable to create evaluated graph: %s"), *OutEvaluationError);
			return OutputResolution;
		}

		OutputResolution = UMovieGraphBlueprintLibrary::GetDesiredOutputResolution(EvaluatedGraph);
	}
	else
	{
		const UMoviePipelineOutputSetting* OutputSetting = Cast<UMoviePipelineOutputSetting>(InOutputData.Job->GetConfiguration()->FindOrAddSettingByClass(UMoviePipelineOutputSetting::StaticClass()));
		if (!OutputSetting)
		{
			OutErrorMsg = FString(TEXT("Unable to find the Output setting in the configuration."));
			return OutputResolution;
		}
		
		OutputResolution = OutputSetting->OutputResolution;
	}

	return OutputResolution;
}

template<typename IdentifierType, typename OutputDataType>
void AMoviePipelineFunctionalTestBase::UpdateGroundTruthToTestImageMap(
	const TMap<IdentifierType, OutputDataType>& InGroundTruthData,
	const TMap<IdentifierType, OutputDataType>& InTestData,
	const FString& InReportDirectory,
	TMap<FString, FString>& OutGroundTruthToTestImageMap)
{
	for (const TPair<IdentifierType, OutputDataType>& GroundTruthPair : InGroundTruthData)
	{
		const OutputDataType* TestOutputData = InTestData.Find(GroundTruthPair.Key);
		if (!TestOutputData)
		{
			FinishTest(EFunctionalTestResult::Failed, FString::Printf(TEXT("Did not find render pass [%s] that is in the Ground Truth."), *LexToString(GroundTruthPair.Key)));
			return;
		}
		
		// Now we can check that they output the same number of files
		const int32 NumOutputFilesTest = TestOutputData->FilePaths.Num();
		const int32 NumOutputFilesGT = GroundTruthPair.Value.FilePaths.Num();
		if (NumOutputFilesTest != NumOutputFilesGT)
		{
			FinishTest(EFunctionalTestResult::Failed, FString::Printf(TEXT("Mismatched number of shots between GroudTruth and New. Expected %d, got %d."), NumOutputFilesGT, NumOutputFilesTest));
			return;
		}
			
		// Now we'll just loop through them in lockstep (as MRQ should put them in order anyways) and just add
		// them to an array to be tested later. We don't test that the sub-file names are exact matches right now.
		for (int32 Index = 0; Index < NumOutputFilesGT; Index++)
		{
			FString AbsoluteGTPath = FPaths::ConvertRelativePathToFull(FPaths::CreateStandardFilename(InReportDirectory / GroundTruthPair.Value.FilePaths[Index]));
			FString AbsoluteNewPath = FPaths::ConvertRelativePathToFull(FPaths::CreateStandardFilename(TestOutputData->FilePaths[Index]));
			OutGroundTruthToTestImageMap.Add(AbsoluteGTPath, AbsoluteNewPath);
		}
	}
}