// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "FunctionalTest.h"
#include "Misc/AutomationTest.h"
#include "MovieRenderPipelineDataTypes.h"
#include "ImageComparer.h"

#include "MoviePipelineFunctionalTestBase.generated.h"

#define UE_API MOVIERENDERPIPELINEEDITOR_API

class UMoviePipelineQueue;
class UMoviePipeline;

/**
* Base class for Movie Pipeline functional tests which render pre-made queues
* and compare their output against pre-existing render outputs.
*/
UCLASS(MinimalAPI, Blueprintable)
class AMoviePipelineFunctionalTestBase : public AFunctionalTest
{
	GENERATED_BODY()

public:
	UE_API AMoviePipelineFunctionalTestBase();

protected:
	// AFunctionalTest
	UE_API virtual void PrepareTest() override;
	UE_API virtual bool IsReady_Implementation() override;
	UE_API virtual void StartTest() override;
	// ~AFunctionalTest
	virtual bool IsEditorOnlyLoadedInPIE() const override
	{
		return true;
	}

	UE_API void OnJobShotFinished(FMoviePipelineOutputData InOutputData);
	UE_API void OnMoviePipelineFinished(FMoviePipelineOutputData InOutputData);
	UE_API void CompareRenderOutputToGroundTruth(FMoviePipelineOutputData InOutputData);

private:
	/**
	 * Gets the resolution that the test-generated images are expected to be.
	 *
	 * If there was an error getting this, OutErrorMsg will be populated with the error message, and the returned point will be (0, 0).
	 */
	FIntPoint GetExpectedOutputResolution(const FMoviePipelineOutputData& InOutputData, FString& OutErrorMsg) const;

	/**
	 * Updates OutGroundTruthToTestImageMap by linking frames in InGroundTruthData to those in InTestData.
	 *
	 * Keys in OutGroundTruthToTestImageMap are the ground-truth frame path (absolute), and values are the test-generated frame path (absolute).
	 */
	template<typename IdentifierType, typename OutputDataType>
	void UpdateGroundTruthToTestImageMap(
		const TMap<IdentifierType, OutputDataType>& InGroundTruthData,
		const TMap<IdentifierType, OutputDataType>& InTestData,
		const FString& InReportDirectory,
		TMap<FString, FString>& OutGroundTruthToTestImageMap);

protected:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Movie Pipeline")
	TObjectPtr<UMoviePipelineQueue> QueuePreset;

	UPROPERTY(EditAnywhere, Category = "Movie Pipeline")
	EImageTolerancePreset ImageToleranceLevel;

	UPROPERTY(EditAnywhere, Category = "Movie Pipeline")
	FImageTolerance CustomImageTolerance;

	/**
	* Boolean flag to toggle screenshot pixel comparison of rendered versus ground truth images.
	* True - screenshot pixel comparison will be performed.
	* False - screenshot pixel comparison will not be performed.  Expects ground truth images to exist to verify that the required number of images were rendered, even if those ground truth images are not used in diffs.
	*/	
	UPROPERTY(EditAnywhere, Category = "Movie Pipeline")
	bool bPerformDiff;

protected:
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineBase> ActiveMoviePipeline;

	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineQueue> ActiveQueue;
};

#undef UE_API
