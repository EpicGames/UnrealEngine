// Copyright Epic Games, Inc. All Rights Reserved.
#include "MoviePipelineNewProcessExecutor.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/StructuredArchive.h"
#include "MoviePipelineQueue.h"
#include "ObjectTools.h"
#include "MoviePipelineInProcessExecutorSettings.h"
#include "MovieRenderPipelineCoreModule.h"
#include "FileHelpers.h"
#include "MovieRenderPipelineSettings.h"
#include "PackageHelperFunctions.h"
#include "UnrealEdMisc.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "MoviePipelineEditorBlueprintLibrary.h"
#include "MoviePipelineGameMode.h"
#include "MoviePipelineUtils.h"
#include "Misc/StringBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineNewProcessExecutor)

#define LOCTEXT_NAMESPACE "MoviePipelineNewProcessExecutor"

void UMoviePipelineNewProcessExecutor::Execute_Implementation(UMoviePipelineQueue* InPipelineQueue)
{
	if (InPipelineQueue->GetJobs().Num() == 0)
	{
		OnExecutorFinishedImpl();
		return;
	}

	// Because it's run in a new process it will load packages from disk, thus we have to save changes to see them.
	bool bPromptUserToSave = true;
	bool bSaveMapPackages = true;
	bool bSaveContentPackages = true;
	if (!FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages))
	{
		OnExecutorFinishedImpl();
		return;
	}

	// Make sure all of the maps in the queue exist on disk somewhere, otherwise the remote process boots up and then fails.
	bool bHasValidMap = UMoviePipelineEditorBlueprintLibrary::IsMapValidForRemoteRender(InPipelineQueue->GetJobs());
	if (!bHasValidMap)
	{
		UMoviePipelineEditorBlueprintLibrary::WarnUserOfUnsavedMap();
		OnExecutorFinishedImpl();
		return;
	}

	if (!ensureMsgf(!ProcessHandle.IsValid(), TEXT("Attempted to start another New Process Executor without the last one quitting. Force killing. This executor only supports one at a time.")))
	{
		FPlatformProcess::TerminateProc(ProcessHandle, true);
	}

	// Arguments to pass to the executable. This can be modified by settings in the event a setting needs to be applied early. In the format of -foo -bar
	FString CommandLineArgs;
	// In the format of ?arg=1?arg=2. This is appended after the the map name.
	FString UnrealURLParams;

	// Append all of our inherited command line arguments from the editor.
	const UMoviePipelineInProcessExecutorSettings* ExecutorSettings = GetDefault<UMoviePipelineInProcessExecutorSettings>();

	CommandLineArgs += ExecutorSettings->InheritedCommandLineArguments;
	CommandLineArgs += TEXT(" ") + ExecutorSettings->AdditionalCommandLineArguments;

	// Provide our own default arguments
	CommandLineArgs += FString::Printf(TEXT(" -messaging -SessionName=\"%s\""), TEXT("NewProcess Movie Render"));
	CommandLineArgs += TEXT(" -nohmd");
	CommandLineArgs += TEXT(" -windowed");
	CommandLineArgs += FString::Printf(TEXT(" -ResX=%d -ResY=%d"), 1280, 720);

	// Place the Queue in a package and serialize it to disk so we can pass their dynamic object
	// to another process without having to save/check in/etc.
	FString ManifestFilePath;
	UMoviePipelineQueue* DuplicatedQueue = UMoviePipelineEditorBlueprintLibrary::SaveQueueToManifestFile(InPipelineQueue, ManifestFilePath);
	if (!DuplicatedQueue)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Could not save manifest package to disk. Path: %ls", *ManifestFilePath);
		OnExecutorFinishedImpl();
		return;
	}

	// Boot into our custom Game Mode (to go with our custom map). Once booted the in-process Executor will load the correct map with correct gamemode.
	UnrealURLParams += FString::Printf(TEXT("?game=%s"), *AMoviePipelineGameMode::StaticClass()->GetPathName());

	TArray<FString> OutUrlParams;
	TArray<FString> OutCommandLineArgs;
	TArray<FString> OutDeviceProfileCVars;
	TArray<FString> OutExecCmds;

	// Loop through our settings in the job and let them modify the command line arguments/params. Because we could have multiple jobs,
	// we go through all jobs and all settings and hope the user doesn't have conflicting settings.
	for (UMoviePipelineExecutorJob* Job : DuplicatedQueue->GetJobs())
	{
		// Only jobs that are enabled should be able to contribute to the command line being built,
		// as non-enabled jobs won't render.
		if (!Job || !Job->IsEnabled())
		{
			continue;
		}

		// Determine the graph to evaluate. For Basic mode, generate a transient graph from the Basic config.
		UMovieGraphConfig* GraphConfig = nullptr;
		if (Job->IsUsingGraphConfiguration())
		{
			GraphConfig = Job->GetGraphPreset();
		}
		else if (Job->IsUsingBasicConfiguration())
		{
			GraphConfig = UMoviePipelineBasicConfig::GenerateGraph(Job->GetBasicConfig(), GetTransientPackage());
		}

		if (GraphConfig)
		{
			FMovieGraphTraversalContext TraversalContext;
			TraversalContext.Job = Job;

			FString OutTraversalError;
			UMovieGraphEvaluatedConfig* EvaluatedGraph = GraphConfig->CreateFlattenedGraph(TraversalContext, OutTraversalError);
			if (!OutTraversalError.IsEmpty())
			{
				return;
			}

			// Only nodes on the Globals branch can apply command line args. Get CDOs here as well, so all nodes which define command line args
			// will be found, even if they were not instantiated in the graph.
			TArray<UClass*> AllSettingsNodeClasses = UE::MovieRenderPipeline::FindMoviePipelineSettingClasses(UMovieGraphSettingNode::StaticClass(), false);
			for (UClass* NodeClass : AllSettingsNodeClasses)
			{
				constexpr bool bIncludeCDOs = true;
				constexpr bool bExactMatch = true;
				UMovieGraphSettingNode* GlobalsNode = EvaluatedGraph->GetSettingForBranch(NodeClass, UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);
				if (!GlobalsNode || (GlobalsNode->GetBranchRestriction() != EMovieGraphBranchRestriction::Globals))
				{
					continue;
				}

				GlobalsNode->BuildNewProcessCommandLineArgsImpl(OutUrlParams, OutCommandLineArgs, OutDeviceProfileCVars, OutExecCmds);
			}
		}
		else
		{
			Job->GetConfiguration()->InitializeTransientSettings();
			for (const UMoviePipelineSetting* Setting : Job->GetConfiguration()->GetAllSettings())
			{
				Setting->BuildNewProcessCommandLineArgs(OutUrlParams, OutCommandLineArgs, OutDeviceProfileCVars, OutExecCmds);
			}
		}
	}

	for (const FString& UrlParam : OutUrlParams)
	{
		UnrealURLParams += UrlParam;
	}

	CommandLineArgs += TEXT(" ");
	for (const FString& Arg : OutCommandLineArgs)
	{
		CommandLineArgs += FString::Printf(TEXT(" %s"), *Arg);
	}

	if (OutDeviceProfileCVars.Num() > 0)
	{
		CommandLineArgs += TEXT(" -dpcvars=\"");
		for (const FString& Cvar : OutDeviceProfileCVars)
		{
			CommandLineArgs += FString::Printf(TEXT("%s,"), *Cvar);
		}
		CommandLineArgs += TEXT("\"");
	}

	if (OutExecCmds.Num() > 0)
	{
		CommandLineArgs += TEXT(" -execcmds=\"");
		for (const FString& Cmd : OutExecCmds)
		{
			CommandLineArgs += FString::Printf(TEXT("%s,"), *Cmd);
		}
		CommandLineArgs += TEXT("\"");
	}

	FString GameNameOrProjectFile;
	if (FPaths::IsProjectFilePathSet())
	{
		GameNameOrProjectFile = FString::Printf(TEXT("\"%s\""), *FPaths::GetProjectFilePath());
	}
	else
	{
		GameNameOrProjectFile = FApp::GetProjectName();
	}

	FString MoviePipelineArgs;
	{
		// We will pass the path to the saved manifest file on the command line and parse it on the other end from disk.
		// This is assumed to be relative to the game's Saved directory on load.
		FString PipelineConfig = TEXT("MovieRenderPipeline/QueueManifest.utxt");
		
		// Because the Queue has multiple jobs in it, we don't need to pass which sequence to render. That's only needed if you're rendering a 
		// specific sequence with a specific primary config.
		MoviePipelineArgs = FString::Printf(TEXT("-MoviePipelineConfig=\"%s\""), *PipelineConfig); // -MoviePipeline=\"%s\" -MoviePipelineLocalExecutorClass=\"%s\" -MoviePipelineClass=\"%s\""),
	}

	TMap<FString, FStringFormatArg> NamedArguments;
	NamedArguments.Add(TEXT("GameNameOrProjectFile"), GameNameOrProjectFile);
	NamedArguments.Add(TEXT("PlayWorld"), TEXT("MoviePipelineEntryMap")); // Boot up on an empty map, the executor will immediately transition to the correct one.
	NamedArguments.Add(TEXT("UnrealURL"), UnrealURLParams); // Pass the command line arguments for this job
	TStringBuilder<64> SubprocessCommandLine;
	FCommandLine::BuildSubprocessCommandLine(ECommandLineArgumentFlags::GameContexts, false /*bOnlyInherited*/, SubprocessCommandLine);
	NamedArguments.Add(TEXT("SubprocessCommandLine"), *SubprocessCommandLine);
	NamedArguments.Add(TEXT("CommandLineParams"), CommandLineArgs);
	NamedArguments.Add(TEXT("MoviePipelineArgs"), MoviePipelineArgs);

	FString FinalCommandLine = FString::Format(
		TEXT("{GameNameOrProjectFile} {PlayWorld}{UnrealURL} -game {SubprocessCommandLine} {CommandLineParams} {MoviePipelineArgs}"), NamedArguments);

	// Prefer the -Cmd version of the executable if possible, gracefully falling back to the normal one. This is to help
	// with user education about the -Cmd version which allows piping the output log to the cmd window that launched it.
	const FString ExecutablePath = FUnrealEdMisc::Get().GetExecutableForCommandlets();
		
	uint32 ProcessID = 0;
	const bool bLaunchDetatched = true;
	const bool bLaunchMinimized = false;
	const bool bLaunchWindowHidden = false;
	const uint32 PriorityModifier = 0;

	UE_LOGF(LogMovieRenderPipeline, Log, "Launching a new process to render with the following command line:");
	UE_LOGF(LogMovieRenderPipeline, Log, "%ls %ls", *ExecutablePath, *FinalCommandLine);
	
	ProcessHandle = FPlatformProcess::CreateProc(
		*ExecutablePath, *FinalCommandLine, bLaunchDetatched,
		bLaunchMinimized, bLaunchWindowHidden, &ProcessID,
		PriorityModifier, nullptr, nullptr, nullptr);

	if (!ProcessHandle.IsValid())
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to launch executable for new process render. Executable Path: \"%ls\" Command Line: \"%ls\"", FPlatformProcess::ExecutablePath(), *FinalCommandLine);
		// OnPipelineErrored(nullptr, true, LOCTEXT("ProcessFailedToLaunch", "New Process failed to launch. See log for command line argument used."));
		// OnIndividualPipelineFinished(nullptr);
	}
	else
	{
		if (ExecutorSettings->bCloseEditor)
		{
			FPlatformMisc::RequestExit(false);
		}
		else
		{
			// Register a tick handler to listen every frame to see if the process shut down gracefully, we'll use return codes to tell success vs cancel.
			FCoreDelegates::OnBeginFrame.AddUObject(this, &UMoviePipelineNewProcessExecutor::CheckForProcessFinished);
		}
	}
}

void UMoviePipelineNewProcessExecutor::CheckForProcessFinished()
{
	if (!ensureMsgf(ProcessHandle.IsValid(), TEXT("CheckForProcessFinished called without a valid process handle. This should only be called if the process was originally valid!")))
	{
		return;
	}

	int32 ReturnCode;
	if (FPlatformProcess::GetProcReturnCode(ProcessHandle, &ReturnCode))
	{
		ProcessHandle.Reset();
		FCoreDelegates::OnBeginFrame.RemoveAll(this);

		// Log an error for now
		if (ReturnCode != 0)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Process exited with non-success return code. Return Code; %d", ReturnCode);
			// OnPipelineErrored(nullptr, true, LOCTEXT("ProcessNonZeroReturn", "Non-success return code returned. See log for Return Code."));
		}

		OnExecutorFinishedImpl();
	}
	else
	{
		// Process is still running, spin wheels.
	}
}

void UMoviePipelineNewProcessExecutor::CancelAllJobs_Implementation()
{
	if (!ensureMsgf(ProcessHandle.IsValid(), TEXT("Attempting to cancel UMoviePipelineNewProcessExecutor job without a valid process handle. This should only be called if the process was originally valid!")))
	{
		return;
	}

	if (FPlatformProcess::IsProcRunning(ProcessHandle))
	{
		// Process is still running, try to kill it.
		FPlatformProcess::TerminateProc(ProcessHandle);
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Warning, "Attempting to cancel UMoviePipelineNewProcessExecutor job but process has already exited.");
	}
}

#undef LOCTEXT_NAMESPACE // "MoviePipelineNewProcessExecutor"

