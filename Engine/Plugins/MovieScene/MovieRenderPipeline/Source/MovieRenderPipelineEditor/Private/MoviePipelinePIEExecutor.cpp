// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelinePIEExecutor.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineShotConfig.h"
#include "MoviePipeline.h"
#include "MoviePipelineGameOverrideSetting.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePostRenderFileDisplayProcessor.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Framework/Application/SlateApplication.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MovieRenderPipelineSettings.h"
#include "Editor/EditorEngine.h"
#include "Editor.h"
#include "HAL/FileManager.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipelinePIEExecutorSettings.h"
#include "MoviePipelineEditorBlueprintLibrary.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "MessageLogModule.h"
#include "Logging/MessageLog.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieRenderGraphEditorSettings.h"
#include "Graph/Nodes/MovieGraphGlobalGameOverrides.h"
#include "LevelSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelinePIEExecutor)

#define LOCTEXT_NAMESPACE "MoviePipelinePIEExecutor"


const TArray<FString> UMoviePipelinePIEExecutor::FValidationMessageGatherer::AllowList = { "LogMovieRenderPipeline", "LogMovieRenderPipelineIO", "LogMoviePipelineExecutor", "LogImageWriteQueue", "LogAppleProResMedia", "LogAvidDNxMedia"};

UMoviePipelinePIEExecutor::FValidationMessageGatherer::FValidationMessageGatherer()
	: FOutputDevice()
	, ExecutorLog()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions MessageLogOptions;
	MessageLogOptions.bShowPages = true;
	MessageLogOptions.bAllowClear = true;
	MessageLogOptions.MaxPageCount = 10;
	MessageLogOptions.bShowFilters = true;
	MessageLogModule.RegisterLogListing("MoviePipelinePIEExecutor", LOCTEXT("MoviePipelineExecutorLogLabel", "High Quality Media Export"));

	ExecutorLog = MakeUnique<FMessageLog>("MoviePipelinePIEExecutor");
}

void UMoviePipelinePIEExecutor::FValidationMessageGatherer::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	for (const FString& AllowedCategory : AllowList)
	{
		if (Category.ToString().Equals(AllowedCategory))
		{
			if (Verbosity == ELogVerbosity::Warning)
			{
				ExecutorLog->Warning(FText::FromString(FString(V)));
			}
			else if (Verbosity == ELogVerbosity::Error)
			{
				ExecutorLog->Error(FText::FromString(FString(V)));
			}
			return;
		}
	}
}

UMoviePipelinePIEExecutor::UMoviePipelinePIEExecutor()
	: UMoviePipelineLinearExecutorBase()
	, bRenderOffscreen(false)
	, RemainingInitializationFrames(-1)
	, bPreviousUseFixedTimeStep(false)
	, PreviousFixedTimeStepDelta(1 / 30.0)
	, bAllowUnsavedLevels(false)
{
	if (!IsTemplate() && FSlateApplication::IsInitialized())
	{
		if (!FApp::CanEverRender() || FSlateApplication::Get().IsRenderingOffScreen())
		{
			SetIsRenderingOffscreen(true);
		}
	}
}

void UMoviePipelinePIEExecutor::SetAllowUsingUnsavedLevels(const bool bShouldAllow)
{
	UE_LOGF(LogMovieRenderPipeline, Log, "Updating the PIE Executor to allow using unsaved levels.");
	
	bAllowUnsavedLevels = bShouldAllow;
}

void UMoviePipelinePIEExecutor::Start(const UMoviePipelineExecutorJob* InJob)
{
	Super::Start(InJob);

	// Check for null sequence
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(InJob->Sequence.TryLoad());
	if (!LevelSequence)
	{
		if(!IsRenderingOffscreen())
		{
			FText FailureReason = LOCTEXT("InvalidSequenceFailureDialog", "One or more jobs in the queue has an invalid/null sequence.");
			FMessageDialog::Open(EAppMsgType::Ok, FailureReason);
		}

		OnExecutorFinishedImpl();
		return;
	}

	// Check for unsaved maps. It's pretty rare that someone actually wants to execute on an unsaved map,
	// and it catches the much more common case of adding the job to an unsaved map and then trying to render
	// from a newly loaded map, PIE startup will fail because the map is no longer valid.
	// Note that opting-in to allowing unsaved levels is permitted (Quick Render is a good example of when this is OK).
	const bool bAllMapsValid = UMoviePipelineEditorBlueprintLibrary::IsMapValidForRemoteRender(Queue->GetJobs());
	if (!bAllMapsValid && !bAllowUnsavedLevels)
	{
		if(!IsRenderingOffscreen())
		{
			FText FailureReason = LOCTEXT("UnsavedMapFailureDialog", "One or more jobs in the queue have an unsaved map as their target map. Maps must be saved at least once before rendering, and then the job must be manually updated to point to the newly saved map.");
			FMessageDialog::Open(EAppMsgType::Ok, FailureReason);
		}

		OnExecutorFinishedImpl();
		return;
	}

	// Start capturing logging messages
	ValidationMessageGatherer.StartGathering();

	FVector2D WindowSize = FVector2D(1280, 720);

	// Allow creating the PIE window at the size of the output to work around UMG scaling bugs when the main viewport is one resolution and then we re-render it at another.
	const UMoviePipelinePIEExecutorSettings* ExecutorSettings = GetDefault<UMoviePipelinePIEExecutorSettings>();
	if (ExecutorSettings->bResizePIEWindowToOutputResolution)
	{
		UMovieGraphConfig* GraphConfig = InJob->GetGraphPreset();
		if (InJob->IsUsingGraphConfiguration() && GraphConfig)
		{
			// Like the GameMode below, this isn't a perfect evaluation, and is further complicated by things like Adapt Output Resolution and Overscan
			// but it should work in the basic case.
			FMovieGraphTraversalContext TraversalContext;
			TraversalContext.Job = const_cast<UMoviePipelineExecutorJob*>(InJob);

			FString OutTraversalError;
			if (UMovieGraphEvaluatedConfig* EvaluatedGraph = GraphConfig->CreateFlattenedGraph(TraversalContext, OutTraversalError))
			{
				WindowSize = UMovieGraphBlueprintLibrary::GetOverscannedResolution(EvaluatedGraph);
			}
			else
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to evaluate graph to determine output resolution. Traversal error: %ls", *OutTraversalError);
			}
		}
		else if (InJob->IsUsingBasicConfiguration())
		{
			// Basic-mode jobs generate their graph later in DuplicateJobAndConfiguration().
			// Use the BasicConfig's output resolution directly for the PIE window size.
			if (const UMoviePipelineBasicConfig* BasicCfg = InJob->GetBasicConfig())
			{
				const FMovieGraphNamedResolution& Resolution = BasicCfg->OutputResolution;
				WindowSize = FVector2D(Resolution.Resolution.X, Resolution.Resolution.Y);
			}
		}
		else
		{
			UMoviePipelineOutputSetting* OutputSetting = Cast<UMoviePipelineOutputSetting>(InJob->GetConfiguration()->FindOrAddSettingByClass(UMoviePipelineOutputSetting::StaticClass()));
			WindowSize = FVector2D(OutputSetting->OutputResolution.X, OutputSetting->OutputResolution.Y);
		}
	}


	// Create a Slate window to hold our preview.
	TSharedRef<SWindow> CustomWindow = SNew(SWindow)
		.ClientSize(WindowSize)
		.AutoCenter(EAutoCenter::PrimaryWorkArea)
		.UseOSWindowBorder(true)
		.FocusWhenFirstShown(true)
		.ActivationPolicy(EWindowActivationPolicy::Never)
		.HasCloseButton(true)
		.SupportsMaximize(true)
		.SupportsMinimize(true)
		.SizingRule(ESizingRule::UserSized);

	CustomWindow->SetAcceptsInput(false);

	WeakCustomWindow = CustomWindow;
	FSlateApplication::Get().AddWindow(CustomWindow, !IsRenderingOffscreen());

	// Initialize our own copy of the Editor Play settings which we will adjust defaults on.
	ULevelEditorPlaySettings* PlayInEditorSettings = NewObject<ULevelEditorPlaySettings>();
	PlayInEditorSettings->SetPlayNetMode(EPlayNetMode::PIE_Standalone);
	PlayInEditorSettings->SetPlayNumberOfClients(1);
	PlayInEditorSettings->bLaunchSeparateServer = false;
	PlayInEditorSettings->SetRunUnderOneProcess(true);
	PlayInEditorSettings->LastExecutedPlayModeType = EPlayModeType::PlayMode_InEditorFloating;
	PlayInEditorSettings->bUseNonRealtimeAudioDevice = true;

	FRequestPlaySessionParams Params;
	Params.EditorPlaySettings = PlayInEditorSettings;
	Params.CustomPIEWindow = CustomWindow;
	Params.GlobalMapOverride = InJob->Map.GetAssetPathString();
	// Don't allow the online subsystem to try and authenticate as it will delay PIE startup and no PIE world will exist when PostPIEStarted is called.
	Params.bAllowOnlineSubsystem = false;

	// Initialize the transient settings so that they will exist in time for the GameOverrides check.
	InJob->GetConfiguration()->InitializeTransientSettings();

	if (TSubclassOf<AGameModeBase> GameModeOverride = UMovieGraphGlobalGameOverridesNode::GetGameModeOverride(InJob))
	{
		Params.GameModeOverride = GameModeOverride;
	}

	bPreviousUseFixedTimeStep = FApp::UseFixedTimeStep();
	PreviousFixedTimeStepDelta = FApp::GetFixedDeltaTime();

	// Force the engine into fixed timestep mode. It's going to get overridden on the first frame by the movie pipeline,
	// and everything controlled by Sequencer will use the correct timestep for renders but non-controlled things (such
	// as pawns) use an uncontrolled DT on the first frame which lowers determinism.
	FApp::SetUseFixedTimeStep(true);
	FApp::SetFixedDeltaTime(InJob->GetConfiguration()->GetEffectiveFrameRate(LevelSequence).AsInterval());

	// Kick off an async request to start a play session. This won't happen until the next frame.
	GEditor->RequestPlaySession(Params);

	// Listen for PIE startup since there's no current way to pass a delegate through the request.
	FEditorDelegates::PostPIEStarted.AddUObject(this, &UMoviePipelinePIEExecutor::OnPIEStartupFinished);
}

void UMoviePipelinePIEExecutor::OnPIEStartupFinished(bool)
{
	// Immediately un-bind our delegate so that we don't catch all PIE startup requests in the future.
	FEditorDelegates::PostPIEStarted.RemoveAll(this);

	// Hack to find out the PIE world since it is not provided by the delegate.
	UWorld* ExecutingWorld = nullptr;
	
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			ExecutingWorld = Context.World();
		}
	}
	
	if(!ExecutingWorld)
	{
		// This only happens if PIE startup fails and they've usually gotten a pop-up dialog already.
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to find a PIE UWorld after OnPIEStartupFinished!");
		OnExecutorFinishedImpl();
		return;
	}

	// Only mark us as rendering once we've gotten the OnPIEStartupFinished call. If something were to interrupt PIE
	// startup (such as non-compiled blueprints) the queue would get stuck thinking it's rendering when it's not.
	bIsRendering = true;

	// Allow the user to have overridden which Pipeline is actually run. This is an unlikely scenario but allows
	// the user to create their own implementation while still re-using the rest of our UI and infrastructure.
	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	TSubclassOf<UMoviePipelineBase> PipelineClass = ProjectSettings->DefaultPipeline.TryLoadClass<UMoviePipelineBase>();
	if (!PipelineClass)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to load project specified MoviePipeline class type!");
		OnExecutorFinishedImpl();
		return;
	}

	if (!Queue || !Queue->GetJobs().IsValidIndex(CurrentPipelineIndex))
	{
		const FText FailureReason = FText::Format(LOCTEXT("OutOfBoundsStartupFailure", "The job index is out of bounds while starting PIE render, did you modify the Queue while rendering was happening? Index: {0}, Length: {1}."),
			FText::AsNumber(CurrentPipelineIndex), FText::AsNumber(Queue ? Queue->GetJobs().Num() : 0));
		UE_LOGF(LogMovieRenderPipeline, Error, "%ls", *FailureReason.ToString());
		OnPipelineErrored(nullptr, true, FailureReason);

		// Route through OnPIEEnded so fixed timestep/log gathering/editor state are restored.
		FEditorDelegates::EndPIE.AddUObject(this, &UMoviePipelinePIEExecutor::OnPIEEnded);
		GEditor->RequestEndPlayMap();
		return;
	}

	// Ensure the pipeline class we're using is compatible with the job's configuration type
	UMoviePipelineExecutorJob* CurrentJob = Queue->GetJobs()[CurrentPipelineIndex];
	const bool bNeedsGraphPipeline = CurrentJob->IsUsingGraphConfiguration() || CurrentJob->IsUsingBasicConfiguration();
	if (bNeedsGraphPipeline && !PipelineClass->IsChildOf(UMovieGraphPipeline::StaticClass()))
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Using the default MovieGraphPipeline; the pipeline specified in Project Settings (%ls) is not compatible with graphs.", *PipelineClass->GetName());
		PipelineClass = UMovieGraphPipeline::StaticClass();
	}
	else if (!bNeedsGraphPipeline && !PipelineClass->IsChildOf(UMoviePipeline::StaticClass()))
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Using the default MoviePipeline; the pipeline specified in Project Settings (%ls) is not compatible with normal (non-graph) presets.", *PipelineClass->GetName());
		PipelineClass = UMoviePipeline::StaticClass();
	}

	// This Pipeline belongs to the world being created so that they have context for things they execute.
	ActiveMoviePipeline = NewObject<UMoviePipelineBase>(ExecutingWorld, PipelineClass, NAME_None, RF_Transient);

	UMoviePipeline* PipelineAsLegacy = Cast<UMoviePipeline>(ActiveMoviePipeline);

	
	// We allow users to set a multi-frame delay before we actually run the Initialization function and start thinking.
	// This solves cases where there are engine systems that need to finish loading before we do anything.
	const UMoviePipelinePIEExecutorSettings* ExecutorSettings = GetDefault<UMoviePipelinePIEExecutorSettings>();

	// We tick each frame to update the Window Title, and kick off latent pipeling initialization.
	FCoreDelegates::OnBeginFrame.AddUObject(this, &UMoviePipelinePIEExecutor::OnTick);

	// Listen for when the pipeline thinks it has finished.
	ActiveMoviePipeline->OnMoviePipelineWorkFinished().AddUObject(this, &UMoviePipelinePIEExecutor::OnPIEMoviePipelineFinished);
	ActiveMoviePipeline->OnMoviePipelineShotWorkFinished().AddUObject(this, &UMoviePipelinePIEExecutor::OnJobShotFinished);
	if (PipelineAsLegacy)
	{

		PipelineAsLegacy->SetViewportInitArgs(ViewportInitArgs);
	}
	
	if (ExecutorSettings->InitialDelayFrameCount == 0)
	{
		OnIndividualJobStartedImpl(CurrentJob);
		if (PipelineAsLegacy)
		{
			PipelineAsLegacy->Initialize(CurrentJob);
			if (CustomInitializationTime.IsSet())
			{
				PipelineAsLegacy->SetInitializationTime(CustomInitializationTime.GetValue());
			}
		}
		else if (UMovieGraphPipeline* PipelineAsGraph = Cast<UMovieGraphPipeline>(ActiveMoviePipeline))
		{
			PipelineAsGraph->Initialize(CurrentJob, FMovieGraphInitConfig());
			if (CustomInitializationTime.IsSet())
			{
				PipelineAsGraph->SetInitializationTime(CustomInitializationTime.GetValue());
			}
		}
		RemainingInitializationFrames = -1;
	}
	else
	{
		RemainingInitializationFrames = ExecutorSettings->InitialDelayFrameCount;
	}
	
	// Listen for PIE shutdown in case the user hits escape to close it. 
	FEditorDelegates::EndPIE.AddUObject(this, &UMoviePipelinePIEExecutor::OnPIEEnded);
}

void UMoviePipelinePIEExecutor::OnTick()
{
	// Defensive: a misbehaving delegate listener (BP/Python) could in principle have cleared the
	// queue's jobs out from under us, so abort safely rather than crash on bad indexing.
	if (!Queue || !Queue->GetJobs().IsValidIndex(CurrentPipelineIndex))
	{
		const FText FailureReason = FText::Format(LOCTEXT("OutOfBoundsTickFailure", "The job index is out of bounds while ticking PIE render, did you modify the Queue while rendering was happening? Index: {0}, Length: {1}."),
			FText::AsNumber(CurrentPipelineIndex), FText::AsNumber(Queue ? Queue->GetJobs().Num() : 0));
		UE_LOGF(LogMovieRenderPipeline, Error, "%ls", *FailureReason.ToString());
		FCoreDelegates::OnBeginFrame.RemoveAll(this);
		OnPipelineErrored(nullptr, true, FailureReason);
		GEditor->RequestEndPlayMap();
		return;
	}

	UMoviePipelineExecutorJob* CurrentJob = Queue->GetJobs()[CurrentPipelineIndex];

	if (RemainingInitializationFrames >= 0)
	{
		if (RemainingInitializationFrames == 0)
		{
			OnIndividualJobStartedImpl(CurrentJob);
			
			UMoviePipeline* PipelineAsLegacy = Cast<UMoviePipeline>(ActiveMoviePipeline);
			if (PipelineAsLegacy)
			{
				PipelineAsLegacy->Initialize(CurrentJob);
				if (CustomInitializationTime.IsSet())
				{
					PipelineAsLegacy->SetInitializationTime(CustomInitializationTime.GetValue());
				}
			}
			else if (UMovieGraphPipeline* PipelineAsGraph = Cast<UMovieGraphPipeline>(ActiveMoviePipeline))
			{
				PipelineAsGraph->Initialize(CurrentJob, FMovieGraphInitConfig());
				if (CustomInitializationTime.IsSet())
				{
					PipelineAsGraph->SetInitializationTime(CustomInitializationTime.GetValue());
				}
			}
		}

		RemainingInitializationFrames--;
	}
	
	if (RemainingInitializationFrames <= 0)
	{
		CurrentJob->SetStatusProgress(GetCompletionPercentageFromActivePipeline());
	}
	
	FText WindowTitle = GetWindowTitle();
	TSharedPtr<SWindow> CustomWindow = WeakCustomWindow.Pin();
	if (CustomWindow)
	{
		CustomWindow->SetTitle(WindowTitle);
	}
}

void UMoviePipelinePIEExecutor::OnPIEMoviePipelineFinished(FMoviePipelineOutputData InOutputData)
{
	if (!InOutputData.bSuccess)
	{
		// We do the cast to UMoviePipeline (which will return nullptr during a graph render) because the functions
		// all take a pointer, but it's never actually used, and you can only have one pipeline at once so the API
		// doesn't really need to pass back the pointer to a soon-to-be-destroyed object.
		OnPipelineErrored(Cast<UMoviePipeline>(InOutputData.Pipeline), true, FText());
	}

	// Unsubscribe to the EndPIE event so we don't think the user canceled it.
	FCoreDelegates::OnBeginFrame.RemoveAll(this);

	// Reset progress for UI (regardless of success). Guard against the queue having been mutated
	// by listeners notified above (e.g., via OnPipelineErrored) before indexing back into it.
	if (Queue && Queue->GetJobs().IsValidIndex(CurrentPipelineIndex))
	{
		Queue->GetJobs()[CurrentPipelineIndex]->SetStatusProgress(0.0f);
	}

	if (ActiveMoviePipeline)
	{
		// Unsubscribe in the event that it gets called twice we don't have issues.
		ActiveMoviePipeline->OnMoviePipelineWorkFinished().RemoveAll(this);
	}

	// The End Play will happen on the next frame.
	GEditor->RequestEndPlayMap();
}

void UMoviePipelinePIEExecutor::OnJobShotFinished(FMoviePipelineOutputData InOutputData)
{
	// Just re-broadcast the delegate to our listeners.
	OnIndividualShotWorkFinishedDelegateNative.Broadcast(InOutputData);
	OnIndividualShotWorkFinishedDelegate.Broadcast(InOutputData);
}

void UMoviePipelinePIEExecutor::BeginDestroy()
{
	// Ensure we're no longer gathering, otherwise it tries to call a callback on the now dead uobject.
	ValidationMessageGatherer.StopGathering();

	Super::BeginDestroy();
}

void UMoviePipelinePIEExecutor::OnExecutorFinishedImpl()
{
	Super::OnExecutorFinishedImpl();

	const UMovieRenderGraphEditorSettings* EditorSettings = GetDefault<UMovieRenderGraphEditorSettings>();

	// Now that the queue has finished, play the rendered frames in an external application, open the output directory,
	// or do nothing.
	if (GetPostRenderAction() == EMoviePipelinePostRenderActionType::PlayRenderOutput)
	{
		FMoviePipelinePostRenderFileDisplayProcessor PostRenderFileDisplayProcessor(EditorSettings->PostRenderSettings);

		for (const FMoviePipelineOutputData& OutputData : AllJobOutputData)
		{
			PostRenderFileDisplayProcessor.AddFiles(OutputData);
		}
		
		PostRenderFileDisplayProcessor.OpenFiles();
	}
	else if (GetPostRenderAction() == EMoviePipelinePostRenderActionType::OpenOutputDirectory)
	{
		while (!ResolvedOutputDirectories.IsEmpty())
		{
			const FString ResolvedOutputDirectory = ResolvedOutputDirectories[0];
			ResolvedOutputDirectories.RemoveAt(0);

			if (!ResolvedOutputDirectory.IsEmpty())
			{
				// The directory might not exist yet. Create it (if needed) so ExploreFolder() can open it.
				IFileManager::Get().MakeDirectory(*ResolvedOutputDirectory, true);

				FPlatformProcess::ExploreFolder(*ResolvedOutputDirectory);
			}

			// Depending on the settings, we might only need to open one directory.
			if (EditorSettings->PostRenderSettings.DirectoryOpenBehavior == EMovieGraphDirectoryOpenBehavior::OpenDirectoryForFirstJob)
			{
				break;
			}
		}
	}

	// Reset all cached data tracking rendered output
	ResolvedOutputDirectories.Empty();
	AllJobOutputData.Empty();
}

bool UMoviePipelinePIEExecutor::GetSupportsPostRenderActions_Implementation()
{
	return true;
}

void UMoviePipelinePIEExecutor::OnPIEEnded(bool)
{
	FEditorDelegates::EndPIE.RemoveAll(this);

	CachedOutputDataParams = FMoviePipelineOutputData();

	// Only call Shutdown if the pipeline hasn't been finished.
	if (ActiveMoviePipeline && ActiveMoviePipeline->GetPipelineState() != EMovieRenderPipelineState::Finished)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "PIE Ended while Movie Pipeline was still active. Stalling to do full shutdown.");

		// This will flush any outstanding work on the movie pipeline (file writes) immediately
		ActiveMoviePipeline->Shutdown(true);
		UE_LOGF(LogMovieRenderPipeline, Log, "MoviePipelinePIEExecutor: Stalling finished, pipeline has shut down.");
	}

	// Cache off the output data so we can use it in DelayedFinishNotification, at which point ActiveMoviePipeline is null.	
	if (UMoviePipeline* PipelineAsLegacy = Cast<UMoviePipeline>(ActiveMoviePipeline))
	{
		CachedOutputDataParams = PipelineAsLegacy->GetOutputDataParams();
	}
	else if (UMovieGraphPipeline* GraphPipeline = Cast<UMovieGraphPipeline>(ActiveMoviePipeline))
	{
		// We only set GraphData for the graph pipeline
		CachedOutputDataParams.GraphData = GraphPipeline->GetGeneratedOutputData();
	}

	// We need to null out this reference this frame, otherwise we try to hold onto a PIE
	// object after PIE finishes which causes a GC leak.
	ActiveMoviePipeline = nullptr;

	// Delay for one frame so that PIE can finish shut down. It's not a huge fan of us starting up on the same frame.
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateUObject(this, &UMoviePipelinePIEExecutor::DelayedFinishNotification));

	// Restore the previous settings.
	FApp::SetUseFixedTimeStep(bPreviousUseFixedTimeStep);
	FApp::SetFixedDeltaTime(PreviousFixedTimeStepDelta);

	// Stop capturing logging messages
	ValidationMessageGatherer.StopGathering();
	ValidationMessageGatherer.OpenLog();
}

void UMoviePipelinePIEExecutor::DelayedFinishNotification()
{
	// Get the params for the job (including output info) from the cache. ActiveMoviePipeline is null already.
	OnIndividualJobFinishedImpl(CachedOutputDataParams);
	
	// Now that another frame has passed and we should be OK to start another PIE session, notify our owner.
	OnIndividualPipelineFinished(nullptr);
}

void UMoviePipelinePIEExecutor::OnIndividualJobStartedImpl(UMoviePipelineExecutorJob* InJob)
{
	// Broadcast to both Native and Python/BP
	OnIndividualJobStartedDelegateNative.Broadcast(InJob);
	OnIndividualJobStartedDelegate.Broadcast(InJob);

	// By the time the job finishes, it will have most likely been garbage collected (at least when using the graph, since jobs that use the graph
	// are duplicated). Instead of caching the resolved output directories when the job finishes, cache it when starting while the job is still valid.
	ResolvedOutputDirectories.AddUnique(UMoviePipelineEditorBlueprintLibrary::ResolveOutputDirectoryFromJob(InJob));
}

void UMoviePipelinePIEExecutor::OnIndividualJobFinishedImpl(FMoviePipelineOutputData InOutputData)
{
	// Broadcast to both Native and Python/BP
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnIndividualJobFinishedDelegateNative.Broadcast(InOutputData.Job, !IsAnyJobErrored());
	OnIndividualJobFinishedDelegate.Broadcast(InOutputData.Job, !IsAnyJobErrored());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	OnIndividualJobWorkFinishedDelegateNative.Broadcast(InOutputData);
	OnIndividualJobWorkFinishedDelegate.Broadcast(InOutputData);

	// Null out the pipeline and job pointers in the output data before caching (so they are GC'd correctly after the pipeline finishes)
	InOutputData.Pipeline = nullptr;
	InOutputData.Job = nullptr;
	AllJobOutputData.Add(InOutputData);
}

#undef LOCTEXT_NAMESPACE // "MoviePipelinePIEExecutor"

