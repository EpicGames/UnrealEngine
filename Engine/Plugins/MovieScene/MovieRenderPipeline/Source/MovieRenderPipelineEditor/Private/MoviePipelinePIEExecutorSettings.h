// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"
#include "MoviePipelinePIEExecutorSettings.generated.h"

/**
* This is the implementation responsible for executing the rendering of
* multiple movie pipelines within the editor using PIE.
*/
UCLASS(MinimalAPI, BlueprintType, config = Editor, defaultconfig, meta = (DisplayName = "Movie Pipeline In Editor"))
class UMoviePipelinePIEExecutorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

	/**
	* How long should we wait after being initialized to start doing any work? This can be used
	* to work around situations where the game is not fully loaded by the time the pipeline
	* is automatically started and it is important that the game is fully loaded before we do
	* any work (such as evaluating frames for warm-up).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, meta = (UIMin = "0", ClampMin = "0", UIMax = "150"), Category = "Startup")
	int32 InitialDelayFrameCount = 0;

	/**
	* Should the PIE Window be created at the same resolution as the MRQ Output? By default we create the window at 720p for a nicer
	* user experience, but this can be used to work around a widget scaling issue with UMG Widgets when using the UI Renderer
	* setting. PIE is still limited by your monitor's resolution so you will need a monitor at least as big as your requested output
	* for this to work (or can be combined with launching the editor with -ForceRes). 
	*
	* Warning: Don't use this setting in combination with HighResTiling, as the main backbuffer will have to get resized to your final
	* output resolution which will be too large.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Startup")
	bool bResizePIEWindowToOutputResolution = false;
};
