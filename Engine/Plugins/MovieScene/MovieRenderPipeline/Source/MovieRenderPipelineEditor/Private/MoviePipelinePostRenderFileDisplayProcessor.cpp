// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelinePostRenderFileDisplayProcessor.h"

#include "Graph/MovieRenderGraphEditorSettings.h"
#include "Internationalization/Regex.h"
#include "Misc/PackageName.h"
#include "MovieGraphImageSequenceOutputNode.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "PackageHelperFunctions.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelinePostRenderFileDisplayProcessor)

FMoviePipelinePostRenderFileDisplayProcessor::FMoviePipelinePostRenderFileDisplayProcessor(const FMovieGraphPostRenderSettings& InPostRenderSettings)
	: PostRenderSettings(InPostRenderSettings)
{
	
}

void FMoviePipelinePostRenderFileDisplayProcessor::AddFiles(const FMoviePipelineOutputData& InRenderOutputData)
{
	PipelineOutputData.Add(InRenderOutputData);
}

void FMoviePipelinePostRenderFileDisplayProcessor::OpenFiles() const
{
	TArray<FString> FinalListOfFilesToOpen;
	const FMovieGraphPostRenderVideoPlayOptions* FinalPlayOptions = nullptr;

	// The start and end frames to use for opening files. Currently only one set of start and end frames is supported, mostly due to restrictions on
	// how they can be passed to third-party image viewers.
	FString StartFrame;
	FString EndFrame;
	
	for (FMoviePipelineOutputData& OutputData : PipelineOutputData)
	{
		FFilesToOpen FilesToOpen = FindFilesToOpen(OutputData);

		// FindFilesToOpen() may return files we want to filter out; this is the final list of what should be opened, as well as the settings that
		// contain the application executable to use for opening the files.
		GetFilteredFilesToOpen(FilesToOpen, FinalListOfFilesToOpen, FinalPlayOptions);

		if (StartFrame.IsEmpty() && EndFrame.IsEmpty())
		{
			StartFrame = FilesToOpen.StartFrameString;
			EndFrame = FilesToOpen.EndFrameString;
		}

		// Settings may dictate that only one job's worth of media should be opened
		if (PostRenderSettings.PostRenderImageSequencePlayOptions.JobPlayback != EMovieGraphJobPlaybackRange::AllJobs)
		{
			break;
		}
	}

	// Open the application with the filtered list of files
	if (!FinalListOfFilesToOpen.IsEmpty() && FinalPlayOptions)
	{
		LaunchFilesWithSettings(FinalListOfFilesToOpen, FinalPlayOptions, TTuple<FString, FString>(StartFrame, EndFrame));

		// Persist the last set of pipeline output data so it can be played back later (even in another editor session). The raw output data is
		// saved here, rather than the final list of files, because the user may subsequently change their playback settings, which could generate a
		// different set of files to play back.
		PersistOutputData(PipelineOutputData);
	}
}

void FMoviePipelinePostRenderFileDisplayProcessor::OpenPreviousFiles(const FMovieGraphPostRenderSettings& InPostRenderSettings)
{
	if (!CanOpenPreviousFiles())
	{
		return;
	}

	FMoviePipelinePostRenderFileDisplayProcessor NewProcessor(InPostRenderSettings);

	for (const FMoviePipelineOutputData& OutputData : GetPersistedOutputData()->OutputData)
	{
		NewProcessor.AddFiles(OutputData);
	}

	NewProcessor.OpenFiles();
}

bool FMoviePipelinePostRenderFileDisplayProcessor::CanOpenPreviousFiles()
{
	const UMoviePipelineOutputDataContainer* PersistedData = GetPersistedOutputData();

	return PersistedData && !PersistedData->OutputData.IsEmpty();
}

FMoviePipelinePostRenderFileDisplayProcessor::FFilesToOpen FMoviePipelinePostRenderFileDisplayProcessor::FindFilesToOpen(FMoviePipelineOutputData& InPipelineOutputData) const
{
	FFilesToOpen FilesToOpen;
	
	// Nothing to do if the graph (GraphData) and preset (ShotData) output is empty.
	if (InPipelineOutputData.GraphData.IsEmpty() && InPipelineOutputData.ShotData.IsEmpty())
	{
		return FilesToOpen;
	}

	// Each entry in the array is an individual shot's image/video data. The map is keyed by file extension, and the values are the file paths.
	TArray<TMap<FString, TArray<FString>>> ImagesGroupedByShot;
	TArray<TMap<FString, TArray<FString>>> FrameTemplateImagesGroupedByShot;
	TArray<TMap<FString, TArray<FString>>> VideosGroupedByShot;
	
	// Group all rendered images and videos by extension and by frame templated string. If only playing back the first render layer, only capture
	// images generated from the first layer.
	GroupFilesByShot(InPipelineOutputData, ImagesGroupedByShot, FrameTemplateImagesGroupedByShot, VideosGroupedByShot);

	// If getting images by priority order, find the image and video extensions that were used that have the highest priority. Note that the first
	// shot will take precedence over all other shots for this determination.
	FString HighestPriorityImageExtension;
	FString HighestPriorityVideoExtension;
	GetHighestPriorityExtensions(ImagesGroupedByShot, VideosGroupedByShot, HighestPriorityImageExtension, HighestPriorityVideoExtension);

	auto AddImages = [this, &FilesToOpen, &FrameTemplateImagesGroupedByShot](const TArray<FString>& InImagePaths, const uint32 ShotIndex)
	{
		const EMovieGraphPlaybackMethod PlaybackMethod = PostRenderSettings.PostRenderImageSequencePlayOptions.PlaybackMethod;
		const EMovieGraphImageSequencePlaybackRange PlaybackRange = PostRenderSettings.PostRenderImageSequencePlayOptions.PlaybackRange;

		// Only provide the first non-templated frame in some cases
		if ((PlaybackMethod == EMovieGraphPlaybackMethod::OperatingSystem) || (PlaybackRange == EMovieGraphImageSequencePlaybackRange::FirstFrameOnly))
		{
			FilesToOpen.Images.Add(InImagePaths[0]);
			return;
		}
		
		// Otherwise, provide the frame-templated image paths
		const TMap<FString, TArray<FString>>& ImagesByFrameTemplate = FrameTemplateImagesGroupedByShot[ShotIndex];
		for (const TPair<FString, TArray<FString>>& FrameTemplateToFilePaths : ImagesByFrameTemplate)
		{
			for (const FString& ImagePath : InImagePaths)
			{
				const TArray<FString>& ImagesAssociatedWithFrameTemplate = FrameTemplateToFilePaths.Value;
				const FString& FrameTemplatedPath = FrameTemplateToFilePaths.Key;
				
				// Get the frame range of this image sequence. This will merge this image sequence's frame range with other image sequences found.
				// For example, if shot1 has a frame range of 0-100, and shot2 has a frame range of 101-200, and this method is called on both shots,
				// FFilesToOpen will have a frame range of 0-200.
				GetStartAndEndFrames(FrameTemplatedPath, InImagePaths, FilesToOpen);
				
				// Add the frame-templated path if the templated path is associated with this non-templated path AND the templated path has not already
				// been added to the list of files to open
				if (ImagesAssociatedWithFrameTemplate.Contains(ImagePath) && !FilesToOpen.Images.Contains(FrameTemplatedPath))
				{
					FilesToOpen.Images.Add(FrameTemplatedPath);

					// Found an image that matched this frame templated path; move on to the next frame template and see if a path matches it
					break;
				}
			}
		}
	};

	auto AddVideos = [this, &FilesToOpen](const TArray<FString>& InVideoPaths)
	{
		for (const FString& VideoPath : InVideoPaths)
		{
			const FString& UpperExtension = FPaths::GetExtension(VideoPath).ToUpper();
			
			if (UpperExtension == TEXT("MOV"))
			{
				FilesToOpen.AppleProResMovies.Add(VideoPath);
			}
			else if (UpperExtension == TEXT("MP4"))
			{
				FilesToOpen.MP4Movies.Add(VideoPath);
			}
			else
			{
				FilesToOpen.AvidDNxHRMovies.Add(VideoPath);
			}
		}
	};

	// With all images/videos grouped, and the highest-priority extensions found, find the images or videos that should be returned
	for (int32 ShotIndex = 0; ShotIndex < ImagesGroupedByShot.Num(); ++ShotIndex)
	{
		const TMap<FString, TArray<FString>>& ShotImages = ImagesGroupedByShot[ShotIndex];
		const TMap<FString, TArray<FString>>& ShotVideos = VideosGroupedByShot[ShotIndex];
		
		// Only add the images OR videos with the highest priority if that's what was requested
		if (PostRenderSettings.OutputTypePlayback == EMovieGraphOutputTypePlayback::UsePriorityOrder)
		{
			const int32 ImageExtensionPriorityIndex = GetExtensionPriorityIndex(HighestPriorityImageExtension);
			const int32 VideoExtensionPriorityIndex = GetExtensionPriorityIndex(HighestPriorityVideoExtension);

			// Images have a higher priority
			if (ImageExtensionPriorityIndex < VideoExtensionPriorityIndex)
			{
				if (const TArray<FString>* ImagePaths = ShotImages.Find(HighestPriorityImageExtension))
				{
					AddImages(*ImagePaths, ShotIndex);
				}
			}
			// Videos have a higher priority
			else
			{
				if (const TArray<FString>* VideoPaths = ShotVideos.Find(HighestPriorityVideoExtension))
				{
					AddVideos(*VideoPaths);
				}
			}
		}
		else
		{
			// If not adding just the images/videos with the highest priority extension, add all images/videos that were found
			for (const TPair<FString, TArray<FString>>& ImagesByExtension : ShotImages)
			{
				AddImages(ImagesByExtension.Value, ShotIndex);
			}

			for (const TPair<FString, TArray<FString>>& VideosByExtension : ShotVideos)
			{
				AddVideos(VideosByExtension.Value);
			}
		}
	}

	FilesToOpen.HighestPriorityImageExtension = HighestPriorityImageExtension;
	FilesToOpen.HighestPriorityVideoExtension = HighestPriorityVideoExtension;

	return FilesToOpen;
}

void FMoviePipelinePostRenderFileDisplayProcessor::GetFilteredFilesToOpen(const FFilesToOpen& InFilesToOpen, TArray<FString>& OutFilteredFilesToOpen, const FMovieGraphPostRenderVideoPlayOptions*& OutFilteredPlayOptions) const
{
	// Only update the play options if no filtered files have been provided yet. If there ARE filtered files already, that means that we're processing
	// files that are not part of the first job. The first job's files should dictate what the play options are.
	auto MaybeAssignPlayOptions = [&OutFilteredPlayOptions, &OutFilteredFilesToOpen](const FMovieGraphPostRenderVideoPlayOptions* InNewPlayOptions)
	{
		if (OutFilteredFilesToOpen.IsEmpty())
		{
			OutFilteredPlayOptions = InNewPlayOptions;
		}
	};
	
	// If there are both images and videos returned, and Play All Output Types is specified on the images, find the media type with the highest
	// priority and use the player for that.
	const bool bPlayAllOutputTypes = (PostRenderSettings.OutputTypePlayback == EMovieGraphOutputTypePlayback::PlayAllOutputTypes);
	const bool bHasVideos = !InFilesToOpen.AppleProResMovies.IsEmpty() || !InFilesToOpen.AvidDNxHRMovies.IsEmpty() || !InFilesToOpen.MP4Movies.IsEmpty();
	const bool bHasImagesAndVideos = !InFilesToOpen.Images.IsEmpty() && bHasVideos;
	if (bPlayAllOutputTypes && bHasImagesAndVideos)
	{
		const int32 ImagePriority = GetExtensionPriorityIndex(InFilesToOpen.HighestPriorityImageExtension);
		const int32 ProResPriority = GetExtensionPriorityIndex(TEXT("MOV"));
		const int32 AvidPriority = GetExtensionPriorityIndex(TEXT("MXF"));
		const int32 MP4Priority = GetExtensionPriorityIndex(TEXT("MP4"));

		// Find the highest priority media type
		TArray<int32> Priorities = {ImagePriority, ProResPriority, AvidPriority, MP4Priority};
		Priorities.Sort();
		const int32 HighestPriority = Priorities[0];

		FString PlayerWithPriority;

		// Find the player that should be used, based on the media type with the highest priority
		if (HighestPriority == ImagePriority)
		{
			MaybeAssignPlayOptions(&PostRenderSettings.PostRenderImageSequencePlayOptions);
			PlayerWithPriority = PostRenderSettings.PostRenderImageSequencePlayOptions.PlayerExecutable.FilePath;
		}
		else if (HighestPriority == ProResPriority)
		{
			MaybeAssignPlayOptions(&PostRenderSettings.PostRenderAppleProResPlayOptions);
			PlayerWithPriority = PostRenderSettings.PostRenderAppleProResPlayOptions.PlayerExecutable.FilePath;
		}
		else if (HighestPriority == AvidPriority)
		{
			MaybeAssignPlayOptions(&PostRenderSettings.PostRenderAvidDNxHRPlayOptions);
			PlayerWithPriority = PostRenderSettings.PostRenderAvidDNxHRPlayOptions.PlayerExecutable.FilePath;
		}
		else
		{
			MaybeAssignPlayOptions(&PostRenderSettings.PostRenderMP4PlayOptions);
			PlayerWithPriority = PostRenderSettings.PostRenderMP4PlayOptions.PlayerExecutable.FilePath;
		}

		// Only return the media that opens with the player associated with the highest-priority media
		if (PostRenderSettings.PostRenderImageSequencePlayOptions.PlayerExecutable.FilePath == PlayerWithPriority)
		{
			OutFilteredFilesToOpen.Append(InFilesToOpen.Images);
		}
		if (PostRenderSettings.PostRenderAppleProResPlayOptions.PlayerExecutable.FilePath == PlayerWithPriority)
		{
			OutFilteredFilesToOpen.Append(InFilesToOpen.AppleProResMovies);
		}
		if (PostRenderSettings.PostRenderAvidDNxHRPlayOptions.PlayerExecutable.FilePath == PlayerWithPriority)
		{
			OutFilteredFilesToOpen.Append(InFilesToOpen.AvidDNxHRMovies);
		}
		if (PostRenderSettings.PostRenderMP4PlayOptions.PlayerExecutable.FilePath == PlayerWithPriority)
		{
			OutFilteredFilesToOpen.Append(InFilesToOpen.MP4Movies);
		}
	}
	else
	{
		// If the Playback Type isn't PlayAllOutputTypes, only images or movies will be specified in FilesToOpen; just provide whatever files were
		// given by FilesToOpen in this case.
		if (!InFilesToOpen.Images.IsEmpty())
		{
			MaybeAssignPlayOptions(&PostRenderSettings.PostRenderImageSequencePlayOptions);
			OutFilteredFilesToOpen.Append(InFilesToOpen.Images);
		}
		else if (!InFilesToOpen.AppleProResMovies.IsEmpty())
		{
			MaybeAssignPlayOptions(&PostRenderSettings.PostRenderAppleProResPlayOptions);
			OutFilteredFilesToOpen.Append(InFilesToOpen.AppleProResMovies);
		}
		else if (!InFilesToOpen.AvidDNxHRMovies.IsEmpty())
		{
			MaybeAssignPlayOptions(&PostRenderSettings.PostRenderAvidDNxHRPlayOptions);
			OutFilteredFilesToOpen.Append(InFilesToOpen.AvidDNxHRMovies);
		}
		else
		{
			MaybeAssignPlayOptions(&PostRenderSettings.PostRenderMP4PlayOptions);
			OutFilteredFilesToOpen.Append(InFilesToOpen.MP4Movies);
		}
	}
}

void FMoviePipelinePostRenderFileDisplayProcessor::GroupFilesByShot(
	FMoviePipelineOutputData& InPipelineOutputData,
	TArray<TMap<FString, TArray<FString>>>& InImagesGroupedByShot,
	TArray<TMap<FString, TArray<FString>>>& InFrameTemplateImagesGroupedByShot,
	TArray<TMap<FString, TArray<FString>>>& InVideosGroupedByShot) const
{
	if (!InPipelineOutputData.GraphData.IsEmpty())
	{
		GroupFilesByShot_Graph(InPipelineOutputData, InImagesGroupedByShot, InFrameTemplateImagesGroupedByShot, InVideosGroupedByShot);
	}
	else
	{
		GroupFilesByShot_Preset(InPipelineOutputData, InImagesGroupedByShot, InFrameTemplateImagesGroupedByShot, InVideosGroupedByShot);
	}
}

void FMoviePipelinePostRenderFileDisplayProcessor::GroupFilesByShot_Graph(
	FMoviePipelineOutputData& InPipelineOutputData,
	TArray<TMap<FString, TArray<FString>>>& InImagesGroupedByShot,
	TArray<TMap<FString, TArray<FString>>>& InFrameTemplateImagesGroupedByShot,
	TArray<TMap<FString, TArray<FString>>>& InVideosGroupedByShot) const
{
	// OutputData is an individual shot's collection of rendered output (images + movies)
	for (FMovieGraphRenderOutputData& OutputData : InPipelineOutputData.GraphData)
	{
		TMap<FString, TArray<FString>>& ImagesByExtension = InImagesGroupedByShot.AddDefaulted_GetRef();
		TMap<FString, TArray<FString>>& ImagesByFrameTemplate = InFrameTemplateImagesGroupedByShot.AddDefaulted_GetRef();
		TMap<FString, TArray<FString>>& VideosByExtension = InVideosGroupedByShot.AddDefaulted_GetRef();

		const bool bFirstRenderLayerOnly_Images = (PostRenderSettings.PostRenderImageSequencePlayOptions.RenderLayerPlayback == EMovieGraphRenderLayerPlaybackRange::FirstRenderLayerOnly);
		const bool bFirstRenderLayerOnly_ProRes = (PostRenderSettings.PostRenderAppleProResPlayOptions.RenderLayerPlayback == EMovieGraphRenderLayerPlaybackRange::FirstRenderLayerOnly);
		const bool bFirstRenderLayerOnly_Avid = (PostRenderSettings.PostRenderAvidDNxHRPlayOptions.RenderLayerPlayback == EMovieGraphRenderLayerPlaybackRange::FirstRenderLayerOnly);
		const bool bFirstRenderLayerOnly_MP4 = (PostRenderSettings.PostRenderMP4PlayOptions.RenderLayerPlayback == EMovieGraphRenderLayerPlaybackRange::FirstRenderLayerOnly);
		
		// Sort the render layers by index. This will allow us to only get files from the first render layer rendered if the "First Render Layer Only"
		// setting is turned on.
		OutputData.RenderLayerData.ValueSort([](const FMovieGraphRenderLayerOutputData& A, const FMovieGraphRenderLayerOutputData& B)
		{
			return A.RenderLayerIndex < B.RenderLayerIndex;
		});

		// Keep track of the render layer that media types were first found in
		FString ImagesFirstFoundOnLayer;
		FString ProResFirstFoundOnLayer;
		FString AvidFirstFoundOnLayer;
		FString MP4FirstFoundOnLayer;

		// Add the images/videos for each render layer within the shot (only the first render layer may be considered depending on settings)
		for (const TPair<FMovieGraphRenderDataIdentifier, FMovieGraphRenderLayerOutputData>& PassData : OutputData.RenderLayerData)
		{
			// For each node type that generated files within the render layer, add its files
			for (const TPair<FSoftClassPath, FMovieGraphStringArray>& NodeToFiles : PassData.Value.NodeTypeToFilePaths)
			{
				const UClass* NodeClass = NodeToFiles.Key.ResolveClass();
				if (!NodeClass)
				{
					UE_LOGF(LogMovieRenderPipeline, Warning, "Found an invalid node type while determining the media to display post-render. This node's media will be skipped.");
					continue;
				}

				// It's unfortunate that we have to identify nodes by name here in some cases, but we cannot compare directly against the static class
				// due to some nodes being in plugins.
				const bool bIsImageNode = NodeClass->IsChildOf(UMovieGraphImageSequenceOutputNode::StaticClass());
				const bool bIsProResNode = NodeClass->GetName().Contains(TEXT("ProRes"), ESearchCase::CaseSensitive);
				const bool bIsAvidNode = NodeClass->GetName().Contains(TEXT("Avid"), ESearchCase::CaseSensitive);
				const bool bIsMP4Node = NodeClass->GetName().Contains(TEXT("MP4"), ESearchCase::CaseSensitive);
				
				// Don't continue processing this node if the node's media type has already been found for this render layer. Note that the render
				// layer name check is to allow media of the same category (eg, several image types like PNG and EXR) to be processed for a single
				// render layer. This loop should process all nodes within the render layer.
				if (bIsImageNode && bFirstRenderLayerOnly_Images && (!ImagesFirstFoundOnLayer.IsEmpty() && ImagesFirstFoundOnLayer != PassData.Key.LayerName))
				{
					continue;
				}
				if (bIsProResNode && bFirstRenderLayerOnly_ProRes && (!ProResFirstFoundOnLayer.IsEmpty() && ProResFirstFoundOnLayer != PassData.Key.LayerName))
				{
					continue;
				}
				if (bIsAvidNode && bFirstRenderLayerOnly_Avid && (!AvidFirstFoundOnLayer.IsEmpty() && AvidFirstFoundOnLayer != PassData.Key.LayerName))
				{
					continue;
				}
				if (bIsMP4Node && bFirstRenderLayerOnly_MP4 && (!MP4FirstFoundOnLayer.IsEmpty() && MP4FirstFoundOnLayer != PassData.Key.LayerName))
				{
					continue;
				}

				// For all files generated by the node, categorize it either by extension and/or frame template (for images)
				for (const FString& NodeFile : NodeToFiles.Value.Array)
				{
					const FString Extension = FPaths::GetExtension(NodeFile).ToUpper();
					if (Extension.IsEmpty())
					{
						continue;
					}
					
					if (bIsImageNode)
					{
						ImagesFirstFoundOnLayer = PassData.Key.LayerName;
						ImagesByExtension.FindOrAdd(Extension).Add(NodeFile);

						// The render layer will contain a set of frame templated file paths, in addition to normal file paths. Determine if this
						// file should be associated with any of these frame templated paths (matched by extension).
						for (const FString& FrameTemplatedFilePath : PassData.Value.FrameTemplatedFilePaths)
						{
							if (FPaths::GetExtension(FrameTemplatedFilePath).ToUpper() == Extension)
							{
								ImagesByFrameTemplate.FindOrAdd(FrameTemplatedFilePath).Add(NodeFile);
								break;
							}
						}
					}
					else
					{
						if (bIsProResNode)
						{
							ProResFirstFoundOnLayer = PassData.Key.LayerName;
						}
						else if (bIsAvidNode)
						{
							AvidFirstFoundOnLayer = PassData.Key.LayerName;
						}
						else if (bIsMP4Node)
						{
							MP4FirstFoundOnLayer = PassData.Key.LayerName;
						}
						
						VideosByExtension.FindOrAdd(Extension).Add(NodeFile);
					}
				}
			}
		}
	}
}

void FMoviePipelinePostRenderFileDisplayProcessor::GroupFilesByShot_Preset(
	FMoviePipelineOutputData& InPipelineOutputData,
	TArray<TMap<FString, TArray<FString>>>& InImagesGroupedByShot,
	TArray<TMap<FString, TArray<FString>>>& InFrameTemplateImagesGroupedByShot,
	TArray<TMap<FString, TArray<FString>>>& InVideosGroupedByShot) const
{
	// OutputData is an individual shot's collection of rendered output (images + movies)
	for (FMoviePipelineShotOutputData& OutputData : InPipelineOutputData.ShotData)
	{
		TMap<FString, TArray<FString>>& ImagesByExtension = InImagesGroupedByShot.AddDefaulted_GetRef();
		TMap<FString, TArray<FString>>& ImagesByFrameTemplate = InFrameTemplateImagesGroupedByShot.AddDefaulted_GetRef();
		TMap<FString, TArray<FString>>& VideosByExtension = InVideosGroupedByShot.AddDefaulted_GetRef();

		// Add the images/videos for each render pass within the shot
		for (const TPair<FMoviePipelinePassIdentifier, FMoviePipelineRenderPassOutputData>& PassData : OutputData.RenderPassData)
		{
			// For each setting type that generated files, add its files
			for (const TPair<FSoftClassPath, FMoviePipelineStringArray>& FilesFromSetting : PassData.Value.SettingTypeToFilePaths)
			{
				const UClass* SettingClass = FilesFromSetting.Key.ResolveClass();
				if (!SettingClass)
				{
					UE_LOGF(LogMovieRenderPipeline, Warning, "Found an invalid setting type while determining the media to display post-render. This setting's media will be skipped.");
					continue;
				}
				
				const bool bIsImageNode = SettingClass->IsChildOf(UMoviePipelineImageSequenceOutputBase::StaticClass());

				// For all files generated by the setting, categorize it either by extension and/or frame template (for images)
				for (const FString& SettingFile : FilesFromSetting.Value.Array)
				{
					const FString Extension = FPaths::GetExtension(SettingFile).ToUpper();
					if (Extension.IsEmpty())
					{
						continue;
					}
					
					if (bIsImageNode)
					{
						ImagesByExtension.FindOrAdd(Extension).Add(SettingFile);

						// The render pass will contain a set of frame templated file paths, in addition to normal file paths. Determine if this
						// file should be associated with any of these frame templated paths (matched by extension).
						for (const FString& FrameTemplatedFilePath : PassData.Value.FrameTemplatedFilePaths)
						{
							if (FPaths::GetExtension(FrameTemplatedFilePath).ToUpper() == Extension)
							{
								ImagesByFrameTemplate.FindOrAdd(FrameTemplatedFilePath).Add(SettingFile);
								break;
							}
						}
					}
					else
					{
						VideosByExtension.FindOrAdd(Extension).Add(SettingFile);
					}
				}
			}
		}
	}
}

void FMoviePipelinePostRenderFileDisplayProcessor::GetHighestPriorityExtensions(
	const TArray<TMap<FString, TArray<FString>>>& InImagesGroupedByShot,
	const TArray<TMap<FString, TArray<FString>>>& InVideosGroupedByShot,
	FString& OutHighestPriorityImageExtension,
	FString& OutHighestPriorityVideoExtension) const
{
	// Assign the given extension (InExtensionToTest_Uppercase) as the highest-priority extension (OutHighestPriorityExtension) if any media contained in
	// InMediaGroupedByShot matches the extension.
	auto MaybeAssignExtensionAsHighestPriority = [](const TArray<TMap<FString, TArray<FString>>>& InMediaGroupedByShot, const FString& InExtensionToTest_Uppercase, FString& OutHighestPriorityExtension)
	{
		// The first shot always takes precedence over all other shots as far as which extensions are considered highest priority
		const TMap<FString, TArray<FString>>& FirstShotMedia = InMediaGroupedByShot[0];

		// Media within the shot is grouped by extension (hence why the keys are used)
		TArray<FString> ExtensionsUsedWithinShot;
		TArray<FString> ExtensionsUsedWithinShot_Uppercase;
		FirstShotMedia.GetKeys(ExtensionsUsedWithinShot);
		Algo::Transform(ExtensionsUsedWithinShot, ExtensionsUsedWithinShot_Uppercase, [](const FString& InExtension) { return InExtension.ToUpper(); });

		if (ExtensionsUsedWithinShot_Uppercase.Contains(InExtensionToTest_Uppercase))
		{
			OutHighestPriorityExtension = InExtensionToTest_Uppercase;
		}
	};
	
	// Iterate the priority list in order of highest priority to lowest priority. The first extensions in this list that actually had media generated
	// for them will be deemed the highest priority extensions (eg, if EXR is first in the priority list, and EXR files were actually generated, EXR
	// would be the highest priority image extension).
	for (const FString& Extension : PostRenderSettings.OutputTypePriorityOrder)
	{
		// Do all extension comparisons in uppercase for consistency
		const FString Extension_Uppercase = Extension.ToUpper();

		// Find the highest priority image extension
		if (!InImagesGroupedByShot.IsEmpty() && OutHighestPriorityImageExtension.IsEmpty())
		{
			MaybeAssignExtensionAsHighestPriority(InImagesGroupedByShot, Extension_Uppercase, OutHighestPriorityImageExtension);
		}

		// Find the highest priority video extension
		if (!InVideosGroupedByShot.IsEmpty() && OutHighestPriorityVideoExtension.IsEmpty())
		{
			MaybeAssignExtensionAsHighestPriority(InVideosGroupedByShot, Extension_Uppercase, OutHighestPriorityVideoExtension);
		}
	}
}

void FMoviePipelinePostRenderFileDisplayProcessor::LaunchFilesWithSettings(const TArray<FString>& InFilesToOpen, const FMovieGraphPostRenderVideoPlayOptions* InPlayOptions, const TTuple<FString, FString>& InFrameRangeToOpen) const
{
	const FString StartFrame = InFrameRangeToOpen.Key;
	const FString EndFrame = InFrameRangeToOpen.Value;
	
	auto ReplaceFramePlaceholderInFilePaths = [this, &StartFrame](TArray<FString>& InFilePaths)
	{
		FString FramePlaceholder;
		
		const EMovieGraphFrameRangeNotation FrameRangeNotationType = PostRenderSettings.PostRenderImageSequencePlayOptions.FrameRangeNotation;
		switch (FrameRangeNotationType)
		{
		case EMovieGraphFrameRangeNotation::Hash:
		case EMovieGraphFrameRangeNotation::HashWithStartEndFrame:
			FramePlaceholder = TEXT("#");
			break;
		case EMovieGraphFrameRangeNotation::DollarF:
			FramePlaceholder = TEXT("$F");
			break;
		case EMovieGraphFrameRangeNotation::StartFrame:
			{
				FramePlaceholder = StartFrame;
				break;
			}
		}
		
		// Make sure the paths are in the right format for the platform, and have any frame templating applied
		for (FString& FileToOpen : InFilePaths)
		{
			FPaths::MakePlatformFilename(FileToOpen);

			if (!FramePlaceholder.IsEmpty())
			{
				FileToOpen.ReplaceInline(TEXT("{frame_placeholder}"), *FramePlaceholder);
			}
		}
	};

	auto InjectFrameRange = [this, &StartFrame, &EndFrame](FString& InConcatenatedFilePaths)
	{
		if (!StartFrame.IsEmpty() && !EndFrame.IsEmpty())
		{
			const EMovieGraphFrameRangeNotation FrameRangeNotation = PostRenderSettings.PostRenderImageSequencePlayOptions.FrameRangeNotation;
			if (FrameRangeNotation == EMovieGraphFrameRangeNotation::HashWithStartEndFrame)
			{
				const FString FrameRangeString = FString::Format(TEXT("{0}-{1}"), { StartFrame, EndFrame });
				
				// If using # with frame ranges, specify the frame range after the file path(s). This is how RV expects it.
				InConcatenatedFilePaths = FString::Format(TEXT("{0} {1}"), { InConcatenatedFilePaths, FrameRangeString });
			}
			else if (FrameRangeNotation == EMovieGraphFrameRangeNotation::DollarF)
			{
				const FString FrameRangeString = FString::Format(TEXT("{0} {1}"), { StartFrame, EndFrame });
				
				// $F is used exclusively by MPlay, so provide the frame range via -f with a 1-frame step (eg, "-f 0 150 1")
				InConcatenatedFilePaths = FString::Format(TEXT("-f {0} 1 {1}"), { FrameRangeString, InConcatenatedFilePaths });
			}
		}
	};

	// Add quotes to the file paths (to avoid issues with spaces in the paths). This needs to be done for both the OS viewer and custom viewer.
	TArray<FString> TransformedFilePaths;
	TransformedFilePaths.Reserve(InFilesToOpen.Num());
	for (const FString& Path : InFilesToOpen)
	{
		TransformedFilePaths.Add(FString::Format(TEXT("\"{0}\""), {Path}));
	}
	
	if (InPlayOptions->PlaybackMethod == EMovieGraphPlaybackMethod::OperatingSystem)
	{
		// Only one frame will be displayed if the OS is being used to open images
		FPlatformProcess::LaunchFileInDefaultExternalApplication(*InFilesToOpen[0]);
	}
	else
	{
		// No quotes around the executable path
		const FString ExecutablePath = InPlayOptions->PlayerExecutable.FilePath;
		FString ExecutablePathNoQuotes = ExecutablePath.Replace(TEXT("\""), TEXT(""));
		FPaths::NormalizeFilename(ExecutablePathNoQuotes);
		
		// Replace {frame_placeholder}, if present, with the appropriate symbol (like '#')
		ReplaceFramePlaceholderInFilePaths(TransformedFilePaths);

		// Depending on the frame range notation chosen, the command line arguments may need to be augmented with the frame range that was rendered
		FString ConcatenatedFilePaths = FString::Join(TransformedFilePaths, TEXT(" "));
		InjectFrameRange(ConcatenatedFilePaths);
		
		const FString CommandLineArguments = FString::Format(TEXT("{0} {1}"), {
			InPlayOptions->AdditionalCommandLineArguments,
			ConcatenatedFilePaths
		});
		
		const FString FinalCommandString = FString::Format(TEXT("{0} {1}"), {ExecutablePathNoQuotes, CommandLineArguments});
		UE_LOGF(LogMovieRenderPipeline, Display, "Quick Render: Opening external viewer with command: %ls", *FinalCommandString);
		
		// Open the images in the selected application
		constexpr bool bLaunchDetached = true;
		constexpr bool bLaunchHidden = false;
		constexpr bool bLaunchReallyHidden = false;
		const FProcHandle ProcHandle = FPlatformProcess::CreateProc(*ExecutablePathNoQuotes, *CommandLineArguments, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, nullptr, nullptr, nullptr);
		if (!ProcHandle.IsValid())
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Unable to open the post-render media viewer. Ensure the executable and its command line arguments have been specified correctly in Editor Preferences. The command that was run: [%ls]", *FinalCommandString);
		}
	}
}

int32 FMoviePipelinePostRenderFileDisplayProcessor::GetExtensionPriorityIndex(const FString& InExtension) const
{
	const int32 ExtensionPriority = PostRenderSettings.OutputTypePriorityOrder.Find(InExtension);

	// Extensions that have a lower index in the array have the highest priority, thus if the extension was not found (-1), it should be
	// assigned a very high priority number so it is not considered a high priority (10000 is arbitrary).
	return (ExtensionPriority == INDEX_NONE) ? 10000 : ExtensionPriority;
}

void FMoviePipelinePostRenderFileDisplayProcessor::GetStartAndEndFrames(const FString& InTemplatedPath, const TArray<FString>& InImagePaths, FFilesToOpen& InOutFilesToOpen) const
{
	if (InImagePaths.IsEmpty())
	{
		return;
	}

	// Turn the frame template path into a regex pattern. Eg, C:\SomeFolder\Shot1\Layer1.{frame_placeholder}.exr -> \EC:\SomeFolder\Shot1\Layer1.\Q(\d+)\E.exr\Q
	// Putting the path parts, excluding the frame number (which has been placed in a capture group), inside an \E\Q group allows us to avoid the need
	// to escape different characters within the path.
	// First, surround the entire path with \E\Q, then replace {frame_placeholder} with \Q(\d+)\E to capture the frame number. 
	const FString ReplacementToken = TEXT(R"(\E(\d+)\Q)");
	FString RegexString = FString::Format(TEXT(R"(\Q{0}\E)"), {InTemplatedPath});
	RegexString.ReplaceInline(TEXT("{frame_placeholder}"), *ReplacementToken);
	const FRegexPattern RegexPattern(RegexString);

	// If this is the first time this method is called, init the Start/End frame numbers with bogus numbers so the loop below knows to immediately
	// initialize them with frame numbers found within the image paths.
	if ((InOutFilesToOpen.StartFrame == TNumericLimits<int32>::Min()) && (InOutFilesToOpen.EndFrame == TNumericLimits<int32>::Min()))
	{
		InOutFilesToOpen.StartFrame = 100000;
		InOutFilesToOpen.EndFrame = -100000;
	}

	for (const FString& ImagePathTest : InImagePaths)
	{
		FRegexMatcher FrameMatcher(RegexPattern, ImagePathTest);
		if (FrameMatcher.FindNext())
		{
			const FString FrameNumberString = FrameMatcher.GetCaptureGroup(1);
			const int32 FrameNumber = FCString::Atoi(*FrameNumberString);
			if (FrameNumber < InOutFilesToOpen.StartFrame)
			{
				InOutFilesToOpen.StartFrame = FrameNumber;
				InOutFilesToOpen.StartFrameString = FrameNumberString;
			}
			if (FrameNumber > InOutFilesToOpen.EndFrame)
			{
				InOutFilesToOpen.EndFrame = FrameNumber;
				InOutFilesToOpen.EndFrameString = FrameNumberString;
			}
		}
	}
}

void FMoviePipelinePostRenderFileDisplayProcessor::PersistOutputData(const TArray<FMoviePipelineOutputData>& InOutputData)
{
	// The output data is saved to a uasset rather than using the ini configuration system. The ini configuration system isn't flexible enough
	// to store all of the types of data in FMoviePipelineOutputData. Additionally, the output data has the potential to contain hundreds/thousands of
	// files, and an ini is not an appropriate storage location for that volume of data.
	
	static const FString PackageFileName = FPackageName::LongPackageNameToFilename(*PersistedOutputDataPackagePath, FPackageName::GetAssetPackageExtension());

	// Get a pointer to the existing (or empty) persisted data in the target package (which will be saved to disk).
	UMoviePipelineOutputDataContainer* PersistedData = GetPersistedOutputData();
	if (!PersistedData)
	{
		return;
	}

	// Ensure the persisted data has the correct flags set so it actually gets saved to disk
	PersistedData->OutputData = InOutputData;
	PersistedData->SetFlags(RF_Public | RF_Standalone);

	bool bSuccess = false;

	// Save the settings out to disk. Turn off the behavior that auto-adds new files to source control.
	{
		UEditorLoadingSavingSettings* SaveSettings = GetMutableDefault<UEditorLoadingSavingSettings>();
		const uint32 bSCCAutoAddNewFiles = SaveSettings->bSCCAutoAddNewFiles;
		SaveSettings->bSCCAutoAddNewFiles = 0;

		bSuccess = SavePackageHelper(PersistedData->GetPackage(), *PackageFileName);

		SaveSettings->bSCCAutoAddNewFiles = bSCCAutoAddNewFiles;
	}

	if (!bSuccess)
	{
		// SavePackageHelper() will emit warnings if the save was unsuccessful, but log a separate warning for movie pipeline in case warnings are
		// being specifically filtered for LogMovieRenderPipeline.
		UE_LOGF(LogMovieRenderPipeline, Warning, "Unable to save post-render play data. Could not save to destination file [%ls].", *PackageFileName);
	}
}

UMoviePipelineOutputDataContainer* FMoviePipelinePostRenderFileDisplayProcessor::GetPersistedOutputData()
{
	UMoviePipelineOutputDataContainer* OutputData = nullptr;

	// Use FindPackage first to avoid triggering the async loading path on packages that are already in memory.
	const UPackage* OutputDataPackage = FindPackage(nullptr, *PersistedOutputDataPackagePath);
	if (!OutputDataPackage)
	{
		OutputDataPackage = LoadPackage(nullptr, *PersistedOutputDataPackagePath, LOAD_None);
	}

	if (OutputDataPackage)
	{
		OutputData = Cast<UMoviePipelineOutputDataContainer>(FindObjectWithOuter(OutputDataPackage, UMoviePipelineOutputDataContainer::StaticClass()));
	}

	// The output data asset may not exist on disk yet, or (rarely) the object may be from a future version (eg, if the user went back to a
	// previous build). Both of these situations require a new output data asset to be created.
	if (!OutputData)
	{
		UPackage* NewOutputDataPackage = CreatePackage(*PersistedOutputDataPackagePath);
		OutputData = NewObject<UMoviePipelineOutputDataContainer>(NewOutputDataPackage);
	}

	return OutputData;
}
