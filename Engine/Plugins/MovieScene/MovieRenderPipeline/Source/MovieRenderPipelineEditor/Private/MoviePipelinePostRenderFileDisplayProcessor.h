// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MoviePipelineQueue.h"
#include "MoviePipelinePostRenderSettings.h"

#include "MoviePipelinePostRenderFileDisplayProcessor.generated.h"

/** UObject wrapper for the FMoviePipelineOutputData struct so it can be saved in a UPackage. */
UCLASS()
class UMoviePipelineOutputDataContainer : public UObject
{
	GENERATED_BODY()

public:
	/** The output data that the pipeline generated. Each entry in the array is the output data for one job. */
	UPROPERTY()
	TArray<FMoviePipelineOutputData> OutputData;
};

/** Handles sorting through all of the images/videos generated by a render(s), and opening them according to post-render editor preferences. */
class FMoviePipelinePostRenderFileDisplayProcessor
{
public:
	explicit FMoviePipelinePostRenderFileDisplayProcessor(const FMovieGraphPostRenderSettings& InPostRenderSettings);

	/**
	 * Adds a set of files to the processor. This should only be called once per job, otherwise duplicate data may be added.
	 * Both graph and preset output data is supported.
	 */
	void AddFiles(const FMoviePipelineOutputData& InRenderOutputData);

	/** Opens/displays rendered images/videos according to the post-render settings the class was initialized with. */
	void OpenFiles() const;

	/**
	 * Open the files that OpenFiles() last opened, using the provided post-render settings (the previous files can be played back with
	 * different settings than they were originally opened with).
	 */
	static void OpenPreviousFiles(const FMovieGraphPostRenderSettings& InPostRenderSettings);

	/** Determines if OpenPreviousFiles() has any files that it can open. */
	static bool CanOpenPreviousFiles();

private:
	/** Keeps track of the working set of images/videos that should be opened, and any other associated transient data. */
	struct FFilesToOpen
	{
		TArray<FString> Images;
		TArray<FString> AppleProResMovies;
		TArray<FString> AvidDNxHRMovies;
		TArray<FString> MP4Movies;

		// The start and end frame of the first image sequence contained in Images. Representing these as strings is a little strange. It's
		// to preserve the padding found in the source filename so it doesn't have to be re-derived later in the pipeline (where determining what that
		// padding is may be tricky). For example, instead of storing an int-based start frame of 10, we can store it as "0010", which is how it
		// appears in the filename.
		FString StartFrameString;
		FString EndFrameString;

		int32 StartFrame = TNumericLimits<int32>::Min();
		int32 EndFrame = TNumericLimits<int32>::Min();

		FString HighestPriorityImageExtension;
		FString HighestPriorityVideoExtension;
	};

	/** Sorts through all of the given rendered data and determines which images/videos should be opened. */
	FFilesToOpen FindFilesToOpen(FMoviePipelineOutputData& InPipelineOutputData) const;

	/** Reduces the files returned to FindFilesToOpen() to a set of files that can be opened in the same application. */
	void GetFilteredFilesToOpen(const FFilesToOpen& InFilesToOpen, TArray<FString>& OutFilteredFilesToOpen, const FMovieGraphPostRenderVideoPlayOptions*& OutFilteredPlayOptions) const;

	/** Group all pipeline output data by shot (and additionally extension within each shot) for three categories: images, frame-templated images, and movies. */
	void GroupFilesByShot(
		FMoviePipelineOutputData& InPipelineOutputData,
		TArray<TMap<FString, TArray<FString>>>& InImagesGroupedByShot,
		TArray<TMap<FString, TArray<FString>>>& InFrameTemplateImagesGroupedByShot,
		TArray<TMap<FString, TArray<FString>>>& InVideosGroupedByShot) const;

	/** Helper that implements graph-based logic for GroupFilesByShot(). */
	void GroupFilesByShot_Graph(
		FMoviePipelineOutputData& InPipelineOutputData,
		TArray<TMap<FString, TArray<FString>>>& InImagesGroupedByShot,
		TArray<TMap<FString, TArray<FString>>>& InFrameTemplateImagesGroupedByShot,
		TArray<TMap<FString, TArray<FString>>>& InVideosGroupedByShot) const;

	/** Helper that implements preset-based logic for GroupFilesByShot(). */
	void GroupFilesByShot_Preset(
		FMoviePipelineOutputData& InPipelineOutputData,
		TArray<TMap<FString, TArray<FString>>>& InImagesGroupedByShot,
		TArray<TMap<FString, TArray<FString>>>& InFrameTemplateImagesGroupedByShot,
		TArray<TMap<FString, TArray<FString>>>& InVideosGroupedByShot) const;

	void GetHighestPriorityExtensions(
		const TArray<TMap<FString, TArray<FString>>>& InImagesGroupedByShot,
		const TArray<TMap<FString, TArray<FString>>>& InVideosGroupedByShot,
		FString& OutHighestPriorityImageExtension,
		FString& OutHighestPriorityVideoExtension) const;

	/** Opens the given files according to the playback settings provided. The frame range of files to open must be specified, but can be empty strings (to open all frames). */
	void LaunchFilesWithSettings(const TArray<FString>& InFilesToOpen, const FMovieGraphPostRenderVideoPlayOptions* InPlayOptions, const TTuple<FString, FString>& InFrameRangeToOpen) const;

	/**
	 * Gets the extension's priority according to the Output Type Priority Order list, or 10000 if the extension was not found. A lower index indicates
	 * higher priority.
	 */
	int32 GetExtensionPriorityIndex(const FString& InExtension) const;

	/** Gets the start and end frame of the images provided, based on the frame-templated path. The string and int-based frame numbers in InOutFilesToOpen are updated. */
	void GetStartAndEndFrames(const FString& InTemplatedPath, const TArray<FString>& InImagePaths, FFilesToOpen& InOutFilesToOpen) const;

	/** Persists the provided pipeline output data (one array entry per job) to disk at /Temp/MovieRenderPipeline/MostRecentOutputData. */
	static void PersistOutputData(const TArray<FMoviePipelineOutputData>& InOutputData);

	/**
	 * Gets the most recently persisted pipeline output data (see PersistOutputData()). Note that if data has not been persisted yet, this method
	 * will still return a valid pointer to an output data container (however, it will not contain any file paths within it).
	 */
	static UMoviePipelineOutputDataContainer* GetPersistedOutputData();

private:
	/** The package path for the output data persisted across editor sessions (see PersistOutputData()). */
	inline static const FString PersistedOutputDataPackagePath = TEXT("/Temp/MovieRenderPipeline/MostRecentOutputData");

	/** The post-render settings the class was initialized with. Usually sourced from the editor preferences. */
	FMovieGraphPostRenderSettings PostRenderSettings;

	// Marked as mutable so the map (within the FMoviePipelineOutputData data structure) can be sorted internally to this class
	/** Pipeline output data sourced from AddFiles(). Each entry in the array is one call from AddFiles(). */
	mutable TArray<FMoviePipelineOutputData> PipelineOutputData;
};