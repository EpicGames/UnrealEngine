// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineQueueSubsystem.h"

#include "Framework/Notifications/NotificationManager.h"
#include "LevelSequenceEditorModule.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "LevelSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineQueueSubsystem)

#define LOCTEXT_NAMESPACE "MoviePipelineQueueSubsystem"

bool UMoviePipelineQueueSubsystem::LoadQueue(UMoviePipelineQueue* QueueToLoad, const bool bPromptOnReplacingDirtyQueue)
{
	if (!QueueToLoad)
	{
		return false;
	}

	UMoviePipelineQueue* Queue = GetQueue();
	if (!Queue)
	{
		return false;
	}

	if (bPromptOnReplacingDirtyQueue && IsQueueDirty())
	{
		const FText TitleText = LOCTEXT("UnsavedQueueWarningTitle", "Unsaved Changes to Queue");
		const FText MessageText = LOCTEXT("UnsavedQueueWarningMessage", "The changes made to the current queue will be lost by importing another queue. Do you want to continue with this import?");

		// If the application is in unattended mode, auto-accept the dialog
		constexpr EAppReturnType::Type DefaultReturn = EAppReturnType::Yes;
		
		if (FMessageDialog::Open(EAppMsgType::YesNo, DefaultReturn, MessageText, TitleText) == EAppReturnType::No)
		{
			return false;
		}
	}
	
	Queue->CopyFrom(QueueToLoad);
	Queue->SetQueueOrigin(QueueToLoad);
	Queue->SetIsDirty(false);

	// Update the shot list in case the stored queue being copied is out of date with the sequence
	for (UMoviePipelineExecutorJob* Job : Queue->GetJobs())
	{
		if (ULevelSequence* LoadedSequence = Cast<ULevelSequence>(Job->Sequence.TryLoad()))
		{
			bool bShotsChanged = false;
			UMoviePipelineBlueprintLibrary::UpdateJobShotListFromSequence(LoadedSequence, Job, bShotsChanged);

			if (bShotsChanged)
			{
				FNotificationInfo Info(LOCTEXT("QueueShotsUpdated", "Shots have changed since the queue was saved, please resave the queue"));
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
			}
		}
	}

	OnQueueLoaded.Broadcast();

	return true;
}

bool UMoviePipelineQueueSubsystem::IsQueueDirty() const
{
	const UMoviePipelineQueue* Queue = GetQueue();
	if (!Queue)
	{
		return false;
	}

	// The queue is considered dirty if the current queue has no origin (ie, it has never been saved) or it has been
	// modified since it was loaded. We skip the no-origin check if there are no jobs (to avoid triggering on an
	// empty, first-load queue), but we don't skip the IsDirty check because a queue that had jobs and then
	// removed them will be dirty.
	const bool bShouldCheckQueueOrigin = Queue->GetJobs().Num() > 0;
	bool bHasNoQueueOrigin = false;
	if (bShouldCheckQueueOrigin)
	{
		bHasNoQueueOrigin = !Queue->GetQueueOrigin();
	}
	
	return bHasNoQueueOrigin || Queue->IsDirty();
}

UMoviePipelineExecutorBase* UMoviePipelineQueueSubsystem::RenderQueueInstanceWithExecutor(UMoviePipelineQueue* InQueue, TSubclassOf<UMoviePipelineExecutorBase> InExecutorType)
{
	if(!ensureMsgf(!IsRendering(), TEXT("RenderQueueWithExecutor cannot be called while already rendering!")))
	{
		return nullptr;
	}

	ActiveExecutor = NewObject<UMoviePipelineExecutorBase>(this, InExecutorType);

	if (InQueue)
	{
		RenderQueueInstanceWithExecutorInstance(InQueue, ActiveExecutor);
	}
	else
	{
		RenderQueueWithExecutorInstance(ActiveExecutor);
	}
	
	return ActiveExecutor;
}

UMoviePipelineExecutorBase* UMoviePipelineQueueSubsystem::RenderQueueWithExecutor(TSubclassOf<UMoviePipelineExecutorBase> InExecutorType)
{
	return RenderQueueInstanceWithExecutor(nullptr, InExecutorType);
}

void UMoviePipelineQueueSubsystem::RenderQueueInstanceWithExecutorInstance(UMoviePipelineQueue* InQueue, UMoviePipelineExecutorBase* InExecutor)
{
	if (!ensureMsgf(!IsRendering(), TEXT("RenderQueueWithExecutor cannot be called while already rendering!")))
	{
		FFrame::KismetExecutionMessage(TEXT("Render already in progress."), ELogVerbosity::Error);
		return;
	}

	if (!InExecutor)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid executor supplied."), ELogVerbosity::Error);
		return;
	}

	ILevelSequenceEditorModule* LevelSequenceEditorModule = FModuleManager::GetModulePtr<ILevelSequenceEditorModule>("LevelSequenceEditor");
	if (LevelSequenceEditorModule)
	{
		LevelSequenceEditorModule->OnComputePlaybackContext().AddUObject(this, &UMoviePipelineQueueSubsystem::OnSequencerContextBinding);
	}

	ActiveExecutor = InExecutor;
	ActiveExecutor->OnExecutorFinished().AddUObject(this, &UMoviePipelineQueueSubsystem::OnExecutorFinished);
	ActiveExecutor->Execute(InQueue);
}

void UMoviePipelineQueueSubsystem::RenderQueueWithExecutorInstance(UMoviePipelineExecutorBase* InExecutor)
{
	RenderQueueInstanceWithExecutorInstance(GetQueue(), InExecutor);
}

void UMoviePipelineQueueSubsystem::OnSequencerContextBinding(bool& bAllowBinding)
{
	if (ActiveExecutor && ActiveExecutor->IsRendering())
	{
		bAllowBinding = false;
	}
}

void UMoviePipelineQueueSubsystem::OnExecutorFinished(UMoviePipelineExecutorBase* InPipelineExecutor, bool bSuccess)
{
	ILevelSequenceEditorModule* LevelSequenceEditorModule = FModuleManager::GetModulePtr<ILevelSequenceEditorModule>("LevelSequenceEditorModule");
	if (LevelSequenceEditorModule)
	{
		LevelSequenceEditorModule->OnComputePlaybackContext().RemoveAll(this);
	}

	ActiveExecutor = nullptr;
}

#undef LOCTEXT_NAMESPACE	// MoviePipelineQueueSubsystem