// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineQuickRenderMenu.h"

#include "Customizations/Graph/MovieGraphQuickRenderSettingsCustomization.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Graph/MovieGraphQuickRender.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "MoviePipelineQuickRenderUIState.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieRenderPipelineEditorModule.h"
#include "MovieRenderPipelineEditorUtils.h"
#include "MovieRenderPipelineSettings.h"
#include "MovieRenderPipelineStyle.h"
#include "PropertyEditorModule.h"
#include "SActionButton.h"
#include "Styling/ToolBarStyle.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/SRichTextBlock.h"

class UMovieGraphVariable;
class UMovieGraphConfig;

#define LOCTEXT_NAMESPACE "FMoviePipelineQuickRenderMenu"

void FMoviePipelineQuickRenderMenu::AddQuickRenderButtonToToolMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	// Use a lambda to fetch the button icon so it can change dynamically with the active mode
	const TAttribute<FSlateIcon> ToolbarIconAttribute = TAttribute<FSlateIcon>::CreateLambda([]()
	{
		const EMovieGraphQuickRenderButtonMode ButtonMode = FMoviePipelineQuickRenderUIState::GetQuickRenderButtonMode();
		const EMovieGraphQuickRenderMode QuickRenderMode = FMoviePipelineQuickRenderUIState::GetQuickRenderMode();

		if (ButtonMode == EMovieGraphQuickRenderButtonMode::QuickRender)
		{
			return GetIconForQuickRenderMode(QuickRenderMode);
		}
		
		return FSlateIcon(FMovieRenderPipelineStyle::Get().GetStyleSetName(), "MovieRenderPipeline.QuickRender.Icon.MovieRenderQueueMode");
	});
	
	// Add to the existing "Content" section in the Assets toolbar
	FToolMenuSection& Section = InMenu->FindOrAddSection("Content");
	
	const FToolMenuEntry QuickRenderLaunchButton =
		FToolMenuEntry::InitToolBarButton(
			"QuickRender",
			FUIAction(
				FExecuteAction::CreateStatic(&FMoviePipelineQuickRenderMenu::QuickRenderButtonPressed)),
			LOCTEXT("QuickRenderButtonName", "Quick Render"),
			TAttribute<FText>::Create(
				TAttribute<FText>::FGetter::CreateLambda([]()
				{
					return LOCTEXT("BeginQuickRender", 
									"Begin Render\n\nThere are two modes to choose from:"
									"\n- Movie Render Queue. This uses the job(s) that are active in the Movie Render Queue editor as the source of the render."
									"\n\n- Quick Render. This performs a render without having to manually configure a queue and a graph. A typical quick render uses "
									"the current map and level sequence, the level sequence's playback range, and the viewport's look, to generate frames. Several different types of quick "
									"renders are available -- see the options drop-down.");
				})
			),
			ToolbarIconAttribute
		);
	
	Section.AddEntry(QuickRenderLaunchButton);

	const FToolMenuEntry QuickRenderOptionsButton = FToolMenuEntry::InitComboButton(
		"QuickRenderOptions",
		FUIAction(),
		FOnGetContent::CreateStatic(&FMoviePipelineQuickRenderMenu::GenerateQuickRenderOptionsMenu),
		LOCTEXT("QuickRenderOptionsLabel", "Quick Render Options"),
		LOCTEXT("QuickRenderOptionsToolTip", "Quick Render Options"),
		TAttribute<FSlateIcon>(),
		true
	);

	Section.AddEntry(QuickRenderOptionsButton);
}

void FMoviePipelineQuickRenderMenu::RemoveQuickRenderButtonToolMenu()
{
	if (UToolMenus* ToolMenus = UToolMenus::TryGet())
	{
		ToolMenus->RemoveEntry("LevelEditor.LevelEditorToolBar.AssetsToolBar", "Content", "QuickRender");
	}
}

void FMoviePipelineQuickRenderMenu::LoadQuickRenderSettings()
{
	InitQuickRenderModeSettingsFromMode(FMoviePipelineQuickRenderUIState::GetQuickRenderMode());
}

void FMoviePipelineQuickRenderMenu::QuickRenderButtonPressed()
{
	LoadQuickRenderSettings();

	if (FMoviePipelineQuickRenderUIState::GetQuickRenderButtonMode() == EMovieGraphQuickRenderButtonMode::QuickRender)
	{
		// The settings window will have its own Render button, so don't perform the render if showing the settings first
		if (FMoviePipelineQuickRenderUIState::GetShouldShowSettingsBeforeRender())
		{
			OpenQuickRenderSettingsWindow(FToolMenuContext());
		}
		else
		{
			UMovieGraphQuickRenderSubsystem* QuickRenderSubsystem = GEditor->GetEditorSubsystem<UMovieGraphQuickRenderSubsystem>();
			QuickRenderSubsystem->BeginQuickRender(FMoviePipelineQuickRenderUIState::GetQuickRenderMode(), QuickRenderModeSettings.Get());
		}
	}
	else
	{
		// If there's an existing Movie Render Queue tab open, try to do a render with what's in the queue. Otherwise, just open the MRQ editor tab.
		if (FGlobalTabmanager::Get()->FindExistingLiveTab(IMovieRenderPipelineEditorModule::MoviePipelineQueueTabName))
		{
			if (!UE::MovieRenderPipelineEditor::Private::PerformLocalRender())
			{
				FMessageDialog::Open(
					EAppMsgType::Ok,
					LOCTEXT("UnableToStartLocalRender", "Unable to start local render. Make sure a job is present in the queue, a render is not currently running, and an executor is specified in Project Settings."));
			}
		}
		else
		{
			FGlobalTabmanager::Get()->TryInvokeTab(IMovieRenderPipelineEditorModule::MoviePipelineQueueTabName);
		}
	}
}

TSharedRef<SWidget> FMoviePipelineQuickRenderMenu::GenerateQuickRenderOptionsMenu()
{
	static const FName MenuName("MoviePipeline.QuickRenderOptionsMenu");

	LoadQuickRenderSettings();

	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);

		GenerateModesMenuSection(Menu);
		GenerateQuickRenderMenuSection(Menu);
		GenerateQuickRenderConfigurationMenuSection(Menu);
		GenerateOutputMenuSection(Menu);
		GenerateSettingsMenuSection(Menu);
	}
	
	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext());
}

void FMoviePipelineQuickRenderMenu::OpenQuickRenderSettingsWindow(const FToolMenuContext& InToolMenuContext)
{
	LoadQuickRenderSettings();
	
	if (const TSharedPtr<SWindow> SettingsWindow = WeakQuickRenderSettingsWindow.Pin())
	{
		SettingsWindow->DrawAttention(FWindowDrawAttentionParameters());
		return;
	}

	// Before the window opens, sync up the mode that the Settings window is using to be the mode that the toolbar shows
	FMoviePipelineQuickRenderUIState::SetWindowQuickRenderMode(FMoviePipelineQuickRenderUIState::GetQuickRenderMode());

	// Init the modes that are available to switch to
	QuickRenderModes.Empty();
	const UEnum* ModesEnum = StaticEnum<EMovieGraphQuickRenderMode>();
	for (int32 EnumIndex = 0; EnumIndex < ModesEnum->NumEnums() - 1; ++EnumIndex)	// -1 to avoid MAX
	{
		QuickRenderModes.Add(MakeShared<FName>(ModesEnum->GetNameByIndex(EnumIndex)));
	}
	
	// Update the graph preset's variable assignments in case the graph was updated since the last time the window opened
	UpdateVariableAssignmentsForCurrentGraph();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bCustomNameAreaLocation = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.ViewIdentifier = "MoviePipelineQuickRenderSettings";
	DetailsViewArgs.bLockable = false;

	// Create the details panel and display the quick render settings
	const TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->RegisterInstancedCustomPropertyLayout(
		UMovieGraphQuickRenderModeSettings::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphQuickRenderSettingsCustomization::MakeInstance));
	DetailsView->SetObject(QuickRenderModeSettings.Get());
	
	const TSharedRef<SWindow> QuickRenderSettingsWindow = SNew(SWindow)
		.Title(LOCTEXT("QuickRenderSettingsWindow_Title", "Quick Render"))
		.SupportsMaximize(false)
		.ClientSize(FVector2D(600.f, 510.f))
		.Content()
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				GenerateQuickRenderSettingsWindowToolbar()
			]

			+ SVerticalBox::Slot()
			[
				DetailsView.ToSharedRef()
			]
		];

	if (const TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(QuickRenderSettingsWindow, RootWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(QuickRenderSettingsWindow);
	}

	WeakDetailsPanel = DetailsView;
	WeakQuickRenderSettingsWindow = QuickRenderSettingsWindow;
}

TSharedRef<SWidget> FMoviePipelineQuickRenderMenu::GenerateQuickRenderSettingsWindowToolbar()
{
	auto GetWidgetForQuickRenderMode = [](TFunction<EMovieGraphQuickRenderMode()>&& GetQuickRenderMode)
	{
		return
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin{0.0f, 0.0f, 5.0f, 0.0f})
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image_Lambda([GetQuickRenderMode]()
				{
					return GetIconForQuickRenderMode(GetQuickRenderMode()).GetIcon();
				})
				.DesiredSizeOverride(FVector2d(16, 16))
			]

			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text_Lambda([GetQuickRenderMode]()
				{
					return StaticEnum<EMovieGraphQuickRenderMode>()->GetDisplayNameTextByValue(static_cast<int64>(GetQuickRenderMode()));
				})
				.ToolTipText_Lambda([GetQuickRenderMode]()
				{
					const UEnum* ModeEnum = StaticEnum<EMovieGraphQuickRenderMode>();
					
					return ModeEnum->GetToolTipTextByIndex(ModeEnum->GetIndexByValue(static_cast<int64>(GetQuickRenderMode())));
				})
			];
	};
	
	FToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.BeginSection("Quick Render");
	ToolbarBuilder.AddWidget(
		SNew(SComboBox<TSharedPtr<FName>>)
		.OptionsSource(&FMoviePipelineQuickRenderMenu::QuickRenderModes)
		.OnGenerateWidget_Lambda([GetWidgetForQuickRenderMode](TSharedPtr<FName> InItem)
		{
			return GetWidgetForQuickRenderMode([InItem]()
			{
				const int64 ModeValue = StaticEnum<EMovieGraphQuickRenderMode>()->GetValueByName(*InItem);
				return static_cast<EMovieGraphQuickRenderMode>(ModeValue);
			});
		})
		.OnSelectionChanged_Lambda([](TSharedPtr<FName> InSelectedItem, ESelectInfo::Type)
		{
			const EMovieGraphQuickRenderMode NewRenderMode = static_cast<EMovieGraphQuickRenderMode>(
				StaticEnum<EMovieGraphQuickRenderMode>()->GetValueByName(*InSelectedItem));
			FMoviePipelineQuickRenderUIState::SetWindowQuickRenderMode(NewRenderMode);

			// Also inform the details panel of this change. It needs to display new settings for the chosen mode.
			if (const TSharedPtr<IDetailsView> DetailsPanel = WeakDetailsPanel.Pin())
			{
				InitQuickRenderModeSettingsFromMode(NewRenderMode);
				UpdateVariableAssignmentsForCurrentGraph();

				DetailsPanel->SetObject(QuickRenderModeSettings.Get());
			}
		})
		[
			GetWidgetForQuickRenderMode([]()
			{
				return FMoviePipelineQuickRenderUIState::GetWindowQuickRenderMode();
			})
		], NAME_None, false, HAlign_Left);

	ToolbarBuilder.AddWidget(
		SNew(SActionButton)
		.Text(LOCTEXT("QuickRenderSettingsWindow_QuickRenderButtonText", "Quick Render"))
		.Icon(FAppStyle::Get().GetBrush("LevelEditor.OpenCinematic"))
		.ActionButtonType(EActionButtonType::Primary)
		.OnClicked_Lambda([]()
		{
			// Sync the Setting window's mode to the toolbar mode. This is only done when a render is performed, not when the window is closed.
			FMoviePipelineQuickRenderUIState::SetQuickRenderMode(FMoviePipelineQuickRenderUIState::GetWindowQuickRenderMode());
			
			UMovieGraphQuickRenderSubsystem* QuickRenderSubsystem = GEditor->GetEditorSubsystem<UMovieGraphQuickRenderSubsystem>();
			QuickRenderSubsystem->BeginQuickRender(FMoviePipelineQuickRenderUIState::GetQuickRenderMode(), QuickRenderModeSettings.Get());
			
			return FReply::Handled();
		}),
		NAME_None, false, HAlign_Right);

	ToolbarBuilder.EndSection();

	return ToolbarBuilder.MakeWidget();
}

void FMoviePipelineQuickRenderMenu::GenerateModesMenuSection(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("MoviePipelineQuickRenderModes", LOCTEXT("QuickRenderModesSection", "Modes"));

	auto AddModeAction = [&Section](const FName& InName, const FText& InLabel, const FText& InToolTip, const FSlateIcon& InIcon, EMovieGraphQuickRenderButtonMode CheckedValue)
	{
		FToolUIAction NewAction;
		NewAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([CheckedValue](const FToolMenuContext&)
		{
			FMoviePipelineQuickRenderUIState::SetQuickRenderButtonMode(CheckedValue);
			QuickRenderModeSettings->PostEditChange();
		});
		NewAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&) { return true; });
		NewAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([CheckedValue](const FToolMenuContext&)
		{
			return (FMoviePipelineQuickRenderUIState::GetQuickRenderButtonMode() == CheckedValue)
				? ECheckBoxState::Checked
				: ECheckBoxState::Unchecked;
		});
		Section.AddMenuEntry(InName, InLabel, InToolTip, InIcon, NewAction, EUserInterfaceActionType::RadioButton);
	};

	AddModeAction(
		FName("QuickRender_MovieRenderQueueMode"),
		LOCTEXT("QuickRenderMode_MovieRenderQueueLabel", "Movie Render Queue"),
		LOCTEXT("QuickRenderMode_MovieRenderQueueToolTip", "Performs a render with the job(s) currently active in the Movie Render Queue editor."),
		FSlateIcon(FMovieRenderPipelineStyle::Get().GetStyleSetName(), "MovieRenderPipeline.QuickRender.Icon.MovieRenderQueueMode"),
		EMovieGraphQuickRenderButtonMode::NormalMovieRenderQueue);

	AddModeAction(
		FName("QuickRender_QuickRenderMode"),
		LOCTEXT("QuickRenderMode_QuickRenderLabel", "Quick Render"),
		LOCTEXT("QuickRenderMode_QuickRenderToolTip", "Performs a render using the Quick Render settings."),
		FSlateIcon(FMovieRenderPipelineStyle::Get().GetStyleSetName(), "MovieRenderPipeline.QuickRender.Icon.QuickRenderMode"),
		EMovieGraphQuickRenderButtonMode::QuickRender);
}

void FMoviePipelineQuickRenderMenu::GenerateQuickRenderMenuSection(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("MoviePipelineQuickRenderType", LOCTEXT("QuickRenderType", "Quick Render"));

	auto AddQuickRenderModeAction = [&Section](const FName& InName, EMovieGraphQuickRenderMode CheckedValue)
	{
		const UEnum* ModeEnum = StaticEnum<EMovieGraphQuickRenderMode>();
		const int32 ValueIndex = ModeEnum->GetIndexByValue(static_cast<int64>(CheckedValue));
		
		const FText ActionTooltip = ModeEnum->GetToolTipTextByIndex(ValueIndex);
		const FText ActionLabel = ModeEnum->GetDisplayNameTextByIndex(ValueIndex);
		
		FToolUIAction NewAction;
		NewAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([CheckedValue](const FToolMenuContext&)
		{
			FMoviePipelineQuickRenderUIState::SetQuickRenderMode(CheckedValue);
			QuickRenderModeSettings->PostEditChange();
		});
		NewAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&) { return true; });
		NewAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([CheckedValue](const FToolMenuContext&)
		{
			return (FMoviePipelineQuickRenderUIState::GetQuickRenderMode() == CheckedValue)
				? ECheckBoxState::Checked
				: ECheckBoxState::Unchecked;
		});
		Section.AddMenuEntry(InName, ActionLabel, ActionTooltip, GetIconForQuickRenderMode(CheckedValue), NewAction, EUserInterfaceActionType::RadioButton);
	};

	AddQuickRenderModeAction(
		FName("QuickRender_CurrentSequence"),
		EMovieGraphQuickRenderMode::CurrentSequence);

	AddQuickRenderModeAction(
		FName("QuickRender_UseViewportCamera"),
		EMovieGraphQuickRenderMode::UseViewportCameraInSequence);

	// The "Current Shot at Playhead" mode is disabled for now, may be added in the future.
	/*AddQuickRenderModeAction(
		FName("QuickRender_CurrentShotAtPlayhead"),
		EMovieGraphQuickRenderMode::CurrentShotAtPlayhead);*/

	AddQuickRenderModeAction(
		FName("QuickRender_CurrentViewport"),
		EMovieGraphQuickRenderMode::CurrentViewport);

	AddQuickRenderModeAction(
		FName("QuickRender_SelectedCameras"),
		EMovieGraphQuickRenderMode::SelectedCameras);
}

void FMoviePipelineQuickRenderMenu::GenerateQuickRenderConfigurationMenuSection(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("MoviePipelineQuickRenderConfiguration", LOCTEXT("QuickRenderConfiguration", "Quick Render Configuration"));
	
	Section.AddEntry(FToolMenuEntry::InitWidget(
		"QuickRenderConfiguration",
		SNew(SBox)
		.Padding(FMargin(16.f, 3.f))
		[
			SNew(SRichTextBlock)
			.TextStyle(FAppStyle::Get(), "NormalText.Subdued")
			.Text_Lambda([]()
			{
				// Generate "Configuration" text
				const FText GraphName = !QuickRenderModeSettings->GraphPreset.IsNull()
					? FText::FromString(QuickRenderModeSettings->GraphPreset.GetAssetName())
					: LOCTEXT("QuickRenderConfigMenu_InvalidGraph", "Invalid");
				
				// Generate "After Render" text
				const FText AfterRender = (QuickRenderModeSettings->PostRenderBehavior == EMoviePipelinePostRenderActionType::PlayRenderOutput)
					? LOCTEXT("QuickRenderConfigMenu_PlayRenderOutput", "Play Render Output")
					: LOCTEXT("QuickRenderConfigMenu_DoNothing", "Do Nothing");
				
				// Generate "Viewport Look" text
				const EMovieGraphQuickRenderViewportLookFlags ViewportLookFlags = static_cast<EMovieGraphQuickRenderViewportLookFlags>(QuickRenderModeSettings->ViewportLookFlags);
				TArray<FText> ViewportFlags;
				if (EnumHasAnyFlags(ViewportLookFlags, EMovieGraphQuickRenderViewportLookFlags::Ocio))
				{
					ViewportFlags.Add(LOCTEXT("QuickRenderConfigMenu_OCIO", "OCIO"));
				}
				if (EnumHasAnyFlags(ViewportLookFlags, EMovieGraphQuickRenderViewportLookFlags::ShowFlags))
				{
					ViewportFlags.Add(LOCTEXT("QuickRenderConfigMenu_ShowFlags", "Show Flags"));
				}
				if (EnumHasAnyFlags(ViewportLookFlags, EMovieGraphQuickRenderViewportLookFlags::ViewMode))
				{
					ViewportFlags.Add(LOCTEXT("QuickRenderConfigMenu_ViewMode", "View Mode"));
				}
				if (ViewportLookFlags == EMovieGraphQuickRenderViewportLookFlags::None)
				{
					ViewportFlags.Add(LOCTEXT("QuickRenderConfigMenu_DontApply", "Don't Apply"));
				}
				
				// Generate "Sequencer Frame Range" text
				const FText SequencerFrameRange = StaticEnum<EMovieGraphQuickRenderFrameRangeType>()->GetDisplayNameTextByValue(static_cast<int64>(QuickRenderModeSettings->FrameRangeType));

				// TODO: Add in graph variables summary as well (although this might make a summary that's too long in some cases)
				FText ConfigurationSummaryText = FText::Format(
					LOCTEXT("QuickRenderConfigurationSummary", "Configuration: <RichTextBlock.BoldHighlight>{0}</>\nAfter Render: {1}\nApply Viewport Look: {2}\nSequencer Frame Range: {3}"),
					{
						GraphName,
						AfterRender,
						FText::Join(LOCTEXT("CommaDelim", ", "), ViewportFlags),
						SequencerFrameRange,
					});

				return ConfigurationSummaryText;
			})
			.LineHeightPercentage(1.3f)
		],
		FText::GetEmpty()
	));
}

void FMoviePipelineQuickRenderMenu::GenerateOutputMenuSection(UToolMenu* InMenu)
{
	// Only show the section if Quick Render mode is active
	FToolMenuSection& Section = InMenu->AddSection("MoviePipelineQuickRenderOutput", LOCTEXT("QuickRenderOutput", "Output"));
	Section.Visibility = TAttribute<EVisibility>::CreateLambda([]() -> EVisibility
	{
		return (FMoviePipelineQuickRenderUIState::GetQuickRenderButtonMode() == EMovieGraphQuickRenderButtonMode::QuickRender)
			? EVisibility::Visible
			: EVisibility::Collapsed;
	});

	// Menu entry for "Play Last Render"
	// ---------------------------------
	FToolUIAction PlayLastRenderAction;
	PlayLastRenderAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext&)
	{
		UMovieGraphQuickRenderSubsystem* QuickRenderSubsystem = GEditor->GetEditorSubsystem<UMovieGraphQuickRenderSubsystem>();
		QuickRenderSubsystem->PlayLastRender();
	});
	PlayLastRenderAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&)
	{
		UMovieGraphQuickRenderSubsystem* QuickRenderSubsystem = GEditor->GetEditorSubsystem<UMovieGraphQuickRenderSubsystem>();
		return QuickRenderSubsystem->CanPlayLastRender();
	});
	Section.AddMenuEntry(
		"QuickRender_PlayLastRender",
		LOCTEXT("QuickRender_PlayLastRenderLabel", "Play Last Render"),
		LOCTEXT("QuickRender_PlayLastRenderToolTip", "Play the media from the last time that Quick Render ran."),
		FSlateIcon(FMovieRenderPipelineStyle::Get().GetStyleSetName(), "MovieRenderPipeline.QuickRender.Icon.PlayLastRender"),
		PlayLastRenderAction);

	// Menu entry for "Open Output Directory"
	// --------------------------------------
	FToolUIAction OpenOutputDirectoryAction;
	OpenOutputDirectoryAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext&)
	{
		UMovieGraphQuickRenderSubsystem* QuickRenderSubsystem = GEditor->GetEditorSubsystem<UMovieGraphQuickRenderSubsystem>();
		QuickRenderSubsystem->OpenOutputDirectory(QuickRenderModeSettings.Get());
	});
	OpenOutputDirectoryAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&) { return true; });
	Section.AddMenuEntry(
		"QuickRender_OpenOutputDirectory",
		LOCTEXT("QuickRender_OpenOutputDirectoryLabel", "Open Output Directory"),
		LOCTEXT("QuickRender_OpenOutputDirectoryToolTip", "Open the output directory that Quick Render is currently configured to save media into."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderOpen"),
		OpenOutputDirectoryAction);
}

void FMoviePipelineQuickRenderMenu::GenerateSettingsMenuSection(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("MoviePipelineQuickRenderSettings");

	Section.AddSeparator("SettingsSeparator");

	FToolUIAction ShowSettingsAction;
        ShowSettingsAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext&)
        {
			FMoviePipelineQuickRenderUIState::SetShouldShowSettingsBeforeRender(!FMoviePipelineQuickRenderUIState::GetShouldShowSettingsBeforeRender());
        	QuickRenderModeSettings->PostEditChange();
        });
        ShowSettingsAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&) { return true; });
        ShowSettingsAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([](const FToolMenuContext&)
        {
            return FMoviePipelineQuickRenderUIState::GetShouldShowSettingsBeforeRender()
				? ECheckBoxState::Checked
				: ECheckBoxState::Unchecked;
        });
        Section.AddMenuEntry(
            "QuickRender_ShowSettingsBefore",
            LOCTEXT("QuickRender_ShowSettingsBeforeLabel", "Show Settings Before Quick Render"),
            LOCTEXT("QuickRender_ShowSettingsBeforeToolTip", "Show the quick render settings before starting a quick render."),
            FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.OpenInExternalEditor"),
            ShowSettingsAction,
            EUserInterfaceActionType::ToggleButton);

	FToolUIAction OpenSettingsAction;
	OpenSettingsAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&FMoviePipelineQuickRenderMenu::OpenQuickRenderSettingsWindow);
	OpenSettingsAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&) { return true; });
	Section.AddMenuEntry(
		"QuickRender_OpenSettings",
		LOCTEXT("QuickRender_OpenSettingsLabel", "Quick Render Settings"),
		LOCTEXT("QuickRender_OpenSettingsToolTip", "Open the quick render settings."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"),
		OpenSettingsAction);
}

FSlateIcon FMoviePipelineQuickRenderMenu::GetIconForQuickRenderMode(const EMovieGraphQuickRenderMode QuickRenderMode)
{
	switch(QuickRenderMode)
	{
	case EMovieGraphQuickRenderMode::CurrentSequence:
		return FSlateIcon(FMovieRenderPipelineStyle::Get().GetStyleSetName(), "MovieRenderPipeline.QuickRender.Icon.CurrentSequenceMode");
	case EMovieGraphQuickRenderMode::CurrentViewport:
		return FSlateIcon(FMovieRenderPipelineStyle::Get().GetStyleSetName(), "MovieRenderPipeline.QuickRender.Icon.CurrentViewportMode");
	case EMovieGraphQuickRenderMode::SelectedCameras:
		return FSlateIcon(FMovieRenderPipelineStyle::Get().GetStyleSetName(), "MovieRenderPipeline.QuickRender.Icon.SelectedCamerasMode");
	case EMovieGraphQuickRenderMode::UseViewportCameraInSequence:
		return FSlateIcon(FMovieRenderPipelineStyle::Get().GetStyleSetName(), "MovieRenderPipeline.QuickRender.Icon.ViewportCameraInSequenceMode");
	default:
		return FSlateIcon();
	}
}

void FMoviePipelineQuickRenderMenu::InitQuickRenderModeSettingsFromMode(const EMovieGraphQuickRenderMode QuickRenderMode)
{
	QuickRenderModeSettings = TStrongObjectPtr(UMovieGraphQuickRenderSettings::GetSavedQuickRenderModeSettings(QuickRenderMode));
}

void FMoviePipelineQuickRenderMenu::UpdateVariableAssignmentsForCurrentGraph()
{
	MoviePipeline::RefreshVariableAssignments(QuickRenderModeSettings->GraphPreset.LoadSynchronous(), QuickRenderModeSettings->GraphVariableAssignments, QuickRenderModeSettings.Get());
}

#undef LOCTEXT_NAMESPACE