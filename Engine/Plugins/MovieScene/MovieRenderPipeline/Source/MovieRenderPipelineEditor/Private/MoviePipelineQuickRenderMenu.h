// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Graph/MovieGraphQuickRenderSettings.h"
#include "Textures/SlateIcon.h"

struct FToolMenuContext;
class SWidget;
class SWindow;
class UToolMenu;

/** Generates the Quick Render button, options menu, and settings dialog. */
class FMoviePipelineQuickRenderMenu
{
public:
	FMoviePipelineQuickRenderMenu() = default;

	/** Adds the Quick Render button to the specified tool menu. */
	static void AddQuickRenderButtonToToolMenu(UToolMenu* InMenu);

	/** Removes the Quick Render button from level editor toolbar. */
	static void RemoveQuickRenderButtonToolMenu();

private:
	static void LoadQuickRenderSettings();
	
	static void QuickRenderButtonPressed();
	
	static TSharedRef<SWidget> GenerateQuickRenderOptionsMenu();
	static void OpenQuickRenderSettingsWindow(const FToolMenuContext& InToolMenuContext);
	static TSharedRef<SWidget> GenerateQuickRenderSettingsWindowToolbar();

	static void GenerateModesMenuSection(UToolMenu* InMenu);
	static void GenerateQuickRenderMenuSection(UToolMenu* InMenu);
	static void GenerateQuickRenderConfigurationMenuSection(UToolMenu* InMenu);
	static void GenerateOutputMenuSection(UToolMenu* InMenu);
	static void GenerateSettingsMenuSection(UToolMenu* InMenu);

	static FSlateIcon GetIconForQuickRenderMode(EMovieGraphQuickRenderMode QuickRenderMode);

	/** Populates the QuickRenderModeSettings member with the settings for the provided mode. */
	static void InitQuickRenderModeSettingsFromMode(const EMovieGraphQuickRenderMode QuickRenderMode);

	/** Updates the variable assignments for the graph preset in the currently active mode. */
	static void UpdateVariableAssignmentsForCurrentGraph();

private:
	/** Weak pointer to the Settings window widget. */
	inline static TWeakPtr<SWindow> WeakQuickRenderSettingsWindow;

	/** Weak pointer to the details panel within the Settings window. */
	inline static TWeakPtr<class IDetailsView> WeakDetailsPanel;

	/** The settings for the currently-active mode within Quick Render. */
	inline static TStrongObjectPtr<UMovieGraphQuickRenderModeSettings> QuickRenderModeSettings = nullptr;

	/** All of the different mode names that are available to be used. */
	inline static TArray<TSharedPtr<FName>> QuickRenderModes;
};
