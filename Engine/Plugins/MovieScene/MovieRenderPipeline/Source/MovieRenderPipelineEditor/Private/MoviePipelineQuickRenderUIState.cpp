// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineQuickRenderUIState.h"

#include "Misc/ConfigCacheIni.h"

EMovieGraphQuickRenderButtonMode FMoviePipelineQuickRenderUIState::GetQuickRenderButtonMode()
{
	constexpr EMovieGraphQuickRenderButtonMode DefaultButtonMode = EMovieGraphQuickRenderButtonMode::QuickRender;
	
	return static_cast<EMovieGraphQuickRenderButtonMode>(
		GConfig->GetIntOrDefault(QuickRenderIniSection, QuickRenderIniSettingName_ButtonMode, static_cast<int32>(DefaultButtonMode), GEditorPerProjectIni));
}

void FMoviePipelineQuickRenderUIState::SetQuickRenderButtonMode(const EMovieGraphQuickRenderButtonMode NewButtonMode)
{
	GConfig->SetInt(QuickRenderIniSection, QuickRenderIniSettingName_ButtonMode, static_cast<int32>(NewButtonMode), GEditorPerProjectIni);
}

EMovieGraphQuickRenderMode FMoviePipelineQuickRenderUIState::GetQuickRenderMode()
{
	constexpr EMovieGraphQuickRenderMode DefaultMode = EMovieGraphQuickRenderMode::CurrentSequence;
	
	return static_cast<EMovieGraphQuickRenderMode>(
		GConfig->GetIntOrDefault(QuickRenderIniSection, QuickRenderIniSettingName_Mode, static_cast<int32>(DefaultMode), GEditorPerProjectIni));
}

EMovieGraphQuickRenderMode FMoviePipelineQuickRenderUIState::GetWindowQuickRenderMode()
{
	return WindowRenderMode;
}

void FMoviePipelineQuickRenderUIState::SetWindowQuickRenderMode(const EMovieGraphQuickRenderMode NewMode)
{
	WindowRenderMode = NewMode;
}

void FMoviePipelineQuickRenderUIState::SetQuickRenderMode(const EMovieGraphQuickRenderMode NewMode)
{
	GConfig->SetInt(QuickRenderIniSection, QuickRenderIniSettingName_Mode, static_cast<int32>(NewMode), GEditorPerProjectIni);
}

bool FMoviePipelineQuickRenderUIState::GetShouldShowSettingsBeforeRender()
{
	constexpr bool bDefaultShowSettingsBeforeRender = false;
	
	return GConfig->GetBoolOrDefault(QuickRenderIniSection, QuickRenderIniSettingName_ShowSettingsBeforeQuickRender, bDefaultShowSettingsBeforeRender, GEditorPerProjectIni);
}

void FMoviePipelineQuickRenderUIState::SetShouldShowSettingsBeforeRender(const bool bNewValue)
{
	GConfig->SetInt(QuickRenderIniSection, QuickRenderIniSettingName_ShowSettingsBeforeQuickRender, bNewValue, GEditorPerProjectIni);
}