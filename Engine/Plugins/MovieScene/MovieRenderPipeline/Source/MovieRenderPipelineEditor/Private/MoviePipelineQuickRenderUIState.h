// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphQuickRenderSettings.h"

/**
 * UI state management for Quick Render.
 *
 * Some of these setting values returned here may differ from what Quick Render is actively using. For example, the mode returned here may be
 * different if Quick Render was started from scripting with a mode that differs from the UI's current mode. Also note that the settings here are
 * distinct from the settings in UMovieGraphQuickRenderSettings. Those are persisted to an asset, whereas these are persisted to an INI. 
 */
struct FMoviePipelineQuickRenderUIState
{
	/** Gets the mode that the Quick Render button is currently in. */
	static EMovieGraphQuickRenderButtonMode GetQuickRenderButtonMode();

	/** Sets the mode that the Quick Render button is currently using. */
	static void SetQuickRenderButtonMode(const EMovieGraphQuickRenderButtonMode NewButtonMode);

	/**
	 * Gets the mode that Quick Render will use when starting a new render. Sometimes called the "system" render mode, as opposed to the "window"
	 * render mode (see GetWindowQuickRenderMode()).
	 */
	static EMovieGraphQuickRenderMode GetQuickRenderMode();

	/** Sets the mode that Quick Render will use when starting a new render. */
	static void SetQuickRenderMode(const EMovieGraphQuickRenderMode NewMode);

	/**
	 * Gets the render mode that the Quick Render settings window currently displays. This is different than GetQuickRenderMode(), or the "system"
	 * render mode. When the settings window opens, this mode is independent of the system render mode. When a render starts, the window mode is
	 * copied to the system render mode. This setting is transient and not persisted across editor sessions.
	 */
	static EMovieGraphQuickRenderMode GetWindowQuickRenderMode();

	/** Sets the "window" render mode. See notes on GetWindowQuickRenderMode(). */
	static void SetWindowQuickRenderMode(const EMovieGraphQuickRenderMode NewMode);

	/** Gets whether Quick Render should show the Settings dialog before a render starts. */
	static bool GetShouldShowSettingsBeforeRender();

	/** Sets whether Quick Render should show the Settings dialog before a render starts. */
	static void SetShouldShowSettingsBeforeRender(const bool bNewValue);

private:
	/** The name of the section that Quick Render UI settings are stored in within the target ini file. */
	inline static const TCHAR* QuickRenderIniSection = TEXT("MovieRenderPipeline.QuickRender");

	/** Ini setting name for: What mode the Quick Render button is using. */
	inline static const TCHAR* QuickRenderIniSettingName_ButtonMode = TEXT("ButtonMode");

	/** Ini setting name for: The mode that Quick Render will use when a render begins. Generally dictates the level sequence and camera(s) that will be used. */
	inline static const TCHAR* QuickRenderIniSettingName_Mode = TEXT("Mode");

	/** Ini setting name for: Whether the settings dialog should be shown before a Quick Render is started. */
	inline static const TCHAR* QuickRenderIniSettingName_ShowSettingsBeforeQuickRender = TEXT("bShowSettingsBeforeQuickRender");

	/** The transient "window" quick render mode. This is not persisted to ini. */
	static inline EMovieGraphQuickRenderMode WindowRenderMode = EMovieGraphQuickRenderMode::CurrentViewport;
};
