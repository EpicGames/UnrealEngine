// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderPipelineEditorModule.h"

#include "EdGraphUtilities.h"
#include "EditorWidgetsModule.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineExecutor.h"
#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructure.h"
#include "Widgets/SMoviePipelineConfigTabContent.h"
#include "Widgets/SMoviePipelineQueueTabContent.h"
#include "ISettingsModule.h"
#include "ISequencerModule.h"
#include "IMovieRendererInterface.h"
#include "MovieRenderPipelineSettings.h"
#include "HAL/IConsoleManager.h"
#include "MoviePipelineExecutor.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/TabManager.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Misc/PackageName.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "Templates/SubclassOf.h"
#include "MovieRenderPipelineStyle.h"
#include "MoviePipelineCommands.h"
#include "MoviePipelineDeferredPasses.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineQueueSubsystem.h"
#include "MoviePipelineQuickRenderMenu.h"
#include "MoviePipelineEditorBlueprintLibrary.h"
#include "MovieSceneSection.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "MoviePipelineConsoleVariableSetting.h"
#include "ObjectNameEditSinkRegistry.h"
#include "PropertyEditorModule.h"
#include "Customizations/ConsoleVariableCustomization.h"
#include "Customizations/Graph/MovieGraphNamedResolutionCustomization.h"
#include "Customizations/MoviePipelineNameEditSinks.h"
#include "LevelSequence.h"
#include "ToolMenus.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphNamedResolution.h"
#include "Graph/MovieGraphPinFactory.h"

#define LOCTEXT_NAMESPACE "FMovieRenderPipelineEditorModule"

FName IMovieRenderPipelineEditorModule::MoviePipelineQueueTabName = "MoviePipelineQueue";
FText IMovieRenderPipelineEditorModule::MoviePipelineQueueTabLabel = LOCTEXT("MovieRenderQueueTab_Label", "Movie Render Queue");
FName IMovieRenderPipelineEditorModule::MoviePipelineConfigEditorTabName = "MovieRenderPipeline";
FText IMovieRenderPipelineEditorModule::MoviePipelineConfigEditorTabLabel = LOCTEXT("MovieRenderPipelineTab_Label", "Movie Render Pipeline");

namespace
{
	static TSharedRef<SDockTab> SpawnMovieRenderPipelineTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		TSharedRef<SDockTab> NewTab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SMoviePipelineConfigTabContent)
			];
		
		return NewTab;
	}

	static TSharedRef<SDockTab> SpawnMoviePipelineQueueTab(const FSpawnTabArgs& InSpawnTabArgs)
	{
		TSharedRef<SDockTab> NewTab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SMoviePipelineQueueTabContent)
			];

		return NewTab;
	}

	static void RegisterTabImpl()
	{
		// FTabSpawnerEntry& MRPConfigTabSpawner = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(IMovieRenderPipelineEditorModule::MoviePipelineConfigEditorTabName, FOnSpawnTab::CreateStatic(SpawnMovieRenderPipelineTab));
		// 
		// MRPConfigTabSpawner
		// 	.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
		// 	.SetDisplayName(IMovieRenderPipelineEditorModule::MoviePipelineConfigEditorTabLabel)
		// 	.SetTooltipText(LOCTEXT("MovieRenderPipelineConfigTab_Tooltip", "Open the Movie Render Config UI for creating and editing presets."))
		// 	.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.TabIcon"));

		FTabSpawnerEntry& MRPQueueTabSpawner = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(IMovieRenderPipelineEditorModule::MoviePipelineQueueTabName, FOnSpawnTab::CreateStatic(SpawnMoviePipelineQueueTab));

		MRPQueueTabSpawner
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
			.SetDisplayName(IMovieRenderPipelineEditorModule::MoviePipelineQueueTabLabel)
			.SetTooltipText(LOCTEXT("MovieRenderPipelineQueueTab_Tooltip", "Open the Movie Render Queue to render Sequences to disk at a higher quality than realtime allows."))
			.SetIcon(FSlateIcon(FMovieRenderPipelineStyle::StyleName, "MovieRenderPipeline.TabIcon"));
	}

}

void FMovieRenderPipelineEditorModule::RegisterSettings()
{
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");

	SettingsModule.RegisterSettings("Project", "Plugins", "Movie Render Pipeline",
		LOCTEXT("ProjectSettings_Label", "Movie Render Pipeline"),
		LOCTEXT("ProjectSettings_Description", "Configure project-wide defaults for the movie render pipeline."),
		GetMutableDefault<UMovieRenderPipelineProjectSettings>()
	);
}

void FMovieRenderPipelineEditorModule::UnregisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Movie Render Pipeline");
	}

	IConsoleManager::Get().UnregisterConsoleObject(TEXT("MovieRenderPipeline.TestRenderSequence"));
}

class FMovieRenderPipelineRenderer : public IMovieRendererInterface
{
	virtual void RenderMovie(UMovieSceneSequence* InSequence, const TArray<UMovieSceneCinematicShotSection*>& InSections) override
	{
		UMoviePipelineQueue* ActiveQueue = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>()->GetQueue();
		check(ActiveQueue);

		UMoviePipelineExecutorJob* ActiveJob = nullptr;
		for (UMoviePipelineExecutorJob* Job : ActiveQueue->GetJobs())
		{
			if (Job->Sequence == FSoftObjectPath(InSequence))
			{
				ActiveJob = Job;
				break;
			}
		}

		if (!ActiveJob)
		{
			GEditor->BeginTransaction(FText::Format(LOCTEXT("CreateJob_Transaction", "Add {0}|plural(one=Job, other=Jobs)"), 1));
				
			ActiveJob = UMoviePipelineEditorBlueprintLibrary::CreateJobFromSequence(ActiveQueue, Cast<ULevelSequence>(InSequence));
			UMoviePipelineEditorBlueprintLibrary::ApplyDefaultConfigurationTypeToJob(ActiveJob);
		}

		TArray<FString> SequenceNames;
		for (UMovieSceneCinematicShotSection* ShotSection : InSections)
		{
			if (ShotSection && ShotSection->GetSequence())
			{
				SequenceNames.Add(ShotSection->GetSequence()->GetName());
			}
		}

		if (!GEditor->IsTransactionActive())
		{
			GEditor->BeginTransaction(LOCTEXT("ModifyJob_Transaction", "Modifying shots in existing job"));
		}

		ActiveJob->Modify();

		for (UMoviePipelineExecutorShot* Shot : ActiveJob->ShotInfo)
		{
			// If no specified shots, enabled them all
			if (Shot)
			{
				if (SequenceNames.Num() == 0)
				{
					Shot->bEnabled = true;
				}
				else
				{
					Shot->bEnabled = false;
					for (const FString& SequenceName : SequenceNames)
					{
						if (Shot->OuterName.Contains(SequenceName))
						{
							Shot->bEnabled = true;
							break;
						}
					}
				}
			}
		}

		if (GEditor->IsTransactionActive())
		{
			GEditor->EndTransaction();
		}

		FGlobalTabmanager::Get()->TryInvokeTab(IMovieRenderPipelineEditorModule::MoviePipelineQueueTabName);
	};

	virtual FString GetDisplayName() const override { return IMovieRenderPipelineEditorModule::MoviePipelineQueueTabLabel.ToString(); }
};

void FMovieRenderPipelineEditorModule::RegisterMovieRenderer()
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");

	MovieRendererDelegate = SequencerModule.RegisterMovieRenderer(TUniquePtr<IMovieRendererInterface>(new FMovieRenderPipelineRenderer));
}

void FMovieRenderPipelineEditorModule::UnregisterMovieRenderer()
{
	ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
	if (SequencerModule)
	{
		SequencerModule->UnregisterMovieRenderer(MovieRendererDelegate);
	}
}

void FMovieRenderPipelineEditorModule::RegisterTypeCustomizations()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	PropertyModule.RegisterCustomPropertyTypeLayout(FMoviePipelineConsoleVariableEntry::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(FConsoleVariablesDetailsCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(FMovieGraphNamedResolution::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieGraphNamedResolutionCustomization::MakeInstance));
}

void FMovieRenderPipelineEditorModule::UnregisterTypeCustomizations()
{
	if (UObjectInitialized())
	{
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyModule)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(FMoviePipelineConsoleVariableEntry::StaticStruct()->GetFName());
			PropertyModule->UnregisterCustomPropertyTypeLayout(FMovieGraphNamedResolution::StaticStruct()->GetFName());
		}
	}
}

void FMovieRenderPipelineEditorModule::RegisterToolbarItem()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.AssetsToolBar");
	
	UnregisterToolbarItem();
	
	FMoviePipelineQuickRenderMenu::AddQuickRenderButtonToToolMenu(Menu);
}

void FMovieRenderPipelineEditorModule::UnregisterToolbarItem()
{
	FMoviePipelineQuickRenderMenu::RemoveQuickRenderButtonToolMenu();
}

void FMovieRenderPipelineEditorModule::StartupModule()
{
	// Initialize our custom style
	FMovieRenderPipelineStyle::Get();
	FMoviePipelineCommands::Register();

	RegisterTabImpl();
	RegisterSettings();
	RegisterTypeCustomizations();
	RegisterMovieRenderer();
	RegisterToolbarItem();

	GraphPanelPinFactory = MakeShared<FMovieGraphPanelPinFactory>();
	FEdGraphUtilities::RegisterVisualPinFactory(GraphPanelPinFactory);

	// Register Object Name sinks for the job/shot so their title appearance is customized in the details panel.
	const FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	JobNameEditSink = MakeShared<FMoviePipelineJobNameEditSink>();
	ShotNameEditSink = MakeShared<FMoviePipelineShotNameEditSink>();
	EditorWidgetsModule.GetObjectNameEditSinkRegistry()->RegisterObjectNameEditSink(JobNameEditSink.ToSharedRef());
	EditorWidgetsModule.GetObjectNameEditSinkRegistry()->RegisterObjectNameEditSink(ShotNameEditSink.ToSharedRef());
}

void FMovieRenderPipelineEditorModule::ShutdownModule()
{
	UnregisterToolbarItem();
	UnregisterMovieRenderer();
	UnregisterTypeCustomizations();
	UnregisterSettings();
	FMoviePipelineCommands::Unregister();
	
	if (GraphPanelPinFactory.IsValid())
	{
		FEdGraphUtilities::UnregisterVisualPinFactory(GraphPanelPinFactory);
		GraphPanelPinFactory.Reset();
	}

	if (FModuleManager::Get().IsModuleLoaded("EditorWidgets"))
	{
		const FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::GetModuleChecked<FEditorWidgetsModule>("EditorWidgets");
		if (JobNameEditSink.IsValid())
		{
			EditorWidgetsModule.GetObjectNameEditSinkRegistry()->UnregisterObjectNameEditSink(JobNameEditSink.ToSharedRef());
		}
		if (ShotNameEditSink.IsValid())
		{
			EditorWidgetsModule.GetObjectNameEditSinkRegistry()->UnregisterObjectNameEditSink(ShotNameEditSink.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMovieRenderPipelineEditorModule, MovieRenderPipelineEditor)
