// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieRenderPipelineEditorModule.h"
#include "IMovieRendererInterface.h"
#include "Delegates/IDelegateInstance.h"

class FMovieGraphPanelPinFactory;
class FMoviePipelineJobNameEditSink;
class FMoviePipelineShotNameEditSink;

class FMovieRenderPipelineEditorModule : public IMovieRenderPipelineEditorModule
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
private:
	void RegisterSettings();
	void UnregisterSettings();

	void RegisterMovieRenderer();
	void UnregisterMovieRenderer();

	void RegisterTypeCustomizations();
	void UnregisterTypeCustomizations();

	void RegisterToolbarItem();
	void UnregisterToolbarItem();

	FDelegateHandle MovieRendererDelegate;

	/** Pin factory for the render graph. */
	TSharedPtr<FMovieGraphPanelPinFactory> GraphPanelPinFactory;

	/** ObjectNameEditSink for UMoviePipelineExecutorJob instances in the details panel. */
	TSharedPtr<FMoviePipelineJobNameEditSink> JobNameEditSink;

	/** ObjectNameEditSink for UMoviePipelineExecutorShot instances in the details panel. */
	TSharedPtr<FMoviePipelineShotNameEditSink> ShotNameEditSink;
};
