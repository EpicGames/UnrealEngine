// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderPipelineEditorUtils.h"

#include "Editor.h"
#include "MoviePipelineQueueSubsystem.h"
#include "MoviePipelineTelemetry.h"
#include "MovieRenderPipelineSettings.h"

namespace UE::MovieRenderPipelineEditor::Private
{
	bool PerformLocalRender()
	{
		if (!CanPerformLocalRender())
		{
			return false;
		}
		
		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);

		const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
		
		const TSubclassOf<UMoviePipelineExecutorBase> ExecutorClass = ProjectSettings->DefaultLocalExecutor.TryLoadClass<UMoviePipelineExecutorBase>();
		check(ExecutorClass != nullptr);
		
		UMoviePipelineExecutorBase* Executor = Subsystem->RenderQueueWithExecutor(ExecutorClass);

		// If the executor encountered an error (like trying to use an unsaved map), then the executor may be null at this point.
		if (!IsValid(Executor))
		{
			return false;
		}

		// Tell the executor to run the post-render action. Some executors may not support this.
		const UMoviePipelineQueueProjectSettings* QueueProjectSettings = GetDefault<UMoviePipelineQueueProjectSettings>();
		Executor->SetPostRenderAction(QueueProjectSettings->PostRenderActionType);

		constexpr bool bIsLocal = true;
		FMoviePipelineTelemetry::SendRendersRequestedTelemetry(bIsLocal, Subsystem->GetQueue()->GetJobs());

		return true;
	}

	bool CanPerformLocalRender()
	{
		const UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);

		const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
		const bool bHasExecutor = ProjectSettings->DefaultLocalExecutor.TryLoadClass<UMoviePipelineExecutorBase>() != nullptr;
		const bool bNotRendering = !Subsystem->IsRendering();

		bool bAtLeastOneJobAvailable = false;
		for (const UMoviePipelineExecutorJob* Job : Subsystem->GetQueue()->GetJobs())
		{
			if (!Job->IsConsumed() && Job->IsEnabled())
			{
				bAtLeastOneJobAvailable = true;
				break;
			}
		}

		const bool bWorldIsActive = GEditor->IsPlaySessionInProgress();
		
		return bHasExecutor && bNotRendering && bAtLeastOneJobAvailable && !bWorldIsActive;
	}
}
