// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::MovieRenderPipelineEditor::Private
{
	/** Performs a local render using the queue currently loaded up in the Movie Render Queue editor. Returns true if a render was started, else false. */
	bool PerformLocalRender();

	/** Determines if a local render is currently able to be performed. */
	bool CanPerformLocalRender();
}