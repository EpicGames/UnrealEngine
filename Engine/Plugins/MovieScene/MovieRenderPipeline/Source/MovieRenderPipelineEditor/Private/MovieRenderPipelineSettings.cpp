// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderPipelineSettings.h"

#include "MoviePipelinePIEExecutor.h"
#include "MoviePipelineNewProcessExecutor.h"
#include "MoviePipeline.h"
#include "MoviePipelineImageSequenceOutput.h"
#include "MoviePipelineDeferredPasses.h"
#include "MoviePipelineQueue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieRenderPipelineSettings)

UMovieRenderPipelineProjectSettings::UMovieRenderPipelineProjectSettings()
{
	DefaultPipeline = UMoviePipeline::StaticClass();
	PresetSaveDir.Path = TEXT("/Game/Cinematics/MoviePipeline/Presets/");
	DefaultLocalExecutor = UMoviePipelinePIEExecutor::StaticClass();
	DefaultRemoteExecutor = UMoviePipelineNewProcessExecutor::StaticClass();
	DefaultExecutorJob = UMoviePipelineExecutorJob::StaticClass();
	DefaultGraph = GetDefaultGraphPath();
	DefaultQuickRenderGraph = GetDefaultQuickRenderGraphPath();

	DefaultClasses.Add(UMoviePipelineImageSequenceOutput_JPG::StaticClass());
	DefaultClasses.Add(UMoviePipelineDeferredPassBase::StaticClass());
}

FSoftObjectPath UMovieRenderPipelineProjectSettings::GetDefaultGraphPath()
{
	static const FSoftObjectPath DefaultGraphPath(TEXT("/MovieRenderPipeline/DefaultRenderGraph.DefaultRenderGraph"));
	
	return DefaultGraphPath;
}

FSoftObjectPath UMovieRenderPipelineProjectSettings::GetDefaultQuickRenderGraphPath()
{
	static const FSoftObjectPath DefaultQuickRenderGraphPath(TEXT("/MovieRenderPipeline/DefaultQuickRenderGraph.DefaultQuickRenderGraph"));
	
	return DefaultQuickRenderGraphPath;
}
