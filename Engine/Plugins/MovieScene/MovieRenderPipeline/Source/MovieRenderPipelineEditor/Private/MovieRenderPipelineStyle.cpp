// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderPipelineStyle.h"

#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"

FName FMovieRenderPipelineStyle::StyleName("MovieRenderPipelineStyle");

FMovieRenderPipelineStyle::FMovieRenderPipelineStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon48x48(48.0f, 48.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FLinearColor White(FLinearColor::White);
	const FLinearColor AlmostWhite( FColor(200, 200, 200) );
	const FLinearColor VeryLightGrey( FColor(128, 128, 128) );
	const FLinearColor LightGrey( FColor(96, 96, 96) );
	const FLinearColor MediumGrey( FColor(62, 62, 62) );
	const FLinearColor DarkGrey( FColor(30, 30, 30) );
	const FLinearColor Black(FLinearColor::Black);

	const FLinearColor SelectionColor( 0.728f, 0.364f, 0.003f );
	const FLinearColor SelectionColor_Subdued( 0.807f, 0.596f, 0.388f );
	const FLinearColor SelectionColor_Inactive( 0.25f, 0.25f, 0.25f );
	const FLinearColor SelectionColor_Pressed( 0.701f, 0.225f, 0.003f );

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("MovieScene/MovieRenderPipeline/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FButtonStyle Button = FButtonStyle()
		.SetNormal(FSlateBoxBrush(RootToContentDir("ButtonHoverHint.png"), FMargin(4/16.0f), FLinearColor(1,1,1,0.15f)))
		.SetHovered(FSlateBoxBrush(RootToContentDir("ButtonHoverHint.png"), FMargin(4/16.0f), FLinearColor(1,1,1,0.25f)))
		.SetPressed(FSlateBoxBrush(RootToContentDir("ButtonHoverHint.png"), FMargin(4/16.0f), FLinearColor(1,1,1,0.30f)))
		.SetNormalPadding( FMargin(0,0,0,1) )
		.SetPressedPadding( FMargin(0,1,0,0) );

	FButtonStyle FlatButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton.Success");
	FlatButton.SetNormalPadding(FMargin(0,0,0,1));
	FlatButton.SetNormalPadding(FMargin(0,1,0,0));

	FComboButtonStyle ComboButton = FComboButtonStyle()
		.SetButtonStyle(Button.SetNormal(FSlateNoResource()))
		.SetDownArrowImage(FSlateImageBrush(RootToCoreContentDir(TEXT("Common/ComboArrow.png")), Icon8x8))
		.SetMenuBorderBrush(FSlateBoxBrush(RootToCoreContentDir(TEXT("Old/Menu_Background.png")), FMargin(8.0f/64.0f)))
		.SetMenuBorderPadding(FMargin(0.0f));

	FButtonStyle PressHintOnly = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource())
		.SetNormalPadding( FMargin(0,0,0,1) )
		.SetPressedPadding( FMargin(0,1,0,0) );

	FTextBlockStyle TextStyle = FTextBlockStyle()
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 9))
		.SetColorAndOpacity(AlmostWhite);

	FEditableTextBoxStyle EditableTextStyle = FEditableTextBoxStyle()
		.SetTextStyle(FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 9))
		.SetBackgroundImageNormal(FSlateNoResource())
		.SetBackgroundImageHovered(FSlateNoResource())
		.SetBackgroundImageFocused(FSlateNoResource())
		.SetBackgroundImageReadOnly(FSlateNoResource())
		.SetBackgroundColor(FLinearColor::Transparent)
		.SetForegroundColor(FSlateColor::UseForeground());

	FCheckBoxStyle SwitchStyle = FCheckBoxStyle()
		.SetForegroundColor(FLinearColor::White)
		.SetUncheckedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetUncheckedHoveredImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetUncheckedPressedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedHoveredImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedPressedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")), FVector2D(28.0F, 14.0F)))
		.SetPadding(FMargin(0, 0, 0, 1));

	Set("WhiteBrush", new FSlateColorBrush(FLinearColor::White));
	Set("Button", Button);
	Set("ComboButton", ComboButton);
	Set("FlatButton.Success", FlatButton);

	Set("PressHintOnly", PressHintOnly);

	Set("MovieRenderPipeline.TextBox", TextStyle);
	Set("MovieRenderPipeline.EditableTextBox", EditableTextStyle);

	Set("MovieRenderPipeline.Setting.Switch", SwitchStyle);

	Set("MovieRenderPipeline.Config.TypeLabel", FTextBlockStyle()
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 32))
		.SetColorAndOpacity(FSlateColor::UseSubduedForeground())
	);

	const FEditableTextBoxStyle NormalEditableTextBoxStyle = FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
	const FTextBlockStyle NormalTextBlockStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");
	
	FEditableTextBoxStyle AutocompleteEditableText = FEditableTextBoxStyle(NormalEditableTextBoxStyle)
		.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
	Set("MovieRenderPipeline.AutocompleteTextStyle", AutocompleteEditableText);
	
	FTextBlockStyle AutocompleteArgumentEditableText = FTextBlockStyle(NormalTextBlockStyle)
		.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")));
	Set("MovieRenderPipeline.AutocompleteArgumentTextStyle", AutocompleteArgumentEditableText);

	Set("MovieRenderPipeline.TabIcon", new FSlateImageBrush(RootToContentDir(TEXT("TabIcon_24x.png")), Icon16x16));

	// Class icons so that SDetailNameArea shows the correct icon for jobs and shots
	Set("ClassIcon.MoviePipelineExecutorJob", new FSlateBrush(*FAppStyle::Get().GetBrush("ClassIcon.CameraComponent")));
	Set("ClassIcon.MoviePipelineExecutorShot", new FSlateBrush(*FAppStyle::Get().GetBrush("Sequencer.Tracks.CinematicShot")));

	Set("MovieRenderGraph.CollectionsTree.SmallComboButton", FComboButtonStyle(FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
		.SetDownArrowPadding(FMargin(0)));

	// Toolbar style for the queue
	{
		FToolBarStyle MRQToolBar = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("AssetEditorToolbar");
		MRQToolBar.SetBackgroundPadding(FMargin(4.0f, 0.0f));
		MRQToolBar.SetSeparatorPadding(FMargin(2.0f, 3.0f));
		MRQToolBar.SetComboButtonPadding(FMargin(4.0f, 0.0f));

		FTextBlockStyle MRQLabelStyle = MRQToolBar.LabelStyle;
		MRQLabelStyle.SetColorAndOpacity(FStyleColors::ForegroundHover);
		MRQToolBar.SetLabelStyle(MRQLabelStyle);

		FButtonStyle MRQSettingsButtonStyle = MRQToolBar.SettingsButtonStyle;
		MRQSettingsButtonStyle
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Primary, FVector4(4.0f, 0.0f, 0.0f, 4.0f)))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, FVector4(4.0f, 0.0f, 0.0f, 4.0f)))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::PrimaryPress, FVector4(4.0f, 0.0f, 0.0f, 4.0f)))
			.SetNormalForeground(FStyleColors::ForegroundHover)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover);
		MRQToolBar.SetSettingsButtonStyle(MRQSettingsButtonStyle);

		FComboButtonStyle MRQSettingsComboButtonStyle = MRQToolBar.SettingsComboButton;
		FButtonStyle MRQSettingsComboButtonButtonStyle = MRQSettingsComboButtonStyle.ButtonStyle;
		MRQSettingsComboButtonButtonStyle.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f), FStyleColors::Header, 1.0f));
		MRQSettingsComboButtonStyle.SetButtonStyle(MRQSettingsComboButtonButtonStyle);
		MRQToolBar.SetSettingsComboButtonStyle(MRQSettingsComboButtonStyle);

		Set("MovieRenderPipeline.ToolBar", MRQToolBar);
	}

	// Quick Render icons
	Set("MovieRenderPipeline.QuickRender.Icon.MovieRenderQueueMode", new IMAGE_BRUSH_SVG("Icons/MovieRenderQueueMode_16", Icon16x16));
	Set("MovieRenderPipeline.QuickRender.Icon.QuickRenderMode", new IMAGE_BRUSH_SVG("Icons/QuickRenderMode_16", Icon16x16));
	Set("MovieRenderPipeline.QuickRender.Icon.CurrentSequenceMode", new IMAGE_BRUSH_SVG("Icons/CurrentSequenceMode_16", Icon16x16));
	Set("MovieRenderPipeline.QuickRender.Icon.ViewportCameraInSequenceMode", new IMAGE_BRUSH_SVG("Icons/ViewportCameraInSequenceMode_16", Icon16x16));
	Set("MovieRenderPipeline.QuickRender.Icon.CurrentViewportMode", new IMAGE_BRUSH_SVG("Icons/CurrentViewportMode_16", Icon16x16));
	Set("MovieRenderPipeline.QuickRender.Icon.SelectedCamerasMode", new IMAGE_BRUSH_SVG("Icons/SelectedCamerasMode_16", Icon16x16));
	Set("MovieRenderPipeline.QuickRender.Icon.PlayLastRender", new IMAGE_BRUSH_SVG("Icons/PlayLastRender_16", Icon16x16));

	// Graph icons
	Set("MovieRenderPipeline.Graph.Icon.RenderLayer", new IMAGE_BRUSH_SVG("Icons/RenderLayer_16", Icon16x16));
	Set("MovieRenderPipeline.Graph.Icon.RenderMovieFile", new IMAGE_BRUSH_SVG("Icons/RenderMovieFile_16", Icon16x16));
	Set("MovieRenderPipeline.Graph.Icon.RenderSequenceOutput", new IMAGE_BRUSH_SVG("Icons/RenderSequenceOutput_16", Icon16x16));
	Set("MovieRenderPipeline.Graph.Icon.WarmUpSettings", new IMAGE_BRUSH_SVG("Icons/WarmUpSettings_16", Icon16x16));

	// Pipeline / General
	Set("MovieRenderPipeline.Icon.RenderRemote", new IMAGE_BRUSH_SVG("Icons/RenderRemote_16", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FMovieRenderPipelineStyle::~FMovieRenderPipelineStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FMovieRenderPipelineStyle& FMovieRenderPipelineStyle::Get()
{
	static FMovieRenderPipelineStyle Inst;
	return Inst;
}


