// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

/**
 * Slate style set that defines all the styles for the Movie Render Pipeline UI
 */
class FMovieRenderPipelineStyle
	: public FSlateStyleSet
{
public:
	static FName StyleName;

	/** Access the singleton instance for this style set */
	static FMovieRenderPipelineStyle& Get();

private:

	FMovieRenderPipelineStyle();
	~FMovieRenderPipelineStyle();
};
