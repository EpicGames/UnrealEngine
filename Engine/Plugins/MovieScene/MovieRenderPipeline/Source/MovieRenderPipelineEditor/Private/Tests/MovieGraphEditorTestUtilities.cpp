// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphEditorTestUtilities.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphConfigFactory.h"
#include "Graph/MovieGraphNode.h"

#include "Algo/RemoveIf.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Misc/AutomationTest.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphEditorTestUtilities)

namespace UE::MovieGraph::Private::Tests
{
	UMovieGraphConfig* CreateNewMovieGraphConfig(const FName InName)
	{
		return NewObject<UMovieGraphConfig>(
			GetTransientPackage(), UMovieGraphConfig::StaticClass(), InName, RF_Transient);
	}

	UMovieGraphConfig* CreateDefaultMovieGraphConfig()
	{
		static const FName NAME_AssetTools = "AssetTools";
		IAssetTools* AssetTools = &FModuleManager::GetModuleChecked<FAssetToolsModule>(NAME_AssetTools).Get();

		UMovieGraphConfigFactory* ConfigFactory =
			NewObject<UMovieGraphConfigFactory>(UMovieGraphConfigFactory::StaticClass());

		UMovieGraphConfig* GraphConfig =
			Cast<UMovieGraphConfig>(
				AssetTools->CreateAsset(
					"GraphTestConfig", "/Game/", UMovieGraphConfig::StaticClass(), ConfigFactory
				)
			);

		return GraphConfig;
	}

	void OpenGraphConfigInEditor(UMovieGraphConfig* InGraphConfig)
	{
		static const FName NAME_AssetTools = "AssetTools";
		IAssetTools* AssetTools = &FModuleManager::GetModuleChecked<FAssetToolsModule>(NAME_AssetTools).Get();

		AssetTools->OpenEditorForAssets({InGraphConfig});
	}

	TArray<UClass*> GetAllDerivedClasses(UClass* BaseClass, bool bRecursive)
	{
		TArray<UClass*> DerivedClasses = GetNativeClasses(BaseClass, bRecursive);
		DerivedClasses.Append(GetBlueprintClasses(BaseClass, bRecursive));
		return DerivedClasses;
	}

	void RemoveAbstractClasses(TArray<UClass*>& InClassArray)
	{
		InClassArray.SetNum(Algo::StableRemoveIf(InClassArray,
			[](const UClass* Class)
			{
				return !Class ||Class->HasAnyClassFlags(CLASS_Abstract);
			}));
	}

	TArray<UClass*> GetNativeClasses(UClass* BaseClass, bool bRecursive)
	{
		TArray<UClass*> DerivedClasses;
		GetDerivedClasses(BaseClass, DerivedClasses, bRecursive);

		DerivedClasses.Remove(BaseClass);

		RemoveAbstractClasses(DerivedClasses);

		return DerivedClasses;
	}

	bool IsBlueprintChildOfBaseClass(const FAssetData& BlueprintClassData, UClass* BaseClass)
	{		
		const FString NativeParentClassPath =
			BlueprintClassData.GetTagValueRef<FString>(FBlueprintTags::NativeParentClassPath);
		const FSoftClassPath ClassPath(NativeParentClassPath);
		const UClass* NativeParentClass = ClassPath.ResolveClass();

		return NativeParentClass && (NativeParentClass == BaseClass || NativeParentClass->IsChildOf(BaseClass));
	}

	TArray<UClass*> GetBlueprintClasses(UClass* BaseClass, bool bRecursive)
	{
		const FAssetRegistryModule& AssetRegistryModule =
			FModuleManager::LoadModuleChecked< FAssetRegistryModule >(FName("AssetRegistry"));
		const IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		TArray<UClass*> DerivedClasses;
		TArray< FAssetData > Assets;
		AssetRegistry.GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), Assets);	
		for (const FAssetData& Asset : Assets)
		{
			if (IsBlueprintChildOfBaseClass(Asset, BaseClass))
			{
				if (const UBlueprint* BlueprintAsset = Cast<UBlueprint>(Asset.GetAsset()))
				{
					UClass* LoadedClass = BlueprintAsset->GeneratedClass;
					if (ensure(LoadedClass && BlueprintAsset->ParentClass))
					{
						DerivedClasses.AddUnique(LoadedClass);
					}
				}
			}
		}

		RemoveAbstractClasses(DerivedClasses);

		return DerivedClasses;
	}
	
	void SuppressLogWarnings(FAutomationTestBase* InTestBase)
	{
		check(InTestBase);
		InTestBase->bSuppressLogWarnings = true;
	}

	void SuppressLogErrors(FAutomationTestBase* InTestBase)
	{
		check(InTestBase);
		InTestBase->bSuppressLogErrors = true;
	}

	void SetupTest(
		FAutomationTestBase* InTestBase, const bool bSuppressLogWarnings, const bool bSuppressLogErrors)
	{
		check(InTestBase);
		
		if (bSuppressLogWarnings)
		{
			SuppressLogWarnings(InTestBase);
		}

		if (bSuppressLogErrors)
		{
			SuppressLogErrors(InTestBase);
		}
	}
}
