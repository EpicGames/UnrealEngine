// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Graph/MovieGraphNode.h"

#include "MovieGraphEditorTestUtilities.generated.h"

class FAutomationTestBase;
class UMovieGraphConfig;
class UMovieGraphNode;

/**
 * A dummy render layer only node. Currently MRG does not ship with any nodes which are restricted to just render layer branches, so this node
 * exists solely to test functionality of restricting nodes to just render layer branches (EMovieGraphBranchRestriction::RenderLayer). Can be removed
 * once a shipped node with this restriction exists.
 */
UCLASS(Hidden, HideDropdown, Deprecated)
class UDEPRECATED_DummyRenderLayerOnlyNode : public UMovieGraphSettingNode
{
	GENERATED_BODY()

public:
	UDEPRECATED_DummyRenderLayerOnlyNode() = default;

	//~ Begin UMovieGraphNode interface
#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override
	{
		static const FText DummyTitle = FText::FromString(TEXT("Dummy Render Layer Only Node"));
		return DummyTitle;
	}

	virtual FText GetMenuCategory() const override
	{
		static const FText DummyCategory = FText::FromString(TEXT("Dummy Nodes"));
		return DummyCategory;
	}
#endif

	virtual EMovieGraphBranchRestriction GetBranchRestriction() const override
	{
		return EMovieGraphBranchRestriction::RenderLayer;
	}
	//~ UMovieGraphNode interface
};

namespace UE::MovieGraph::Private::Tests
{
	UMovieGraphConfig* CreateNewMovieGraphConfig(const FName InName = "GraphTestConfig");
	UMovieGraphConfig* CreateDefaultMovieGraphConfig();
	void OpenGraphConfigInEditor(UMovieGraphConfig* InGraphConfig);
	TArray<UClass*> GetAllDerivedClasses(UClass* BaseClass, bool bRecursive);
	TArray<UClass*> GetNativeClasses(UClass* BaseClass, bool bRecursive);
	TArray<UClass*> GetBlueprintClasses(UClass* BaseClass, bool bRecursive);
	void SuppressLogWarnings(FAutomationTestBase* InTestBase);
	void SuppressLogErrors(FAutomationTestBase* InTestBase);
	void SetupTest(
		FAutomationTestBase* InTestBase, const bool bSuppressLogWarnings = true, const bool bSuppressLogErrors = true);
}
