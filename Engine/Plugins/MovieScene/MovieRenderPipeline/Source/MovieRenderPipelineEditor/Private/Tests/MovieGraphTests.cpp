// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphEditorTestUtilities.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/Nodes/MovieGraphApplyCVarPresetNode.h"
#include "Graph/Nodes/MovieGraphBranchNode.h"
#include "Graph/Nodes/MovieGraphDeferredPassNode.h"
#include "Graph/Nodes/MovieGraphInputNode.h"
#include "Graph/Nodes/MovieGraphOutputNode.h"
#include "Graph/Nodes/MovieGraphSetCVarValueNode.h"
#include "Graph/Nodes/MovieGraphSubgraphNode.h"
#include "Graph/Nodes/MovieGraphVariableNode.h"

#include "Algo/RemoveIf.h"
#include "Misc/AutomationTest.h"

namespace UE::MovieGraph::Private::Tests
{
	void FilterOutUndesirableClasses(TArray<UClass*>& InClassArray)
	{
		static const TSet<UClass*> UndesirableClasses = {
			UMovieGraphVariableNode::StaticClass(),	// Will be tested separately in this test
			UMovieGraphInputNode::StaticClass(),	// Should not be able to be added via API
			UMovieGraphOutputNode::StaticClass(),	// Should not be able to be added via API
			UMovieGraphSubgraphNode::StaticClass()	// Will be tested separately in another test
		};

		InClassArray.SetNum(Algo::StableRemoveIf(InClassArray,
			[&](const UClass* Class)
			{
				return UndesirableClasses.Contains(Class);
			}));
	}

	void OpenGraphConfigInEditorIfDesired(UMovieGraphConfig* InGraphConfig)
	{
		/**
		* A static switch which governs whether the graph config is opened
		* useful for test development
		* just remember to set it to false when done
		*/
		static bool bIsDesired = false;

		if (bIsDesired)
		{
			OpenGraphConfigInEditor(InGraphConfig);
		}
	}

	void RemoveAllEdgesFromAllNodes(UMovieGraphConfig* GraphConfig)
	{
		TArray<UMovieGraphNode*> Nodes = GraphConfig->GetNodes();
		Nodes.Add(GraphConfig->GetInputNode());
		Nodes.Add(GraphConfig->GetOutputNode());
		
		for (UMovieGraphNode* Node : Nodes)
		{
			GraphConfig->RemoveAllInboundEdges(Node);
			GraphConfig->RemoveAllOutboundEdges(Node);
		}
	}

	void AddVerifyAndRemoveVariableNode(
		FAutomationTestBase* InTestBase,
		UMovieGraphConfig* GraphConfig)
	{
		// Test adding and removing a variable member and variable node
		UMovieGraphVariable* Variable = GraphConfig->AddVariable(TEXT("A_Cool_Variable"));
		InTestBase->TestTrue(TEXT("Variable member added"), Variable != nullptr && GraphConfig->GetVariables().Contains(Variable));
		if (!Variable)
		{
			return;
		}
		
		UMovieGraphVariableNode* VariableNode = GraphConfig->ConstructRuntimeNode<UMovieGraphVariableNode>(UMovieGraphVariableNode::StaticClass());
		InTestBase->TestTrue(TEXT("Variable node type added"), VariableNode != nullptr && GraphConfig->GetNodes().Contains(VariableNode));
		if (!VariableNode)
		{
			return;
		}
		
		VariableNode->SetVariable(Variable);
		InTestBase->TestTrue(TEXT("Variable node variable member set"), VariableNode->GetVariable() == Variable);
		
		VariableNode->SetVariable(nullptr);
		InTestBase->TestTrue(TEXT("Calling SetVariable with nullptr should do nothing"), VariableNode->GetVariable() == Variable);
		
		GraphConfig->RemoveNode(VariableNode);
		InTestBase->TestTrue(TEXT("Variable node removed"), GraphConfig->GetNodes().Num() == 1);
		
		GraphConfig->DeleteMember(Variable);
		InTestBase->TestTrue(TEXT("Variable member removed"), !GraphConfig->GetVariables().Contains(Variable));
	}

	void SeparateNodesIntoInputAndOutputTypes(
		FAutomationTestBase* InTestBase,
		const UMovieGraphConfig* GraphConfig,
		TArray<TObjectPtr<UMovieGraphNode>>& OutNodesWithOutputs, 
		TArray<TObjectPtr<UMovieGraphNode>>& OutNodesWithInputs)
	{
		for (const TObjectPtr<UMovieGraphNode>& Node : GraphConfig->GetNodes())
		{
			if (!Node->GetInputPins().IsEmpty())
			{
				OutNodesWithInputs.Add(Node);
			}

			if (!Node->GetOutputPins().IsEmpty())
			{
				OutNodesWithOutputs.Add(Node);
			}

			const bool bHasNoInputsOrOutputs = Node->GetInputPins().Num() == 0 && Node->GetOutputPins().Num() == 0;
			
			InTestBase->TestTrue(*FString::Printf(TEXT("Node %s has inputs or outputs"), *Node.GetName()), !bHasNoInputsOrOutputs);
		}
	};

	void ConnectAndTestEdges_ExpectedSuccess(
		FAutomationTestBase* InTestBase,
		UMovieGraphConfig* GraphConfig,
		UMovieGraphNode* NodeFrom, UMovieGraphNode* NodeTo,
		UMovieGraphPin* OutputPin, UMovieGraphPin* InputPin)
	{
		GraphConfig->AddLabeledEdge(
			NodeFrom, OutputPin->Properties.Label,
			NodeTo, InputPin->Properties.Label);

		InTestBase->TestTrue(
			*FString::Printf(TEXT("Edge added between %s and %s"), *NodeFrom->GetName(), *NodeTo->GetName()),
			OutputPin->GetAllConnectedPins().Contains(InputPin));
	}
	
	void ConnectAndTestEdges_ExpectedFailure(
		FAutomationTestBase* InTestBase,
		UMovieGraphConfig* GraphConfig,
		UMovieGraphNode* NodeFrom, UMovieGraphNode* NodeTo,
		UMovieGraphPin* OutputPin, UMovieGraphPin* InputPin)
	{
		GraphConfig->AddLabeledEdge(
			NodeFrom, OutputPin->Properties.Label,
			NodeTo, InputPin->Properties.Label);

		if (OutputPin->GetAllConnectedPins().Contains(InputPin))
		{
			InTestBase->AddError(
				*FString::Printf(TEXT("Edge added between %s and %s, but edge connection was expected to fail."),
					*NodeFrom->GetName(), *NodeTo->GetName()));
		}
	}

	void RemoveEdgeAndVerify(
		FAutomationTestBase* InTestBase,
		UMovieGraphConfig* GraphConfig,
		UMovieGraphNode* NodeFrom, UMovieGraphNode* NodeTo,
		UMovieGraphPin* OutputPin, UMovieGraphPin* InputPin)
	{
		GraphConfig->RemoveLabeledEdge(NodeFrom, OutputPin->Properties.Label,
			NodeTo, InputPin->Properties.Label);

		InTestBase->TestTrue(
			*FString::Printf(
				TEXT("Edge removed between %s and %s"), *NodeFrom->GetName(), *NodeTo->GetName()),
			!OutputPin->GetAllConnectedPins().Contains(InputPin));
	}

	void ConnectAllEdgesToAllNodes(
		FAutomationTestBase* InTestBase, 
		UMovieGraphConfig* GraphConfig,
		const TArray<TObjectPtr<UMovieGraphNode>>& NodesWithOutputs, 
		const TArray<TObjectPtr<UMovieGraphNode>>& NodesWithInputs)
	{
		for (const TObjectPtr<UMovieGraphNode>& NodeFrom : NodesWithOutputs)
		{
			for (UMovieGraphPin* OutputPin : NodeFrom->GetOutputPins())
			{
				for (const TObjectPtr<UMovieGraphNode>& NodeTo : NodesWithInputs)
				{
					if (NodeFrom == NodeTo) // Don't try to self-connect
					{
						continue;
					}
					
					for (UMovieGraphPin* InputPin : NodeTo->GetInputPins())
					{
						if (OutputPin->CanCreateConnection(InputPin))
						{
							ConnectAndTestEdges_ExpectedSuccess(
								InTestBase, GraphConfig, NodeFrom, NodeTo, OutputPin, InputPin);
						}
					}
				}
			}
		}
	};

	/**
	* 1) Add a null node, verify that NumNodes == 0
	* 2) Add an unsupported node, verify that NumNodes == 0
	* 3) Add a supported node, Verify that NumNodes == NumSupportedNodes
	* 4) Add a variable and variable node, verify the variable node exists includes the variable member
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAddGraphNodeTest, "VirtualProduction.MovieGraph.AddGraphNodeTest", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter));
	bool FAddGraphNodeTest::RunTest(const FString& Parameters)
	{
		SetupTest(this);
		
		// Create Config
		UMovieGraphConfig* GraphConfig = CreateNewMovieGraphConfig("AddGraphNodeTest");
		TestTrue(TEXT("MovieGraphConfig successfully created"), GraphConfig != nullptr);
		if (!GraphConfig)
		{
			return false;
		}

		GraphConfig->ConstructRuntimeNode<UMovieGraphNode>(nullptr);
		TestTrue(TEXT("Null Node not added"), GraphConfig->GetNodes().IsEmpty());

		// Ensure input and output nodes can't be added via the API
		const UMovieGraphInputNode* TestInputRuntimeNode = GraphConfig->ConstructRuntimeNode<UMovieGraphInputNode>(
			UMovieGraphInputNode::StaticClass());
		if(TestInputRuntimeNode && GraphConfig->GetNodes().Contains(TestInputRuntimeNode))
		{
			AddError(
				*FString::Printf(
					TEXT("Graph allowed user construction of %s"), *TestInputRuntimeNode->GetClass()->GetName()));
		}
		const UMovieGraphOutputNode* TestOutputRuntimeNode = GraphConfig->ConstructRuntimeNode<UMovieGraphOutputNode>(
			UMovieGraphOutputNode::StaticClass());
		if(TestOutputRuntimeNode && GraphConfig->GetNodes().Contains(TestOutputRuntimeNode))
		{
			AddError(
				*FString::Printf(
					TEXT("Graph allowed user construction of %s"), *TestOutputRuntimeNode->GetClass()->GetName()));
		}
		
		// Add one node of each type, excepting variables, inputs and outputs
		TArray<UClass*> AllNodeClasses = GetAllDerivedClasses(UMovieGraphNode::StaticClass(), true);
		FilterOutUndesirableClasses(AllNodeClasses);
		for (UClass* Class : AllNodeClasses)
		{
			GraphConfig->ConstructRuntimeNode<UMovieGraphNode>(Class);
		}
		TestTrue(
			TEXT("All supported nodes types added"), GraphConfig->GetNodes().Num() == AllNodeClasses.Num());

		// Test adding a variable node
		UMovieGraphVariable* Variable = GraphConfig->AddVariable(TEXT("A_Cool_Variable"));
		TestTrue(
			TEXT("Variable member added"),
			Variable != nullptr && GraphConfig->GetVariables().Contains(Variable));
		if (!Variable)
		{
			return false;
		}
		UMovieGraphVariableNode* VariableNode = GraphConfig->ConstructRuntimeNode<UMovieGraphVariableNode>(
			UMovieGraphVariableNode::StaticClass());
		TestTrue(
			TEXT("Variable node type added"),
			VariableNode != nullptr && GraphConfig->GetNodes().Contains(VariableNode));
		if (!VariableNode)
		{
			return false;
		}
		VariableNode->SetVariable(Variable);
		TestTrue(TEXT("Variable node variable member set"), VariableNode->GetVariable() == Variable);

		OpenGraphConfigInEditorIfDesired(GraphConfig);

		return true;
	}

	/**
	* 1) Add two nodes to the graph
	* 2) Remove one node, verify NumNodes == 1
	* 3) Try to remove a nullptr, verify NumNodes == 1
	* 4) Try to remove a node that doesn't exist in the graph, verify NumNodes == 1
	* 5) Remove a variable, variable node, and verify it's not in GetNodes()
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRemoveGraphNodeTest, "VirtualProduction.MovieGraph.RemoveGraphNodeTest", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter));
	bool FRemoveGraphNodeTest::RunTest(const FString& Parameters)
	{
		SetupTest(this);
		
		// Create Config
		UMovieGraphConfig* GraphConfig = CreateNewMovieGraphConfig("RemoveGraphNodeTest");
		TestTrue(TEXT("MovieGraphConfig successfully created"), GraphConfig != nullptr);
		if (!GraphConfig)
		{
			return false;
		}
		
		// Add two nodes
		TArray<UClass*> AllNodeClasses = GetAllDerivedClasses(UMovieGraphNode::StaticClass(), true);
		FilterOutUndesirableClasses(AllNodeClasses);
		UMovieGraphNode* NodeOne = GraphConfig->ConstructRuntimeNode<UMovieGraphNode>(AllNodeClasses[0]);
		UMovieGraphNode* NodeTwo = GraphConfig->ConstructRuntimeNode<UMovieGraphNode>(AllNodeClasses[1]);
		if (!NodeOne || !NodeTwo)
		{
			AddError(TEXT("Unable to create two nodes."));
			return false;
		}
		TestTrue(
			FString::Printf(
				TEXT("Node types added: %s and %s"), *AllNodeClasses[0]->GetName(), *AllNodeClasses[1]->GetName()),
			GraphConfig->GetNodes().Num() == 2);

		// Test removing one node
		GraphConfig->RemoveNode(NodeOne);
		TestTrue(TEXT("Node removed"), GraphConfig->GetNodes().Num() == 1);

		// Test removing a nullptr
		GraphConfig->RemoveNode(nullptr);
		TestTrue(
			TEXT("Calling RemoveNode with nullptr should do nothing"), GraphConfig->GetNodes().Num() == 1);

		// Test removing a node which doesn't exist in the graph
		GraphConfig->RemoveNode(NodeOne);
		TestTrue(
			TEXT("Attempting to remove node which does not exist in the graph should do nothing"),
			GraphConfig->GetNodes().Num() == 1);

		// Test adding and removing a variable member and variable node
		AddVerifyAndRemoveVariableNode(this, GraphConfig);

		OpenGraphConfigInEditorIfDesired(GraphConfig);

		return true;
	}
	
	/**
	* 1) Add nodes to the graph, verify that NumNodes == AllNodeClasses
	* 2) Connect nodes together
	* 3) Use the Pin API to verify that the connection is between the correct nodes
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGraphNodeAddEdgeTest, "VirtualProduction.MovieGraph.GraphNodeAddEdgeTest", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter));
	bool FGraphNodeAddEdgeTest::RunTest(const FString& Parameters)
	{
		SetupTest(this);
		
		// Create Config
		UMovieGraphConfig* GraphConfig = CreateNewMovieGraphConfig("GraphNodeAddEdgeTest");
		TestTrue(TEXT("MovieGraphConfig successfully created"), GraphConfig != nullptr);
		if (!GraphConfig)
		{
			return false;
		}

		// Add all nodes excepting variables, inputs and outputs
		TArray<UClass*> AllNodeClasses = GetAllDerivedClasses(UMovieGraphNode::StaticClass(), true);
		FilterOutUndesirableClasses(AllNodeClasses);
		for (UClass* Class : AllNodeClasses)
		{
			GraphConfig->ConstructRuntimeNode<UMovieGraphNode>(Class);
		}
		TestTrue(
			TEXT("All supported nodes types added"), GraphConfig->GetNodes().Num() == AllNodeClasses.Num());

		// Connect all edges		
		// Some nodes will be duplicated in between these arrays
		
		TArray<TObjectPtr<UMovieGraphNode>> NodesWithOutputs = {GraphConfig->GetInputNode()};
		TArray<TObjectPtr<UMovieGraphNode>> NodesWithInputs = {GraphConfig->GetOutputNode()};
		
		SeparateNodesIntoInputAndOutputTypes(this, GraphConfig, NodesWithOutputs, NodesWithInputs);
		ConnectAllEdgesToAllNodes(this, GraphConfig, NodesWithOutputs, NodesWithInputs);
		
		OpenGraphConfigInEditorIfDesired(GraphConfig);
		
		return true;
	}
	
	/**
	* 1) Add a node, connect it with default input and output nodes
	* 2) Remove an edge
	* 3) Use the Pin API to verify no more connection
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGraphNodeRemoveEdgeTest, "VirtualProduction.MovieGraph.GraphNodeRemoveEdgeTest", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter));
	bool FGraphNodeRemoveEdgeTest::RunTest(const FString& Parameters)
	{
		SetupTest(this);
		
		// Create Config
		UMovieGraphConfig* GraphConfig = CreateNewMovieGraphConfig("GraphNodeRemoveEdgeTest");
		TestTrue(TEXT("MovieGraphConfig successfully created"), GraphConfig != nullptr);
		if (!GraphConfig)
		{
			return false;
		}

		// Add a node with an input and output for connection
		UMovieGraphNode* MiddleNode =
			GraphConfig->ConstructRuntimeNode<UMovieGraphBranchNode>(UMovieGraphBranchNode::StaticClass());
		TestTrue(TEXT("MiddleNode successfully added"), GraphConfig->GetNodes().Contains(MiddleNode));
		if (!MiddleNode)
		{
			return false;
		}

		UMovieGraphNode* InputNode = GraphConfig->GetInputNode();
		UMovieGraphNode* OutputNode = GraphConfig->GetOutputNode();
		if (!InputNode || !OutputNode)
		{
			AddError(TEXT("Unable to find Input and Output nodes."));
			return false;
		}

		const TObjectPtr<UMovieGraphPin> InputNodeOutputPin = InputNode->GetOutputPins()[0];
		const TObjectPtr<UMovieGraphPin> MiddleNodeInputPin = MiddleNode->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> MiddleNodeOutputPin = MiddleNode->GetOutputPins()[0];
		const TObjectPtr<UMovieGraphPin> OutputNodeInputPin = OutputNode->GetInputPins()[0];

		if (!InputNodeOutputPin || !MiddleNodeInputPin || !MiddleNodeOutputPin || !OutputNodeInputPin)
		{
			AddError(TEXT("Unable to find all Pins for Input, Middle and Output Nodes."));
			return false;
		}

		// Add first edge
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, InputNode, MiddleNode, InputNodeOutputPin, MiddleNodeInputPin);

		// Remove edge
		RemoveEdgeAndVerify(
			this, GraphConfig, InputNode, MiddleNode,
			InputNodeOutputPin, MiddleNodeInputPin);

		// Add second edge
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, MiddleNode, OutputNode, MiddleNodeOutputPin, OutputNodeInputPin);

		// Remove edge
		RemoveEdgeAndVerify(
			this, GraphConfig, MiddleNode, OutputNode,
			MiddleNodeOutputPin, OutputNodeInputPin);

		OpenGraphConfigInEditorIfDesired(GraphConfig);
		
		return true;
	}

	/**
	* Tests Node to Node Branch Connection without connection to input or output:
	* Any-> Any, Any->Global, Any->Render Layer,  (all expected to work),
	* Globals-> Globals (should work), Globals-> Any (should work), Globals-> RenderLayer (should fail),
	* RenderLayer-> RenderLayer (should work), RenderLayer-> Any (should work), RenderLayer-> Globals (should fail)
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGraphNodeBranchRestrictionIndividualNodeTest, "VirtualProduction.MovieGraph.GraphNodeBranchRestrictionIndividualNodeTest", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter));
	bool FGraphNodeBranchRestrictionIndividualNodeTest::RunTest(const FString& Parameters)
	{
		SetupTest(this);
		
		// Globals: UMovieGraphSetCVarValueNode, UMovieGraphApplyCVarPresetNode, UMovieGraphSamplingMethodNode,
		// UMovieGraphAudioOutputNode, UMovieGraphWarmUpSettingNode
		// RenderLayer: UDummyRenderLayerOnlyNode
		// Any: All other node types as of October 6, 2023
		
		// Create Config
		UMovieGraphConfig* GraphConfig = CreateNewMovieGraphConfig("GraphNodeRemoveEdgeTest");
		TestTrue(TEXT("MovieGraphConfig successfully created"), GraphConfig != nullptr);
		if (!GraphConfig)
		{
			return false;
		}

		// Add middle nodes of BranchRestriction type 'Any'
		UMovieGraphNode* AnyNodeA =
			GraphConfig->ConstructRuntimeNode<UMovieGraphBranchNode>(
				UMovieGraphBranchNode::StaticClass());
		TestTrue(
			TEXT("AnyNodeA successfully added"), GraphConfig->GetNodes().Contains(AnyNodeA));
		if (!AnyNodeA)
		{
			return false;
		}
		UMovieGraphNode* AnyNodeB =
			GraphConfig->ConstructRuntimeNode<UMovieGraphDeferredRenderPassNode>(
				UMovieGraphDeferredRenderPassNode::StaticClass());
		TestTrue(
			TEXT("AnyNodeB successfully added"), GraphConfig->GetNodes().Contains(AnyNodeB));
		if (!AnyNodeB)
		{
			return false;
		}

		// Add middle nodes of BranchRestriction type 'Globals'
		UMovieGraphNode* GlobalsNodeA =
			GraphConfig->ConstructRuntimeNode<UMovieGraphSetCVarValueNode>(
				UMovieGraphSetCVarValueNode::StaticClass());
		TestTrue(
			TEXT("GlobalsNodeA successfully added"), GraphConfig->GetNodes().Contains(GlobalsNodeA));
		if (!GlobalsNodeA)
		{
			return false;
		}
		UMovieGraphNode* GlobalsNodeB =
			GraphConfig->ConstructRuntimeNode<UMovieGraphApplyCVarPresetNode>(
				UMovieGraphApplyCVarPresetNode::StaticClass());
		TestTrue(
			TEXT("GlobalsNodeB successfully added"), GraphConfig->GetNodes().Contains(GlobalsNodeB));
		if (!GlobalsNodeB)
		{
			return false;
		}

		// Add middle nodes of BranchRestriction type 'RenderLayer'
		// Ideally we'd want two different types but only one exists at present
		UMovieGraphNode* RenderLayerNodeA =
			GraphConfig->ConstructRuntimeNode<UDEPRECATED_DummyRenderLayerOnlyNode>(
				UDEPRECATED_DummyRenderLayerOnlyNode::StaticClass());
		TestTrue(
			TEXT("RenderLayerNodeA successfully added"), GraphConfig->GetNodes().Contains(RenderLayerNodeA));
		if (!RenderLayerNodeA)
		{
			return false;
		}
		UMovieGraphNode* RenderLayerNodeB =
			GraphConfig->ConstructRuntimeNode<UDEPRECATED_DummyRenderLayerOnlyNode>(
				UDEPRECATED_DummyRenderLayerOnlyNode::StaticClass());
		TestTrue(
			TEXT("RenderLayerNodeB successfully added"), GraphConfig->GetNodes().Contains(RenderLayerNodeB));
		if (!RenderLayerNodeB)
		{
			return false;
		}

		// Identify Pins
		// Need two nodes of each Type
		const TObjectPtr<UMovieGraphPin> AnyNodeInputPinA = AnyNodeA->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> AnyNodeOutputPinA = AnyNodeA->GetOutputPins()[0];
		
		const TObjectPtr<UMovieGraphPin> AnyNodeInputPinB = AnyNodeB->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> AnyNodeOutputPinB = AnyNodeB->GetOutputPins()[0];

		const TObjectPtr<UMovieGraphPin> GlobalsNodeInputPinA = GlobalsNodeA->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> GlobalsNodeOutputPinA = GlobalsNodeA->GetOutputPins()[0];
		
		const TObjectPtr<UMovieGraphPin> GlobalsNodeInputPinB = GlobalsNodeB->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> GlobalsNodeOutputPinB = GlobalsNodeB->GetOutputPins()[0];

		const TObjectPtr<UMovieGraphPin> RenderLayerNodeInputPinA = RenderLayerNodeA->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> RenderLayerNodeOutputPinA = RenderLayerNodeA->GetOutputPins()[0];
		
		const TObjectPtr<UMovieGraphPin> RenderLayerNodeInputPinB = RenderLayerNodeB->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> RenderLayerNodeOutputPinB = RenderLayerNodeB->GetOutputPins()[0];

		if (!AnyNodeInputPinA || !AnyNodeOutputPinA ||
			!AnyNodeInputPinB || !AnyNodeOutputPinB ||
			!GlobalsNodeInputPinA || !GlobalsNodeOutputPinA ||
			!GlobalsNodeInputPinB || !GlobalsNodeOutputPinB ||
			!RenderLayerNodeInputPinA || !RenderLayerNodeOutputPinA ||
			!RenderLayerNodeInputPinB || !RenderLayerNodeOutputPinB)
		{
			AddError(TEXT("Unable to find all Pins for each Node."));
			return false;
		}

		// Any-> Any, Any-> Global, Any-> RenderLayer,  (all expected to work),
		// This connects AnyNodeA to AnyNodeB, GlobalsNodeA, and RenderLayerNodeA simultaneously

		// Any-> Any
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, AnyNodeA, AnyNodeB,
			AnyNodeOutputPinA, AnyNodeInputPinB);

		// Any-> Global
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, AnyNodeA, GlobalsNodeA,
			AnyNodeOutputPinA, GlobalsNodeInputPinA);

		// Any-> RenderLayer
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, AnyNodeA, RenderLayerNodeA,
			AnyNodeOutputPinA, RenderLayerNodeInputPinA);

		// Globals-> Globals (should work), Globals-> Any (should work), Globals-> RenderLayer (should fail)
		// This connects GlobalsNodeA to GlobalsNodeB and AnyNodeB simultaneously.
		// It attempts a connection to RenderLayerNodeA but fails.
		// AnyNodeB's connection to AnyNodeA will be severed.
		
		// Globals-> Globals
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, GlobalsNodeA, GlobalsNodeB,
			GlobalsNodeOutputPinA, GlobalsNodeInputPinB);

		// Globals-> Any
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, GlobalsNodeA, AnyNodeB,
			GlobalsNodeOutputPinA, AnyNodeInputPinB);

		// Globals-> RenderLayer
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, GlobalsNodeA, RenderLayerNodeA,
			GlobalsNodeOutputPinA, RenderLayerNodeInputPinA);
		
		// RenderLayer-> RenderLayer (should work), RenderLayer-> Any (should work), RenderLayer-> Globals (should fail)
		// This connects RenderLayerNodeA to RenderLayerNodeB and AnyNodeB simultaneously.
		// It attempts a connection to GlobalsNodeB but fails.
		// AnyNodeB's connection to GlobalsNodeA will be severed.
		
		// RenderLayer-> RenderLayer
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, RenderLayerNodeA, RenderLayerNodeB,
			RenderLayerNodeOutputPinA, RenderLayerNodeInputPinB);

		// RenderLayer-> Any
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, RenderLayerNodeA, AnyNodeB,
			RenderLayerNodeOutputPinA, AnyNodeInputPinB);

		// RenderLayer-> Globals
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, RenderLayerNodeA, GlobalsNodeB,
			RenderLayerNodeOutputPinA, GlobalsNodeInputPinB);

		OpenGraphConfigInEditorIfDesired(GraphConfig);
		
		return true;
	}

	/**
	* Tests Node to Chain Branch Connection:
	* Input Globals-> Globals Node-> Any Node-> Output Globals (should work),
	* Input Globals-> Globals Node-> Any Node-> Output RenderLayer (should fail),
	* Input Globals-> Globals Node-> Any Node-> RenderLayer Node (should fail),
	* Input Globals-> RenderLayer Node (should fail),
	* Unconnected Globals Node-> Output Globals (should work),
	* Unconnected Globals Node-> Output RenderLayer (should fail),
	* Input RenderLayer-> RenderLayer Node-> Any Node-> Output RenderLayer (should work),
	* Input RenderLayer-> RenderLayer Node-> Any Node-> Output Globals (should fail),
	* Input RenderLayer-> RenderLayer Node-> Any Node-> Globals Node (should fail),
	* Input RenderLayer-> Globals Node (should fail),
	* Unconnected RenderLayer Node-> Output Globals (should fail),
	* Unconnected RenderLayer Node-> Output RenderLayer (should work)
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGraphNodeBranchRestrictionUpstreamAndDownstreamTest, "VirtualProduction.MovieGraph.GraphNodeBranchRestrictionUpstreamAndDownstreamTest", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter));
	bool FGraphNodeBranchRestrictionUpstreamAndDownstreamTest::RunTest(const FString& Parameters)
	{
		SetupTest(this);
		
		// Globals: UMovieGraphSetCVarValueNode, UMovieGraphApplyCVarPresetNode, UMovieGraphSamplingMethodNode,
		// UMovieGraphAudioOutputNode, UMovieGraphWarmUpSettingNode
		// RenderLayer: UDummyRenderLayerOnlyNode
		// Any: All other node types as of October 5, 2023
		
		// Create Config
		UMovieGraphConfig* GraphConfig = CreateNewMovieGraphConfig("GraphNodeRemoveEdgeTest");
		TestTrue(TEXT("MovieGraphConfig successfully created"), GraphConfig != nullptr);
		if (!GraphConfig)
		{
			return false;
		}

		// Add inputs and outputs for a render layer
		const UMovieGraphInput* NewInput = GraphConfig->AddInput();
		const UMovieGraphOutput* NewOutput = GraphConfig->AddOutput();
		if (!NewInput || !NewOutput)
		{
			AddError(TEXT("Unable to create new Input and Output."));
			return false;
		}

		UMovieGraphNode* InputNode = GraphConfig->GetInputNode();
		UMovieGraphNode* OutputNode = GraphConfig->GetOutputNode();
		if (!InputNode || !OutputNode)
		{
			AddError(TEXT("Unable to find Input and Output nodes."));
			return false;
		}

		// Add middle nodes with an input and output for connection
		UMovieGraphNode* AnyNode =
			GraphConfig->ConstructRuntimeNode<UMovieGraphBranchNode>(UMovieGraphBranchNode::StaticClass());
		TestTrue(TEXT("AnyNode successfully added"), GraphConfig->GetNodes().Contains(AnyNode));
		if (!AnyNode)
		{
			return false;
		}

		UMovieGraphNode* GlobalsNode =
			GraphConfig->ConstructRuntimeNode<UMovieGraphSetCVarValueNode>(UMovieGraphSetCVarValueNode::StaticClass());
		TestTrue(TEXT("GlobalsNode successfully added"), GraphConfig->GetNodes().Contains(GlobalsNode));
		if (!GlobalsNode)
		{
			return false;
		}

		UMovieGraphNode* RenderLayerNode =
			GraphConfig->ConstructRuntimeNode<UDEPRECATED_DummyRenderLayerOnlyNode>(
				UDEPRECATED_DummyRenderLayerOnlyNode::StaticClass());
		TestTrue(
			TEXT("RenderLayerNode successfully added"), GraphConfig->GetNodes().Contains(RenderLayerNode));
		if (!RenderLayerNode)
		{
			return false;
		}

		// Identify Pins
		const TObjectPtr<UMovieGraphPin> InputNodeGlobalsOutputPin = InputNode->GetOutputPins()[0];
		const TObjectPtr<UMovieGraphPin> InputNodeRenderLayerOutputPin = InputNode->GetOutputPins()[1];
		
		const TObjectPtr<UMovieGraphPin> AnyNodeInputPin = AnyNode->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> AnyNodeOutputPin = AnyNode->GetOutputPins()[0];

		const TObjectPtr<UMovieGraphPin> GlobalsNodeInputPin = GlobalsNode->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> GlobalsNodeOutputPin = GlobalsNode->GetOutputPins()[0];

		const TObjectPtr<UMovieGraphPin> RenderLayerNodeInputPin = RenderLayerNode->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> RenderLayerNodeOutputPin = RenderLayerNode->GetOutputPins()[0];
		
		const TObjectPtr<UMovieGraphPin> OutputNodeGlobalsInputPin = OutputNode->GetInputPins()[0];
		const TObjectPtr<UMovieGraphPin> OutputNodeRenderLayerInputPin = OutputNode->GetInputPins()[1];

		if (!InputNodeGlobalsOutputPin || !InputNodeRenderLayerOutputPin ||!AnyNodeInputPin ||
			!AnyNodeOutputPin || !GlobalsNodeInputPin || !GlobalsNodeOutputPin ||
			!RenderLayerNodeInputPin || !RenderLayerNodeOutputPin ||
			!OutputNodeGlobalsInputPin || !OutputNodeRenderLayerInputPin)
		{
			AddError(TEXT("Unable to find all Pins for Input, Output and other Nodes."));
			return false;
		}

		/**
		* Input Globals-> Globals Node-> Any Node-> Output Globals (should work),
		* 
		* Input Globals-> Globals Node-> Any Node-> Output RenderLayer (should fail),
		* Input Globals-> Globals Node-> Any Node-> RenderLayer Node (should fail),
		* 
		* Input Globals-> RenderLayer Node (should fail)
		*/
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, InputNode, GlobalsNode,
			InputNodeGlobalsOutputPin, GlobalsNodeInputPin);
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, GlobalsNode, AnyNode,
			GlobalsNodeOutputPin, AnyNodeInputPin);
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, AnyNode, OutputNode,
			AnyNodeOutputPin, OutputNodeGlobalsInputPin);
		
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, AnyNode, OutputNode,
			AnyNodeOutputPin, OutputNodeRenderLayerInputPin);
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, AnyNode, RenderLayerNode,
			AnyNodeOutputPin, RenderLayerNodeInputPin);

		RemoveAllEdgesFromAllNodes(GraphConfig);
		
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, InputNode, RenderLayerNode,
			InputNodeGlobalsOutputPin, RenderLayerNodeInputPin);

		RemoveAllEdgesFromAllNodes(GraphConfig);

		/**
		* Unconnected Globals Node-> Output Globals (should work),
		* Unconnected Globals Node-> Output RenderLayer (should fail)
		*/
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, GlobalsNode, OutputNode,
			GlobalsNodeOutputPin, OutputNodeGlobalsInputPin);
		
		RemoveAllEdgesFromAllNodes(GraphConfig);
		
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, GlobalsNode, OutputNode,
			GlobalsNodeOutputPin, OutputNodeRenderLayerInputPin);

		/**
		* Input RenderLayer-> RenderLayer Node-> Any Node-> Output RenderLayer (should work),
		* 
		* Input RenderLayer-> RenderLayer Node-> Any Node-> Output Globals (should fail),
		* Input RenderLayer-> RenderLayer Node-> Any Node-> Globals Node (should fail),
		* 
		* Input RenderLayer-> Globals Node (should fail)
		*/
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, InputNode, RenderLayerNode,
			InputNodeRenderLayerOutputPin, RenderLayerNodeInputPin);
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, RenderLayerNode, AnyNode,
			RenderLayerNodeOutputPin, AnyNodeInputPin);
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, AnyNode, OutputNode,
			AnyNodeOutputPin, OutputNodeRenderLayerInputPin);
		
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, AnyNode, OutputNode,
			AnyNodeOutputPin, OutputNodeGlobalsInputPin);
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, AnyNode, GlobalsNode,
			AnyNodeOutputPin, GlobalsNodeInputPin);

		RemoveAllEdgesFromAllNodes(GraphConfig);
		
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, InputNode, GlobalsNode,
			InputNodeRenderLayerOutputPin, GlobalsNodeInputPin);

		RemoveAllEdgesFromAllNodes(GraphConfig);

		/**
		* Unconnected RenderLayer Node-> Output Globals (should fail),
		* Unconnected RenderLayer Node-> Output RenderLayer (should work)
		*/
		ConnectAndTestEdges_ExpectedFailure(
			this, GraphConfig, RenderLayerNode, OutputNode,
			RenderLayerNodeOutputPin, OutputNodeGlobalsInputPin);
		
		RemoveAllEdgesFromAllNodes(GraphConfig);
		
		ConnectAndTestEdges_ExpectedSuccess(
			this, GraphConfig, RenderLayerNode, OutputNode,
			RenderLayerNodeOutputPin, OutputNodeRenderLayerInputPin);

		OpenGraphConfigInEditorIfDesired(GraphConfig);
		
		return true;
	}
}