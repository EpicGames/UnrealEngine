// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Graph/SModifierCollectionsHeader.h"

#include "ScopedTransaction.h"
#include "Graph/Nodes/MovieGraphModifierNode.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Graph/SMovieGraphCollectionPicker.h"

#define LOCTEXT_NAMESPACE "MovieGraphModifierCollectionsHeader"

void SMovieGraphCollectionsHeaderWidget::Construct(const FArguments& InArgs)
{
	WeakModifierInterface = InArgs._WeakModifierInterface.Get();
	OnCollectionPicked = InArgs._OnCollectionPicked;

	// Generate a (multi-layered) icon for the "Add" menu
	const TSharedRef<SLayeredImage> AddIcon =
		SNew(SLayeredImage)
		.ColorAndOpacity(FSlateColor::UseForeground())
		.Image(FAppStyle::GetBrush("LevelEditor.OpenAddContent.Background"));
	AddIcon->AddLayer(FAppStyle::GetBrush("LevelEditor.OpenAddContent.Overlay"));

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CollectionsHeaderText", "Collections"))
			.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(5.f, 0, 0, 0)
		[
			SNew(SComboButton)
			.ToolTipText(LOCTEXT("AddCollectionToModifierTooltip", "Add a collection that will be affected by the configured modifiers."))
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.ContentPadding(0)
			.HasDownArrow(false)
			.OnGetMenuContent_Lambda([this]()
			{
				const UObject* ModifierNode = WeakModifierInterface.GetObject();
				
				return
					SNew(SMovieGraphCollectionPicker)
					.Graph(IsValid(ModifierNode) ? ModifierNode->GetTypedOuter<UMovieGraphConfig>() : nullptr)
					.OnFilter_Lambda([this](const FName CollectionName)
					{
						if (WeakModifierInterface.IsValid())
						{
							return !WeakModifierInterface.Get()->GetAllCollections().Contains(CollectionName);
						}

						return false;
					})
					.OnCollectionPicked_Lambda([this](const FName PickedCollectionName)
					{
						if (WeakModifierInterface.IsValid())
						{
							const FScopedTransaction Transaction(LOCTEXT("AddCollectionToModifier", "Add Collection to Modifier"));
							
							WeakModifierInterface.Get()->AddCollection(PickedCollectionName);

							if (OnCollectionPicked.IsBound())
							{
								OnCollectionPicked.Execute(PickedCollectionName);
							}
						}
					});
			})
			.ButtonContent()
			[
				SNew(SBox)
				.WidthOverride(16)
				.HeightOverride(16)
				[
					AddIcon
				]
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
