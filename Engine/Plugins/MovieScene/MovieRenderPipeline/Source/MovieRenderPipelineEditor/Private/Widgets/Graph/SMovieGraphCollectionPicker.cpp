// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Graph/SMovieGraphCollectionPicker.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphCollectionNode.h"

#define LOCTEXT_NAMESPACE "MovieGraphCollectionPicker"

void SMovieGraphCollectionPicker::Construct(const FArguments& InArgs)
{
	CurrentGraph = InArgs._Graph.Get();
	OnCollectionPicked = InArgs._OnCollectionPicked;
	OnFilter = InArgs._OnFilter;

	SMovieGraphSimplePicker::Construct(SMovieGraphSimplePicker::FArguments()
		.OnGetRowText_Lambda([](FName InCollectionName) { return FText::FromName(InCollectionName); })
		.Title(LOCTEXT("PickCollectionHelpText", "Pick a Collection"))
		.DataSourceEmptyMessage(LOCTEXT("NoCollectionsFoundWarning", "No collections found."))
		.OnItemPicked(OnCollectionPicked));

	// Update the data source *after* calling Construct() above because Construct() will populate DataSource based on the widget arguments, but
	// we want to control/update it manually.
	UpdateDataSource();
}

void SMovieGraphCollectionPicker::UpdateDataSource()
{
	if (!CurrentGraph)
	{
		return;
	}

	TSet<UMovieGraphConfig*> Subgraphs;
	CurrentGraph->GetAllContainedSubgraphs(Subgraphs);

	TArray<UMovieGraphConfig*> AllGraphs = { CurrentGraph };
	for (UMovieGraphConfig* Subgraph : Subgraphs)
	{
		AllGraphs.Add(Subgraph);
	}

	for (const UMovieGraphConfig* Graph : AllGraphs)
	{
		for (const TObjectPtr<UMovieGraphNode>& Node : Graph->GetNodes())
		{
			if (const TObjectPtr<UMovieGraphCollectionNode> CollectionNode = Cast<UMovieGraphCollectionNode>(Node))
			{
				const FName CollectionName = FName(CollectionNode->Collection->GetCollectionName());
				bool bIncludeCollection = true;

				if (OnFilter.IsBound())
				{
					bIncludeCollection = OnFilter.Execute(CollectionName);
				}

				if (bIncludeCollection)
				{
					DataSource.AddUnique(CollectionName);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE