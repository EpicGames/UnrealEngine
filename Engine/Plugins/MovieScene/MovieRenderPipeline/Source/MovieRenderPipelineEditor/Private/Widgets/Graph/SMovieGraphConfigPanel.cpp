// Copyright Epic Games, Inc. All Rights Reserved.

// Movie Pipeline Includes
#include "Widgets/Graph/SMovieGraphConfigPanel.h"
#include "Widgets/MoviePipelineWidgetConstants.h"
#include "Widgets/SMoviePipelineQueueEditor.h"
#include "Widgets/SMoviePipelineConfigPanel.h"
#include "MovieRenderPipelineSettings.h"
#include "MovieRenderPipelineStyle.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineQueueSubsystem.h"
#include "LevelSequence.h"

// Slate Includes
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWindow.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SPrimaryButton.h"
#include "Framework/Commands/GenericCommands.h"
#include "SNodePanel.h"

// ContentBrowser Includes
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

// Misc
#include "Editor.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"

// UnrealEd Includes
#include "ScopedTransaction.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "FileHelpers.h"
#include "AssetToolsModule.h"
#include "Misc/FileHelper.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphUtilities.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphVariableNode.h"
#include "Graph/MovieEdGraph.h"
#include "Graph/MovieEdGraphNode.h"
#include "Graph/MovieGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "GraphEditor.h"
#include "GraphEditorActions.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineGraphPanel"

UE_DISABLE_OPTIMIZATION_SHIP
void SMoviePipelineGraphPanel::Construct(const FArguments& InArgs)
{
	OnGraphSelectionChangedEvent = InArgs._OnGraphSelectionChanged;

	// Create the child widgets that need to know about our pipeline
	PipelineQueueEditorWidget = SNew(SMoviePipelineQueueEditor)
		.OnEditConfigRequested(this, &SMoviePipelineGraphPanel::OnEditJobConfigRequested)
		.OnPresetChosen(this, &SMoviePipelineGraphPanel::OnJobPresetChosen)
		.OnJobSelectionChanged(this, &SMoviePipelineGraphPanel::OnSelectionChanged);


	{
		// Automatically select the first job in the queue
		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);

		TArray<UMoviePipelineExecutorJob*> Jobs;
		if (Subsystem->GetQueue()->GetJobs().Num() > 0)
		{
			Jobs.Add(Subsystem->GetQueue()->GetJobs()[0]);
		}

		// Go through the UI so it updates the UI selection too and then this will loop back
		// around to OnSelectionChanged to update ourself.
		PipelineQueueEditorWidget->SetSelectedJobs(Jobs);
	}

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &SMoviePipelineGraphPanel::OnSelectedNodesChanged);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &SMoviePipelineGraphPanel::OnNodeDoubleClicked);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &SMoviePipelineGraphPanel::OnNodeTitleCommitted);

	CurrentGraph = InArgs._Graph;
	
	UMoviePipelineEdGraph* EdGraph = Cast<UMoviePipelineEdGraph>(CurrentGraph->PipelineEdGraph);

	// Create the EdGraph if it has not yet been created. It is saved as part of the runtime graph to prevent it from being re-created every time the
	// graph is opened (and therefore dirtying the package).
	//
	// Note: Now that the editor graph is saved with the runtime graph, the !EdGraph condition will rarely be true (because when the "template"
	// runtime graph is duplicated into a new runtime graph, the editor graph comes along as well). There may be other scenarios where a new runtime
	// graph is created without a duplication though, so this code path remains.
	if (!EdGraph)
	{
		EdGraph = Cast<UMoviePipelineEdGraph>(FBlueprintEditorUtils::CreateNewGraph(CurrentGraph, TEXT("MoviePipelineEdGraph"), UMoviePipelineEdGraph::StaticClass(), UMovieGraphSchema::StaticClass()));

		// Probably not ideal.. USoundCue has a CreateGraph() node that does this (#if WITH_EDITOR) but then requires an interface
		// for the editor half to avoid the circular dependency.
		CurrentGraph->PipelineEdGraph = EdGraph;
		EdGraph->InitFromRuntimeGraph(CurrentGraph);

		const UEdGraphSchema* Schema = EdGraph->GetSchema();
		Schema->CreateDefaultNodesForGraph(*EdGraph);
	}
	else
	{
		EdGraph->InitFromRuntimeGraph(CurrentGraph);
	}
	
	MakeEditorCommands();

	ChildSlot
	[
		SAssignNew(GraphEditorWidget, SGraphEditor)
		.IsEditable(true)
		.GraphToEdit(EdGraph)
		.AdditionalCommands(GraphEditorCommands)
		.GraphEvents(InEvents)
		.Appearance(this, &SMoviePipelineGraphPanel::GetGraphAppearanceInfo)
	];
}

UE_ENABLE_OPTIMIZATION_SHIP

void SMoviePipelineGraphPanel::MakeEditorCommands()
{
	GraphEditorCommands = MakeShareable(new FUICommandList);
	{
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().CreateComment,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnCreateComment)
		);
		
		// Editing commands
		GraphEditorCommands->MapAction(FGenericCommands::Get().SelectAll,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::SelectAllNodes),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CanSelectAllNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::DeleteSelectedNodes),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CanDeleteSelectedNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CopySelectedNodes),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CanCopySelectedNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Cut,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CutSelectedNodes),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CanCutSelectedNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Paste,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::PasteNodes),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CanPasteNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Duplicate,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::DuplicateNodes),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CanDuplicateNodes)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().EnableNodes,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::SetEnabledStateForSelectedNodes, ENodeEnabledState::Enabled),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CanDisableSelectedNodes),
			FGetActionCheckState::CreateSP(this, &SMoviePipelineGraphPanel::CheckEnabledStateForSelectedNodes, ENodeEnabledState::Enabled)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DisableNodes,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::SetEnabledStateForSelectedNodes, ENodeEnabledState::Disabled),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::CanDisableSelectedNodes),
			FGetActionCheckState::CreateSP(this, &SMoviePipelineGraphPanel::CheckEnabledStateForSelectedNodes, ENodeEnabledState::Disabled)
		);

		// Alignment Commands
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesTop,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnAlignTop)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnAlignMiddle)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesBottom,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnAlignBottom)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesLeft,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnAlignLeft)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesCenter,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnAlignCenter)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesRight,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnAlignRight)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StraightenConnections,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnStraightenConnections)
		);

		// Distribution Commands
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnDistributeNodesH)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically,
			FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnDistributeNodesV)
		);
	}
}

void SMoviePipelineGraphPanel::OnCreateComment() const
{
	if (CurrentGraph)
	{
		FMovieGraphSchemaAction_NewComment CommentAction;
		
		const TSharedPtr<SGraphEditor> GraphEditorPtr = SGraphEditor::FindGraphEditorForGraph(CurrentGraph->PipelineEdGraph);
		FVector2f Location = FVector2f::ZeroVector;
		if (GraphEditorPtr)
		{
			Location = GraphEditorPtr->GetPasteLocation2f();
		}
		
		CommentAction.PerformAction(CurrentGraph->PipelineEdGraph, nullptr, Location);
	}
}

void SMoviePipelineGraphPanel::SelectAllNodes()
{
	GraphEditorWidget->SelectAllNodes();
}


bool SMoviePipelineGraphPanel::CanSelectAllNodes() const
{
	return GraphEditorWidget.IsValid();
}

bool SMoviePipelineGraphPanel::CanDeleteSelectedNodes() const
{
	if (GraphEditorWidget->GetSelectedNodes().IsEmpty())
	{
		return false;
	}

	for (UObject* Object : GraphEditorWidget->GetSelectedNodes())
	{
		const UMoviePipelineEdGraphNodeBase* GraphNode = Cast<UMoviePipelineEdGraphNodeBase>(Object);

		// If the editor node doesn't derive from UMoviePipelineEdGraphNodeBase, it's a node from outside MRG (eg, a comment node); these are
		// always deletable. They will not have a runtime representation. Ideally we could check to see if this is in CurrentGraph->GetEditorOnlyNodes(),
		// but that array is only updated after a save is performed.
		if (!GraphNode)
		{
			continue;
		}

		// Some nodes in the graph are explicitly marked as non-deletable (eg, inputs/outputs).
		if (!GraphNode->CanUserDeleteNode())
		{
			return false;
		}

		// Additionally, if the associated runtime node is not valid (likely from an unloaded plugin), then the node cannot be deleted.
		// Editor-only nodes do not have an associated runtime node, but those are accounted for earlier.
		if (!GraphNode->GetRuntimeNode())
		{
			return false;
		}
	}

	return true;
}

void SMoviePipelineGraphPanel::DeleteSelectedNodes()
{
	TArray<UMovieGraphNode*> NodesToDelete;
	TArray<UEdGraphNode*> EdNodesToDelete;
	
	for (UObject* Object : GraphEditorWidget->GetSelectedNodes())
	{
		const UMoviePipelineEdGraphNodeBase* GraphNode = Cast<UMoviePipelineEdGraphNodeBase>(Object);
		if (GraphNode && GraphNode->CanUserDeleteNode())
		{
			NodesToDelete.Add(GraphNode->GetRuntimeNode());
			continue;
		}

		UEdGraphNode* EdGraphNode = Cast<UEdGraphNode>(Object);
		if (EdGraphNode && EdGraphNode->CanUserDeleteNode())
		{
			EdNodesToDelete.Add(EdGraphNode);
		}
	}

	UMoviePipelineEdGraph* Graph = Cast<UMoviePipelineEdGraph>(GraphEditorWidget->GetCurrentGraph());
	
	const bool bShouldActuallyTransact = Graph && (NodesToDelete.Num() > 0 || EdNodesToDelete.Num() > 0);
	const FScopedTransaction Transaction(
		FGenericCommands::Get().Delete->GetDescription(), bShouldActuallyTransact);
	
	if (bShouldActuallyTransact)
	{
		// Remove all runtime graph nodes
		if (!NodesToDelete.IsEmpty())
		{
			for (UMovieGraphNode* Node : NodesToDelete)
			{
				Graph->GetPipelineGraph()->RemoveNode(Node);
			}
		}
		
		// Remove all editor nodes (nodes not backed by a runtime node, like comments)
		if (!EdNodesToDelete.IsEmpty())
		{
			for (UEdGraphNode* EdGraphNode : EdNodesToDelete)
			{
				Graph->RemoveNode(EdGraphNode);
			}
		}
	}

	ClearGraphSelection();
}

void SMoviePipelineGraphPanel::CopySelectedNodes()
{
	if (GraphEditorWidget.IsValid())
	{
		const FGraphPanelSelectionSet SelectedNodes = GraphEditorWidget->GetSelectedNodes();

		for (UObject* SelectedNode : SelectedNodes)
		{
			UEdGraphNode* GraphNode = CastChecked<UEdGraphNode>(SelectedNode);

			// This causes the editor node to become the temporary owner of the underlying data model node
			GraphNode->PrepareForCopying();
		}

		// Serialize it all to text, the linked data model nodes are copied too due to (temporarily) being owned
		// by the editor node we're copying. 
		FString ExportedText;
		FEdGraphUtilities::ExportNodesToText(SelectedNodes, ExportedText);
		FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

		// Now go back through our nodes and restore their proper ownership. There's no matching PostPrepareForCopying
		// in UEdGraphNode, so we have to cast it to our type first.
		for (UObject* SelectedNode : SelectedNodes)
		{
			if (UMoviePipelineEdGraphNodeBase* MovieGraphNode = Cast<UMoviePipelineEdGraphNodeBase>(SelectedNode))
			{
				MovieGraphNode->PostCopy();
			}
		}
	}
}

bool SMoviePipelineGraphPanel::CanCopySelectedNodes() const
{
	// We handle non-copyable nodes on paste so the toast warning happens on paste
	// to warn the user that some nodes (Input/Outputs) weren't copied.
	bool bAtLeastOneCopyableNode = false;
	if (GraphEditorWidget.IsValid())
	{
		for (UObject* Object : GraphEditorWidget->GetSelectedNodes())
		{
			if (const UMoviePipelineEdGraphNodeBase* GraphNode = Cast<UMoviePipelineEdGraphNodeBase>(Object))
			{
				// Invalid runtime nodes (likely from an unloaded plugin) cannot be copied.
				if (!GraphNode->GetRuntimeNode())
				{
					return false;
				}
				
				if (GraphNode->CanDuplicateNode())
				{
					bAtLeastOneCopyableNode = true;
					break;
				}
			}
			else if (const UEdGraphNode* EdGraphNode = Cast<UEdGraphNode>(Object))
			{
				// Covers non-MRG nodes, but is likely a Comment node.
				if (EdGraphNode->CanDuplicateNode())
				{
					bAtLeastOneCopyableNode = true;
					break;
				}
			}
		}
	}

	return bAtLeastOneCopyableNode;
}

void SMoviePipelineGraphPanel::CutSelectedNodes()
{
	CopySelectedNodes();
	DeleteSelectedNodes();
}

bool SMoviePipelineGraphPanel::CanCutSelectedNodes() const
{
	return CanCopySelectedNodes() && CanDeleteSelectedNodes();
}

void SMoviePipelineGraphPanel::PasteNodes()
{
	if (GraphEditorWidget.IsValid())
	{
		PasteNodesHere(GraphEditorWidget->GetPasteLocation2f());
	}
}

void SMoviePipelineGraphPanel::PasteNodesHere(const FVector2D& Location)
{
	if (!GraphEditorWidget.IsValid() || !CurrentGraph->PipelineEdGraph)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("PasteNodes_Transaction", "Movie Graph Paste Node(s)"));
	UMoviePipelineEdGraph* EdGraph = Cast<UMoviePipelineEdGraph>(CurrentGraph->PipelineEdGraph);
	EdGraph->Modify();

	// Clear the selection set (newly pasted stuff will be selected)
	GraphEditorWidget->ClearSelectionSet();

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Import the nodes
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(EdGraph, TextToImport, PastedNodes);

	//Average position of nodes so that resulting nodes are centered around the paste location
	FVector2D AvgNodePosition(0.0f, 0.0f);

	int32 AvgCount = 0;

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		if (PastedNode)
		{
			AvgNodePosition.X += PastedNode->NodePosX;
			AvgNodePosition.Y += PastedNode->NodePosY;
			++AvgCount;
		}
	}

	if (AvgCount > 0)
	{
		float InvNumNodes = 1.0f / float(AvgCount);
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;
	}

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		GraphEditorWidget->SetNodeSelection(PastedNode, true);

		// Offset to center the nodes on the paste location
		PastedNode->NodePosX = (PastedNode->NodePosX - AvgNodePosition.X) + Location.X;
		PastedNode->NodePosY = (PastedNode->NodePosY - AvgNodePosition.Y) + Location.Y;

		PastedNode->SnapToGrid(SNodePanel::GetSnapGridSize());

		PastedNode->CreateNewGuid();

		if (UMoviePipelineEdGraphNodeBase* PastedEdGraphNode = Cast<UMoviePipelineEdGraphNodeBase>(PastedNode))
		{
			if (UMovieGraphNode* PastedRuntimeNode = PastedEdGraphNode->GetRuntimeNode())
			{
				// AddNode fixes up ownership to ensure the graph owns this node
				CurrentGraph->AddNode(PastedRuntimeNode);

				// Rebuild the runtime node connections based on the editor node pins
				PastedEdGraphNode->PostPaste();
			}
		}
	}

	GraphEditorWidget->NotifyGraphChanged();
}

bool SMoviePipelineGraphPanel::CanPasteNodes() const
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FEdGraphUtilities::CanImportNodesFromText(CurrentGraph->PipelineEdGraph, ClipboardContent);
}

void SMoviePipelineGraphPanel::DuplicateNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

bool SMoviePipelineGraphPanel::CanDuplicateNodes() const
{
	return CanCopySelectedNodes();
}

void SMoviePipelineGraphPanel::SetEnabledStateForSelectedNodes(const ENodeEnabledState NewState) const
{
	if (!GraphEditorWidget.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetNodeEnabledState", "Set Node Enabled State"));
	
	for (UObject* SelectedNode : GraphEditorWidget->GetSelectedNodes())
	{
		if (UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode))
		{
			SelectedGraphNode->Modify();
			SelectedGraphNode->SetEnabledState(NewState);
		}
	}
	
	GraphEditorWidget->NotifyGraphChanged();
}

ECheckBoxState SMoviePipelineGraphPanel::CheckEnabledStateForSelectedNodes(const ENodeEnabledState EnabledStateToCheck) const
{
	if (!GraphEditorWidget.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}
	
	ECheckBoxState CheckBoxState = ECheckBoxState::Undetermined;
	
	for (UObject* SelectedNode : GraphEditorWidget->GetSelectedNodes())
	{
		if (const UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode))
		{
			const ENodeEnabledState NodeEnabledState = SelectedGraphNode->GetDesiredEnabledState();
			const ECheckBoxState NewCheckBoxState = (NodeEnabledState == EnabledStateToCheck) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

			// Initialize CheckBoxState to either Checked/Unchecked; this only happens when the state is Undetermined
			if (CheckBoxState == ECheckBoxState::Undetermined)
			{
				CheckBoxState = NewCheckBoxState;
				continue;
			}

			// If the new/old checkbox states don't match, revert to an undetermined state
			if (NewCheckBoxState != CheckBoxState)
			{
				CheckBoxState = ECheckBoxState::Undetermined;
				break;
			}
			
			CheckBoxState = NewCheckBoxState;
		}
	}

	return CheckBoxState;
}

bool SMoviePipelineGraphPanel::CanDisableSelectedNodes() const
{
	for (const UMovieGraphNode* SelectedModelNode : GetSelectedModelNodes())
	{
		if (!SelectedModelNode->CanBeDisabled())
		{
			return false;
		}
	}

	return true;
}

void SMoviePipelineGraphPanel::OnAlignTop()
{
	if (GraphEditorWidget.IsValid())
	{
		GraphEditorWidget->OnAlignTop();
	}
}

void SMoviePipelineGraphPanel::OnAlignMiddle()
{
	if (GraphEditorWidget.IsValid())
	{
		GraphEditorWidget->OnAlignMiddle();
	}
}

void SMoviePipelineGraphPanel::OnAlignBottom()
{
	if (GraphEditorWidget.IsValid())
	{
		GraphEditorWidget->OnAlignBottom();
	}
}

void SMoviePipelineGraphPanel::OnAlignLeft()
{
	if (GraphEditorWidget.IsValid())
	{
		GraphEditorWidget->OnAlignLeft();
	}
}

void SMoviePipelineGraphPanel::OnAlignCenter()
{
	if (GraphEditorWidget.IsValid())
	{
		GraphEditorWidget->OnAlignCenter();
	}
}

void SMoviePipelineGraphPanel::OnAlignRight()
{
	if (GraphEditorWidget.IsValid())
	{
		GraphEditorWidget->OnAlignRight();
	}
}

void SMoviePipelineGraphPanel::OnStraightenConnections()
{
	if (GraphEditorWidget.IsValid())
	{
		GraphEditorWidget->OnStraightenConnections();
	}
}

void SMoviePipelineGraphPanel::OnDistributeNodesH()
{
	if (GraphEditorWidget.IsValid())
	{
		GraphEditorWidget->OnDistributeNodesH();
	}
}

void SMoviePipelineGraphPanel::OnDistributeNodesV()
{
	if (GraphEditorWidget.IsValid())
	{
		GraphEditorWidget->OnDistributeNodesV();
	}
}

void SMoviePipelineGraphPanel::ClearGraphSelection() const
{
	GraphEditorWidget->ClearSelectionSet();
}

TWeakPtr<SGraphEditor> SMoviePipelineGraphPanel::GetGraphEditor() const
{
	return GraphEditorWidget.ToWeakPtr();
}

bool SMoviePipelineGraphPanel::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	// ToDo: If we wanted to rebuild the graph less often, we could make all of our transactions have a context
	// and only handle PostUndo if the undo actually matches the context. For now we'll just always rebuild the widget.
	return true;
}

void SMoviePipelineGraphPanel::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		if (CurrentGraph)
		{
			CurrentGraph->OnGraphChangedDelegate.Broadcast();
		}

		if (GraphEditorWidget.IsValid())
		{
			// Undo has the potential to leave behind an invalid graph or schema. To be on the safe side, close out
			// the editor if this is the case.
			if (!GraphEditorWidget->GetCurrentGraph() || !GraphEditorWidget->GetCurrentGraph()->GetSchema())
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(CurrentGraph);
				return;
			}

			// This unfortunately causes the graph to flicker for one frame, but all Blueprint Graphs do this.
			// This is needed so that when a node is pasted, then undone, the widgets on the graph for the now
			// deleted nodes also get removed, otherwise we try to draw them and they're pointed to invalid uobjects.
			GraphEditorWidget->NotifyGraphChanged();

			FSlateApplication::Get().DismissAllMenus();
		}
	}
}

void SMoviePipelineGraphPanel::OnSelectedNodesChanged(const TSet<UObject*>& NewSelection)
{
	TArray<UObject*> SelectedObjects;
	for (UObject* Obj : NewSelection)
	{
		// The selected objects will be editor objects. Get the underlying runtime object for each editor object.
		if (const UMoviePipelineEdGraphNodeBase* NodeBase = Cast<UMoviePipelineEdGraphNodeBase>(Obj))
		{
			if (UMovieGraphNode* GraphNode = NodeBase->GetRuntimeNode())
			{
				// For variable nodes, select the underlying variable instead. Otherwise, just select the runtime node.
				if (const UMovieGraphVariableNode* VariableNode = Cast<UMovieGraphVariableNode>(GraphNode))
				{
					SelectedObjects.Add(VariableNode->GetVariable());
				}
				else
				{
					SelectedObjects.Add(GraphNode);
				}
			}
		}
		else if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(Obj))
		{
			// Comment nodes are not MRQ-specific nodes, so they don't have a runtime node.
			// Add the comment node directly so its properties are displayed in the details panel.
			SelectedObjects.Add(CommentNode);
		}
	}

	OnGraphSelectionChangedEvent.ExecuteIfBound(SelectedObjects);
}

void SMoviePipelineGraphPanel::OnNodeDoubleClicked(class UEdGraphNode* Node)
{
	if (Node != nullptr)
	{
		if (UObject* Object = Node->GetJumpTargetForDoubleClick())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);
		}
	}
}

void SMoviePipelineGraphPanel::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("K2_RenameNode", "RenameNode", "Rename Node"));
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewText.ToString());
	}
}

FGraphAppearanceInfo SMoviePipelineGraphPanel::GetGraphAppearanceInfo() const
{
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("EditorCornerText", "Movie Graph Config");
	AppearanceInfo.PIENotifyText = LOCTEXT("PIENotifyText", "RENDERING");

	return AppearanceInfo;
}

FReply SMoviePipelineGraphPanel::OnRenderLocalRequested()
{
	UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);

	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	TSubclassOf<UMoviePipelineExecutorBase> ExecutorClass = ProjectSettings->DefaultLocalExecutor.TryLoadClass<UMoviePipelineExecutorBase>();

	// OnRenderLocalRequested should only get called if IsRenderLocalEnabled() returns true, meaning there's a valid class.
	check(ExecutorClass != nullptr);
	Subsystem->RenderQueueWithExecutor(ExecutorClass);
	return FReply::Handled();
}

bool SMoviePipelineGraphPanel::IsRenderLocalEnabled() const
{
	UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);

	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	const bool bHasExecutor = ProjectSettings->DefaultLocalExecutor.TryLoadClass<UMoviePipelineExecutorBase>() != nullptr;
	const bool bNotRendering = !Subsystem->IsRendering();
	const bool bConfigWindowIsOpen = WeakEditorWindow.IsValid();

	bool bAtLeastOneJobAvailable = false;
	for (UMoviePipelineExecutorJob* Job : Subsystem->GetQueue()->GetJobs())
	{
		if (!Job->IsConsumed() && Job->IsEnabled())
		{
			bAtLeastOneJobAvailable = true;
			break;
		}
	}

	const bool bWorldIsActive = GEditor->IsPlaySessionInProgress();
	return bHasExecutor && bNotRendering && bAtLeastOneJobAvailable && !bWorldIsActive && !bConfigWindowIsOpen;
}

FReply SMoviePipelineGraphPanel::OnRenderRemoteRequested()
{
	UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);

	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	TSubclassOf<UMoviePipelineExecutorBase> ExecutorClass = ProjectSettings->DefaultRemoteExecutor.TryLoadClass<UMoviePipelineExecutorBase>();

	// OnRenderRemoteRequested should only get called if IsRenderRemoteEnabled() returns true, meaning there's a valid class.
	check(ExecutorClass != nullptr);

	Subsystem->RenderQueueWithExecutor(ExecutorClass);
	return FReply::Handled();
}

bool SMoviePipelineGraphPanel::IsRenderRemoteEnabled() const
{
	UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);

	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	const bool bHasExecutor = ProjectSettings->DefaultRemoteExecutor.TryLoadClass<UMoviePipelineExecutorBase>() != nullptr;
	const bool bNotRendering = !Subsystem->IsRendering();
	const bool bConfigWindowIsOpen = WeakEditorWindow.IsValid();

	bool bAtLeastOneJobAvailable = false;
	for (UMoviePipelineExecutorJob* Job : Subsystem->GetQueue()->GetJobs())
	{
		if (!Job->IsConsumed() && Job->IsEnabled())
		{
			bAtLeastOneJobAvailable = true;
			break;
		}
	}

	return bHasExecutor && bNotRendering && bAtLeastOneJobAvailable && !bConfigWindowIsOpen;
}

void SMoviePipelineGraphPanel::OnJobPresetChosen(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot)
{
	UMovieRenderPipelineProjectSettings* ProjectSettings = GetMutableDefault<UMovieRenderPipelineProjectSettings>();
	if (!InShot.IsValid() && InJob.IsValid())
	{
		// Store the preset so the next job they make will use it.
		ProjectSettings->LastPresetOrigin = InJob->GetPresetOrigin();
	}
	ProjectSettings->SaveConfig();
}

void SMoviePipelineGraphPanel::OnEditJobConfigRequested(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot)
{
	if (!InJob.IsValid())
	{
		return;
	}

	// Only allow one editor open at once for now.
	if (WeakEditorWindow.IsValid())
	{
		FWidgetPath ExistingWindowPath;
		if (FSlateApplication::Get().FindPathToWidget(WeakEditorWindow.Pin().ToSharedRef(), ExistingWindowPath, EVisibility::All))
		{
			WeakEditorWindow.Pin()->BringToFront();
			FSlateApplication::Get().SetAllUserFocus(ExistingWindowPath, EFocusCause::SetDirectly);
		}

		return;
	}

	TSubclassOf<UMoviePipelineConfigBase> ConfigType;
	UMoviePipelineConfigBase* BasePreset = nullptr;
	UMoviePipelineConfigBase* BaseConfig = nullptr;
	if (InShot.IsValid())
	{
		ConfigType = UMoviePipelineShotConfig::StaticClass();
		BasePreset = InShot->GetShotOverridePresetOrigin();
		BaseConfig = InShot->GetShotOverrideConfiguration();
	}
	else
	{
		ConfigType = UMoviePipelinePrimaryConfig::StaticClass();
		BasePreset = InJob->GetPresetOrigin();
		BaseConfig = InJob->GetConfiguration();
	}

	TSharedRef<SWindow> EditorWindow =
		SNew(SWindow)
		.ClientSize(FVector2D(700, 600));

	TSharedRef<SMoviePipelineConfigPanel> ConfigEditorPanel =
		SNew(SMoviePipelineConfigPanel, ConfigType)
		.Job(InJob)
		.Shot(InShot)
		.OnConfigurationModified(this, &SMoviePipelineGraphPanel::OnConfigUpdatedForJob)
		.OnConfigurationSetToPreset(this, &SMoviePipelineGraphPanel::OnConfigUpdatedForJobToPreset)
		.BasePreset(BasePreset)
		.BaseConfig(BaseConfig);

	EditorWindow->SetContent(ConfigEditorPanel);


	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(EditorWindow, ParentWindow.ToSharedRef());
	}

	WeakEditorWindow = EditorWindow;
}

void SMoviePipelineGraphPanel::OnConfigWindowClosed()
{
	if (WeakEditorWindow.IsValid())
	{
		WeakEditorWindow.Pin()->RequestDestroyWindow();
	}
}

void SMoviePipelineGraphPanel::OnConfigUpdatedForJob(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot, UMoviePipelineConfigBase* InConfig)
{
	if (InJob.IsValid())
	{
		if (InShot.IsValid())
		{
			if (UMoviePipelineShotConfig* ShotConfig = Cast<UMoviePipelineShotConfig>(InConfig))
			{
				InShot->SetShotOverrideConfiguration(ShotConfig);
			}
		}
		else
		{
			if (UMoviePipelinePrimaryConfig* PrimaryConfig = Cast<UMoviePipelinePrimaryConfig>(InConfig))
			{
				InJob->SetConfiguration(PrimaryConfig);
			}
		}
	}

	OnConfigWindowClosed();
}

void SMoviePipelineGraphPanel::OnConfigUpdatedForJobToPreset(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot, UMoviePipelineConfigBase* InConfig)
{
	if (InJob.IsValid())
	{
		if (InShot.IsValid())
		{
			if (UMoviePipelineShotConfig* ShotConfig = Cast<UMoviePipelineShotConfig>(InConfig))
			{
				InShot->SetShotOverridePresetOrigin(ShotConfig);
			}
		}
		else
		{
			if (UMoviePipelinePrimaryConfig* PrimaryConfig = Cast<UMoviePipelinePrimaryConfig>(InConfig))
			{
				InJob->SetPresetOrigin(PrimaryConfig);
			}
		}
	}

	// Store the preset they used as the last set one
	OnJobPresetChosen(InJob, InShot);

	OnConfigWindowClosed();
}

void SMoviePipelineGraphPanel::OnSelectionChanged(const TArray<UMoviePipelineExecutorJob*>& InSelectedJobs, const TArray<UMoviePipelineExecutorShot*>& InSelectedShots)
{
	TArray<UObject*> Jobs;
	for (UMoviePipelineExecutorJob* Job : InSelectedJobs)
	{
		Jobs.Add(Job);
	}
	
	NumSelectedJobs = InSelectedJobs.Num();
}

TArray<UMovieGraphNode*> SMoviePipelineGraphPanel::GetSelectedModelNodes() const
{
	TArray<UMovieGraphNode*> SelectedModelNodes;
	
	for (UObject* SelectedNode : GraphEditorWidget->GetSelectedNodes())
	{
		if (const UMoviePipelineEdGraphNodeBase* SelectedGraphNode = Cast<UMoviePipelineEdGraphNodeBase>(SelectedNode))
		{
			if (UMovieGraphNode* SelectedModelNode = SelectedGraphNode->GetRuntimeNode())
			{
				SelectedModelNodes.Add(SelectedModelNode);
			}
		}
	}

	return SelectedModelNodes;
}

TSharedRef<SWidget> SMoviePipelineGraphPanel::OnGenerateSavedQueuesMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SaveAsQueue_Text", "Save As Asset"),
		LOCTEXT("SaveAsQueue_Tip", "Save the current configuration as a new preset that can be shared between multiple jobs, or imported later as the base of a new configuration."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAsset"),
		FUIAction(FExecuteAction::CreateSP(this, &SMoviePipelineGraphPanel::OnSaveAsAsset))
	);

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bSortByPathInColumnView = false;

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoQueueAssets_Warning", "No Queues Found");
		AssetPickerConfig.Filter.ClassPaths.Add(UMoviePipelineQueue::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SMoviePipelineGraphPanel::OnImportSavedQueueAssest);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("LoadQueue_MenuSection", "Load Queue"));
	{
		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.MinDesiredWidth(400.f)
			.MinDesiredHeight(400.f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SMoviePipelineGraphPanel::OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName)
{
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	{
		SaveAssetDialogConfig.DefaultPath = InDefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = InNewNameSuggestion;
		SaveAssetDialogConfig.AssetClassNames.Add(UMoviePipelineQueue::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveQueueAssetDialogTitle", "Save Queue Asset");
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (!SaveObjectPath.IsEmpty())
	{
		OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		return true;
	}

	return false;
}

bool SMoviePipelineGraphPanel::GetSavePresetPackageName(const FString& InExistingName, FString& OutName)
{
	UMovieRenderPipelineProjectSettings* ConfigSettings = GetMutableDefault<UMovieRenderPipelineProjectSettings>();

	// determine default package path
	const FString DefaultSaveDirectory = ConfigSettings->PresetSaveDir.Path;

	FString DialogStartPath;
	FPackageName::TryConvertFilenameToLongPackageName(DefaultSaveDirectory, DialogStartPath);
	if (DialogStartPath.IsEmpty())
	{
		DialogStartPath = TEXT("/Game");
	}

	// determine default asset name
	FString DefaultName = InExistingName;

	FString UniquePackageName;
	FString UniqueAssetName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(DialogStartPath / DefaultName, TEXT(""), UniquePackageName, UniqueAssetName);

	FString DialogStartName = FPaths::GetCleanFilename(UniqueAssetName);

	FString UserPackageName;
	FString NewPackageName;

	// get destination for asset
	bool bFilenameValid = false;
	while (!bFilenameValid)
	{
		if (!OpenSaveDialog(DialogStartPath, DialogStartName, UserPackageName))
		{
			return false;
		}

		NewPackageName = UserPackageName;

		FText OutError;
		bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError);
	}

	// Update to the last location they saved to so it remembers their settings next time.
	ConfigSettings->PresetSaveDir.Path = FPackageName::GetLongPackagePath(UserPackageName);
	ConfigSettings->SaveConfig();
	OutName = MoveTemp(NewPackageName);
	return true;
}

void SMoviePipelineGraphPanel::OnSaveAsAsset()
{
	UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);
	UMoviePipelineQueue* CurrentQueue = Subsystem->GetQueue();

	FString PackageName;
	if (!GetSavePresetPackageName(CurrentQueue->GetName(), PackageName))
	{
		return;
	}
	
	// Saving into a new package
	const FString NewAssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UPackage* NewPackage = CreatePackage(*PackageName);
	NewPackage->MarkAsFullyLoaded();
	UMoviePipelineQueue* DuplicateQueue = DuplicateObject<UMoviePipelineQueue>(CurrentQueue, NewPackage, *NewAssetName);

	if (DuplicateQueue)
	{
		DuplicateQueue->SetFlags(RF_Public | RF_Standalone | RF_Transactional);

		FAssetRegistryModule::AssetCreated(DuplicateQueue);

		FEditorFileUtils::EPromptReturnCode PromptReturnCode = FEditorFileUtils::PromptForCheckoutAndSave({ NewPackage }, false, false);
	}
}

void SMoviePipelineGraphPanel::OnImportSavedQueueAssest(const FAssetData& InPresetAsset)
{
	FSlateApplication::Get().DismissAllMenus();

	UMoviePipelineQueue* SavedQueue = CastChecked<UMoviePipelineQueue>(InPresetAsset.GetAsset());
	if (SavedQueue)
	{
		// Duplicate the queue so we don't start modifying the one in the asset.
		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);

		Subsystem->GetQueue()->CopyFrom(SavedQueue);

		// Update the shot list in case the stored queue being copied is out of date with the sequence
		for (UMoviePipelineExecutorJob* Job : Subsystem->GetQueue()->GetJobs())
		{
			ULevelSequence* LoadedSequence = Cast<ULevelSequence>(Job->Sequence.TryLoad());
			if (LoadedSequence)
			{
				bool bShotsChanged = false;
				UMoviePipelineBlueprintLibrary::UpdateJobShotListFromSequence(LoadedSequence, Job, bShotsChanged);

				if (bShotsChanged)
				{
					FNotificationInfo Info(LOCTEXT("QueueShotsUpdated", "Shots have changed since the queue was saved, please resave the queue"));
					Info.ExpireDuration = 5.0f;
					FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
				}
			}
		}

		// Automatically select the first job in the queue
		TArray<UMoviePipelineExecutorJob*> Jobs;
		if (Subsystem->GetQueue()->GetJobs().Num() > 0)
		{
			Jobs.Add(Subsystem->GetQueue()->GetJobs()[0]);
		}

		// Go through the UI so it updates the UI selection too and then this will loop back
		// around to OnSelectionChanged to update ourself.
		PipelineQueueEditorWidget->SetSelectedJobs(Jobs);
	}
}


#undef LOCTEXT_NAMESPACE // SMoviePipelineGraphPanel