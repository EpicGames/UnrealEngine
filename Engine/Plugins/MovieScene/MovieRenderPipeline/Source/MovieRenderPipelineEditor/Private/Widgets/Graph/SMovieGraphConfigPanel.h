// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "EditorUndoClient.h"
#include "GraphEditor.h"

class UMoviePipelineConfigBase;
class SGraphEditor;
class SMoviePipelineQueueEditor;
class SWindow;
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;
class IDetailsView;
struct FAssetData;
class FUICommandList;
class UMovieGraphNode;
enum class ECheckBoxState : uint8;
enum class ENodeEnabledState : uint8;

/**
 * Outermost widget that is used for adding and removing jobs from the Movie Pipeline Queue Subsystem.
 */
class SMoviePipelineGraphPanel : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
public:
	DECLARE_DELEGATE_OneParam(FOnGraphSelectionChanged, TArray<UObject*>);
	
	SLATE_BEGIN_ARGS(SMoviePipelineGraphPanel)
		: _Graph(nullptr)

		{}

		SLATE_EVENT(FOnGraphSelectionChanged, OnGraphSelectionChanged)

		/** The graph that is initially displayed */
		SLATE_ARGUMENT(class UMovieGraphConfig*, Graph)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Clears the selection in the graph widget being displayed. */
	void ClearGraphSelection() const;

	/** Gets the graph editor that's contained within this widget. */
	TWeakPtr<SGraphEditor> GetGraphEditor() const;

	// FEditorUndoClient interface
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// ~FEditorUndoClient interface

private:
	FReply OnRenderLocalRequested();
	bool IsRenderLocalEnabled() const;
	FReply OnRenderRemoteRequested();
	bool IsRenderRemoteEnabled() const;

	/** When they want to edit the current configuration for the job */
	void OnEditJobConfigRequested(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot);
	/** When an existing preset is chosen for the specified job. */
	void OnJobPresetChosen(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot);
	void OnConfigUpdatedForJob(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot, UMoviePipelineConfigBase* InConfig);
	void OnConfigUpdatedForJobToPreset(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot, UMoviePipelineConfigBase* InConfig);
	void OnConfigWindowClosed();

	void OnSelectionChanged(const TArray<UMoviePipelineExecutorJob*>& InSelectedJobs, const TArray<UMoviePipelineExecutorShot*>& InSelectedShots);
	TArray<UMovieGraphNode*> GetSelectedModelNodes() const;

	TSharedRef<SWidget> OnGenerateSavedQueuesMenu();
	bool OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName);
	bool GetSavePresetPackageName(const FString& InExistingName, FString& OutName);
	void OnSaveAsAsset();
	void OnImportSavedQueueAssest(const FAssetData& InPresetAsset);
	
	void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection);
	void OnNodeDoubleClicked(class UEdGraphNode* Node);
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/** Provides appearance info to the graph widget. */
	FGraphAppearanceInfo GetGraphAppearanceInfo() const;
	
	TObjectPtr<class UMovieGraphConfig> CurrentGraph;
private:
	/** Allocates a transient preset so that the user can use the pipeline without saving it to an asset first. */
	//UMoviePipelineConfigBase* AllocateTransientPreset();
	void MakeEditorCommands();

	/** Creates a new, empty comment in the graph around the currently selected nodes. */
	void OnCreateComment() const;
	
	void SelectAllNodes();
	bool CanSelectAllNodes() const;
	bool CanDeleteSelectedNodes() const;
	void DeleteSelectedNodes();
	void CopySelectedNodes();
	bool CanCopySelectedNodes() const;
	void CutSelectedNodes();
	bool CanCutSelectedNodes() const;
	void PasteNodes();
	void PasteNodesHere(const FVector2D& Location);
	bool CanPasteNodes() const;
	void DuplicateNodes();
	bool CanDuplicateNodes() const;
	
	/** Set the enabled state for the currently selected nodes. */
	void SetEnabledStateForSelectedNodes(const ENodeEnabledState NewState) const;

	/**
	 * Attempt to match the given enable state for currently-selected nodes. If StateToCheck matches all selected
	 * nodes, returns ECheckBoxState::Checked or ECheckBoxState::Unchecked; if there is a mismatch, returns
	 * ECheckBoxState::Undetermined.
	 */
	ECheckBoxState CheckEnabledStateForSelectedNodes(const ENodeEnabledState EnabledStateToCheck) const;

	/** Determines if all of the selected nodes can have their enable/disable state toggled. */
	bool CanDisableSelectedNodes() const;

	void OnAlignTop();
	void OnAlignMiddle();
	void OnAlignBottom();
	void OnAlignLeft();
	void OnAlignCenter();
	void OnAlignRight();
	void OnStraightenConnections();
	void OnDistributeNodesH();
	void OnDistributeNodesV();

private:
	/** The main movie pipeline queue editor widget */
	TSharedPtr<SMoviePipelineQueueEditor> PipelineQueueEditorWidget;

	/** The graph widget displayed in this panel. */
	TSharedPtr<SGraphEditor> GraphEditorWidget;

	TWeakPtr<SWindow> WeakEditorWindow;
	
	int32 NumSelectedJobs;

	FOnGraphSelectionChanged OnGraphSelectionChangedEvent;
	TSharedPtr<FUICommandList> GraphEditorCommands;
};