// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Graph/SMovieGraphModifierCollectionsList.h"

#include "Graph/Nodes/MovieGraphModifierNode.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "MovieGraphModifierCollectionsList"

void SMovieGraphModifierCollectionsList::Construct(const FArguments& InArgs)
{
	WeakModifierInterface = InArgs._WeakModifierInterface.Get();
	
	ChildSlot
	[
		SAssignNew(CollectionsList, SMovieGraphSimpleList<FName>)
        .DataSource(&ListDataSource)
        .DataType(FText::FromString("Collection"))
        .DataTypePlural(FText::FromString("Collections"))
        .OnDelete_Lambda([this](const TArray<FName> DeletedCollectionNames)
        {
        	if (WeakModifierInterface.IsValid())
        	{
        		const FScopedTransaction Transaction(LOCTEXT("RemoveCollectionsFromModifier", "Remove Collections from Modifier"));

        		for (const FName& DeletedCollectionName : DeletedCollectionNames)
        		{
        			WeakModifierInterface.Get()->RemoveCollection(DeletedCollectionName);
        		}
        		
        		Refresh();
        	}
        })
        .OnGetRowText_Static(&GetCollectionRowText)
        .ShowEnableDisable(true)
        .OnGetRowEnableState_Lambda([this](FName InCollectionName)
        {
        	if (WeakModifierInterface.IsValid())
        	{
        		return WeakModifierInterface.Get()->IsCollectionEnabled(InCollectionName);
        	}

        	return true;
        })
        .OnSetRowEnableState_Lambda([this](FName InCollectionName, bool bNewEnableState)
        {
        	if (WeakModifierInterface.IsValid())
        	{
        		const FScopedTransaction Transaction(LOCTEXT("ChangeCollectionEnableState", "Change Collection Enable State"));
        		
        		WeakModifierInterface.Get()->SetCollectionEnabled(InCollectionName, bNewEnableState);
        	}
        })
        .OnRefreshDataSourceRequested(this, &SMovieGraphModifierCollectionsList::RefreshListDataSource)
	];

	// Make sure the data source is up-to-date initially
	Refresh();
}

void SMovieGraphModifierCollectionsList::Refresh()
{
	RefreshListDataSource();

	if (CollectionsList)
	{
		CollectionsList->Refresh();
	}
}

FText SMovieGraphModifierCollectionsList::GetCollectionRowText(const FName CollectionName)
{
	return FText::FromName(CollectionName);
}

void SMovieGraphModifierCollectionsList::RefreshListDataSource()
{
	if (WeakModifierInterface.IsValid())
	{
		ListDataSource = WeakModifierInterface.Get()->GetAllCollections();
	}
}

#undef LOCTEXT_NAMESPACE
