// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineQueuePanelUtils.h"

#include "Editor.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphConfigFactory.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineQueueSubsystem.h"
#include "MovieRenderPipelineSettings.h"

#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "FileHelpers.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"

#define LOCTEXT_NAMESPACE "MoviePipelineQueuePanelUtils"

namespace UE::MovieRenderPipelineEditor::Private
{
	UMovieGraphConfig* GenerateNewShotSubgraph(const UMoviePipelineExecutorJob* InJob, UMoviePipelineExecutorShot* InShot)
	{
		UMovieGraphConfigFactory* GraphFactory = NewObject<UMovieGraphConfigFactory>();
		GraphFactory->InitialSubgraphAsset = InJob->GetGraphPreset();

		const FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		UObject* NewAsset = AssetToolsModule.Get().CreateAssetWithDialog(GraphFactory->GetSupportedClass(), GraphFactory);

		// A cancel in the dialog returns null -- don't ensure
		UMovieGraphConfig* NewGraph = Cast<UMovieGraphConfig>(NewAsset);
		if (!NewGraph)
		{
			return nullptr;
		}

		constexpr bool bOnlyDirty = false;
		UEditorLoadingAndSavingUtils::SavePackages({ NewGraph->GetPackage() }, bOnlyDirty);

		InShot->SetGraphPreset(NewGraph);
		return NewGraph;
	}

	void SaveTransientQueueToAsset(UMoviePipelineQueue* DestinationQueue)
	{
		if (!DestinationQueue)
		{
			return;
		}

		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);

		// Copy the live (transient) queue state into the destination asset
		UMoviePipelineQueue* CurrentQueue = Subsystem->GetQueue();
		DestinationQueue->CopyFrom(CurrentQueue);
		DestinationQueue->SetQueueOrigin(nullptr);
		DestinationQueue->MarkPackageDirty();
		DestinationQueue->SetFlags(RF_Public | RF_Standalone | RF_Transactional);

		FAssetRegistryModule::AssetCreated(DestinationQueue);

		constexpr bool bCheckDirty = false;
		constexpr bool bPromptToSave = false;
		const FEditorFileUtils::EPromptReturnCode PromptReturnCode =
			FEditorFileUtils::PromptForCheckoutAndSave({ DestinationQueue->GetPackage() }, bCheckDirty, bPromptToSave);

		if (PromptReturnCode == FEditorFileUtils::EPromptReturnCode::PR_Success)
		{
			// Reload the queue from the newly saved asset so the subsystem tracks it as the origin
			constexpr bool bPromptOnReplacingDirtyQueue = false;
			Subsystem->LoadQueue(DestinationQueue, bPromptOnReplacingDirtyQueue);
		}
	}

	void SaveQueue()
	{
		UMoviePipelineQueue* QueueOrigin = GetQueueOrigin();

		// If the queue has no known origin asset, fall through to "Save As" so the user picks a path
		if (!QueueOrigin)
		{
			SaveQueueAs();
			return;
		}

		SaveTransientQueueToAsset(QueueOrigin);
	}

	void SaveQueueAs()
	{
		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);
		UMoviePipelineQueue* CurrentQueue = Subsystem->GetQueue();

		FString PackageName;
		if (!GetSavePresetPackageName(CurrentQueue->GetName(), PackageName))
		{
			return;
		}

		const FString NewAssetName = FPackageName::GetLongPackageAssetName(PackageName);
		UPackage* NewPackage = CreatePackage(*PackageName);
		NewPackage->MarkAsFullyLoaded();

		UMoviePipelineQueue* DuplicateQueue = DuplicateObject<UMoviePipelineQueue>(CurrentQueue, NewPackage, *NewAssetName);
		SaveTransientQueueToAsset(DuplicateQueue);
	}

	void ImportSavedQueueAsset(const FAssetData& InPresetAsset)
	{
		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);

		UMoviePipelineQueue* SavedQueue = CastChecked<UMoviePipelineQueue>(InPresetAsset.GetAsset());
		Subsystem->LoadQueue(SavedQueue);
	}

	bool IsQueueDirty()
	{
		const UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);
		return Subsystem->IsQueueDirty();
	}

	UMoviePipelineQueue* GetQueueOrigin()
	{
		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);

		const UMoviePipelineQueue* Queue = Subsystem->GetQueue();
		return Queue ? Queue->GetQueueOrigin() : nullptr;
	}

	FString GetQueueOriginName()
	{
		const UMoviePipelineQueue* QueueOrigin = GetQueueOrigin();
		if (!QueueOrigin)
		{
			return FString();
		}

		const UPackage* Package = QueueOrigin->GetPackage();
		if (!Package)
		{
			return FString();
		}

		FString OutPackageRoot, OutPackagePath, OutPackageName;
		if (FPackageName::SplitLongPackageName(Package->GetName(), OutPackageRoot, OutPackagePath, OutPackageName))
		{
			return OutPackageName;
		}

		return FString();
	}

	bool OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName)
	{
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DefaultPath = InDefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = InNewNameSuggestion;
		SaveAssetDialogConfig.AssetClassNames.Add(UMoviePipelineQueue::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveQueueAssetDialogTitle", "Save Queue Asset");

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

		if (!SaveObjectPath.IsEmpty())
		{
			OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
			return true;
		}

		return false;
	}

	bool GetSavePresetPackageName(const FString& InExistingName, FString& OutName)
	{
		UMovieRenderPipelineProjectSettings* ConfigSettings = GetMutableDefault<UMovieRenderPipelineProjectSettings>();

		const FString DefaultSaveDirectory = ConfigSettings->PresetSaveDir.Path;

		FString DialogStartPath;
		FPackageName::TryConvertFilenameToLongPackageName(DefaultSaveDirectory, DialogStartPath);
		if (DialogStartPath.IsEmpty())
		{
			DialogStartPath = TEXT("/Game");
		}

		FString UniquePackageName;
		FString UniqueAssetName;

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(DialogStartPath / InExistingName, TEXT(""), UniquePackageName, UniqueAssetName);

		const FString DialogStartName = FPaths::GetCleanFilename(UniqueAssetName);

		FString UserPackageName;
		FString NewPackageName;

		bool bFilenameValid = false;
		while (!bFilenameValid)
		{
			if (!OpenSaveDialog(DialogStartPath, DialogStartName, UserPackageName))
			{
				return false;
			}

			NewPackageName = UserPackageName;

			FText OutError;
			bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError);
		}

		// Remember the last save location for next time
		ConfigSettings->PresetSaveDir.Path = FPackageName::GetLongPackagePath(UserPackageName);
		ConfigSettings->SaveConfig();
		OutName = MoveTemp(NewPackageName);
		return true;
	}
}

#undef LOCTEXT_NAMESPACE // MoviePipelineQueuePanelUtils
