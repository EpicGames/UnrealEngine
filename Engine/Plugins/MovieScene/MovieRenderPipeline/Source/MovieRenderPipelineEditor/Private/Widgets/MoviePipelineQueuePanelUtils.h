// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;
class UMoviePipelineQueue;
class UMovieGraphConfig;
struct FAssetData;

namespace UE::MovieRenderPipelineEditor::Private
{
	/**
	 * Generates and returns a new shot subgraph based on the given job and shot. Returns nullptr if there was an issue creating the graph. If the
	 * graph was successfully created, it will be assigned to the shot.
	 */
	UMovieGraphConfig* GenerateNewShotSubgraph(const UMoviePipelineExecutorJob* InJob, UMoviePipelineExecutorShot* InShot);

	/** Copies the transient queue to the destination queue asset, registers it, and prompts for save. */
	void SaveTransientQueueToAsset(UMoviePipelineQueue* DestinationQueue);

	/** Saves the queue. If it has no known origin, falls through to SaveQueueAs(). */
	void SaveQueue();

	/** Prompts the user for a new package name and saves the queue as a new asset. */
	void SaveQueueAs();

	/** Loads a previously saved queue asset via the subsystem. Does NOT dismiss menus -- callers handle that. */
	void ImportSavedQueueAsset(const FAssetData& InPresetAsset);

	/** Returns true if the current queue has been modified since the last save. */
	bool IsQueueDirty();

	/** Returns the origin queue asset for the currently loaded queue, or nullptr. */
	UMoviePipelineQueue* GetQueueOrigin();

	/** Returns the short package name of the queue origin, or an empty string. */
	FString GetQueueOriginName();

	/** Opens a save-asset dialog for a queue. Returns true if the user selected a valid path. */
	bool OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName);

	/** Determines the package name when saving a queue preset. Returns true on success. */
	bool GetSavePresetPackageName(const FString& InExistingName, FString& OutName);
}
