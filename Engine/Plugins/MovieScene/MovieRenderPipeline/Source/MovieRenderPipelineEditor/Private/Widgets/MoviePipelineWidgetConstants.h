// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"

namespace MoviePipeline
{
	/** Button offset to apply to buttons on movie pipeline panel toolbars */
	static FMargin ButtonOffset(FMargin(0.f, 0.f, 4.f, 0.f));

	/** Button padding to apply to buttons inside movie pipeline panel toolbars */
	static FMargin ButtonPadding(4.f);
}