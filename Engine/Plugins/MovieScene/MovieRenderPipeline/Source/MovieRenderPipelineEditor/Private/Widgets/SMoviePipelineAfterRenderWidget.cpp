// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMoviePipelineAfterRenderWidget.h"
#include "MoviePipelineQueue.h"

#include "PropertyEditorModule.h"
#include "ISinglePropertyView.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineAfterRenderWidget"

void SMoviePipelineAfterRenderWidget::Construct(const FArguments& InArgs)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	UMoviePipelineQueueProjectSettings* QueueProjectSettings = GetMutableDefault<UMoviePipelineQueueProjectSettings>();

	FSinglePropertyParams PostRenderActionTypeParams;
	PostRenderActionTypeParams.NamePlacement = EPropertyNamePlacement::Hidden;
	PostRenderActionTypeParams.bHideResetToDefault = true;

	PostRenderBehaviorWidget = PropertyEditorModule.CreateSingleProperty(
		QueueProjectSettings, GET_MEMBER_NAME_CHECKED(UMoviePipelineQueueProjectSettings, PostRenderActionType), PostRenderActionTypeParams);

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4, 0, 4, 0)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AfterRender_Text", "After Render:"))
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			PostRenderBehaviorWidget.ToSharedRef()
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.ContentPadding(0)
			.OnClicked_Lambda([]()
			{
				FModuleManager::LoadModuleChecked<ISettingsModule>("Settings")
					.ShowViewer("Editor", "Plugins", "MovieRenderGraphEditorSettings");

				return FReply::Handled();
			})
			.ToolTipText(LOCTEXT("OpenPlaybackPrefs_Tooltip", "Open up the preferences which dictate how media is opened after a render completes."))
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("EditorPreferences.TabIcon"))
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
