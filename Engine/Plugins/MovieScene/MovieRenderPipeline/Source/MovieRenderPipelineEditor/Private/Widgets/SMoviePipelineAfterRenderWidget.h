// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ISinglePropertyView;

/**
 * Compact toolbar widget that shows the "After Render" behavior dropdown
 * and a settings button to open the post-render preferences.
 */
class SMoviePipelineAfterRenderWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMoviePipelineAfterRenderWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	TSharedPtr<ISinglePropertyView> PostRenderBehaviorWidget;
};
