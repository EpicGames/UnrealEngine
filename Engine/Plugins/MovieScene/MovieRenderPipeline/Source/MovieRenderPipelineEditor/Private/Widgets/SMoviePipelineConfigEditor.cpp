// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMoviePipelineConfigEditor.h"

#include "MoviePipelineConsoleVariableSetting.h"
#include "MoviePipelineDeferredPasses.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineSetting.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineUtils.h"
#include "SMoviePipelineFormatTokenAutoCompleteBox.h"
#include "Widgets/SMoviePipelineConfigSettings.h"

// Core includes
#include "Modules/ModuleManager.h"
#include "Templates/SubclassOf.h"

// AssetRegistry includes
#include "AssetRegistry/AssetData.h"
 
// Slate includes
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Views/SListView.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/SlateIconFinder.h"
#include "SPositiveActionButton.h"

// EditorStyle includes
#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"
#include "ScopedTransaction.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "IDetailCustomization.h"
#include "Editor.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"

#include "FrameNumberDetailsCustomization.h"
#include "Customizations/ConsoleVariableSettingCustomization.h"
#include "Customizations/PostProcessPassCustomization.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineEditor"

class FOutputFormatDetailsCustomization : public IDetailCustomization
{
public:
	/** Creates a detail customization instance */
	static TSharedRef<IDetailCustomization> MakeInstance(TSharedRef<SMoviePipelineConfigEditor> InEditor)
	{
		return MakeShared<FOutputFormatDetailsCustomization>(InEditor);
	}

	FOutputFormatDetailsCustomization(TSharedRef<SMoviePipelineConfigEditor> InEditor)
	{
		OwningEditor = InEditor;
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		OutputFormatPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMoviePipelineOutputSetting, FileNameFormat));
		const bool bShowChildren = false;
		FText InitialText;
		OutputFormatPropertyHandle->GetValueAsDisplayText(InitialText);

		DetailBuilder.EditDefaultProperty(OutputFormatPropertyHandle)->CustomWidget(bShowChildren)
			.NameContent()
			[
				OutputFormatPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(200.0f)
			[
				SAssignNew(AutoCompleteBox, SMoviePipelineFormatTokenAutoCompleteBox)
				.InitialText(InitialText)
				.TextHandle(OutputFormatPropertyHandle)
				.Suggestions(this, &FOutputFormatDetailsCustomization::GetSuggestions)
			];
	}
	//~ End IDetailCustomization interface

	TArray<FString> GetSuggestions() const
	{
		TSharedPtr<SMoviePipelineConfigEditor> ConfigEditor = OwningEditor.Pin();
		if (ConfigEditor.IsValid())
		{
			TArray<FString> Suggestions;
			if (ConfigEditor->CachedPipelineConfig.IsValid())
			{
				FMoviePipelineFormatArgs FormatArgs;
				FormatArgs.InJob = ConfigEditor->CachedOwningJob.Get();

				// Find the primary configuration that owns us (we won't necessairly have a job, but
				// we might be able to find the primary config that this setting is outered to.
				UMoviePipelinePrimaryConfig* PrimaryConfig = Cast<UMoviePipelinePrimaryConfig>(ConfigEditor->CachedPipelineConfig.Get());
				if (PrimaryConfig)
				{
					PrimaryConfig->GetFormatArguments(FormatArgs);
				}

				for (const TPair<FString, FString>& KVP : FormatArgs.FilenameArguments)
				{
					Suggestions.Add(KVP.Key);
				}
			}

			return Suggestions;
		}

		return TArray<FString>();
	}

	TSharedPtr<IPropertyHandle> OutputFormatPropertyHandle;
	TWeakPtr<SMoviePipelineConfigEditor> OwningEditor;
	TSharedPtr<SMoviePipelineFormatTokenAutoCompleteBox> AutoCompleteBox;
};

/**
 * Widget used to edit a Movie Render Pipeline Shot Config.
 */
UE_DISABLE_OPTIMIZATION_SHIP
void SMoviePipelineConfigEditor::Construct(const FArguments& InArgs)
{
	bRequestDetailsRefresh = true;
	PipelineConfigAttribute = InArgs._PipelineConfig;
	OwningJobAttribute = InArgs._OwningJob;
    
	DetailsBox = SNew(SScrollBox);
	DetailsBox->SetScrollBarRightClickDragAllowed(true);
     
	SettingsWidget = SNew(SMoviePipelineConfigSettings)
	.OnSelectionChanged(this, &SMoviePipelineConfigEditor::OnSettingsSelectionChanged);
    
	CheckForNewSettingsObject();
     
	// Automatically try to select the Output setting as it's the most commonly edited one.
	// If that fails, we fall back to the first setting.
	{
		UMoviePipelineOutputSetting* OutputSetting = CachedPipelineConfig->FindSetting<UMoviePipelineOutputSetting>();
		if (OutputSetting)
		{
			TArray<UMoviePipelineSetting*> SelectedSettings;
			SelectedSettings.Add(OutputSetting);
			SettingsWidget->SetSelectedSettings(SelectedSettings);
		}
		// Shot overrides may not have Output Settings.
		else
		{
			if (CachedPipelineConfig->GetUserSettings().Num() > 0)
			{
				SettingsWidget->SetSelectedSettings({ CachedPipelineConfig->GetUserSettings()[0] });
			}
		}
	}
	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)
    
		+ SSplitter::Slot()
		.Value(.33f)
		[
			SNew(SBorder)
			.Padding(4)
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			[
				SettingsWidget.ToSharedRef()
			]
		]
    
		+ SSplitter::Slot()
		.Value(.67f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				DetailsBox.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.Padding(FMargin(0, 4, 0, 0))
			.AutoHeight()
			[
				SNew(SBorder)
				.Visibility(this, &SMoviePipelineConfigEditor::IsSettingFooterVisible)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SBox)
					.MaxDesiredHeight(96)
					[
						SNew(SScrollBox)
						+ SScrollBox::Slot()
						[
							SNew(SMultiLineEditableText)
							.IsReadOnly(true)
							.AutoWrapText(true)
							.SelectWordOnMouseDoubleClick(true)
							.Text(this, &SMoviePipelineConfigEditor::GetSettingsFooterText)
						]
					]	
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Visibility(this, &SMoviePipelineConfigEditor::IsValidationWarningVisible)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2, 0, 4, 0)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
						.Text(FEditorFontGlyphs::Exclamation_Triangle)
						.ColorAndOpacity(FLinearColor::Yellow)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Text(this, &SMoviePipelineConfigEditor::GetValidationWarningText)
					]
				]
			]
		]
	];

	// Register a callback so that we can refresh the details panel after it is reinstanced.
	GEditor->OnBlueprintReinstanced().AddSP(this, &SMoviePipelineConfigEditor::OnBlueprintReinstanced);
}

TSharedRef<SWidget> SMoviePipelineConfigEditor::MakeAddSettingButton()
{
	return SNew(SPositiveActionButton)
		.OnGetMenuContent(this, &SMoviePipelineConfigEditor::OnGenerateSettingsMenu)
		.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		.Text(LOCTEXT("AddNewSetting_Text", "Setting"));
}
UE_ENABLE_OPTIMIZATION_SHIP

void SMoviePipelineConfigEditor::OnBlueprintReinstanced()
{
	SettingsWidget->InvalidateCachedSettingsSerialNumber();
	bRequestDetailsRefresh = true;
}

void SMoviePipelineConfigEditor::CheckForNewSettingsObject()
{
	UMoviePipelineConfigBase* NewMoviePipeline = PipelineConfigAttribute.Get();
	if (CachedPipelineConfig != NewMoviePipeline)
	{
		CachedPipelineConfig = NewMoviePipeline;

		SettingsWidget->SetShotConfigObject(NewMoviePipeline);
		bRequestDetailsRefresh = true;
	}

	UMoviePipelineExecutorJob* NewOwningJob = OwningJobAttribute.Get();
	if (CachedOwningJob != NewOwningJob)
	{
		CachedOwningJob = NewOwningJob;
		bRequestDetailsRefresh = true;
	}
}

void SMoviePipelineConfigEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	CheckForNewSettingsObject();
	if (bRequestDetailsRefresh)
	{
		UpdateDetails();
		bRequestDetailsRefresh = false;
	}
}

TSharedRef<SWidget> SMoviePipelineConfigEditor::OnGenerateSettingsMenu()
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	FMenuBuilder MenuBuilder(true, nullptr, Extender);

	// Put the different categories into different sections
	TArray<UClass*> SourceClasses = UE::MovieRenderPipeline::FindMoviePipelineSettingClasses(UMoviePipelineSetting::StaticClass());

	TMap<FString, TArray<UClass*>> CategorizedClasses;

	for (UClass* Class : SourceClasses)
	{
		const UMoviePipelineSetting* SettingDefaultObject = GetDefault<UMoviePipelineSetting>(Class);
		CategorizedClasses.FindOrAdd(SettingDefaultObject->GetCategoryText().ToString()).Add(Class);
	}

	for (TPair <FString, TArray<UClass*>> KVP : CategorizedClasses)
	{
		MenuBuilder.BeginSection(NAME_None, FText::FromString(KVP.Key));

		// Sort the classes by their CDO's GetDisplayText and not the classes GetDisplayName.
		Algo::Sort(KVP.Value, [](UClass* A, UClass* B)
			{
				const UMoviePipelineSetting* DefaultA = GetDefault<UMoviePipelineSetting>(A);
				const UMoviePipelineSetting* DefaultB = GetDefault<UMoviePipelineSetting>(B);

				return DefaultA->GetDisplayText().CompareTo(DefaultB->GetDisplayText()) < 0;
			});

		for (UClass* Class : KVP.Value)
		{
			// Get a display name for the setting from the CDO.
			const UMoviePipelineSetting* SettingDefaultObject = GetDefault<UMoviePipelineSetting>(Class);
			if (!SettingDefaultObject)
			{
				continue;
			}

			// Depending on the type of config we're editing, some settings may not be eligible. If this is the case, we omit them from the list.
			bool bCanSettingBeAdded = CachedPipelineConfig->CanSettingBeAdded(SettingDefaultObject);
			if (!bCanSettingBeAdded)
			{
				continue;
			}

			// If the setting already exists and it only allows a single instance, we omit them from the list.
			bool bAllowDuplicates = true;
			for (UMoviePipelineSetting* ExistingSetting : CachedPipelineConfig->GetUserSettings())
			{
				// If we found a setting with the same class as ours, ask the CDO if multiple are valid.
				if (ExistingSetting->GetClass() == Class)
				{
					bAllowDuplicates = !SettingDefaultObject->IsSolo();
					break;
				}
			}
			if (!bAllowDuplicates)
			{
				continue;
			}

			FText SettingDisplayName = SettingDefaultObject->GetDisplayText();

			TSubclassOf<UMoviePipelineSetting> SubclassOf = Class;
			MenuBuilder.AddMenuEntry(
				SettingDisplayName,
				Class->GetToolTipText(true),
				FSlateIconFinder::FindIconForClass(Class),
				FUIAction(
					FExecuteAction::CreateSP(this, &SMoviePipelineConfigEditor::AddSettingFromClass, SubclassOf),
					FCanExecuteAction::CreateSP(this, &SMoviePipelineConfigEditor::CanAddSettingFromClass, SubclassOf)
				)
			);
		}

		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SMoviePipelineConfigEditor::UpdateDetails()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowScrollBar = false;
	DetailsViewArgs.ColumnWidth = 0.5f;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;

	TArray<UMoviePipelineSetting*> SelectedSources;
	SettingsWidget->GetSelectedSettings(SelectedSources);

	
	// Create 1 details panel per source class type
	TSortedMap<const UClass*, TArray<UObject*>> ClassToSources;
	for (UMoviePipelineSetting* Source : SelectedSources)
	{
		ClassToSources.FindOrAdd(Source->GetClass()).Add(Source);
	}
	
	TArray<FObjectKey> PreviousClasses;
	ClassToDetailsView.GenerateKeyArray(PreviousClasses);

	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface = MakeShareable(new TDefaultNumericTypeInterface<double>);

	for (auto& Pair : ClassToSources)
	{
		PreviousClasses.Remove(Pair.Key);
	
		TSharedPtr<IDetailsView> ExistingDetails = ClassToDetailsView.FindRef(Pair.Key);
		if (ExistingDetails.IsValid())
		{
			ExistingDetails->SetObjects(Pair.Value);
		}
		else
		{
			TSharedRef<IDetailsView> Details = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
			Details->RegisterInstancedCustomPropertyTypeLayout("FrameNumber", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFrameNumberDetailsCustomization::MakeInstance, NumericTypeInterface));
			Details->RegisterInstancedCustomPropertyTypeLayout(FMoviePipelinePostProcessPass::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(FPostProcessPassCustomization_MRQ::MakeInstance));
			Details->RegisterInstancedCustomPropertyLayout(UMoviePipelineOutputSetting::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(FOutputFormatDetailsCustomization::MakeInstance, SharedThis(this)));
			Details->RegisterInstancedCustomPropertyLayout(UMoviePipelineConsoleVariableSetting::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(FConsoleVariablesSettingDetailsCustomization::MakeInstance));

			Details->SetObjects(Pair.Value);
			DetailsBox->AddSlot()
				[
					Details
				];
	
			ClassToDetailsView.Add(Pair.Key, Details);
		}
	}
	
	for (FObjectKey StaleClass : PreviousClasses)
	{
		TSharedPtr<IDetailsView> Details = ClassToDetailsView.FindRef(StaleClass);
		DetailsBox->RemoveSlot(Details.ToSharedRef());
		ClassToDetailsView.Remove(StaleClass);
	}
}

void SMoviePipelineConfigEditor::AddSettingFromClass(TSubclassOf<UMoviePipelineSetting> SettingClass)
{
	UMoviePipelineConfigBase* Pipeline = PipelineConfigAttribute.Get();

	if (*SettingClass && Pipeline)
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("AddNewSetting", "Add New {0} Setting"), SettingClass->GetDisplayNameText()));

		const bool bIncludeDisabledSettings = false;
		const bool bExactMatch = true;
		UMoviePipelineSetting* NewSetting = Pipeline->FindOrAddSettingByClass(SettingClass, bIncludeDisabledSettings, bExactMatch);

		SettingsWidget->SetSelectedSettings({ NewSetting });
	}
}

bool SMoviePipelineConfigEditor::CanAddSettingFromClass(TSubclassOf<UMoviePipelineSetting> SettingClass)
{
	return true;
}

EVisibility SMoviePipelineConfigEditor::IsSettingFooterVisible() const
{
	return (GetSettingsFooterText().IsEmpty()) ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SMoviePipelineConfigEditor::GetSettingsFooterText() const
{
	TArray<UMoviePipelineSetting*> SelectedSettings;
	SettingsWidget->GetSelectedSettings(SelectedSettings);

	FTextBuilder TextBuilder;

	if (CachedOwningJob.IsValid())
	{
		for (const UMoviePipelineSetting* Setting : SelectedSettings)
		{
			const FText FooterText = Setting->GetFooterText(CachedOwningJob.Get());
			if (!FooterText.IsEmpty())
			{
				TextBuilder.AppendLine(FooterText);
			}
		}
	}

	return TextBuilder.ToText();
}

EVisibility SMoviePipelineConfigEditor::IsValidationWarningVisible() const
{
	TArray<UMoviePipelineSetting*> SelectedSettings;
	SettingsWidget->GetSelectedSettings(SelectedSettings);
	
	EMoviePipelineValidationState ValidationResult = EMoviePipelineValidationState::Valid;
	for (const UMoviePipelineSetting* Setting : SelectedSettings)
	{
		// Don't show warnings for disabled settings as the invalid setting won't be used when rendering.
		if (!Setting->IsEnabled())
		{
			continue;
		}

		if ((int32)Setting->GetValidationState() > (int32)ValidationResult)
		{
			ValidationResult = Setting->GetValidationState();
		}
	}

	return (ValidationResult != EMoviePipelineValidationState::Valid) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SMoviePipelineConfigEditor::GetValidationWarningText() const
{
	TArray<UMoviePipelineSetting*> SelectedSettings;
	SettingsWidget->GetSelectedSettings(SelectedSettings);

	FTextBuilder TextBuilder;
	for (const UMoviePipelineSetting* Setting : SelectedSettings)
	{
		for (const FText& Result : Setting->GetValidationResults())
		{
			TextBuilder.AppendLine(Result);
		}
	}

	return TextBuilder.ToText();
}
#undef LOCTEXT_NAMESPACE