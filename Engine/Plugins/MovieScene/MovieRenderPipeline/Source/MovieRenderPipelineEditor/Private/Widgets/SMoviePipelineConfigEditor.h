// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"

struct FAssetData;
struct IMoviePipelineSettingTreeItem;

class SScrollBox;
class SMoviePipelineConfigSettings;
class UMoviePipelineConfigBase;
class IDetailsView;
class UMoviePipelineSetting;

template<class> class TSubclassOf;
class UMoviePipelineExecutorJob;

/**
 * Widget used to edit a Movie Render Pipeline Shot Config.
 */
class SMoviePipelineConfigEditor : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMoviePipelineConfigEditor)
		: _PipelineConfig(nullptr)
		, _OwningJob(nullptr)
		{}

		SLATE_ATTRIBUTE(UMoviePipelineConfigBase*, PipelineConfig)
		SLATE_ATTRIBUTE(UMoviePipelineExecutorJob*, OwningJob)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/**
	 * Construct a button that can add sources to this widget's preset
	 */
	TSharedRef<SWidget> MakeAddSettingButton();

private:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	/**
	 * Check to see whether the configuration we're editing has changed, and propagate that change if necessary
	 */
	void CheckForNewSettingsObject();

	/**
	 * Update the details panel for the current selection
	 */
	void UpdateDetails();

private:

	TSharedRef<SWidget> OnGenerateSettingsMenu();
     
	void AddSettingFromClass(TSubclassOf<UMoviePipelineSetting> SettingClass);
	bool CanAddSettingFromClass(TSubclassOf<UMoviePipelineSetting> SettingClass);
	
	void OnSettingsSelectionChanged(TSharedPtr<IMoviePipelineSettingTreeItem>, ESelectInfo::Type) { bRequestDetailsRefresh = true; }
	FText GetSettingsFooterText() const;
	EVisibility IsSettingFooterVisible() const;

	EVisibility IsValidationWarningVisible() const;
	FText GetValidationWarningText() const;
	void OnBlueprintReinstanced();
private:

	bool bRequestDetailsRefresh;

	// Let the customization access these so we can get the options for auto-complete.
	friend class FOutputFormatDetailsCustomization; 

	// Edited Pipeline
	TAttribute<UMoviePipelineConfigBase*> PipelineConfigAttribute;
	TWeakObjectPtr<UMoviePipelineConfigBase> CachedPipelineConfig;

	// Owning Job
	TAttribute<UMoviePipelineExecutorJob*> OwningJobAttribute;
	TWeakObjectPtr<UMoviePipelineExecutorJob> CachedOwningJob;

    
	TSharedPtr<SMoviePipelineConfigSettings> SettingsWidget;
	TSharedPtr<SScrollBox> DetailsBox;
	TMap<FObjectKey, TSharedPtr<IDetailsView>> ClassToDetailsView;
};