// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMoviePipelineConfigPanel.h"
#include "Widgets/SMoviePipelineConfigEditor.h"
#include "Widgets/MoviePipelineWidgetConstants.h"
#include "Misc/MessageDialog.h"
#include "MoviePipelineShotConfig.h"
#include "MovieRenderPipelineSettings.h"
#include "MoviePipelineConfigBase.h"
#include "MoviePipelineOutputSetting.h"

// Slate Includes
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"
#include "MovieRenderPipelineStyle.h"
#include "SPrimaryButton.h"

// ContentBrowser Includes
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

// UnrealEd Includes
#include "ScopedTransaction.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "FileHelpers.h"
#include "AssetToolsModule.h"
#include "Misc/FileHelper.h"


#define LOCTEXT_NAMESPACE "SMoviePipelineConfigPanel"

SMoviePipelineConfigPanel::~SMoviePipelineConfigPanel()
{
}

UE_DISABLE_OPTIMIZATION_SHIP
void SMoviePipelineConfigPanel::Construct(const FArguments& InArgs, TSubclassOf<UMoviePipelineConfigBase> InConfigType)
{
	ConfigAssetType = InConfigType;

	if (!InArgs._AssetToEdit)
	{
		// Allocate a transient preset automatically so they can start editing without having to create an asset.
		TransientPreset = AllocateTransientPreset();

		// Copy the base preset into the transient preset if it was provided.
		if (InArgs._BasePreset)
		{
			TransientPreset->CopyFrom(InArgs._BasePreset);
			OriginPreset = InArgs._BasePreset;
		}
		else if (InArgs._BaseConfig)
		{
			TransientPreset->CopyFrom(InArgs._BaseConfig);
			OriginPreset = nullptr;
		}
	}
	else
	{
		// If they want to edit an asset directly, we don't copy from other sources
		TransientPreset = InArgs._AssetToEdit;
	}

	// Listen for modificatiosn to our edited configuration so we can determine if they've modified the preset
	// which only changes the display (the actual preset asset is not modified)
	FCoreUObjectDelegates::OnObjectModified.AddSP(this, &SMoviePipelineConfigPanel::OnAnyObjectModified);
	
	WeakJob = InArgs._Job;
	WeakShot = InArgs._Shot;
	OnConfigurationModified = InArgs._OnConfigurationModified;
	OnConfigurationSetToPreset = InArgs._OnConfigurationSetToPreset;

	// Create the child widgets that need to know about our pipeline
	MoviePipelineEditorWidget = SNew(SMoviePipelineConfigEditor)
	.PipelineConfig(this, &SMoviePipelineConfigPanel::GetPipelineConfig)
	.OwningJob(this, &SMoviePipelineConfigPanel::GetOwningJob);

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);
	ChildSlot
	[
		VerticalBox
	];

	// Direct editing of assets doesn't support 'transient' preset management toolbars.
	if (!InArgs._AssetToEdit)
	{
		// Create the toolbar for adding new stuff, choosing/saving a preset and resetting to the preset defaults.
		VerticalBox->AddSlot()
		.Padding(FMargin(0.f, 1.0f))
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
			[
				SNew(SHorizontalBox)

				// Adding things to the Pipeline
				+ SHorizontalBox::Slot()
				.Padding(MoviePipeline::ButtonOffset)
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					MoviePipelineEditorWidget->MakeAddSettingButton()
				]

				// Spacer
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.FillWidth(1.f)
				[
					SNullWidget::NullWidget
				]

				// Presets Management Button
				+ SHorizontalBox::Slot()
				.Padding(MoviePipeline::ButtonOffset)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					SNew(SComboButton)
					.ToolTipText(LOCTEXT("PresetManagementButton_Tooltip", "Export the current settings to a preset, or import a copy of an existing preset."))
					.ContentPadding(MoviePipeline::ButtonPadding)
					.ComboButtonStyle(FMovieRenderPipelineStyle::Get(), "ComboButton")
					.OnGetMenuContent(this, &SMoviePipelineConfigPanel::OnGeneratePresetsMenu)
					.ForegroundColor(FSlateColor::UseForeground())
					.ButtonContent()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.Padding(0, 1, 4, 0)
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("AssetEditor.SaveAsset"))
						]

						+ SHorizontalBox::Slot()
						.Padding(0, 1, 0, 0)
						[
							SNew(STextBlock)
							.Text(this, &SMoviePipelineConfigPanel::GetPresetsMenuButtonText)
						]
					]
				]
			]
		];
	}

	// Large Label Telling You What You're Editing
	VerticalBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.TextStyle(FMovieRenderPipelineStyle::Get(), "MovieRenderPipeline.Config.TypeLabel")
		.Text(this, &SMoviePipelineConfigPanel::GetConfigTypeLabel)
		.Visibility(EVisibility::Collapsed)
	];

		
	// Main Editor 
	VerticalBox->AddSlot()
	.FillHeight(1.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			MoviePipelineEditorWidget.ToSharedRef()
		]
	];

	// Direct editing of assets doesn't support 'transient' preset management footers.
	if (!InArgs._AssetToEdit)
	{
		// Footer Bar
		VerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
			.Padding(MoviePipeline::ButtonPadding)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SMoviePipelineConfigPanel::GetValidationWarningText)
				]

				// Cancel Changes
				+ SHorizontalBox::Slot()
				.Padding(MoviePipeline::ButtonOffset)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("CancelChangesButton_Text", "Cancel"))
					.ToolTipText(LOCTEXT("CancelChangesButton_Tooltip", "Discards changes made and does not modify anything."))
					.OnClicked(this, &SMoviePipelineConfigPanel::OnCancelChanges)
				]

				// Accept Changes
				+ SHorizontalBox::Slot()
				.Padding(MoviePipeline::ButtonOffset)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("ConfirmChangesButton_Text", "Accept"))
					.ToolTipText(LOCTEXT("ConfirmChangesButton_Tooltip", "Accepts the changes made and applies them to the particular job instance."))
					.OnClicked(this, &SMoviePipelineConfigPanel::OnConfirmChanges)
					.IsEnabled(this, &SMoviePipelineConfigPanel::CanAcceptChanges)
				]
			]
		];
	}
}

UE_ENABLE_OPTIMIZATION_SHIP

FText SMoviePipelineConfigPanel::GetPresetsMenuButtonText() const
{
	const FString PresetName = GetConfigOriginName();
	if (!PresetName.IsEmpty())
	{
		return FText::Format(FText::FromString(TEXT("{0}{1}")),
			FText::FromString(PresetName),
			FText::FromString(IsConfigDirty() ? TEXT(" *") : TEXT("")));
	}
	
	return LOCTEXT("PresetsSaveMenuUnsavedConfig_Text", "Unsaved Config *");
}

FText SMoviePipelineConfigPanel::GetConfigTypeLabel() const
{
	return LOCTEXT("PrimaryConfigType_Label", "PRIMARY");
}

FReply SMoviePipelineConfigPanel::OnConfirmChanges()
{
	// Confirming changes is slightly complicated. If the last action the user
	// did was to import or export a preset then their expectation is that hitting
	// accept will simply assign that preset to the resulting job. Because the UI
	// works on a copy it is hard to tell so we separately track the last preset-based
	// action and then clear it if they've modified 'it' (which is actually our UI copy).
	if (OriginPreset.IsValid() && !bHasModifiedLoadedPreset)
	{
		// They just want to use the preset. We'll call a different callback so it's easier for
		// the caller to determine the intent.
		OnConfigurationSetToPreset.ExecuteIfBound(WeakJob, WeakShot, OriginPreset.Get());
	}
	else
	{
		// If the panel was originally loaded up with an origin preset, but changes have been
		// made, update the transient preset to indicate where it came from
		if (OriginPreset.IsValid())
		{
			TransientPreset->SetConfigOrigin(OriginPreset.Get());
		}
		
		// They modified the object after the last preset action, then we'll just use 
		// the transient ui copy and the owning job will know it has been modified.
		OnConfigurationModified.ExecuteIfBound(WeakJob, WeakShot, TransientPreset);
	}
	
	return FReply::Handled();
}

bool SMoviePipelineConfigPanel::CanAcceptChanges() const
{
	EMoviePipelineValidationState ValidationResult = EMoviePipelineValidationState::Valid;
	for (const UMoviePipelineSetting* Setting : TransientPreset->GetUserSettings())
	{
		if ((int32)Setting->GetValidationState() > (int32)ValidationResult)
		{
			ValidationResult = Setting->GetValidationState();
		}
	}

	return (int32)ValidationResult < (int32)EMoviePipelineValidationState::Errors;
}

FReply SMoviePipelineConfigPanel::OnCancelChanges()
{
	TSharedPtr<SWindow> OwningWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (OwningWindow)
	{
		OwningWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FText SMoviePipelineConfigPanel::GetValidationWarningText() const
{
	EMoviePipelineValidationState ValidationResult = EMoviePipelineValidationState::Valid;
	for (const UMoviePipelineSetting* Setting : TransientPreset->FindSettings<UMoviePipelineSetting>())
	{
		int32 CurResult = (int32)ValidationResult;
		int32 NewResult = (int32)Setting->GetValidationState();
		if (NewResult > CurResult)
		{
			ValidationResult = (EMoviePipelineValidationState)NewResult;
		}
	}

	switch (ValidationResult)
	{
	case EMoviePipelineValidationState::Errors:
		return LOCTEXT("ValidationSettingError_Text", "One or more settings has a configuration error and must be resolved.");
	case EMoviePipelineValidationState::Warnings:
		return LOCTEXT("ValidationSettingWarning_Text", "One or more settings have validation warnings. This may produce undesired results.");
	}

	return FText();
}

UMoviePipelineConfigBase* SMoviePipelineConfigPanel::AllocateTransientPreset()
{
	static const TCHAR* PackageName = TEXT("/Temp/MoviePipeline/PendingConfig");
	FString DesiredName = FString::Printf(TEXT("Pending_%s"), *ConfigAssetType->GetName());

	// Return a cached transient if it exists
	UMoviePipelineConfigBase* ExistingPreset = FindObject<UMoviePipelineConfigBase>(nullptr, TEXT("/Temp/MoviePipeline/PendingConfig.PendingConfig"));
	if (ExistingPreset)
	{
		return ExistingPreset;
	}

	UPackage* NewPackage = CreatePackage(PackageName);
	NewPackage->SetFlags(RF_Transient);
	NewPackage->AddToRoot();

	UMoviePipelineConfigBase* NewPreset = NewObject<UMoviePipelineConfigBase>(NewPackage, ConfigAssetType, *DesiredName, RF_Transient | RF_Transactional | RF_Standalone);

	return NewPreset;
}

TSharedRef<SWidget> SMoviePipelineConfigPanel::OnGeneratePresetsMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	static const FText NoLoadedPresetText = LOCTEXT("NoLoadedPreset", "None");
	static const FText LoadedPresetFormatText = LOCTEXT("LoadedPresetFormat", "Current Preset: {0}");
	const FString PresetName = GetConfigOriginName();
	const FText LoadedPresetName = FText::Format(
		LoadedPresetFormatText, !PresetName.IsEmpty() ? FText::FromString(PresetName) : NoLoadedPresetText);

	MenuBuilder.AddMenuEntry(
		LoadedPresetName,
		LoadedPresetName,
		FSlateIcon(),
		FUIAction(
			FExecuteAction(),
			FCanExecuteAction::CreateLambda([]() { return false; })),
		NAME_None,
		EUserInterfaceActionType::None
	);

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SavePreset_Text", "Save Preset"),
		LOCTEXT("SavePreset_Tip", "Save the current preset if one has been loaded."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAsset"),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMoviePipelineConfigPanel::OnSavePreset),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineConfigPanel::CanSavePreset)
		)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SaveAsPreset_Text", "Save As Preset"),
		LOCTEXT("SaveAsPreset_Tip", "Save the current configuration as a new preset that can be shared between multiple jobs, or imported later as the base of a new configuration."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAssetAs"),
		FUIAction(FExecuteAction::CreateSP(this, &SMoviePipelineConfigPanel::OnSaveAsPreset))
	);

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bSortByPathInColumnView = false;
		AssetPickerConfig.bForceShowEngineContent = false;
		AssetPickerConfig.bForceShowPluginContent = false;

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoPresets_Warning", "No Presets Found");
		AssetPickerConfig.Filter.ClassPaths.Add(ConfigAssetType->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SMoviePipelineConfigPanel::OnImportPreset);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ImportPreset_MenuSection", "Import Preset"));
	{
		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.MinDesiredWidth(400.f)
			.MinDesiredHeight(400.f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SMoviePipelineConfigPanel::OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName)
{
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	{
		SaveAssetDialogConfig.DefaultPath = InDefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = InNewNameSuggestion;
		SaveAssetDialogConfig.AssetClassNames.Add(ConfigAssetType->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveConfigPresetDialogTitle", "Save Config Preset");
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (!SaveObjectPath.IsEmpty())
	{
		OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		return true;
	}

	return false;
}

bool SMoviePipelineConfigPanel::GetSavePresetPackageName(const FString& InExistingName, FString& OutName)
{
	UMovieRenderPipelineProjectSettings* ConfigSettings = GetMutableDefault<UMovieRenderPipelineProjectSettings>();

	// determine default package path
	const FString DefaultSaveDirectory = ConfigSettings->PresetSaveDir.Path;

	FString DialogStartPath;
	FPackageName::TryConvertFilenameToLongPackageName(DefaultSaveDirectory, DialogStartPath);
	if (DialogStartPath.IsEmpty())
	{
		DialogStartPath = TEXT("/Game");
	}

	// determine default asset name
	FString DefaultName = InExistingName;

	FString UniquePackageName;
	FString UniqueAssetName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(DialogStartPath / DefaultName, TEXT(""), UniquePackageName, UniqueAssetName);

	FString DialogStartName = FPaths::GetCleanFilename(UniqueAssetName);

	FString UserPackageName;
	FString NewPackageName;

	// get destination for asset
	bool bFilenameValid = false;
	while (!bFilenameValid)
	{
		if (!OpenSaveDialog(DialogStartPath, DialogStartName, UserPackageName))
		{
			return false;
		}

		NewPackageName = UserPackageName; // FString::Format(*UserPackageName, FormatArgs);

		FText OutError;
		bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError);
	}

	// Update to the last location they saved to so it remembers their settings next time.
	ConfigSettings->PresetSaveDir.Path = FPackageName::GetLongPackagePath(UserPackageName);
	ConfigSettings->SaveConfig();
	OutName = MoveTemp(NewPackageName);
	return true;
}

void SMoviePipelineConfigPanel::OnSavePreset()
{
	UMoviePipelineConfigBase* ConfigOrigin = GetConfigOrigin();
	
	// If the preset origin is not known, then treat the "Save" as a "Save As"
	if (!ConfigOrigin)
	{
		OnSaveAsPreset();
		return;
	}
	
	SaveTransientPresetToAsset(ConfigOrigin);
}

void SMoviePipelineConfigPanel::OnSaveAsPreset()
{
	FString PackageName;
	if (!GetSavePresetPackageName(TransientPreset->GetName(), PackageName))
	{
		return;
	}

	// Saving into a new package
	const FString NewAssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UPackage* NewPackage = CreatePackage(*PackageName);
	NewPackage->MarkAsFullyLoaded();
	UMoviePipelineConfigBase* NewPreset = NewObject<UMoviePipelineConfigBase>(NewPackage, ConfigAssetType, *NewAssetName, RF_Public | RF_Standalone | RF_Transactional);

	SaveTransientPresetToAsset(NewPreset);
}

void SMoviePipelineConfigPanel::SaveTransientPresetToAsset(UMoviePipelineConfigBase* DestinationPreset)
{
	if (TransientPreset && DestinationPreset)
	{
		DestinationPreset->CopyFrom(TransientPreset);
		DestinationPreset->SetConfigOrigin(nullptr);
		DestinationPreset->MarkPackageDirty();

		// We need to be able to save an empty string into the file. When we duplicate the Output Setting in CopyFrom, the PostLoad is fired which fixes it back up
		// to point to the local output directory. A bit unfortunate but we need to do this after the CopyFrom since it needs to only happen on asset save.
		UMoviePipelineOutputSetting* ExistingOutputSetting = TransientPreset->FindSetting<UMoviePipelineOutputSetting>();
		UMoviePipelineOutputSetting* NewOutputSetting = DestinationPreset->FindSetting<UMoviePipelineOutputSetting>();
		if (ExistingOutputSetting && NewOutputSetting)
		{
			if (ExistingOutputSetting->OutputDirectory.Path.Len() == 0)
			{
				// If an empty string, it will get filled via PostLoad either next time the preset is brought into the UI or the asset is loaded.
				NewOutputSetting->OutputDirectory.Path = ExistingOutputSetting->OutputDirectory.Path;
			}
		}
		
		// Make the Display Name match the preset name so when imported later they look like the asset.
		DestinationPreset->DisplayName = DestinationPreset->GetName();

		FAssetRegistryModule::AssetCreated(DestinationPreset);

		const bool bCheckDirty = false;
		const bool bPromptToSave = false;
		FEditorFileUtils::EPromptReturnCode PromptReturnCode = FEditorFileUtils::PromptForCheckoutAndSave({ DestinationPreset->GetPackage() }, bCheckDirty, bPromptToSave);
		if (PromptReturnCode == FEditorFileUtils::EPromptReturnCode::PR_Success)
		{
			// If the user has exported a new preset, they expect that hitting accept will
			// just use that preset directly, unless they further modify the ui. Because this
			// ui uses a copy of the object there's no good way to compare them after closing
			// the dialog.
			bHasModifiedLoadedPreset = false;
			OriginPreset = DestinationPreset;

			// We also want to rename our transient preset. The name is used to indicate what preset
			// it originally came from, and now the closest preset to it is not the original but the
			// newly exported preset.
			TransientPreset->DisplayName = DestinationPreset->DisplayName;
		}

		// CockpitWidget->GetMetaData()->SetPresetOrigin(NewPreset);
	}
}

void SMoviePipelineConfigPanel::OnAnyObjectModified(UObject* InModifiedObject)
{
	// Check to see if our TransientPreset is the outer to the modified object. The preset
	// owns all of the settings and the settings are often the one having Modified called on them.
	if (InModifiedObject)
	{
		if (InModifiedObject->IsIn(TransientPreset) || (InModifiedObject == TransientPreset))
		{
			bHasModifiedLoadedPreset = true;
		}
	}
}

void SMoviePipelineConfigPanel::OnImportPreset(const FAssetData& InPresetAsset)
{
	if (IsConfigDirty())
	{
		const FText TitleText = LOCTEXT("UnsavedPresetWarningTitle", "Unsaved Changes to Preset");
		const FText MessageText = LOCTEXT("UnsavedPresetWarningMessage", "The changes made to the current preset will be lost by importing another preset. Do you want to continue with this import?");

		if (FMessageDialog::Open(EAppMsgType::YesNo, MessageText, TitleText) == EAppReturnType::No)
		{
			return;
		}
	}
	
	FSlateApplication::Get().DismissAllMenus();

	UMoviePipelineConfigBase* Preset = CastChecked<UMoviePipelineConfigBase>(InPresetAsset.GetAsset());
	if (Preset)
	{
		FScopedTransaction Transaction(LOCTEXT("ImportPreset_Transaction", "Import Config Preset"));
		
		TransientPreset->Modify();
		TransientPreset->CopyFrom(Preset);

		bHasModifiedLoadedPreset = false;
		OriginPreset = Preset;
		// CockpitWidget->GetMetaData()->SetPresetOrigin(Take);
	}
}

bool SMoviePipelineConfigPanel::CanSavePreset() const
{
	// If a transient preset is loaded, "Save" will always be available (although it may trigger a "Save As" if the
	// transient's origin preset is not known)
	if (!OriginPreset.IsValid() && TransientPreset)
	{
		return true;
	}

	// If a non-transient preset is loaded, it can be saved as long as it has been modified
	return bHasModifiedLoadedPreset;
}

void SMoviePipelineConfigPanel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(TransientPreset);
}

UMoviePipelineConfigBase* SMoviePipelineConfigPanel::GetPipelineConfig() const
{
	return TransientPreset;
}

UMoviePipelineExecutorJob* SMoviePipelineConfigPanel::GetOwningJob() const
{
	return WeakJob.Get();
}

UMoviePipelineExecutorShot* SMoviePipelineConfigPanel::GetOwningShot() const
{
	return WeakShot.Get();
}

TSharedRef<SWidget> SMoviePipelineConfigPanel::MakeSettingsWidget()
{
	return MoviePipelineEditorWidget->MakeAddSettingButton();
}

bool SMoviePipelineConfigPanel::IsConfigDirty() const
{
	// The preset is dirty if...
	// 1) There was never an origin preset specified, meaning this is a purely transient config, or
	// 2) There *was* an origin preset, but it has been modified after being loaded in this dialog
	return !OriginPreset.IsValid() || (OriginPreset.IsValid() && bHasModifiedLoadedPreset);
}

UMoviePipelineConfigBase* SMoviePipelineConfigPanel::GetConfigOrigin() const
{
	if (OriginPreset.IsValid())
	{
		return OriginPreset.Get();
	}
	
	if (TransientPreset->GetConfigOrigin())
	{
		return TransientPreset->GetConfigOrigin();
	}

	return nullptr;
}

FString SMoviePipelineConfigPanel::GetConfigOriginName() const
{
	if (const UMoviePipelineConfigBase* ConfigOrigin = GetConfigOrigin())
	{
		return ConfigOrigin->DisplayName;
	}

	return FString();
}

#undef LOCTEXT_NAMESPACE // SMoviePipelinePanel