// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"
#include "MoviePipelineConfigBase.h"
#include "MoviePipelineQueue.h"

class UMoviePipelineConfigBase;
class SMoviePipelineConfigEditor;
struct FAssetData;
class UMoviePipelineExecutorJob;

DECLARE_DELEGATE_ThreeParams(FOnMoviePipelineConfigChanged, TWeakObjectPtr<UMoviePipelineExecutorJob>, TWeakObjectPtr<UMoviePipelineExecutorShot>, UMoviePipelineConfigBase*)

/**
 * Outermost widget that is used for setting up a new movie render pipeline config. Operates on a transient UMovieRenderShotConfig that is internally owned and maintained 
 */
class SMoviePipelineConfigPanel : public SCompoundWidget, public FGCObject
{
public:

	~SMoviePipelineConfigPanel();

	SLATE_BEGIN_ARGS(SMoviePipelineConfigPanel)
		: _BasePreset(nullptr)
		,  _BaseConfig(nullptr)
		, _AssetToEdit(nullptr)

		{}
		SLATE_ARGUMENT(TWeakObjectPtr<UMoviePipelineExecutorJob>, Job)
		SLATE_ARGUMENT(TWeakObjectPtr<UMoviePipelineExecutorShot>, Shot)

		SLATE_EVENT(FOnMoviePipelineConfigChanged, OnConfigurationModified)
		SLATE_EVENT(FOnMoviePipelineConfigChanged, OnConfigurationSetToPreset)

		/*~ All following arguments are mutually-exclusive */
		/*-------------------------------------------------*/
		/** A preset asset to copy into the transient UI object. This will not get modified */
		SLATE_ARGUMENT(UMoviePipelineConfigBase*, BasePreset)

		/** An existing configuration to copy into the transient UI object. This will not get modified */
		SLATE_ARGUMENT(UMoviePipelineConfigBase*, BaseConfig)

		/** An existing asset to edit directly with no copy. Hides some of the UI. */
		SLATE_ARGUMENT(UMoviePipelineConfigBase*, AssetToEdit)

		/*-------------------------------------------------*/

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSubclassOf<UMoviePipelineConfigBase> InConfigType);
	UMoviePipelineConfigBase* GetPipelineConfig() const;
	UMoviePipelineExecutorJob* GetOwningJob() const;
	UMoviePipelineExecutorShot* GetOwningShot() const;

private:
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SMoviePipelineConfigPanel");
	}
private:
	/** Attempts to work with the user to find a suitable package path to save the asset under. */
	bool GetSavePresetPackageName(const FString& InExistingName, FString& OutName);

	bool OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName);

	/** Generate the widget that is visible in the Choose Preset dropdown. */
	TSharedRef<SWidget> OnGeneratePresetsMenu();
	
	FText GetPresetsMenuButtonText() const;
	FText GetConfigTypeLabel() const;

	FReply OnCancelChanges();
	FReply OnConfirmChanges();
	bool CanAcceptChanges() const;


	/** Allocates a transient preset so that the user can use the pipeline without saving it to an asset first. */
	UMoviePipelineConfigBase* AllocateTransientPreset();

	/** Called when any object has Modify() called on it. Used to track if user edits transient object after exporting a preset. */
	void OnAnyObjectModified(UObject* InModifiedObject);
	
	/** When a user wants to import an existing preset asset over the current config. */
	void OnImportPreset(const FAssetData& InPresetAsset);

	/** Returns true if the user can save the current preset, else false. */
	bool CanSavePreset() const;
	
	/** Saves the current preset to its existing asset, or triggers a "Save As" if a transient preset is being saved. */
	void OnSavePreset();
	
	/** Saves the current preset out to a new or existing asset. */
	void OnSaveAsPreset();

	/** Saves the transient preset to DestinationPreset. */
	void SaveTransientPresetToAsset(UMoviePipelineConfigBase* DestinationPreset);

	FText GetValidationWarningText() const;

public:
	TSharedRef<SWidget> MakeSettingsWidget();

private:
	bool IsConfigDirty() const;
	UMoviePipelineConfigBase* GetConfigOrigin() const;
	FString GetConfigOriginName() const;

private:
	/** The transient preset that we use - kept alive by AddReferencedObjects */
	TObjectPtr<UMoviePipelineConfigBase> TransientPreset = nullptr;

	/** The preset that is in use, if any. Will be null if the preset was never saved and is only transient. */
	TSoftObjectPtr<UMoviePipelineConfigBase> OriginPreset;

	/** Whether the preset (transient or otherwise) has been modified while loaded in this panel. */
	bool bHasModifiedLoadedPreset = false;

	/** The job this editing panel is for. Kept alive externally. */
	TWeakObjectPtr<UMoviePipelineExecutorJob> WeakJob;

	/** The job this editing panel is for. Kept alive externally. */
	TWeakObjectPtr<UMoviePipelineExecutorShot> WeakShot;

	/** The main movie pipeline editor widget */
	TSharedPtr<SMoviePipelineConfigEditor> MoviePipelineEditorWidget;

	/** What type of asset are we editing? Could be a primary config or a per-shot override config. */
	TSubclassOf<UMoviePipelineConfigBase> ConfigAssetType;

	FOnMoviePipelineConfigChanged OnConfigurationModified;
	FOnMoviePipelineConfigChanged OnConfigurationSetToPreset;
};
