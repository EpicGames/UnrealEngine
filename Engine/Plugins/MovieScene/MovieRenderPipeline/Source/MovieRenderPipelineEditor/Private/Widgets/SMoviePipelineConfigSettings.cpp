// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMoviePipelineConfigSettings.h"
#include "MoviePipelineShotConfig.h"
#include "MoviePipelineSetting.h"

// Core includes
#include "Algo/Sort.h"
#include "UObject/UObjectIterator.h"

// AssetRegistry includes
#include "AssetRegistry/AssetRegistryModule.h"

// Slate includes
#include "Widgets/SBoxPanel.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Commands/GenericCommands.h"

// UnrealEd includes
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"
#include "MovieRenderPipelineStyle.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineConfigSettings"

struct FMoviePipelineSettingCategory;
struct FMoviePipelineSettingTreeItem;

struct IMoviePipelineSettingTreeItem : TSharedFromThis<IMoviePipelineSettingTreeItem>
{
	virtual ~IMoviePipelineSettingTreeItem() {}

	virtual void Delete(UMoviePipelineConfigBase* Owner) = 0;

	virtual TSharedPtr<FMoviePipelineSettingCategory> AsCategory() { return nullptr; }
	virtual TSharedPtr<FMoviePipelineSettingTreeItem> AsSetting()   { return nullptr; }

	virtual TSharedRef<SWidget> ConstructWidget(TWeakPtr<SMoviePipelineConfigSettings> SettingsWidget) = 0;
};

struct FMoviePipelineSettingTreeItem : IMoviePipelineSettingTreeItem
{
	/** Weak pointer to the setting that this tree item represents */
	TWeakObjectPtr<UMoviePipelineSetting> WeakSetting;

	explicit FMoviePipelineSettingTreeItem(UMoviePipelineSetting* InSetting)
		: WeakSetting(InSetting)
	{}

	FText GetLabel() const
	{
		UMoviePipelineSetting* Setting = WeakSetting.Get();
		return Setting ? Setting->GetDisplayText() : FText();
	}

	virtual void Delete(UMoviePipelineConfigBase* Owner) override
	{
		UMoviePipelineSetting* Setting = WeakSetting.Get();
		if (Setting)
		{
			Owner->RemoveSetting(Setting);
		}
	}

	virtual TSharedPtr<FMoviePipelineSettingTreeItem> AsSetting()
	{
		return SharedThis(this);
	}

	virtual TSharedRef<SWidget> ConstructWidget(TWeakPtr<SMoviePipelineConfigSettings> SettingsWidget) override
	{
		return SNew(SOverlay)

		+SOverlay::Slot()
		[
		
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(32)
				.HeightOverride(32)
		  		[
					SNew(SCheckBox)
					.Style(FMovieRenderPipelineStyle::Get(), "MovieRenderPipeline.Setting.Switch")
					.IsFocusable(false)
					.IsChecked(this, &FMoviePipelineSettingTreeItem::GetCheckState)
					.OnCheckStateChanged(this, &FMoviePipelineSettingTreeItem::SetCheckState, SettingsWidget)
					.Visibility(this, &FMoviePipelineSettingTreeItem::GetEnabledStateVisibility)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8, 0)
			[
				SNew(STextBlock)
				.Text(this, &FMoviePipelineSettingTreeItem::GetLabel)
				//.TextStyle(FTakeRecorderStyle::Get(), "TakeRecorder.Source.Label")
				.ColorAndOpacity(this, &FMoviePipelineSettingTreeItem::GetColorAndOpacity)
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SBox)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8, 0, 24, 0)
			[
				SNew(STextBlock)
				.Text(this, &FMoviePipelineSettingTreeItem::GetDescription)
				//.TextStyle(FTakeRecorderStyle::Get(), "TakeRecorder.Source.Label")
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				.ColorAndOpacity(this, &FMoviePipelineSettingTreeItem::GetColorAndOpacity)
			]
		]

		+SOverlay::Slot()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SBox)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 4, 0)
			[
				SNew(STextBlock)
				.ToolTipText(this, &FMoviePipelineSettingTreeItem::GetSettingErrorTooltipText)
				.Visibility(this, &FMoviePipelineSettingTreeItem::GetSettingErrorVisibility)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FEditorFontGlyphs::Exclamation_Triangle)
				.ColorAndOpacity(FLinearColor::Yellow)
			]

		];

	}

private:
	FText GetDescription() const
	{
		UMoviePipelineSetting* Setting = WeakSetting.Get();
		return Setting ? Setting->GetDescriptionText() : FText();
	}

	ECheckBoxState GetCheckState() const
	{
		UMoviePipelineSetting* Setting = WeakSetting.Get();
		return Setting && Setting->IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void SetCheckState(const ECheckBoxState NewState, TWeakPtr<SMoviePipelineConfigSettings> WeakSettingsWidget)
	{
		const bool bEnable = NewState == ECheckBoxState::Checked;

		TSharedPtr<SMoviePipelineConfigSettings>		SettingsWidget = WeakSettingsWidget.Pin();
		UMoviePipelineSetting*            ThisSetting    = WeakSetting.Get();

		if (ThisSetting && SettingsWidget.IsValid())
		{
			TArray<UMoviePipelineSetting*> SelectedSettings;
			SettingsWidget->GetSelectedSettings(SelectedSettings);

			FText TransactionFormat = bEnable
				? LOCTEXT("EnableSetting", "Enable {0}|plural(one=Setting, other=Settings)")
				: LOCTEXT("DisableSetting", "Disable {0}|plural(one=Setting, other=Settings)");

			if (!SelectedSettings.Contains(ThisSetting))
			{
				FScopedTransaction Transaction(FText::Format(TransactionFormat, 1));

				ThisSetting->Modify();
				ThisSetting->SetIsEnabled(bEnable);
			}
			else 
			{
				FScopedTransaction Transaction(FText::Format(TransactionFormat, SelectedSettings.Num()));

				for (UMoviePipelineSetting* SelectedSetting : SelectedSettings)
				{
					SelectedSetting->Modify();
					SelectedSetting->SetIsEnabled(bEnable);
				}
			}
		}
	}

	EVisibility GetEnabledStateVisibility() const
	{
		UMoviePipelineSetting* Setting = WeakSetting.Get();
		return Setting && Setting->CanBeDisabled() ? EVisibility::Visible : EVisibility::Hidden;
	}

	FSlateColor GetColorAndOpacity() const
	{
		UMoviePipelineSetting* Setting = WeakSetting.Get();
		return Setting && Setting->IsEnabled() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground();
	}

	FSlateColor GetImageColorAndOpacity() const
	{
		UMoviePipelineSetting* Setting = WeakSetting.Get();
		return Setting && Setting->IsEnabled() ? FLinearColor::White : FLinearColor::White.CopyWithNewOpacity(0.3f);
	}

	FText GetSettingErrorTooltipText() const
	{
		UMoviePipelineSetting* Setting = WeakSetting.Get();
		if (Setting)
		{
			switch (Setting->GetValidationState())
			{
			case EMoviePipelineValidationState::Errors:
				return LOCTEXT("ValidationSettingError_Tooltip", "This setting has an invalid setting that must be corrected before attempting to render.");
			case EMoviePipelineValidationState::Warnings:
				return LOCTEXT("ValidationSettingWarning_Tooltip", "This setting is configured in a way that may produce undesired results.");
			}
		}

		return FText();
	}

	EVisibility GetSettingErrorVisibility() const
	{
		UMoviePipelineSetting* Setting = WeakSetting.Get();
		return Setting && (Setting->GetValidationState() != EMoviePipelineValidationState::Valid) ? EVisibility::Visible : EVisibility::Hidden;
	}
};

struct FMoviePipelineSettingCategory : IMoviePipelineSettingTreeItem
{
	/** The title of this category */
	FText Category;

	/** Sorted list of this category's children */
	TArray<TSharedPtr<FMoviePipelineSettingTreeItem>> Children;

	explicit FMoviePipelineSettingCategory(const FString& InCategory)
		: Category(FText::FromString(InCategory))
	{}

	virtual void Delete(UMoviePipelineConfigBase* Owner) override
	{
		for (TSharedPtr<FMoviePipelineSettingTreeItem> Child : Children)
		{
			Child->Delete(Owner);
		}
	}

	virtual TSharedPtr<FMoviePipelineSettingCategory> AsCategory()
	{
		return SharedThis(this);
	}

	virtual TSharedRef<SWidget> ConstructWidget(TWeakPtr<SMoviePipelineConfigSettings> SettingsWidget) override
	{
		return SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(20, 4)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &FMoviePipelineSettingCategory::GetLabel)
			];
	}

private:

	FText GetLabel() const
	{
		return FText::Format(LOCTEXT("CategoryFormatString", "{0}"),
			Category);
	}
};


void SMoviePipelineConfigSettings::Construct(const FArguments& InArgs)
{
	CachedSettingsSerialNumber = uint32(-1);

	TreeView = SNew(STreeView<TSharedPtr<IMoviePipelineSettingTreeItem>>)
		.TreeItemsSource(&RootNodes)
		.OnSelectionChanged(InArgs._OnSelectionChanged)
		.OnGenerateRow(this, &SMoviePipelineConfigSettings::OnGenerateRow)
		.OnGetChildren(this, &SMoviePipelineConfigSettings::OnGetChildren)
		.OnIsSelectableOrNavigable(this, &SMoviePipelineConfigSettings::IsSelectableOrNavigable) // Only works for Keyboard Navigation
		.ClearSelectionOnClick(false);

	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SMoviePipelineConfigSettings::OnDeleteSelected),
		FCanExecuteAction()
	);

	ChildSlot
	[
		TreeView.ToSharedRef()
	];

	// When undo occurs, get a notification so we can make sure our view is up to date
	GEditor->RegisterForUndo(this);
}

void SMoviePipelineConfigSettings::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UMoviePipelineConfigBase* ShotConfig = WeakShotConfig.Get();

	// If we have a settings ptr, we expect its serial number to match our cached one, if not, we rebuild the tree
	if (ShotConfig)
	{
		if (CachedSettingsSerialNumber != ShotConfig->GetSettingsSerialNumber())
		{
			ReconstructTree();
		}
	}
	// The settings are no longer valid, so we expect our cached serial number to be -1. If not, we haven't reset the tree yet.
	else if (CachedSettingsSerialNumber != uint32(-1))
	{
		ReconstructTree();
	}

	// One the tree has been reconstructed (if needed) see if there's any pending settings to try and select.
	if (PendingSettingsToSelect.Num() > 0)
	{
		SetSelectedSettings_Impl(PendingSettingsToSelect);
		PendingSettingsToSelect.Empty();
	}
}

FReply SMoviePipelineConfigSettings::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SMoviePipelineConfigSettings::GetSelectedSettings(TArray<UMoviePipelineSetting*>& OutSettings) const
{
	TArray<TSharedPtr<IMoviePipelineSettingTreeItem>> SelectedItems;

	TreeView->GetSelectedItems(SelectedItems);
	for (TSharedPtr<IMoviePipelineSettingTreeItem> Item : SelectedItems)
	{
		TSharedPtr<FMoviePipelineSettingTreeItem> SettingItem = Item->AsSetting();
		UMoviePipelineSetting* SettingPtr = SettingItem.IsValid() ? SettingItem->WeakSetting.Get() : nullptr;
		if (SettingPtr)
		{
			OutSettings.Add(SettingPtr);
		}
	}
}

void SMoviePipelineConfigSettings::SetSelectedSettings(const TArray<UMoviePipelineSetting*>& Settings)
{
	PendingSettingsToSelect = Settings;
}

void SMoviePipelineConfigSettings::SetSelectedSettings_Impl(const TArray<UMoviePipelineSetting*>& Settings)
{
	TreeView->ClearSelection();

	TArray<TSharedPtr<IMoviePipelineSettingTreeItem>> AllSettingTreeItems;

	// Get all of our items first
	for (TSharedPtr<IMoviePipelineSettingTreeItem> Item : RootNodes)
	{
		AllSettingTreeItems.Add(Item);
		OnGetChildren(Item, AllSettingTreeItems);
	}


	TArray<TSharedPtr<IMoviePipelineSettingTreeItem>> SelectedTreeItems;
	for (TSharedPtr<IMoviePipelineSettingTreeItem> Item : AllSettingTreeItems)
	{
		TSharedPtr<FMoviePipelineSettingTreeItem> SettingTreeItem = Item->AsSetting();
		if (SettingTreeItem.IsValid())
		{
			if (Settings.Contains(SettingTreeItem->WeakSetting.Get()))
			{
				SelectedTreeItems.Add(Item);
			}
		}
	}

	TreeView->SetItemSelection(SelectedTreeItems, true, ESelectInfo::Direct);
}

void SMoviePipelineConfigSettings::SetShotConfigObject(UMoviePipelineConfigBase* InShotConfig)
{
	WeakShotConfig = InShotConfig;
	ReconstructTree();
}

void SMoviePipelineConfigSettings::PostUndo(bool bSuccess)
{
	CachedSettingsSerialNumber++;
}


void SMoviePipelineConfigSettings::ReconstructTree()
{
	UMoviePipelineConfigBase* ShotConfig = WeakShotConfig.Get();
	if (!ShotConfig)
	{
		CachedSettingsSerialNumber = uint32(-1);
		RootNodes.Reset();
		return;
	}

	CachedSettingsSerialNumber = ShotConfig->GetSettingsSerialNumber();

	TSortedMap<FString, TSharedPtr<FMoviePipelineSettingCategory>> RootCategories;
	for (TSharedPtr<IMoviePipelineSettingTreeItem> RootItem : RootNodes)
	{
		TSharedPtr<FMoviePipelineSettingCategory> RootCategory = RootItem->AsCategory();
		if (RootCategory.IsValid())
		{
			RootCategory->Children.Reset();
			RootCategories.Add(RootCategory->Category.ToString(), RootCategory);
		}
	}

	RootNodes.Reset();

	// We attempt to re-use tree items in order to maintain selection states on them
	TMap<FObjectKey, TSharedPtr<FMoviePipelineSettingTreeItem>> OldSettingToTreeItem;
	Swap(SettingToTreeItem, OldSettingToTreeItem);


	static const FName CategoryName = "Category";

	for (UMoviePipelineSetting* Setting : ShotConfig->GetUserSettings())
	{
		if (!Setting)
		{
			continue;
		}

		// The category in the UI is taken from the class itself
		FString Category = Setting->GetCategoryText().ToString();
		if (Category.IsEmpty())
		{
			Category = Setting->GetClass()->GetMetaData(CategoryName);
		}

		// Attempt to find an existing category node, creating one if necessary
		TSharedPtr<FMoviePipelineSettingCategory> CategoryNode = RootCategories.FindRef(Category);
		if (!CategoryNode.IsValid())
		{
			CategoryNode = RootCategories.Add(Category, MakeShared<FMoviePipelineSettingCategory>(Category));

			TreeView->SetItemExpansion(CategoryNode, true);
		}

		// Attempt to find an existing setting item node from the previous data, creating one if necessary
		FObjectKey SettingKey(Setting);
		TSharedPtr<FMoviePipelineSettingTreeItem> SettingItem = SettingToTreeItem.FindRef(SettingKey);
		if (!SettingItem.IsValid())
		{
			SettingItem = OldSettingToTreeItem.FindRef(Setting);
			if (SettingItem.IsValid())
			{
				SettingToTreeItem.Add(SettingKey, SettingItem);
			}
			else
			{
				SettingItem = MakeShared<FMoviePipelineSettingTreeItem>(Setting);
				SettingToTreeItem.Add(SettingKey, SettingItem);

				TreeView->SetItemExpansion(SettingItem, true);
			}
		}

		check(SettingItem.IsValid());
		CategoryNode->Children.Add(SettingItem);
	}


	RootNodes.Reset(RootCategories.Num());
	for(TTuple<FString, TSharedPtr<FMoviePipelineSettingCategory>>& Pair : RootCategories)
	{
		if (Pair.Value->Children.Num() == 0)
		{
			continue;
		}

		// Sort children by name. Work with a tuple of index and string to avoid excessively calling GetLabel().ToString()
		TArray<TTuple<int32, FString>> SortData;
		SortData.Reserve(Pair.Value->Children.Num());
		for (TSharedPtr<FMoviePipelineSettingTreeItem> Item : Pair.Value->Children)
		{
			SortData.Add(MakeTuple(SortData.Num(), Item->GetLabel().ToString()));
		}

		auto SortPredicate = [](TTuple<int32, FString>& A, TTuple<int32, FString>& B)
		{
			return A.Get<1>() < B.Get<1>();
		};

		Algo::Sort(SortData, SortPredicate);

		// Create a new sorted list of the child entries
		TArray<TSharedPtr<FMoviePipelineSettingTreeItem>> NewChildren;
		NewChildren.Reserve(SortData.Num());
		for (const TTuple<int32, FString>& Item : SortData)
		{
			NewChildren.Add(Pair.Value->Children[Item.Get<0>()]);
		}

		Swap(Pair.Value->Children, NewChildren);

		// Add the category
		RootNodes.Add(Pair.Value);
	}

	TreeView->RequestTreeRefresh();
}

TSharedRef<ITableRow> SMoviePipelineConfigSettings::OnGenerateRow(TSharedPtr<IMoviePipelineSettingTreeItem> Item, const TSharedRef<STableViewBase>& Tree)
{
	return
		SNew(STableRow<TSharedPtr<IMoviePipelineSettingTreeItem>>, Tree)
		[
			Item->ConstructWidget(SharedThis(this))
		];
}

void SMoviePipelineConfigSettings::OnGetChildren(TSharedPtr<IMoviePipelineSettingTreeItem> Item, TArray<TSharedPtr<IMoviePipelineSettingTreeItem>>& OutChildItems)
{
	TSharedPtr<FMoviePipelineSettingCategory> Category = Item->AsCategory();
	if (Category.IsValid())
	{
		OutChildItems.Append(Category->Children);
	}
}

bool SMoviePipelineConfigSettings::IsSelectableOrNavigable(TSharedPtr<IMoviePipelineSettingTreeItem> Item) const
{
	return !Item->AsCategory().IsValid();
}

void SMoviePipelineConfigSettings::OnDeleteSelected()
{
	UMoviePipelineConfigBase* ShotConfig = WeakShotConfig.Get();
	if (ShotConfig)
	{
		TArray<TSharedPtr<IMoviePipelineSettingTreeItem>> Items = TreeView->GetSelectedItems();

		FScopedTransaction Transaction(FText::Format(LOCTEXT("DeleteSelection", "Delete Selected {0}|plural(one=Setting, other=Settings)"), Items.Num()));
		ShotConfig->Modify();

		for (TSharedPtr<IMoviePipelineSettingTreeItem> Item : Items)
		{
			Item->Delete(ShotConfig);
		}
	}
}

#undef LOCTEXT_NAMESPACE