// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/ObjectKey.h"
#include "Styling/SlateTypes.h" 
#include "EditorUndoClient.h"

class ITableRow;
class FUICommandList;
class STableViewBase;
class UMoviePipelineSetting;
class UMoviePipelineConfigBase;
struct IMoviePipelineSettingTreeItem;
struct FMoviePipelineSettingTreeItem;
template<typename> class STreeView;

DECLARE_DELEGATE_TwoParams(
	FOnSettingSelectionChanged,
	TSharedPtr<IMoviePipelineSettingTreeItem>,
	ESelectInfo::Type
)

/** Main widget for the Movie Render Shot Config panel */
class SMoviePipelineConfigSettings : public SCompoundWidget, public FEditorUndoClient
{
public:

	SLATE_BEGIN_ARGS(SMoviePipelineConfigSettings){}
		SLATE_EVENT(FOnSettingSelectionChanged, OnSelectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetShotConfigObject(UMoviePipelineConfigBase* InShotConfig);

	void GetSelectedSettings(TArray<UMoviePipelineSetting*>& OutSettings) const;
	void SetSelectedSettings(const TArray<UMoviePipelineSetting*>& Settings);
	void InvalidateCachedSettingsSerialNumber() { CachedSettingsSerialNumber = -1; }
private:

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	void SetSelectedSettings_Impl(const TArray<UMoviePipelineSetting*>& Settings);

	// FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// ~FEditorUndoClient

private:

	void ReconstructTree();

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<IMoviePipelineSettingTreeItem> Item, const TSharedRef<STableViewBase>& Tree);

	void OnGetChildren(TSharedPtr<IMoviePipelineSettingTreeItem> Item, TArray<TSharedPtr<IMoviePipelineSettingTreeItem>>& OutChildItems);

	void OnDeleteSelected();
	bool IsSelectableOrNavigable(TSharedPtr<IMoviePipelineSettingTreeItem> Item) const;

private:
	TArray<UMoviePipelineSetting*> PendingSettingsToSelect;

	uint32 CachedSettingsSerialNumber;
	TWeakObjectPtr<UMoviePipelineConfigBase> WeakShotConfig;

	TArray<TSharedPtr<IMoviePipelineSettingTreeItem>> RootNodes;
	TMap<FObjectKey, TSharedPtr<FMoviePipelineSettingTreeItem>> SettingToTreeItem;
	TSharedPtr<STreeView<TSharedPtr<IMoviePipelineSettingTreeItem>>> TreeView;

	TSharedPtr<FUICommandList> CommandList;
};