// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMoviePipelineConfigTabContent.h"
#include "Widgets/SMoviePipelineConfigPanel.h"


// Analytics
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "MoviePipelineConfigBase.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineTabContent"


void SMoviePipelineConfigTabContent::Construct(const FArguments& InArgs)
{    
	// Delay one tick before opening the default pipeline setup panel.
	// this allows anything that just invoked the tab to customize it without the default UI being created
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SMoviePipelineConfigTabContent::OnActiveTimer));
     
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MoviePipeline.PanelOpened"));
	}
}

EActiveTimerReturnType SMoviePipelineConfigTabContent::OnActiveTimer(double InCurrentTime, float InDeltaTime)
{
	SetupForPipeline((UMoviePipelineConfigBase*)nullptr);
	return EActiveTimerReturnType::Stop;
}

void SMoviePipelineConfigTabContent::SetupForPipeline(UMoviePipelineConfigBase* BasePreset)
{
	// Null out the tab content to ensure that all references have been cleaned up before constructing the new one
	ChildSlot [ SNullWidget::NullWidget ];

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SAssignNew(WeakPanel, SMoviePipelineConfigPanel, UMoviePipelineConfigBase::StaticClass())
		.BasePreset(BasePreset)
	];

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MoviePipeline.SetupForPipelineFromPreset"));
	}
}

#undef LOCTEXT_NAMESPACE // SMoviePipelineTabContent
