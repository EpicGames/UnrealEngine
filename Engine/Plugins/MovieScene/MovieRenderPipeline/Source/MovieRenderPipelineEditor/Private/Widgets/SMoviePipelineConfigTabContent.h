// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UMoviePipelineConfigBase;
class SMoviePipelineConfigPanel;

class SMoviePipelineConfigTabContent : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMoviePipelineConfigTabContent){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Set this tab up for a specific Pipeline preset. Passing nullptr will initialize a default pipeline. */
	void SetupForPipeline(UMoviePipelineConfigBase* BasePreset);

private:
	EActiveTimerReturnType OnActiveTimer(double InCurrentTime, float InDeltaTime);

private:
	TWeakPtr<SMoviePipelineConfigPanel> WeakPanel;
};
