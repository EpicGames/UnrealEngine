// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMoviePipelineFormatTokenAutoCompleteBox.h"

#include "DetailLayoutBuilder.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphFilenameResolveParams.h"
#include "Layout/WidgetPath.h"

void SMoviePipelineFormatTokenAutoCompleteBox::Construct(const FArguments& InArgs)
{
	TextHandle = InArgs._TextHandle;
	
	ChildSlot
	[
		SAssignNew(MenuAnchor, SMenuAnchor)
		.Placement(MenuPlacement_ComboBox)
		[
			SAssignNew(TextBox, SMultiLineEditableTextBox)
			.Text_Lambda([this]()
			{
				FString TextValue;
				TextHandle->GetValue(TextValue);
				
				return FText::FromString(TextValue);
			})
			.HintText(InArgs._HintText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.OnKeyDownHandler(this, &SMoviePipelineFormatTokenAutoCompleteBox::OnKeyDown)
			.OnTextChanged(this, &SMoviePipelineFormatTokenAutoCompleteBox::HandleTextBoxTextChanged)
			.SelectWordOnMouseDoubleClick(true)
			.AllowMultiLine(false)
			.IsEnabled(InArgs._IsEnabled)
		]
		.MenuContent
		(
			SNew(SBorder)
			.Padding(FMargin(2))
			[
				SAssignNew(VerticalBox, SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(SuggestionListView, SListView<TSharedPtr<FString>>)
					.ListItemsSource(&Suggestions)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow(this, &SMoviePipelineFormatTokenAutoCompleteBox::HandleSuggestionListViewGenerateRow)
					.OnMouseButtonClick(this, &SMoviePipelineFormatTokenAutoCompleteBox::OnItemClicked)
				]
			]
		)
	];

	// We just call it once and cache it for now as the selection code isn't tested against
	// the amount of suggestions changing.
	AllSuggestions.Append(InArgs._Suggestions.Get());
}

void SMoviePipelineFormatTokenAutoCompleteBox::OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
	// If the suggestion list view gets focus when the autocomplete was previously focused, do not process the focus change. This will cause a
	// commit, which we don't want yet.
	if (NewWidgetPath.ContainsWidget(SuggestionListView.Get()))
	{
		return;
	}

	// Close the suggestion list if the text box has lost focus to anything other than the suggestion list
	if (PreviousFocusPath.ContainsWidget(TextBox.Get()) && !NewWidgetPath.ContainsWidget(SuggestionListView.Get()) && MenuAnchor->IsOpen())
	{
		CloseMenuAndReset();
	}

	// If the autocomplete loses focus, commit so an undo entry is created. This is generally only important if the user is typing in something
	// manually and not choosing an entry from the autocomplete (choosing an entry will cause a commit).
	FString TextValue;
	TextHandle->GetValue(TextValue);
	HandleTextBoxTextCommitted(FText::FromString(TextValue), ETextCommit::Default);
	
	SWidget::OnFocusChanging(PreviousFocusPath, NewWidgetPath, InFocusEvent);
}

FReply SMoviePipelineFormatTokenAutoCompleteBox::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent)
{
	if (MenuAnchor->IsOpen())
	{
		if (KeyEvent.GetKey() == EKeys::Up)
		{
			// Because the pop-up dialog is below the text, 'up' actually goes to an earlier item in the list.
			int32 NewSuggestionIndex = CurrentSuggestionIndex - 1;
			if (NewSuggestionIndex < 0)
			{
				NewSuggestionIndex = Suggestions.Num() - 1;
			}

			SetActiveSuggestionIndex(NewSuggestionIndex);
			return FReply::Handled();
		}
		else if(KeyEvent.GetKey() == EKeys::Down)
		{
			int32 NewSuggestionIndex = CurrentSuggestionIndex + 1;
			if (NewSuggestionIndex > Suggestions.Num() - 1)
			{
				NewSuggestionIndex = 0;
			}

			SetActiveSuggestionIndex(NewSuggestionIndex);
			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Escape)
		{
			CloseMenuAndReset();
			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Tab || KeyEvent.GetKey() == EKeys::Enter)
		{
			if(CurrentSuggestionIndex >= 0 && CurrentSuggestionIndex <= Suggestions.Num() - 1)
			{
				// Trigger the auto-complete for the highlighted suggestion
				const FString SuggestionText = *Suggestions[CurrentSuggestionIndex];
				ReplaceRelevantTextWithSuggestion(SuggestionText);
				CloseMenuAndReset();
				return FReply::Handled();
			}
		}
	}
	else if ((KeyEvent.GetKey() == EKeys::LeftBracket) && KeyEvent.IsShiftDown())
	{
		// Start showing token suggestions immediately when { is typed
		FilterVisibleSuggestions(FString(), false);
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

void SMoviePipelineFormatTokenAutoCompleteBox::SetText(const FText& InText)
{
	// SetText always sets the cursor location to the end of the line, so we need to cache and restore the location
	const FTextLocation OriginalCursorLocation = TextBox->GetCursorLocation();
	TextBox->SetText(InText);
	TextBox->GoTo(OriginalCursorLocation);
}

void SMoviePipelineFormatTokenAutoCompleteBox::OnItemClicked(TSharedPtr<FString> Item)
{
	ReplaceRelevantTextWithSuggestion(*Item);
	CloseMenuAndReset();
}

void SMoviePipelineFormatTokenAutoCompleteBox::FindAutoCompletableTextAtPos(const FString& InWholeString, int32 InCursorPos, FString& OutStr, bool& bShowAutoComplete)
{
	OutStr = FString();
	bShowAutoComplete = false;

	int32 StartingBracePos, EndBracePos;
	GetBracePositionsForCursor(InWholeString, InCursorPos, StartingBracePos, EndBracePos);

	FString AutoCompleteText;

	// Now that we found a {, take the substring between it and either the next } or {, or the end of the string.
	if (StartingBracePos >= 0)
	{
		int32 Count = InWholeString.Len() - StartingBracePos;
		if (EndBracePos >= 0)
		{
			Count = EndBracePos - StartingBracePos;
		}

		AutoCompleteText = InWholeString.Mid(StartingBracePos + 1, Count - 1);
	}

	OutStr = AutoCompleteText;
	bShowAutoComplete = StartingBracePos >= 0 && OutStr.Len() == 0;
}

void SMoviePipelineFormatTokenAutoCompleteBox::ReplaceRelevantTextWithSuggestion(const FString& InSuggestionText)
{
	FString TextBoxText = TextBox->GetText().ToString();
	int32 CursorPos = TextBoxText.Len();
	const FTextLocation CursorLoc = TextBox->GetCursorLocation();
	if (CursorLoc.IsValid())
	{
		CursorPos = FMath::Clamp(CursorLoc.GetOffset(), 0, CursorPos);
	}

	int32 StartingBracePos, EndBracePos;
	GetBracePositionsForCursor(TextBoxText, CursorPos, StartingBracePos, EndBracePos);

	// Insert the suggestion text after the opening brace, and before the ending brace (if any). Replace any text that currently exists between the
	// braces with the suggestion text.
	int32 NewCursorPos = 0;
	if (StartingBracePos >= 0)	
	{
		// +1 to keep the left { brace
		const FString Left = TextBoxText.Left(StartingBracePos+1);
		FString Right;

		if (EndBracePos >= 0)
		{
			Right = TextBoxText.RightChop(EndBracePos);
		}

		// Since the user chose the suggestion ensure there's already a } brace to close off the pair.
		if (!Right.StartsWith(TEXT("}")))
		{
			Right = FString::Printf(TEXT("}%s"), *Right);
		}

		TextBoxText = Left + InSuggestionText + Right;
		
		// We subtract 1 from the Right as we want to put the cursor after the automatically generated "}" token.
		NewCursorPos = TextBoxText.Len() - (Right.Len() - 1);
	}

	// Once the text replacement has been made, commit it so an undo entry is made
	HandleTextBoxTextCommitted(FText::FromString(TextBoxText), ETextCommit::Default);

	// Seemingly due to some focus event oddities, we need to manually refresh the text box after it regains focus from the suggestion list. Text will
	// be committed at this point, but the text box may not show the update.
	TextBox->Refresh();
	
	TextBox->GoTo(FTextLocation(0, NewCursorPos));
}

void SMoviePipelineFormatTokenAutoCompleteBox::HandleTextBoxTextChanged(const FText& InText)
{
	TextHandle->SetValue(InText.ToString(), EPropertyValueSetFlags::InteractiveChange);

	const FString TextAsStr = InText.ToString();
	int32 CursorPos = TextAsStr.Len();
	const FTextLocation CursorLoc = TextBox->GetCursorLocation();
	if (CursorLoc.IsValid())
	{
		CursorPos = FMath::Clamp(CursorLoc.GetOffset(), 0, CursorPos);
	}

	FString OutStr;
	bool bShowAutoComplete;
	FindAutoCompletableTextAtPos(TextAsStr, CursorPos, OutStr, bShowAutoComplete);
	FilterVisibleSuggestions(OutStr, bShowAutoComplete);
}

void SMoviePipelineFormatTokenAutoCompleteBox::HandleTextBoxTextCommitted(const FText& InText, ETextCommit::Type CommitInfo) const
{
	TextHandle->SetValue(InText.ToString(), EPropertyValueSetFlags::DefaultFlags);
}

void SMoviePipelineFormatTokenAutoCompleteBox::FilterVisibleSuggestions(const FString& StrToMatch, const bool bForceShowAll)
{
	// If the { is not immediately before the cursor and the suggestion string is empty (ie, the user has not started typing in a format token name),
	// do not show the autocomplete menu at all. This is something that mostly occurs when the text box is empty.
	if (StrToMatch.IsEmpty())
	{
		bool bForceHideAutocomplete = false;
		
		const FString TextBoxString = TextBox->GetText().ToString();
		const int32 CursorLocation = TextBox->GetCursorLocation().GetOffset();
		if (TextBoxString.IsValidIndex(CursorLocation - 1))
		{
			const TCHAR& CharBeforeCursor = TextBoxString[CursorLocation - 1];
			if (CharBeforeCursor != '{')
			{
				bForceHideAutocomplete = true;
			}
		}
		else
		{
			// Could happen if the cursor is at the very start of the text box
			bForceHideAutocomplete = true;
		}

		if (bForceHideAutocomplete)
		{
			CloseMenuAndReset();
			return;
		}
	}
	
	Suggestions.Reset();
	for (const FString& Suggestion : AllSuggestions)
	{
		if (Suggestion.Contains(StrToMatch) || bForceShowAll)
		{
			Suggestions.Add(MakeShared<FString>(Suggestion));
		}
	}

	if (Suggestions.Num() > 0)
	{
		// We don't focus the menu (because then you can't type on the keyboard) and instead
		// keep the focus on the text field and bubble the keyboard commands to it.
		constexpr bool bIsOpen = true;
		constexpr bool bFocusMenu = false;
		MenuAnchor->SetIsOpen(bIsOpen, bFocusMenu);
		SuggestionListView->RequestScrollIntoView(Suggestions[0]);
	}
	else
	{
		CloseMenuAndReset();
	}
}

void SMoviePipelineFormatTokenAutoCompleteBox::CloseMenuAndReset()
{
	constexpr bool bIsOpen = false;
	MenuAnchor->SetIsOpen(bIsOpen);

	// Reset their index when the drawer closes so that the first item is always selected when we re-open.
	CurrentSuggestionIndex = -1;
}

void SMoviePipelineFormatTokenAutoCompleteBox::SetActiveSuggestionIndex(int32 InIndex)
{
	if (InIndex < 0 || InIndex >= Suggestions.Num())
	{
		return;
	}

	const TSharedPtr<FString> Suggestion = Suggestions[InIndex];
	SuggestionListView->SetSelection(Suggestion);
	if (!SuggestionListView->IsItemVisible(Suggestion))
	{
		SuggestionListView->RequestScrollIntoView(Suggestion);
	}
	CurrentSuggestionIndex = InIndex;
}

TSharedRef<ITableRow> SMoviePipelineFormatTokenAutoCompleteBox::HandleSuggestionListViewGenerateRow(
	TSharedPtr<FString> Text, const TSharedRef<STableViewBase>& OwnerTable) const
{
	const FString SuggestionText = *Text;

	return SNew(STableRow<TSharedPtr<FString> >, OwnerTable)
		[
			SNew(SBox)
			[
				SNew(STextBlock)
				.Text(FText::FromString(SuggestionText))
			]
		];
}

TArray<FString> SMoviePipelineFormatTokenAutoCompleteBox::GetFileNameFormatSuggestions()
{
	TArray<FString> FileNameFormatSuggestions;
	
	// Just fetch the format arguments (by keeping the format string empty). The tokens themselves will not be resolved correctly here (no context is
	// provided in the resolve params), but all we care about here is the token list, not the resolved token values.
	const FString FormatString;
	const FMovieGraphFilenameResolveParams ResolveParams;
	FMovieGraphResolveArgs FormatArgs;
	UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FormatString, ResolveParams, FormatArgs);

	// Display the token names alphabetically
	FormatArgs.FilenameArguments.GetKeys(FileNameFormatSuggestions);
	FileNameFormatSuggestions.Sort();

	return FileNameFormatSuggestions;
}

void SMoviePipelineFormatTokenAutoCompleteBox::GetBracePositionsForCursor(const FString& InText, int32 CursorPos, int32& OutStartingBracePos, int32& OutEndBracePos)
{
	OutStartingBracePos = InText.Find(
		TEXT("{"), ESearchCase::Type::IgnoreCase, ESearchDir::Type::FromEnd, CursorPos);
	
	const int32 NextStartingBracePos = InText.Find(
		TEXT("{"), ESearchCase::Type::IgnoreCase, ESearchDir::Type::FromStart, CursorPos);
	const int32 EndBracePos = InText.Find(
		TEXT("}"), ESearchCase::Type::IgnoreCase, ESearchDir::Type::FromStart, CursorPos);

	// Use the position of the next { if it comes before the next }. This could happen if, for example, the user has a format string like {layer_name},
	// and they want to prepend a format token to the front, so the text would look like {{layer_name}, and the cursor is right after the initial {.
	OutEndBracePos = (NextStartingBracePos != -1) && (NextStartingBracePos < EndBracePos) ? NextStartingBracePos : EndBracePos;
}