// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PropertyHandle.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Views/SListView.h"

/** This is a specific auto-complete widget made for the MoviePipeline that isn't very flexible. It's
* similar to SSuggestionTextBox, but SSuggestionTextBox doesn't handle more than one word/suggestions
* mid string. This widget is hardcoded to look for '{' characters (for {format_tokens}) and then auto
* completes them from a list and fixes up the {} braces.
*/
class SMoviePipelineFormatTokenAutoCompleteBox : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMoviePipelineFormatTokenAutoCompleteBox){}

	SLATE_ARGUMENT(FText, InitialText)
	SLATE_ARGUMENT(FText, HintText)
	SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, TextHandle)
	SLATE_ATTRIBUTE(TArray<FString>, Suggestions)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// SWidget interface
	virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent) override;
	// End of SWidget interface

	void SetText(const FText& InText);

	void OnItemClicked(TSharedPtr<FString>);

	static void FindAutoCompletableTextAtPos(const FString& InWholeString, int32 InCursorPos, FString& OutStr, bool& bShowAutoComplete);

	void ReplaceRelevantTextWithSuggestion(const FString& InSuggestionText);

	void HandleTextBoxTextChanged(const FText& InText);

	void HandleTextBoxTextCommitted(const FText& InText, ETextCommit::Type CommitInfo) const;

	void FilterVisibleSuggestions(const FString& StrToMatch, const bool bForceShowAll);

	void CloseMenuAndReset();

	void SetActiveSuggestionIndex(int32 InIndex);

	TSharedRef<ITableRow> HandleSuggestionListViewGenerateRow(TSharedPtr<FString> Text, const TSharedRef<STableViewBase>& OwnerTable) const;

	/**
	 * A helper to get file name format suggestions. Can be passed to the "Suggestions" argument.
	 */
	static TArray<FString> GetFileNameFormatSuggestions();

private:
	/** Get the relevant brace positions for the given cursor position within the text. */
	static void GetBracePositionsForCursor(const FString& InText, int32 CursorPos, int32& OutStartingBracePos, int32& OutEndBracePos);

private:
	TSharedPtr<SListView<TSharedPtr<FString>>> SuggestionListView;
	TSharedPtr<SMultiLineEditableTextBox> TextBox;
	TSharedPtr<SMenuAnchor> MenuAnchor;
	TSharedPtr<SVerticalBox> VerticalBox;
	TSharedPtr<IPropertyHandle> TextHandle;

	// The pool of suggestions to show
	TArray<FString> AllSuggestions;
	// The currently filtered suggestion list
	TArray<TSharedPtr<FString>> Suggestions;
	int32 CurrentSuggestionIndex = -1;
};
