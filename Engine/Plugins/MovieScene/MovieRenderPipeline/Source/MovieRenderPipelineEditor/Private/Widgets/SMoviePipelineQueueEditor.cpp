// Copyright Epic Games, Inc. All Rights Reserved.

// Movie Pipeline
#include "Widgets/SMoviePipelineQueueEditor.h"
#include "Widgets/MoviePipelineWidgetConstants.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineBasicConfig.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineQueueSubsystem.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MovieRenderPipelineStyle.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "MoviePipelineCommands.h"
#include "MoviePipelineEditorBlueprintLibrary.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphConfigFactory.h"

// Slate Includes
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Input/SHyperlink.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"
#include "SDropTarget.h"
#include "SPositiveActionButton.h"

// Editor
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "ScopedTransaction.h"
#include "PropertyCustomizationHelpers.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Commands/GenericCommands.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "AssetRegistry/AssetData.h"

// Misc
#include "AssetToolsModule.h"
#include "LevelSequence.h"
#include "Engine/EngineTypes.h"
#include "Framework/Application/SlateApplication.h"
#include "SMoviePipelineConfigPanel.h"
#include "Graph/MovieGraphPipeline.h"
#include "Widgets/SWindow.h"
#include "HAL/FileManager.h"
#include "Widgets/Layout/SBox.h"


#define LOCTEXT_NAMESPACE "SMoviePipelineQueueEditor"

struct FMoviePipelineQueueJobTreeItem;
struct FMoviePipelineMapTreeItem;
struct FMoviePipelineShotItem;
class SQueueJobListRow;

namespace UE::MovieGraph::Private
{
	/** Returns a new saved UMovieGraphConfig if one could be created, else nullptr. */
	UMovieGraphConfig* CreateNewSavedGraphAsset()
	{
		if (UMovieGraphConfigFactory* GraphFactory = NewObject<UMovieGraphConfigFactory>())
		{
			// Make the new graph via save dialog
			const FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
			UObject* NewAsset = AssetToolsModule.Get().CreateAssetWithDialog(GraphFactory->GetSupportedClass(), GraphFactory);

			// Don't ensure here because a "cancel" in the dialog can cause the returned asset to be null
			if (UMovieGraphConfig* NewGraph = Cast<UMovieGraphConfig>(NewAsset))
			{
				return NewGraph;
			}
		}

		return nullptr;
	}
}

struct IMoviePipelineQueueTreeItem : TSharedFromThis<IMoviePipelineQueueTreeItem>
{
	virtual ~IMoviePipelineQueueTreeItem() {}

	virtual TSharedPtr<FMoviePipelineQueueJobTreeItem> AsJob() { return nullptr; }
	virtual TSharedPtr<FMoviePipelineShotItem> AsShot() { return nullptr; }
	virtual UMoviePipelineExecutorJob* GetOwningJob() { return nullptr; }
	virtual UMoviePipelineExecutorShot* GetOwningShot() { return nullptr; }
	virtual void Delete(UMoviePipelineQueue* InOwningQueue) {}
	virtual void ResetStatus() {}
	virtual UMoviePipelineExecutorJob* Duplicate(UMoviePipelineQueue* InOwningQueue) { return nullptr; }

	virtual TSharedRef<ITableRow> ConstructWidget(TWeakPtr<SMoviePipelineQueueEditor> InQueueWidget, const TSharedRef<STableViewBase>& OwnerTable) = 0;	

protected:
	TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> GetMultiSelectAffectedItems(TWeakPtr<SMoviePipelineQueueEditor> WeakQueueEditor, bool bIsShot) const
	{
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems;

		TSharedPtr<SMoviePipelineQueueEditor> QueueEditor = WeakQueueEditor.Pin();
		if (!QueueEditor)
		{
			return AffectedItems;
		}
		
		// We always want to apply to ourselves
		TSharedPtr<IMoviePipelineQueueTreeItem> NonConstSelf = ConstCastSharedRef<IMoviePipelineQueueTreeItem>(AsShared());
		AffectedItems.Add(NonConstSelf);

		// If the item you clicked on (the one running this code) isn't part of the multi-selected items then we only apply the requested change
		// to the current item itself.
		if (!QueueEditor->GetSelectedItems().Contains(AsShared()))
		{
			return AffectedItems;
		}

		// Otherwise we add all the other items from the selection (of the appropriate type)
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : QueueEditor->GetSelectedItems())
		{
			if (Item->AsJob() && !bIsShot)
			{
				AffectedItems.Add(Item);
			}
			else if (Item->AsShot() && bIsShot)
			{
				AffectedItems.Add(Item);
			}
		}

		return AffectedItems;
	}
};

class SQueueJobListRow : public SMultiColumnTableRow<TSharedPtr<IMoviePipelineQueueTreeItem>>
{
public:
	SLATE_BEGIN_ARGS(SQueueJobListRow) {}
		SLATE_ARGUMENT(TSharedPtr<FMoviePipelineQueueJobTreeItem>, Item)
		SLATE_EVENT(FOnMoviePipelineEditConfig, OnEditConfigRequested)
	SLATE_END_ARGS()

	static const FName NAME_Enabled;
	static const FName NAME_JobName;
	static const FName NAME_Settings;
	static const FName NAME_Output;
	static const FName NAME_Status;

	TSharedPtr<FMoviePipelineQueueJobTreeItem> Item;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

protected:
	FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);
	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<IMoviePipelineQueueTreeItem> InItem);
	FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<IMoviePipelineQueueTreeItem> InItem);

private:
	FOnMoviePipelineEditConfig OnEditConfigRequested;
};

struct FMoviePipelineQueueJobTreeItem : IMoviePipelineQueueTreeItem
{
	/** The job that this tree item represents */
	TWeakObjectPtr<UMoviePipelineExecutorJob> WeakJob;

	TWeakPtr<SMoviePipelineQueueEditor> WeakQueueEditor;

	/** Sorted list of this category's children */
	TArray<TSharedPtr<IMoviePipelineQueueTreeItem>> Children;

	FOnMoviePipelineEditConfig OnEditConfigCallback;
	FOnMoviePipelineEditConfig OnChosePresetCallback;

	explicit FMoviePipelineQueueJobTreeItem(UMoviePipelineExecutorJob* InJob, FOnMoviePipelineEditConfig InOnEditConfigCallback, FOnMoviePipelineEditConfig InOnChosePresetCallback)
		: WeakJob(InJob)
		, OnEditConfigCallback(InOnEditConfigCallback)
		, OnChosePresetCallback(InOnChosePresetCallback)
	{}

	virtual TSharedRef<ITableRow> ConstructWidget(TWeakPtr<SMoviePipelineQueueEditor> InQueueWidget, const TSharedRef<STableViewBase>& OwnerTable) override
	{
		WeakQueueEditor = InQueueWidget;

		return SNew(SQueueJobListRow, OwnerTable)
			.Item(SharedThis(this));
	}

	virtual TSharedPtr<FMoviePipelineQueueJobTreeItem> AsJob() override
	{
		return SharedThis(this);
	}

	virtual UMoviePipelineExecutorJob* GetOwningJob() override
	{ 
		return WeakJob.Get();
	}

	virtual void Delete(UMoviePipelineQueue* InOwningQueue) override
	{
		InOwningQueue->DeleteJob(WeakJob.Get());
	}

	virtual UMoviePipelineExecutorJob* Duplicate(UMoviePipelineQueue* InOwningQueue) override
	{
		return InOwningQueue->DuplicateJob(WeakJob.Get());
	}

	virtual void ResetStatus() override
	{
		if (WeakJob.Get())
		{
			WeakJob->SetConsumed(false);
		}
	}

public:
	ECheckBoxState GetCheckState() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		if (Job)
		{
			return Job->IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Unchecked;
	}

	void SetCheckState(const ECheckBoxState NewState)
	{
		const bool bIsShot = false;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);
		
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineQueueJobTreeItem> JobTreeItem = Item->AsJob();
			if (JobTreeItem.IsValid())
			{
				if (UMoviePipelineExecutorJob* Job = JobTreeItem->WeakJob.Get())
				{
					Job->SetIsEnabled(NewState == ECheckBoxState::Checked);
				}
			}
		}
	}

	FText GetJobName() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		if (Job)
		{
			return FText::FromString(Job->JobName);
		}

		return FText();
	}

	FText GetPrimaryConfigLabel() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		if (Job)
		{
			if (Job->IsUsingBasicConfiguration())
			{
				return LOCTEXT("BasicConfigLabel", "Basic");
			}

			if (Job->IsUsingGraphConfiguration())
			{
				if (Job->GetGraphPreset())
				{
					return FText::FromString(Job->GetGraphPreset()->GetName());
				}
				else
				{
					return LOCTEXT("QueueEditorDefaultJobGraph_Text", "Default Graph");
				}
			}

			// If the job has a preset origin (ie, its config is based off a preset w/o any modifications), use its
			// display name. If the config has a preset origin (ie, it's based off a preset, but has modifications), use
			// that display name. Otherwise, fall back to the config's display name.
			UMoviePipelineConfigBase* Config = Job->GetPresetOrigin();
			if (!Config)
			{
				Config = Job->GetConfiguration();

				if (Config && Config->GetConfigOrigin())
				{
					Config = Config->GetConfigOrigin();
				}
			}

			if (Config)
			{
				return FText::FromString(Config->DisplayName);
			}
		}

		return FText();
	}

	void OnPickPresetFromAsset(const FAssetData& AssetData)
	{	
		// Close the dropdown menu that showed them the assets to pick from.
		FSlateApplication::Get().DismissAllMenus();

		UMoviePipelineExecutorJob* CurrentJob = WeakJob.Get();
		if (!CurrentJob)
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("PickJobPresetAsset_Transaction", "Set Job Configuration Asset"));
					
		const bool bIsShot = false;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);
		
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineQueueJobTreeItem> JobTreeItem = Item->AsJob();
			if (JobTreeItem.IsValid())
			{
				UMoviePipelineExecutorJob* Job = JobTreeItem->WeakJob.Get();
				if (Job)
				{
					// Only apply changes to jobs of the same configuration type as our selected item.
					const bool bSelectedJobIsGraphJob = CurrentJob->IsUsingGraphConfiguration();
					const bool bCurrentJobIsGraphJob = Job->IsUsingGraphConfiguration();

					if (bSelectedJobIsGraphJob && bCurrentJobIsGraphJob)
					{
						Job->Modify();
						Job->SetGraphPreset(CastChecked<UMovieGraphConfig>(AssetData.GetAsset()));
					}
					else if (!bSelectedJobIsGraphJob && !bCurrentJobIsGraphJob)
					{
						Job->Modify();
						Job->SetPresetOrigin(CastChecked<UMoviePipelinePrimaryConfig>(AssetData.GetAsset()));
					}
				}
			}

		}

		OnChosePresetCallback.ExecuteIfBound(WeakJob, nullptr);
	}

	void OnClearNonGraphPreset()
	{	
		// Close the dropdown menu that showed them the assets to pick from.
		FSlateApplication::Get().DismissAllMenus();

		UMoviePipelineExecutorJob* CurrentJob = WeakJob.Get();
		if (!CurrentJob)
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("ClearJobPreset_Transaction", "Clear Job Configuration Preset"));


		const bool bIsShot = false;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);
		
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineQueueJobTreeItem> JobTreeItem = Item->AsJob();
			if (JobTreeItem.IsValid())
			{
				UMoviePipelineExecutorJob* Job = JobTreeItem->WeakJob.Get();
				if (Job)
				{
					// Only apply changes to jobs that aren't using graph configurations
					if (!Job->IsUsingGraphConfiguration())
					{
						Job->SetConfiguration(GetMutableDefault<UMoviePipelineExecutorJob>()->GetConfiguration());
						UMoviePipelineEditorBlueprintLibrary::EnsureJobHasDefaultSettings(Job, /*bForce=*/true);
					}
				}
			}

		}

	}

	void OnSwitchToGraph()
	{
		FScopedTransaction Transaction(LOCTEXT("SetGraphConfig_Transaction", "Set Graph Configuration"));

		const bool bIsShot = false;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);

		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineQueueJobTreeItem> JobTreeItem = Item->AsJob();
			if (JobTreeItem.IsValid())
			{
				UMoviePipelineExecutorJob* Job = JobTreeItem->WeakJob.Get();
				if (Job && !Job->SetConfigMode(EMoviePipelineConfigMode::Graph))
				{
					SMoviePipelineQueueEditor::AssignDefaultGraphPresetToJob(Job);
				}
			}
		}

		FSlateApplication::Get().DismissAllMenus();
	}

	void OnCreateNewGraphAndAssign()
	{
		FScopedTransaction Transaction(LOCTEXT("CreateNewGraphAndAssign_Transaction", "Create Graph"));

		const bool bIsShot = false;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);
		
		// Create only one graph for all affected items.
		if (const UMovieGraphConfig* NewGraph = UE::MovieGraph::Private::CreateNewSavedGraphAsset())
		{
			for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
			{
				TSharedPtr<FMoviePipelineQueueJobTreeItem> JobTreeItem = Item->AsJob();
				if (JobTreeItem.IsValid())
				{
					UMoviePipelineExecutorJob* Job = JobTreeItem->WeakJob.Get();
					if (Job)
					{
						Job->SetGraphPreset(NewGraph);
					}
				}
			}
		}
	}

	void OnSwitchToPreset()
	{
		FScopedTransaction Transaction(LOCTEXT("SetPresetConfig_Transaction", "Set Preset Configuration"));

		const bool bIsShot = false;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);

		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineQueueJobTreeItem> JobTreeItem = Item->AsJob();
			if (JobTreeItem.IsValid())
			{
				UMoviePipelineExecutorJob* Job = JobTreeItem->WeakJob.Get();
				if (Job)
				{
					Job->SetConfigMode(EMoviePipelineConfigMode::Preset);
					UMoviePipelineEditorBlueprintLibrary::EnsureJobHasDefaultSettings(Job);
				}
			}
		}

		FSlateApplication::Get().DismissAllMenus();
	}

	EVisibility GetPrimaryConfigModifiedVisibility() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		if (Job)
		{
			// Basic-mode configs are always saved directly on the job, so there's no external asset to be dirty.
			if (Job->IsUsingBasicConfiguration())
			{
				return EVisibility::Collapsed;
			}

			if (const UMovieGraphConfig* GraphPreset = Job->GetGraphPreset())
			{
				return GraphPreset->GetPackage()->IsDirty() ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return (Job->GetPresetOrigin() == nullptr) ? EVisibility::Visible : EVisibility::Collapsed;
		}

		return EVisibility::Collapsed;
	}

	int32 GetConfigLabelWidgetIndex() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		return (Job && Job->IsUsingBasicConfiguration()) ? 0 : 1;
	}

	FText GetPrimaryConfigLabelWithModifiedIndicator() const
	{
		if (GetPrimaryConfigModifiedVisibility() == EVisibility::Visible)
		{
			return FText::Format(INVTEXT("{0} *"), GetPrimaryConfigLabel());
		}
		return GetPrimaryConfigLabel();
	}

	void OnEditPrimaryConfigForJob()
	{
		OnEditConfigCallback.ExecuteIfBound(WeakJob, nullptr);
	}

	FText GetOutputLabel() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		FString OutputDir;
		if (UMoviePipelineEditorBlueprintLibrary::GetDisplayOutputPathFromJob(Job, OutputDir))
		{
			return FText::FromString(OutputDir);
		}

		return LOCTEXT("MissingConfigOutput_Label", "[No Config Set]");
	}

	void BrowseToOutputFolder()
	{
		if (UMoviePipelineExecutorJob* Job = WeakJob.Get())
		{
			FText ResolveError;
			const FString ResolvedOutputDir = UMoviePipelineEditorBlueprintLibrary::TryResolveOutputDirectoryFromJob(Job, ResolveError);

			if (!ResolvedOutputDir.IsEmpty())
			{
				// Attempt to make the directory. The user can see the output folder before they render so the folder
				// may not have been created yet and the ExploreFolder call will fail.
				IFileManager::Get().MakeDirectory(*ResolvedOutputDir, true);

				FPlatformProcess::ExploreFolder(*ResolvedOutputDir);
			}
			else if (!ResolveError.IsEmpty())
			{
				FNotificationInfo Info(LOCTEXT("BrowseToOutputFolder_FailedNotification", "Unable to Open Output Folder"));
				Info.SubText = ResolveError;
				Info.Image = FAppStyle::GetBrush(TEXT("Icons.Warning"));
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
		}
	}

	int32 GetStatusIndex() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		if (Job)
		{
			// If the progress is zero we want to show the status message instead.
			return Job->GetStatusProgress() > 0;
		}

		return 0;
	}

	bool IsEnabled() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		if (Job)
		{
			return Job->IsEnabled() && !Job->IsConsumed();
		}
		return false;
	}

	bool IsConfigEditingEnabled() const
	{
		// Don't allow editing the UI while a job is running as it will change job parameters mid-job!
		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);
		const bool bNotRendering = !Subsystem->IsRendering();

		return IsEnabled() && bNotRendering;
	}

	TOptional<float> GetProgressPercent() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		return Job ? Job->GetStatusProgress() : TOptional<float>();
	}

	FText GetStatusMessage() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		return Job ? FText::FromString(Job->GetStatusMessage()) : FText();
	}

	TSharedRef<SWidget> OnGenerateSequencePickerMenu()
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

		FAssetPickerConfig AssetPickerConfig;
		{
			AssetPickerConfig.SelectionMode = ESelectionMode::Single;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.bShowBottomToolbar = true;
			AssetPickerConfig.bAutohideSearchBar = false;
			AssetPickerConfig.bAllowDragging = false;
			AssetPickerConfig.bCanShowClasses = false;
			AssetPickerConfig.bShowPathInColumnView = true;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.bSortByPathInColumnView = false;
			AssetPickerConfig.ThumbnailScale = 0.1f;
			AssetPickerConfig.SaveSettingsName = TEXT("MoviePipelineSequenceAsset");

			AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoSequencesInPicker_Warning", "No Level Sequences Found");
			AssetPickerConfig.Filter.ClassPaths.Add(ULevelSequence::StaticClass()->GetClassPathName());
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FMoviePipelineQueueJobTreeItem::OnPickSequenceFromAsset);
		}

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("PickSequence_MenuSection", "Level Sequence"));
		{
			TSharedRef<SWidget> PresetPicker = SNew(SBox)
				.WidthOverride(300.f)
				.HeightOverride(300.f)
				[
					ContentBrowser.CreateAssetPicker(AssetPickerConfig)
				];

			MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void OnPickSequenceFromAsset(const FAssetData& AssetData)
	{
		FSlateApplication::Get().DismissAllMenus();

		UMoviePipelineExecutorJob* CurrentJob = WeakJob.Get();
		if (!CurrentJob)
		{
			return;
		}

		ULevelSequence* LevelSequence = Cast<ULevelSequence>(AssetData.GetAsset());
		if (!LevelSequence)
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("PickJobSequence_Transaction", "Set Job Sequence"));

		const bool bIsShot = false;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);

		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineQueueJobTreeItem> JobTreeItem = Item->AsJob();
			if (JobTreeItem.IsValid())
			{
				UMoviePipelineExecutorJob* Job = JobTreeItem->WeakJob.Get();
				if (Job)
				{
					Job->Modify();
					Job->SetSequence(FSoftObjectPath(LevelSequence));
					Job->JobName = LevelSequence->GetName();
				}
			}
		}
	}

	TSharedRef<SWidget> OnGenerateConfigPresetPickerMenu()
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		if (!Job)
		{
			return SNullWidget::NullWidget;
		}

		UClass* ConfigType = Job->IsUsingGraphConfiguration() ? UMovieGraphConfig::StaticClass() : UMoviePipelinePrimaryConfig::StaticClass();
		constexpr bool bIsShot = false;
		
		return OnGenerateConfigPresetPickerMenuFromClass(
			ConfigType,
			WeakJob,
			bIsShot,
			FOnAssetSelected::CreateRaw(this, &FMoviePipelineQueueJobTreeItem::OnPickPresetFromAsset),
			FExecuteAction::CreateRaw(this, &FMoviePipelineQueueJobTreeItem::OnClearNonGraphPreset),
			FExecuteAction::CreateRaw(this, &FMoviePipelineQueueJobTreeItem::OnSwitchToGraph),
			FExecuteAction::CreateRaw(this, &FMoviePipelineQueueJobTreeItem::OnCreateNewGraphAndAssign),
			FExecuteAction::CreateRaw(this, &FMoviePipelineQueueJobTreeItem::OnSwitchToPreset)
			);
	}

	static TSharedRef<SWidget> OnGenerateConfigPresetPickerMenuFromClass(UClass* InClass,
		TWeakObjectPtr<UMoviePipelineExecutorJob> TargetJob, const bool bIsShot, FOnAssetSelected InOnAssetSelected, FExecuteAction InClearNonGraphConfig,
		FExecuteAction InSwitchToGraph, FExecuteAction InCreateNewGraphAndAssign, FExecuteAction InSwitchToPreset)
	{
		if (!TargetJob.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		FMenuBuilder MenuBuilder(true, nullptr);
		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

		FAssetPickerConfig AssetPickerConfig;
		{
			const FText NoAssetsFoundWarning = InClass->IsChildOf(UMoviePipelineConfigBase::StaticClass())
				? LOCTEXT("NoConfigs_Warning", "No Configurations Found")
				: LOCTEXT("NoGraphs_Warning", "No Render Graphs Found");

			AssetPickerConfig.SelectionMode = ESelectionMode::Single;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.bShowBottomToolbar = true;
			AssetPickerConfig.bAutohideSearchBar = false;
			AssetPickerConfig.bAllowDragging = false;
			AssetPickerConfig.bCanShowClasses = false;
			AssetPickerConfig.bShowPathInColumnView = true;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.bSortByPathInColumnView = false;
			AssetPickerConfig.ThumbnailScale = 0.1f;
			AssetPickerConfig.SaveSettingsName = TEXT("MoviePipelineConfigAsset");

			AssetPickerConfig.AssetShowWarningText = NoAssetsFoundWarning;
			AssetPickerConfig.Filter.ClassPaths.Add(InClass->GetClassPathName());
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.OnAssetSelected = InOnAssetSelected;
		}

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("CurrentConfig_MenuSection", "Current Configuration"));
		{
			if (!TargetJob->IsUsingGraphConfiguration())
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("ResetConfig_Label", "Reset Configuration"),
					LOCTEXT("ResetConfig_Tooltip", "Resets the changes to the config and goes back to the defaults."),
					FSlateIcon(),
					FUIAction(InClearNonGraphConfig),
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
			else
			{
				if (bIsShot)
				{
					// Shots can clear their graph, meaning they will inherit from the parent graph.
					MenuBuilder.AddMenuEntry(
						LOCTEXT("ClearGraph_Label", "Clear Graph"),
						LOCTEXT("ClearGraph_Tooltip", "Remove the graph assigned to this shot. After removal, the shot will inherit the graph from the primary job."),
						FSlateIcon(),
						FUIAction(InSwitchToPreset),
						NAME_None,
						EUserInterfaceActionType::Button
					);
				}
				else
				{
					// Primary jobs in graph mode: "Reset Configuration" reassigns the default graph from project settings.
					MenuBuilder.AddMenuEntry(
						LOCTEXT("ResetGraphConfig_Label", "Reset Configuration"),
						LOCTEXT("ResetGraphConfig_Tooltip", "Resets the graph to the default render graph specified in project settings."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([TargetJob]()
						{
							if (TargetJob.IsValid())
							{
								SMoviePipelineQueueEditor::AssignDefaultGraphPresetToJob(TargetJob.Get());
								FSlateApplication::Get().DismissAllMenus();
							}
						})),
						NAME_None,
						EUserInterfaceActionType::Button
					);
				}
			}
		}
		MenuBuilder.EndSection();

		if (!bIsShot)
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("ConfigurationType_MenuSection", "Configuration Type"));
			{
				// Basic radio button
				FUIAction SwitchToBasicAction;
				SwitchToBasicAction.ExecuteAction = FExecuteAction::CreateLambda([TargetJob]()
				{
					if (TargetJob.IsValid())
					{
						FScopedTransaction Transaction(LOCTEXT("SetBasicConfig", "Set Basic Configuration"));
						TargetJob->SetConfigMode(EMoviePipelineConfigMode::Basic);
						FSlateApplication::Get().DismissAllMenus();
					}
				});
				SwitchToBasicAction.GetActionCheckState = FGetActionCheckState::CreateLambda([TargetJob]() -> ECheckBoxState
				{
					return (TargetJob.IsValid() && TargetJob->IsUsingBasicConfiguration()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				});

				MenuBuilder.AddMenuEntry(
					LOCTEXT("ConfigurationType_BasicLabel", "Basic"),
					LOCTEXT("ConfigurationType_BasicTooltip", "Use a simplified set of rendering settings."),
					FSlateIcon(),
					SwitchToBasicAction,
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);

				// Preset radio button
				FUIAction SwitchToPresetAction;
				SwitchToPresetAction.ExecuteAction = InSwitchToPreset;
				SwitchToPresetAction.GetActionCheckState = FGetActionCheckState::CreateLambda([TargetJob]() -> ECheckBoxState
				{
					return (TargetJob.IsValid() && !TargetJob->IsUsingGraphConfiguration() && !TargetJob->IsUsingBasicConfiguration())
						? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				});

				// Graph radio button
				FUIAction SwitchToGraphAction;
				SwitchToGraphAction.ExecuteAction = InSwitchToGraph;
				SwitchToGraphAction.GetActionCheckState = FGetActionCheckState::CreateLambda([TargetJob]() -> ECheckBoxState
				{
					return (TargetJob.IsValid() && TargetJob->IsUsingGraphConfiguration()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				});

				MenuBuilder.AddMenuEntry(
					LOCTEXT("ConfigurationType_GraphLabel", "Movie Render Graph"),
					LOCTEXT("ConfigurationType_GraphTooltip", "Updates this job or shot to use a graph-based configuration."),
					FSlateIcon(),
					SwitchToGraphAction,
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);

				// Preset radio button
				MenuBuilder.AddMenuEntry(
					LOCTEXT("ConfigurationType_PresetLabel", "Legacy Preset"),
					LOCTEXT("ConfigurationType_PresetTooltip", "Updates this job or shot to use a preset-based configuration."),
					FSlateIcon(),
					SwitchToPresetAction,
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}
			MenuBuilder.EndSection();
		}

		// "Create New Asset" is only relevant in Graph mode. Shots additionally require the parent job to be in Graph mode.
		if (TargetJob->IsUsingGraphConfiguration())
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("CreateNewAsset_MenuSection", "Create New Asset"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("NewGraph_Label", "Movie Render Graph"),
					LOCTEXT("NewGraph_Tooltip", "Creates a new graph asset and assigns it to this job or shot."),
					FSlateIcon(),
					FUIAction(InCreateNewGraphAndAssign),
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
			MenuBuilder.EndSection();
		}

		// Asset picker is not applicable in Basic mode (Basic config is edited directly in the details panel).
		if (!TargetJob->IsUsingBasicConfiguration())
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("ImportConfig_MenuSection", "Import Configuration"));
			{
				TSharedRef<SWidget> PresetPicker = SNew(SBox)
					.WidthOverride(300.f)
					.HeightOverride(300.f)
					[
						ContentBrowser.CreateAssetPicker(AssetPickerConfig)
					];

				MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
			}
			MenuBuilder.EndSection();
		}

		return MenuBuilder.MakeWidget();
	}
};

void SQueueJobListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
{
	Item = InArgs._Item;
	OnEditConfigRequested = InArgs._OnEditConfigRequested;

	FSuperRowType::FArguments SuperArgs = FSuperRowType::FArguments();
	FSuperRowType::Construct(SuperArgs 
		.OnDragDetected(this, &SQueueJobListRow::OnDragDetected)
		.OnCanAcceptDrop(this, &SQueueJobListRow::OnCanAcceptDrop)
		.OnAcceptDrop(this, &SQueueJobListRow::OnAcceptDrop),		
		OwnerTable);
}

TSharedRef<SWidget> SQueueJobListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == NAME_Enabled)
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
		  	[
				SNew(SCheckBox)
				.IsFocusable(false)
				.IsChecked(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetCheckState)
				.OnCheckStateChanged(Item.Get(), &FMoviePipelineQueueJobTreeItem::SetCheckState)
			];
	}
	else if (ColumnName == NAME_JobName)
	{
		return SNew(SBox)
			.Padding(2.0f)
			.IsEnabled(Item.Get(), &FMoviePipelineQueueJobTreeItem::IsEnabled)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 10, 0)
				[
					SNew(SExpanderArrow, SharedThis(this))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 4, 0)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("ClassIcon.CameraComponent"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetJobName)
				]

				// Sequence picker dropdown
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(4, 0, 4, 0)
				[
					SNew(SComboButton)
					.ContentPadding(1)
					.IsEnabled(Item.Get(), &FMoviePipelineQueueJobTreeItem::IsConfigEditingEnabled)
					.OnGetMenuContent(Item.Get(), &FMoviePipelineQueueJobTreeItem::OnGenerateSequencePickerMenu)
					.HasDownArrow(false)
					.ButtonContent()
					[
						SNew(SBox)
						.Padding(FMargin(2, 0))
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "NormalText.Important")
							.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
							.Text(FEditorFontGlyphs::Caret_Down)
						]
					]
				]
			];
	}
	else if (ColumnName == NAME_Settings)
	{
		return SNew(SHorizontalBox)
		.IsEnabled(Item.Get(), &FMoviePipelineQueueJobTreeItem::IsConfigEditingEnabled)

		// Preset Label
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4, 0)
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetConfigLabelWidgetIndex)

			// Index 0: Basic config — plain text, no hyperlink
			+ SWidgetSwitcher::Slot()
			[
				SNew(STextBlock)
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetPrimaryConfigLabel)))
			]

			// Index 1: Preset/graph config — hyperlink to open preset picker
			+ SWidgetSwitcher::Slot()
			[
				SNew(SHyperlink)
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetPrimaryConfigLabelWithModifiedIndicator)))
				.OnNavigate(Item.Get(), &FMoviePipelineQueueJobTreeItem::OnEditPrimaryConfigForJob)
			]
		]

		// Dropdown Arrow
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(4,0,4,0)
		[
			SNew(SComboButton)
			.ContentPadding(1)
			.OnGetMenuContent(Item.Get(), &FMoviePipelineQueueJobTreeItem::OnGenerateConfigPresetPickerMenu)
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SBox)
				.Padding(FMargin(2, 0))
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "NormalText.Important")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
					.Text(FEditorFontGlyphs::Caret_Down)
				]
			]
		];
	}
	else if (ColumnName == NAME_Output)
	{
		return SNew(SBox)
			.IsEnabled(Item.Get(), &FMoviePipelineQueueJobTreeItem::IsEnabled)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(5, 0))
			[
				SNew(SHyperlink)
					.Text(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetOutputLabel)
					.ToolTipText(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetOutputLabel)
					.OnNavigate(Item.Get(), &FMoviePipelineQueueJobTreeItem::BrowseToOutputFolder)
			];

	}
	else if(ColumnName == NAME_Status)
	{
		return SNew(SBox)
			.Padding(FMargin(5, 0))
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetStatusIndex)
				.IsEnabled(Item.Get(), &FMoviePipelineQueueJobTreeItem::IsEnabled)

				// Status Message Label
				+ SWidgetSwitcher::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetStatusMessage)
				]

				// Progress Bar
				+ SWidgetSwitcher::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(SProgressBar)
					.Percent(Item.Get(), &FMoviePipelineQueueJobTreeItem::GetProgressPercent)
				]
			];

	}

	return SNullWidget::NullWidget;
}

const FName SQueueJobListRow::NAME_Enabled = FName(TEXT("Enabled"));
const FName SQueueJobListRow::NAME_JobName = FName(TEXT("Job Name"));
const FName SQueueJobListRow::NAME_Settings = FName(TEXT("Settings"));
const FName SQueueJobListRow::NAME_Output = FName(TEXT("Output"));
const FName SQueueJobListRow::NAME_Status = FName(TEXT("Status"));

/**
 * This drag drop operation allows us to move around queues in the widget tree
 */
class FQueueJobListDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FQueueJobListDragDropOp, FDecoratedDragDropOp)

	/** The template to create an instance */
	TSharedPtr<IMoviePipelineQueueTreeItem> ListItem;

	/** Constructs the drag drop operation */
	static TSharedRef<FQueueJobListDragDropOp> New(const TSharedPtr<IMoviePipelineQueueTreeItem>& InListItem, FText InDragText)
	{
		TSharedRef<FQueueJobListDragDropOp> Operation = MakeShared<FQueueJobListDragDropOp>();
		Operation->ListItem = InListItem;
		Operation->DefaultHoverText = InDragText;
		Operation->CurrentHoverText = InDragText;
		Operation->Construct();

		return Operation;
	}
};

/** Called whenever a drag is detected by the tree view. */
FReply SQueueJobListRow::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (Item.IsValid())
	{
		FText DefaultText = LOCTEXT("DefaultDragDropFormat", "Move 1 item(s)");
		return FReply::Handled().BeginDragDrop(FQueueJobListDragDropOp::New(Item, DefaultText));
	}
	return FReply::Unhandled();
}

/** Called to determine whether a current drag operation is valid for this row. */
TOptional<EItemDropZone> SQueueJobListRow::OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<IMoviePipelineQueueTreeItem> InItem)
{
	TSharedPtr<FQueueJobListDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FQueueJobListDragDropOp>();
	if (DragDropOp.IsValid())
	{
		if (InItemDropZone == EItemDropZone::OntoItem)
		{
			DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
		}
		else
		{
			DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Ok"));
		}
		return InItemDropZone;
	}
	return TOptional<EItemDropZone>();
}

/** Called to complete a drag and drop onto this drop. */
FReply SQueueJobListRow::OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<IMoviePipelineQueueTreeItem> InItem)
{
	TSharedPtr<FQueueJobListDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FQueueJobListDragDropOp>();
	if (!DragDropOp.IsValid() || !DragDropOp->ListItem.IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FMoviePipelineQueueJobTreeItem> DragDropJobItem = StaticCastSharedPtr<FMoviePipelineQueueJobTreeItem>(DragDropOp->ListItem);
	UMoviePipelineExecutorJob* DragDropJob = DragDropJobItem->GetOwningJob();

	TSharedPtr<FMoviePipelineQueueJobTreeItem> JobItem = StaticCastSharedPtr<FMoviePipelineQueueJobTreeItem>(InItem);
	UMoviePipelineExecutorJob* Job = JobItem->GetOwningJob();

	UMoviePipelineQueue* ActiveQueue = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>()->GetQueue();
	if (!ActiveQueue || !DragDropJob || !Job)
	{
		return FReply::Unhandled();
	}
	
	int32 Index = 0;
	ActiveQueue->GetJobs().Find(Job, Index);

	if (InItemDropZone == EItemDropZone::BelowItem)
	{
		++Index;
		if (Index > ActiveQueue->GetJobs().Num())
		{
			Index = ActiveQueue->GetJobs().Num()-1;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("ReorderJob_Transaction", "Reorder Job"));

	ActiveQueue->Modify();

	ActiveQueue->SetJobIndex(DragDropJob, Index);

	return FReply::Handled();
}

class SQueueShotListRow : public SMultiColumnTableRow<TSharedPtr<IMoviePipelineQueueTreeItem>>
{
public:
	SLATE_BEGIN_ARGS(SQueueShotListRow) {}
	SLATE_ARGUMENT(TSharedPtr<FMoviePipelineShotItem>, Item)
		SLATE_EVENT(FOnMoviePipelineEditConfig, OnEditConfigRequested)
		SLATE_END_ARGS()

	TSharedPtr<FMoviePipelineShotItem> Item;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	FOnMoviePipelineEditConfig OnEditConfigRequested;
};

struct FMoviePipelineShotItem : IMoviePipelineQueueTreeItem
{
	/** The job that this tree item represents */
	TWeakObjectPtr<UMoviePipelineExecutorJob> WeakJob;

	/** The identifier in the job for which shot this is. */
	TWeakObjectPtr<UMoviePipelineExecutorShot> WeakShot;

	TWeakPtr<SMoviePipelineQueueEditor> WeakQueueEditor;

	FOnMoviePipelineEditConfig OnEditConfigCallback;
	FOnMoviePipelineEditConfig OnChosePresetCallback;

	explicit FMoviePipelineShotItem(UMoviePipelineExecutorJob* InJob, UMoviePipelineExecutorShot* InShot, FOnMoviePipelineEditConfig InOnEditConfigCallback, FOnMoviePipelineEditConfig InOnChosePresetCallback)
		: WeakJob(InJob)
		, WeakShot(InShot)
		, OnEditConfigCallback(InOnEditConfigCallback)
		, OnChosePresetCallback(InOnChosePresetCallback)
	{}

	virtual UMoviePipelineExecutorJob* GetOwningJob() override
	{
		return WeakJob.Get();
	}

	virtual UMoviePipelineExecutorShot* GetOwningShot() override
	{
		return WeakShot.Get();
	}

	virtual TSharedPtr<FMoviePipelineShotItem> AsShot() override
	{
		return SharedThis(this);
	}

	virtual TSharedRef<ITableRow> ConstructWidget(TWeakPtr<SMoviePipelineQueueEditor> InQueueWidget, const TSharedRef<STableViewBase>& OwnerTable) override
	{
		WeakQueueEditor = InQueueWidget;

		return SNew(SQueueShotListRow, OwnerTable)
			.Item(SharedThis(this));
	}

	ECheckBoxState GetCheckState() const
	{
		if (UMoviePipelineExecutorShot* Shot = WeakShot.Get())
		{
			return Shot->bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		
		return ECheckBoxState::Unchecked;
	}

	void SetCheckState(ECheckBoxState InNewState)
	{
		const bool bIsShot = true;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);
		
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineShotItem> ShotTreeItem = Item->AsShot();
			if (ShotTreeItem.IsValid())
			{
				if (UMoviePipelineExecutorShot* Shot = ShotTreeItem->WeakShot.Get())
				{
					Shot->bEnabled = InNewState == ECheckBoxState::Checked;
				}
			}
		}
	}

	FText GetShotLabel() const
	{
		UMoviePipelineExecutorShot* Shot = WeakShot.Get();
		if (Shot)
		{
			FString FormattedTitle = FString::Printf(TEXT("%s %s"), *Shot->OuterName, *Shot->InnerName);
			return FText::FromString(FormattedTitle);
		}
		return FText();
	}

	FText GetPresetLabel() const
	{
		return FText();
	}

	int32 GetStatusIndex() const
	{
		UMoviePipelineExecutorShot* Shot = WeakShot.Get();
		if (Shot)
		{
			// If the progress is zero we want to show the status message instead.
			return Shot->GetStatusProgress() > 0;
		}

		return 0;
	}

	bool IsEnabled() const
	{
		UMoviePipelineExecutorJob* Job = WeakJob.Get();
		if (Job)
		{
			return Job->IsEnabled()  && !Job->IsConsumed();
		}
		return false;
	}

	bool IsConfigEditingEnabled() const
	{
		// Don't allow editing the UI while a job is running as it will change job parameters mid-job!
		UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
		check(Subsystem);
		const bool bNotRendering = !Subsystem->IsRendering();

		return IsEnabled() && bNotRendering;
	}

	TOptional<float> GetProgressPercent() const
	{
		UMoviePipelineExecutorShot* Shot = WeakShot.Get();
		return Shot ? Shot->GetStatusProgress() : TOptional<float>();
	}

	FText GetStatusMessage() const
	{
		UMoviePipelineExecutorShot* Shot = WeakShot.Get();
		return Shot ? FText::FromString(Shot->GetStatusMessage()) : FText();
	}

	FText GetShotConfigLabel() const
	{
		UMoviePipelineExecutorShot* Shot = WeakShot.Get();
		if (Shot)
		{
			if (Shot->IsUsingBasicConfiguration())
			{
				const UMoviePipelineBasicConfig* ShotConfig = Shot->GetBasicShotConfig();
				if (ShotConfig && ShotConfig->HasAnyOverrides())
				{
					return LOCTEXT("BasicShotOverrideLabel", "Basic (Override)");
				}

				return LOCTEXT("BasicShotInheritLabel", "Basic");
			}

			if (Shot->IsUsingGraphConfiguration())
			{
				if (const UMovieGraphConfig* GraphConfig = Shot->GetGraphPreset())
				{
					return FText::FromString(GraphConfig->GetName());
				}
				
				return LOCTEXT("QueueEditorMakeNewShotSubgraph", "Make Subgraph");
			}
			
			// If the shot has a preset origin (ie, its config is based off a preset w/o any modifications), use its
			// display name. If the config has a preset origin (ie, it's based off a preset, but has modifications), use
			// that display name. Otherwise, fall back to the config's display name.
			UMoviePipelineShotConfig* Config = Shot->GetShotOverridePresetOrigin();
			if (!Config)
			{
				Config = Shot->GetShotOverrideConfiguration();

				if (Config && Config->GetConfigOrigin())
				{
					Config = Cast<UMoviePipelineShotConfig>(Config->GetConfigOrigin());
				}
			}

			if (Config)
			{
				return FText::FromString(Config->DisplayName);
			}
		}

		return FText::FromString(TEXT("Edit"));
	}

	void OnPickShotPresetFromAsset(const FAssetData& AssetData)
	{
		// Close the dropdown menu that showed them the assets to pick from.
		FSlateApplication::Get().DismissAllMenus();
		
		UMoviePipelineExecutorShot* SelectedShot = WeakShot.Get();
		if(!SelectedShot)
		{
			return;
		}
								
		FScopedTransaction Transaction(LOCTEXT("PickShotPresetAsset_Transaction", "Set Shot Configuration Asset"));

		const bool bIsShot = true;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);
		
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineShotItem> ShotTreeItem = Item->AsShot();
			if (ShotTreeItem.IsValid())
			{
				UMoviePipelineExecutorShot* Shot = ShotTreeItem->WeakShot.Get();
				if(Shot)
				{				
					const bool bSelectedShotIsGraph = SelectedShot->IsUsingGraphConfiguration();
					const bool bCurrentShotIsGraph = Shot->IsUsingGraphConfiguration();
						
					// Only apply changes to jobs of the same configuration type as our selected item.	
					if (bSelectedShotIsGraph && bCurrentShotIsGraph)
					{
						Shot->Modify();
						Shot->SetGraphPreset(CastChecked<UMovieGraphConfig>(AssetData.GetAsset()));
					}
					else if (!bSelectedShotIsGraph && !bCurrentShotIsGraph)
					{
						Shot->Modify();
						Shot->SetShotOverridePresetOrigin(CastChecked<UMoviePipelineShotConfig>(AssetData.GetAsset()));
					}
				}
			}
		}


		OnChosePresetCallback.ExecuteIfBound(WeakJob, WeakShot);
	}

	void OnClearNonGraphPreset()
	{
		// Close the dropdown menu that showed them the assets to pick from.
		FSlateApplication::Get().DismissAllMenus();
		
		UMoviePipelineExecutorShot* SelectedShot = WeakShot.Get();
		if(!SelectedShot)
		{
			return;
		}
		
		FScopedTransaction Transaction(LOCTEXT("PickShotClearConfig_Transaction", "Clear Configuration Asset"));
		
		const bool bIsShot = true;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);
		
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineShotItem> ShotTreeItem = Item->AsShot();
			if (ShotTreeItem.IsValid())
			{
				UMoviePipelineExecutorShot* Shot = ShotTreeItem->WeakShot.Get();
				if(Shot)
				{
					// Only clear jobs that aren't using the graph configuration
					if(!Shot->IsUsingGraphConfiguration())
					{
						Shot->Modify();
						Shot->SetShotOverrideConfiguration(nullptr);
					}
				}
			}
		}
					

		OnChosePresetCallback.ExecuteIfBound(WeakJob, WeakShot);
	}



	void OnCreateNewGraphAndAssign()
	{
		UMoviePipelineExecutorShot* SelectedShot = WeakShot.Get();
		if (!SelectedShot)
		{
			return;
		}

		if (const UMovieGraphConfig* NewGraph = UE::MovieGraph::Private::CreateNewSavedGraphAsset())
		{
			FScopedTransaction Transaction(LOCTEXT("PickCreateNewGraphAsset_Transaction", "Assign Graph Configuration Asset"));

			const bool bIsShot = true;
			TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);
		
			for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
			{
				TSharedPtr<FMoviePipelineShotItem> ShotTreeItem = Item->AsShot();
				if (ShotTreeItem.IsValid())
				{
					UMoviePipelineExecutorShot* Shot = ShotTreeItem->WeakShot.Get();
					if (Shot)
					{
						// Only assign jobs that are using the graph configuration
						if (Shot->IsUsingGraphConfiguration())
						{
							Shot->Modify();
							Shot->SetGraphPreset(NewGraph);
						}
					}
				}
			}
		}

		
	}		
		

	void OnClearGraph()
	{
		UMoviePipelineExecutorShot* SelectedShot = WeakShot.Get();
		if (!SelectedShot)
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("ClearShotGraph_Transaction", "Clear Graph Config"));

		const bool bIsShot = true;
		TSet<TSharedPtr<IMoviePipelineQueueTreeItem>> AffectedItems = GetMultiSelectAffectedItems(WeakQueueEditor, bIsShot);
		
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AffectedItems)
		{
			TSharedPtr<FMoviePipelineShotItem> ShotTreeItem = Item->AsShot();
			if (ShotTreeItem.IsValid())
			{
				UMoviePipelineExecutorShot* Shot = ShotTreeItem->WeakShot.Get();
				if (Shot)
				{
					// Only clear jobs that are using the graph configuration
					if (Shot->IsUsingGraphConfiguration())
					{
						Shot->Modify();
						Shot->SetGraphPreset(nullptr);
					}
				}
			}
		}
	}

	EVisibility GetShotConfigModifiedVisibility() const
	{
		if (const UMoviePipelineExecutorShot* Shot = WeakShot.Get())
		{
			// Basic-mode configs are always saved directly on the job, so there's no external asset to be dirty.
			if (Shot->IsUsingBasicConfiguration())
			{
				return EVisibility::Collapsed;
			}

			if (const UMovieGraphConfig* GraphPreset = Shot->GetGraphPreset())
			{
				return GraphPreset->GetPackage()->IsDirty() ? EVisibility::Visible : EVisibility::Collapsed;
			}

			if (Shot->GetShotOverrideConfiguration() && (Shot->GetShotOverridePresetOrigin() == nullptr))
			{
				return EVisibility::Visible;
			}
		}

		return EVisibility::Collapsed;
	}

	FText GetShotConfigLabelWithModifiedIndicator() const
	{
		if (GetShotConfigModifiedVisibility() == EVisibility::Visible)
		{
			return FText::Format(INVTEXT("{0} *"), GetShotConfigLabel());
		}
		return GetShotConfigLabel();
	}

	int32 GetShotConfigLabelWidgetIndex() const
	{
		UMoviePipelineExecutorShot* Shot = WeakShot.Get();
		return (Shot && Shot->IsUsingBasicConfiguration()) ? 0 : 1;
	}

	void OnEditConfigForShot()
	{
		OnEditConfigCallback.ExecuteIfBound(WeakJob, WeakShot);
	}

	TSharedRef<SWidget> OnGenerateShotConfigPresetPickerMenu()
	{
		const UMoviePipelineExecutorShot* Shot = WeakShot.Get();
		UClass* ConfigType = Shot && Shot->IsUsingGraphConfiguration() ? UMovieGraphConfig::StaticClass() : UMoviePipelineShotConfig::StaticClass();
		constexpr bool bIsShot = true;
		const FExecuteAction SwitchToGraph = nullptr;

		return FMoviePipelineQueueJobTreeItem::OnGenerateConfigPresetPickerMenuFromClass(
			ConfigType,
			WeakJob,
			bIsShot,
			FOnAssetSelected::CreateRaw(this, &FMoviePipelineShotItem::OnPickShotPresetFromAsset),
			FExecuteAction::CreateRaw(this, &FMoviePipelineShotItem::OnClearNonGraphPreset),
			SwitchToGraph,
			FExecuteAction::CreateRaw(this, &FMoviePipelineShotItem::OnCreateNewGraphAndAssign),
			FExecuteAction::CreateRaw(this, &FMoviePipelineShotItem::OnClearGraph)
			);
	}
};

void SQueueShotListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
{
	Item = InArgs._Item;
	OnEditConfigRequested = InArgs._OnEditConfigRequested;

	FSuperRowType::FArguments SuperArgs = FSuperRowType::FArguments();
	FSuperRowType::Construct(SuperArgs, OwnerTable);
}

TSharedRef<SWidget> SQueueShotListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == SQueueJobListRow::NAME_Enabled)
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsFocusable(false)
				.IsChecked(Item.Get(), &FMoviePipelineShotItem::GetCheckState)
				.OnCheckStateChanged(Item.Get(), &FMoviePipelineShotItem::SetCheckState)
			];
	}
	else if (ColumnName == SQueueJobListRow::NAME_JobName)
	{
		return SNew(SBox)
		.Padding(2.0f)
		[
			SNew(SHorizontalBox)
			.IsEnabled(Item.Get(), &FMoviePipelineShotItem::IsEnabled)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 0, 0)
			[
				SNew(SExpanderArrow, SharedThis(this))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Sequencer.Tracks.CinematicShot"))
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

			// Shot Name Label
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Item.Get(), &FMoviePipelineShotItem::GetShotLabel)
			]
		];
	}
	else if (ColumnName == SQueueJobListRow::NAME_Settings)
	{
		return SNew(SHorizontalBox)
		.IsEnabled(Item.Get(), &FMoviePipelineShotItem::IsConfigEditingEnabled)

		// Preset Label
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4, 0)
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(Item.Get(), &FMoviePipelineShotItem::GetShotConfigLabelWidgetIndex)

			// Index 0: Basic config — plain text, no hyperlink
			+ SWidgetSwitcher::Slot()
			[
				SNew(STextBlock)
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(Item.Get(), &FMoviePipelineShotItem::GetShotConfigLabel)))
			]

			// Index 1: Preset/graph config — hyperlink to open preset picker
			+ SWidgetSwitcher::Slot()
			[
				SNew(SHyperlink)
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(Item.Get(), &FMoviePipelineShotItem::GetShotConfigLabelWithModifiedIndicator)))
				.OnNavigate(Item.Get(), &FMoviePipelineShotItem::OnEditConfigForShot)
			]
		]

		// Dropdown Arrow
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(4, 0, 4, 0)
		[
			SNew(SComboButton)
			.ContentPadding(1)
			.OnGetMenuContent(Item.Get(), &FMoviePipelineShotItem::OnGenerateShotConfigPresetPickerMenu)
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SBox)
				.Padding(FMargin(2, 0))
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "NormalText.Important")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
					.Text(FEditorFontGlyphs::Caret_Down)
				]
			]
		];
	}
	else if (ColumnName == SQueueJobListRow::NAME_Output)
	{
		return SNullWidget::NullWidget;
	}
	else if (ColumnName == SQueueJobListRow::NAME_Status)
	{
		return SNew(SBox)
			.Padding(FMargin(5, 0))
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(Item.Get(), &FMoviePipelineShotItem::GetStatusIndex)
				.IsEnabled(Item.Get(), &FMoviePipelineShotItem::IsEnabled)

				// Ready Label
				+ SWidgetSwitcher::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PendingJobStatusReady_Label", "Ready"))
				]

				// Progress Bar
				+ SWidgetSwitcher::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(SProgressBar)
					.Percent(Item.Get(), &FMoviePipelineShotItem::GetProgressPercent)
				]

				// Completed
				+ SWidgetSwitcher::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PendingJobStatusCompleted_Label", "Completed!"))
				]
			];
	}

	return SNullWidget::NullWidget;
}


UE_DISABLE_OPTIMIZATION_SHIP
void SMoviePipelineQueueEditor::Construct(const FArguments& InArgs)
{
	CachedQueueSerialNumber = uint32(-1);
	OnEditConfigRequested = InArgs._OnEditConfigRequested;
	OnPresetChosen = InArgs._OnPresetChosen;
	OnJobSelectionChanged = InArgs._OnJobSelectionChanged;

	TreeView = SNew(STreeView<TSharedPtr<IMoviePipelineQueueTreeItem>>)
		.TreeItemsSource(&RootNodes)
		.ClearSelectionOnClick(false)
		.OnSelectionChanged(this, &SMoviePipelineQueueEditor::OnJobSelectionChanged_Impl)
		.OnGenerateRow(this, &SMoviePipelineQueueEditor::OnGenerateRow)
		.OnGetChildren(this, &SMoviePipelineQueueEditor::OnGetChildren)
		.OnContextMenuOpening(this, &SMoviePipelineQueueEditor::GetContextMenuContent)
		.IsEnabled_Lambda([]()
		{
			const UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
			check(Subsystem);
			
			return !Subsystem->IsRendering();
		})
		.HeaderRow
		(
			SNew(SHeaderRow)

			+ SHeaderRow::Column(SQueueJobListRow::NAME_Enabled)
			.FixedWidth(24.f)
			.DefaultLabel(FText::FromString(TEXT(" ")))

			+ SHeaderRow::Column(SQueueJobListRow::NAME_JobName)
			.FillWidth(0.25f)
			.MinSize(80.f)
			.DefaultLabel(LOCTEXT("QueueHeaderJobName_Text", "Job"))

			+ SHeaderRow::Column(SQueueJobListRow::NAME_Settings)
			.FillWidth(0.25f)
			.MinSize(80.f)
			.DefaultLabel(LOCTEXT("QueueHeaderSettings_Text", "Configuration"))

			+ SHeaderRow::Column(SQueueJobListRow::NAME_Output)
			.FillWidth(0.45f)
			.MinSize(80.f)
			.DefaultLabel(LOCTEXT("QueueHeaderOutput_Text", "Output"))

			+ SHeaderRow::Column(SQueueJobListRow::NAME_Status)
			.FixedWidth(80)
			.DefaultLabel(LOCTEXT("QueueHeaderStatus_Text", "Status"))
		);

	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SMoviePipelineQueueEditor::OnDeleteSelected),
		FCanExecuteAction::CreateSP(this, &SMoviePipelineQueueEditor::CanDeleteSelected)
	);

	CommandList->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &SMoviePipelineQueueEditor::OnDuplicateSelected),
		FCanExecuteAction::CreateSP(this, &SMoviePipelineQueueEditor::CanDuplicateSelected)
	);

	CommandList->MapAction(
		FMoviePipelineCommands::Get().ResetStatus,
		FExecuteAction::CreateSP(this, &SMoviePipelineQueueEditor::OnResetStatus)
	);

	ChildSlot
	[
		SNew(SDropTarget)
		.OnDropped(this, &SMoviePipelineQueueEditor::OnDragDropTarget)
		.OnAllowDrop(this, &SMoviePipelineQueueEditor::CanDragDropTarget)
		.OnIsRecognized(this, &SMoviePipelineQueueEditor::CanDragDropTarget)
		[
			TreeView.ToSharedRef()
		]
	];

	// When undo occurs, get a notification so we can make sure our view is up to date
	GEditor->RegisterForUndo(this);

	// React to when a new queue is loaded via the subsystem
	UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);
	Subsystem->OnQueueLoaded.AddSP(this, &SMoviePipelineQueueEditor::OnQueueLoaded);
}


TSharedPtr<SWidget> SMoviePipelineQueueEditor::GetContextMenuContent()
{
	FMenuBuilder MenuBuilder(true, CommandList);
	MenuBuilder.BeginSection("Edit");
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
	MenuBuilder.AddMenuEntry(FMoviePipelineCommands::Get().ResetStatus);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SMoviePipelineQueueEditor::MakeAddSequenceJobButton()
{
	return SNew(SPositiveActionButton)
		.OnGetMenuContent(this, &SMoviePipelineQueueEditor::OnGenerateNewJobFromAssetMenu)
		.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		.Text(LOCTEXT("AddNewJob_Text", "Render"));
}

TSharedRef<SWidget> SMoviePipelineQueueEditor::RemoveSelectedJobButton()
{
	return SNew(SButton)
		.ContentPadding(MoviePipeline::ButtonPadding)
		.IsEnabled(this, &SMoviePipelineQueueEditor::CanDeleteSelected)
		.OnClicked(this, &SMoviePipelineQueueEditor::DeleteSelected)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "NormalText.Important")
			.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
			.Text(FEditorFontGlyphs::Minus)
		];
}

TSharedRef<SWidget> SMoviePipelineQueueEditor::OnGenerateNewJobFromAssetMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bSortByPathInColumnView = false;
		AssetPickerConfig.ThumbnailScale = 0.4f;
		AssetPickerConfig.SaveSettingsName = TEXT("MoviePipelineQueueJobAsset");

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoSequences_Warning", "No Level Sequences Found");
		AssetPickerConfig.Filter.ClassPaths.Add(ULevelSequence::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SMoviePipelineQueueEditor::OnCreateJobFromAsset);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("NewJob_MenuSection", "New Render Job"));
	{
		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.WidthOverride(300.f)
			.HeightOverride(300.f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SMoviePipelineQueueEditor::AssignDefaultGraphPresetToJob(UMoviePipelineExecutorJob* InJob)
{
	UMoviePipelineEditorBlueprintLibrary::AssignDefaultGraphPresetToJob(InJob);
}

UE_ENABLE_OPTIMIZATION_SHIP

void SMoviePipelineQueueEditor::OnCreateJobFromAsset(const FAssetData& InAsset)
{
	// Close the dropdown menu that showed them the assets to pick from.
	FSlateApplication::Get().DismissAllMenus();

	UMoviePipelineQueue* ActiveQueue = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>()->GetQueue();
	check(ActiveQueue);
	
	FScopedTransaction Transaction(FText::Format(LOCTEXT("CreateJob_Transaction", "Add {0}|plural(one=Job, other=Jobs)"), 1));

	TArray<UMoviePipelineExecutorJob*> NewJobs;
	// Only try to initialize level sequences, in the event they had more than a level sequence selected when drag/dropping.
	if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(InAsset.GetAsset()))
	{		
		UMoviePipelineExecutorJob* NewJob = UMoviePipelineEditorBlueprintLibrary::CreateJobFromSequence(ActiveQueue, LevelSequence);
		if (!NewJob)
		{
			return;
		}

		NewJobs.Add(NewJob);
	}
	else if (UMoviePipelineQueue* Queue = Cast<UMoviePipelineQueue>(InAsset.GetAsset()))
	{
		for (UMoviePipelineExecutorJob* Job : Queue->GetJobs())
		{
			UMoviePipelineExecutorJob* NewJob = ActiveQueue->DuplicateJob(Job);
			if (NewJob)
			{
				NewJobs.Add(NewJob);
			}
		}
	}

	for (UMoviePipelineExecutorJob* NewJob : NewJobs)
	{
		PendingJobsToSelect.Add(NewJob);
		UMoviePipelineEditorBlueprintLibrary::ApplyDefaultConfigurationTypeToJob(NewJob);
	}
}

void SMoviePipelineQueueEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UMoviePipelineQueue* ActiveQueue = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>()->GetQueue();
	check(ActiveQueue);

	if (ActiveQueue)
	{
		if (CachedQueueSerialNumber != ActiveQueue->GetQueueSerialNumber())
		{
			ReconstructTree();
		}
	}
	// The sources are no longer valid, so we expect our cached serial number to be -1. If not, we haven't reset the tree yet.
	else if (CachedQueueSerialNumber != uint32(-1))
	{
		ReconstructTree();
	}

	if (PendingJobsToSelect.Num() > 0)
	{
		SetSelectedJobs_Impl(PendingJobsToSelect);
		PendingJobsToSelect.Empty();
	}
}

void SMoviePipelineQueueEditor::PostUndo(bool bSuccess)
{
	CachedQueueSerialNumber++;
}

void SMoviePipelineQueueEditor::ReconstructTree()
{
	UMoviePipelineQueue* ActiveQueue = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>()->GetQueue();
	check(ActiveQueue);
	if (!ActiveQueue)
	{
		CachedQueueSerialNumber = uint32(-1);
		RootNodes.Reset();
		return;
	}

	CachedQueueSerialNumber = ActiveQueue->GetQueueSerialNumber();

	// TSortedMap<FString, TSharedPtr<FMoviePipelineQueueJobTreeItem>> RootJobs;
	// for (TSharedPtr<IMoviePipelineQueueTreeItem> RootItem : RootNodes)
	// {
	// 	TSharedPtr<FMoviePipelineQueueJobTreeItem> RootCategory = RootItem->AsJob();
	// 	if (RootCategory.IsValid())
	// 	{
	// 		RootCategory->Children.Reset();
	// 		RootJobs.Add(RootCategory->Category.ToString(), RootCategory);
	// 	}
	// }

	RootNodes.Reset();

	// We attempt to re-use tree items in order to maintain selection states on them
	// TMap<FObjectKey, TSharedPtr<FTakeRecorderSourceTreeItem>> OldSourceToTreeItem;
	// Swap(SourceToTreeItem, OldSourceToTreeItem);

	for (UMoviePipelineExecutorJob* Job : ActiveQueue->GetJobs())
	{
		if (!Job)
		{
			continue;
		}

		TSharedPtr<FMoviePipelineQueueJobTreeItem> JobTreeItem = MakeShared<FMoviePipelineQueueJobTreeItem>(Job, OnEditConfigRequested, OnPresetChosen);

		// Add Shots
		for (UMoviePipelineExecutorShot* ShotInfo : Job->ShotInfo)
		{
			TSharedPtr<FMoviePipelineShotItem> Shot = MakeShared<FMoviePipelineShotItem>(Job, ShotInfo, OnEditConfigRequested, OnPresetChosen);
			JobTreeItem->Children.Add(Shot);
		}

		RootNodes.Add(JobTreeItem);
	}

	TreeView->RequestTreeRefresh();
}

FReply SMoviePipelineQueueEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<ITableRow> SMoviePipelineQueueEditor::OnGenerateRow(TSharedPtr<IMoviePipelineQueueTreeItem> Item, const TSharedRef<STableViewBase>& Tree)
{
	// Let the item construct itself.
	return Item->ConstructWidget(SharedThis(this), Tree);
}

void SMoviePipelineQueueEditor::OnGetChildren(TSharedPtr<IMoviePipelineQueueTreeItem> Item, TArray<TSharedPtr<IMoviePipelineQueueTreeItem>>& OutChildItems)
{
	TSharedPtr<FMoviePipelineQueueJobTreeItem> Job = Item->AsJob();
	if (Job.IsValid())
	{
		OutChildItems.Append(Job->Children);
	}
}

FReply SMoviePipelineQueueEditor::OnDragDropTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (TSharedPtr<FAssetDragDropOp> AssetDragDrop = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("CreateJob_Transaction", "Add {0}|plural(one=Job, other=Jobs)"), AssetDragDrop->GetAssets().Num()));

		for (const FAssetData& Asset : AssetDragDrop->GetAssets())
		{
			OnCreateJobFromAsset(Asset);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SMoviePipelineQueueEditor::CanDragDropTarget(TSharedPtr<FDragDropOperation> InOperation)
{
	bool bIsValid = false;
	if (InOperation)
	{
		if (InOperation->IsOfType<FAssetDragDropOp>())
		{
			TSharedPtr<FAssetDragDropOp> AssetDragDrop = StaticCastSharedPtr<FAssetDragDropOp>(InOperation);
			for (const FAssetData& Asset : AssetDragDrop->GetAssets())
			{
				ULevelSequence* LevelSequence = Cast<ULevelSequence>(Asset.GetAsset());
				if (LevelSequence)
				{
					// If at least one of them is a Level Sequence then we'll accept the drop.
					bIsValid = true;
					break;
				}

				UMoviePipelineQueue* QueueAsset = Cast<UMoviePipelineQueue>(Asset.GetAsset());
				if (QueueAsset)
				{
					bIsValid = true;
					break;
				}
			}
		}
	}

	return bIsValid;
}

FReply SMoviePipelineQueueEditor::DeleteSelected()
{
	UMoviePipelineQueue* ActiveQueue = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>()->GetQueue();
	check(ActiveQueue);

	if (ActiveQueue)
	{
		TArray<TSharedPtr<IMoviePipelineQueueTreeItem>> Items = TreeView->GetSelectedItems();

		FScopedTransaction Transaction(FText::Format(LOCTEXT("DeleteSelection", "Delete Selected {0}|plural(one=Job, other=Jobs)"), Items.Num()));
		ActiveQueue->Modify();

		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : Items)
		{
			Item->Delete(ActiveQueue);
		}
	}

	return FReply::Handled();
}

void SMoviePipelineQueueEditor::OnDeleteSelected()
{
	DeleteSelected();
}

bool SMoviePipelineQueueEditor::CanDeleteSelected() const
{
	return true;
}

void SMoviePipelineQueueEditor::OnDuplicateSelected()
{
	UMoviePipelineQueue* ActiveQueue = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>()->GetQueue();
	check(ActiveQueue);

	if (ActiveQueue)
	{
		TArray<TSharedPtr<IMoviePipelineQueueTreeItem>> Items = TreeView->GetSelectedItems();

		FScopedTransaction Transaction(FText::Format(LOCTEXT("DuplicateSelection", "Duplicate Selected {0}|plural(one=Job, other=Jobs)"), Items.Num()));
		ActiveQueue->Modify();

		TArray<UMoviePipelineExecutorJob*> NewJobs;
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : Items)
		{
			UMoviePipelineExecutorJob* NewJob = Item->Duplicate(ActiveQueue);
			if (NewJob)
			{
				NewJobs.Add(NewJob);
			}
		}

		PendingJobsToSelect = NewJobs;
	}
}

bool SMoviePipelineQueueEditor::CanDuplicateSelected() const
{
	return true;
}

void SMoviePipelineQueueEditor::OnResetStatus()
{
	UMoviePipelineQueue* ActiveQueue = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>()->GetQueue();
	check(ActiveQueue);

	if (ActiveQueue)
	{
		TArray<TSharedPtr<IMoviePipelineQueueTreeItem>> Items = TreeView->GetSelectedItems();

		FScopedTransaction Transaction(FText::Format(LOCTEXT("ResetStatus", "Reset Status on {0}|plural(one=Job, other=Jobs)"), Items.Num()));
		ActiveQueue->Modify();

		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : Items)
		{
			Item->ResetStatus();
		}
	}
}

void SMoviePipelineQueueEditor::SetSelectedJobs_Impl(const TArray<UMoviePipelineExecutorJob*>& InJobs)
{
	TreeView->ClearSelection();

	TArray<TSharedPtr<IMoviePipelineQueueTreeItem>> AllTreeItems;

	// Get all of our items first
	for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : RootNodes)
	{
		AllTreeItems.Add(Item);
		OnGetChildren(Item, AllTreeItems);
	}


	TArray<TSharedPtr<IMoviePipelineQueueTreeItem>> SelectedTreeItems;
	for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : AllTreeItems)
	{
		TSharedPtr<FMoviePipelineQueueJobTreeItem> JobTreeItem = Item->AsJob();
		if (JobTreeItem.IsValid())
		{
			if (InJobs.Contains(JobTreeItem->WeakJob.Get()))
			{
				SelectedTreeItems.Add(Item);
			}
		}
	}

	TreeView->SetItemSelection(SelectedTreeItems, true, ESelectInfo::Direct);
}

void SMoviePipelineQueueEditor::OnJobSelectionChanged_Impl(TSharedPtr<IMoviePipelineQueueTreeItem> TreeItem, ESelectInfo::Type SelectInfo)
{
	TArray<UMoviePipelineExecutorJob*> SelectedJobs;
	TArray<UMoviePipelineExecutorShot*> SelectedShots;
	
	if (TreeItem.IsValid())
	{
		// Iterate the tree and get all selected items.
		TArray<TSharedPtr<IMoviePipelineQueueTreeItem>> SelectedTreeItems = TreeView->GetSelectedItems();
		for (TSharedPtr<IMoviePipelineQueueTreeItem> Item : SelectedTreeItems)
		{
			if (UMoviePipelineExecutorJob* Job = Item->GetOwningJob())
			{
				SelectedJobs.AddUnique(Job);
			}

			if (UMoviePipelineExecutorShot* Shot = Item->GetOwningShot())
			{
				SelectedShots.AddUnique(Shot);
			}
		}
	}

	OnJobSelectionChanged.ExecuteIfBound(SelectedJobs, SelectedShots);
}

TArray<UMoviePipelineExecutorJob*> SMoviePipelineQueueEditor::GetSelectedJobs() const
{
	TArray<UMoviePipelineExecutorJob*> Jobs;
	for (const TSharedPtr<IMoviePipelineQueueTreeItem>& Item : TreeView->GetSelectedItems())
	{
		if (UMoviePipelineExecutorJob* Job = Item->GetOwningJob())
		{
			Jobs.AddUnique(Job);
		}
	}
	return Jobs;
}

void SMoviePipelineQueueEditor::OnQueueLoaded()
{
	const UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);
	
	// Automatically select the first job in the queue
	TArray<UMoviePipelineExecutorJob*> Jobs;
	if (Subsystem->GetQueue()->GetJobs().Num() > 0)
	{
		Jobs.Add(Subsystem->GetQueue()->GetJobs()[0]);
	}
	
	// Go through the UI so it updates the UI selection too and then this will loop back
	// around to OnSelectionChanged to update ourself.
	SetSelectedJobs(Jobs);
}

#undef LOCTEXT_NAMESPACE
