// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"
#include "Types/SlateEnums.h"
#include "EditorUndoClient.h"

struct FAssetData;
struct IMoviePipelineQueueTreeItem;
template<typename> class STreeView;
class FUICommandList;
class ITableRow;
class STableViewBase;
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;
class SWindow;
class UMoviePipelineQueue;
class SMoviePipelineQueueEditor;
class UMovieSceneCinematicShotSection;
struct FMoviePipelineQueueJobTreeItem;

DECLARE_DELEGATE_TwoParams(FOnMoviePipelineEditConfig, TWeakObjectPtr<UMoviePipelineExecutorJob>, TWeakObjectPtr<UMoviePipelineExecutorShot>)
DECLARE_DELEGATE_TwoParams(FOnMoviePipelineJobSelection, const TArray<UMoviePipelineExecutorJob*>&, const TArray<UMoviePipelineExecutorShot*>&)

/**
 * Widget used to edit a Movie Pipeline Queue
 */
class SMoviePipelineQueueEditor : public SCompoundWidget, public FEditorUndoClient
{
public:
	

	SLATE_BEGIN_ARGS(SMoviePipelineQueueEditor)
		{}
		SLATE_EVENT(FOnMoviePipelineEditConfig, OnEditConfigRequested)
		SLATE_EVENT(FOnMoviePipelineEditConfig, OnPresetChosen)
		SLATE_EVENT(FOnMoviePipelineJobSelection, OnJobSelectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	TSharedRef<SWidget> MakeAddSequenceJobButton();
	TSharedRef<SWidget> RemoveSelectedJobButton();
	TSharedRef<SWidget> OnGenerateNewJobFromAssetMenu();

	TArray<TSharedPtr<IMoviePipelineQueueTreeItem>> GetSelectedItems() const { return TreeView->GetSelectedItems(); }
	void SetSelectedJobs(const TArray<UMoviePipelineExecutorJob*>& InJobs) { PendingJobsToSelect = InJobs; }

	/** Returns the currently selected jobs. */
	TArray<UMoviePipelineExecutorJob*> GetSelectedJobs() const;

	/** Get the default graph preset from the project settings and assign it to the given job */
	static void AssignDefaultGraphPresetToJob(UMoviePipelineExecutorJob* InJob);

private:
	// SWidget Interface
	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);
	// ~SWidget Interface

	// FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// ~FEditorUndoClient

private:
	
	void OnCreateJobFromAsset(const FAssetData& InAsset);

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<IMoviePipelineQueueTreeItem> Item, const TSharedRef<STableViewBase>& Tree);

	void OnGetChildren(TSharedPtr<IMoviePipelineQueueTreeItem> Item, TArray<TSharedPtr<IMoviePipelineQueueTreeItem>>& OutChildItems);

	FReply OnDragDropTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);

	bool CanDragDropTarget(TSharedPtr<FDragDropOperation> InOperation);

	TSharedPtr<SWidget> GetContextMenuContent();

	void OnDeleteSelected();
	bool CanDeleteSelected() const;
	FReply DeleteSelected();

	void OnDuplicateSelected();
	bool CanDuplicateSelected() const;

	void OnResetStatus();

	void ReconstructTree();
	void SetSelectedJobs_Impl(const TArray<UMoviePipelineExecutorJob*>& InJobs);
	void OnJobSelectionChanged_Impl(TSharedPtr<IMoviePipelineQueueTreeItem> TreeItem, ESelectInfo::Type SelectInfo);

	/** Called when a new queue is loaded in the queue subsystem. */
	void OnQueueLoaded();

private:
	TArray<TSharedPtr<IMoviePipelineQueueTreeItem>> RootNodes;
	TSharedPtr<STreeView<TSharedPtr<IMoviePipelineQueueTreeItem>>> TreeView;
	TSharedPtr<FUICommandList> CommandList;
	uint32 CachedQueueSerialNumber;
	TArray<UMoviePipelineExecutorJob*> PendingJobsToSelect;
	FOnMoviePipelineEditConfig OnEditConfigRequested;
	FOnMoviePipelineEditConfig OnPresetChosen;
	FOnMoviePipelineJobSelection OnJobSelectionChanged;
};