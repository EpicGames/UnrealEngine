// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMoviePipelineQueuePanel.h"

#include "Customizations/Graph/MovieGraphNamedResolutionCustomization.h"
#include "Customizations/JobCustomization.h"
#include "Graph/MovieGraphAssetToolkit.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieRenderGraphEditorSettings.h"
#include "MovieRenderPipelineEditorUtils.h"
#include "MovieRenderPipelineSettings.h"
#include "MovieRenderPipelineStyle.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineQueuePanelUtils.h"
#include "MoviePipelineQueueSubsystem.h"
#include "MoviePipelineTelemetry.h"
#include "SMoviePipelineQueueEditor.h"
#include "SMoviePipelineConfigPanel.h"
#include "Widgets/MoviePipelineWidgetConstants.h"
#include "Widgets/SMoviePipelineAfterRenderWidget.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "SPositiveActionButton.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#include "AssetRegistry/AssetData.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineQueuePanel"

namespace
{
	// GEditorPerProjectIni section and keys for window state persisted by the queue.
	const TCHAR* GQueueSettingsSection = TEXT("MovieRenderPipelineQueue");
	const TCHAR* GDetailsSplitterValueKey = TEXT("DetailsSplitterValue");
	const TCHAR* GLastQueueRenderWasLocalKey = TEXT("LastQueueRenderWasLocal");
}

void SMoviePipelineQueuePanel::Construct(const FArguments& InArgs)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bLockable = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea;
	DetailsViewArgs.bShowObjectLabel = true;
	DetailsViewArgs.bCustomNameAreaLocation = true;
	DetailsViewArgs.bCustomFilterAreaLocation = true;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.ColumnWidth = 0.7f;

	// Extend the built-in gear/settings menu with a "Default Configuration Type" section.
	const TSharedPtr<FExtender> OptionsExtender = MakeShareable(new FExtender);
	OptionsExtender->AddMenuExtension(
		FName("DetailsViewShowOptions"),
		EExtensionHook::Before,
		nullptr,
		FMenuExtensionDelegate::CreateLambda([](FMenuBuilder& MenuBuilder)
		{
			UMovieRenderGraphEditorSettings* Settings = GetMutableDefault<UMovieRenderGraphEditorSettings>();

			MenuBuilder.BeginSection(NAME_None, LOCTEXT("DefaultConfigTypeSection", "Default Configuration Type"));

			auto AddConfigTypeEntry = [&](const FText& Label, EMoviePipelineDefaultConfigType Type)
			{
				MenuBuilder.AddMenuEntry(
					Label, FText::GetEmpty(), FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([Settings, Type]()
						{
							Settings->DefaultConfigType = Type;
							Settings->SaveConfig();
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([Settings, Type]()
						{
							return Settings->DefaultConfigType == Type;
						})
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			};

			AddConfigTypeEntry(LOCTEXT("ConfigType_Basic", "Basic"), EMoviePipelineDefaultConfigType::Basic);
			AddConfigTypeEntry(LOCTEXT("ConfigType_Graph", "Movie Render Graph"), EMoviePipelineDefaultConfigType::MovieRenderGraph);
			AddConfigTypeEntry(LOCTEXT("ConfigType_Preset", "Legacy Preset"), EMoviePipelineDefaultConfigType::LegacyPreset);

			MenuBuilder.EndSection();
		})
	);
	DetailsViewArgs.OptionsExtender = OptionsExtender;

	JobDetailsPanelWidget = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	JobDetailsPanelWidget->RegisterInstancedCustomPropertyLayout(
		UMoviePipelineExecutorJob::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FJobDetailsCustomization::MakeInstance));

	JobDetailsPanelWidget->RegisterInstancedCustomPropertyLayout(
		UMoviePipelineExecutorShot::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FJobDetailsCustomization::MakeInstance));

	JobDetailsPanelWidget->RegisterInstancedCustomPropertyTypeLayout(
		FMovieGraphNamedResolution::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieGraphNamedResolutionCustomization::MakeInstance));

	PipelineQueueEditorWidget = SNew(SMoviePipelineQueueEditor)
		.OnEditConfigRequested(this, &SMoviePipelineQueuePanel::OnEditJobConfigRequested)
		.OnPresetChosen(this, &SMoviePipelineQueuePanel::OnJobPresetChosen)
		.OnJobSelectionChanged(this, &SMoviePipelineQueuePanel::OnSelectionChanged);

	// Restore the persisted details splitter width (per-user, GEditorPerProjectIni). Falls through to default
	// on first launch when the key is absent.
	GConfig->GetFloat(GQueueSettingsSection, GDetailsSplitterValueKey, DetailsSplitterValue, GEditorPerProjectIni);
	DetailsSplitterValue = FMath::Clamp(DetailsSplitterValue, 0.05f, 0.95f);

	const UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);
	UMoviePipelineQueue* Queue = Subsystem->GetQueue();
	check(Queue);

	// Auto-select the first job so the details panel isn't empty on open
	{
		const TArray<UMoviePipelineExecutorJob*> CurrentJobs = Queue->GetJobs();
		if (!CurrentJobs.IsEmpty())
		{
			PipelineQueueEditorWidget->SetSelectedJobs({ CurrentJobs[0] });
		}
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		// Toolbar
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.f, 1.0f))
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
			[
				SNew(SHorizontalBox)

				// Toolbar (FToolBarBuilder-generated)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					MakeToolbar()
				]

				// Spacer
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNullWidget::NullWidget
				]

				// Queue save/load dropdown (right-aligned, stays outside FToolBarBuilder)
				+ SHorizontalBox::Slot()
				.Padding(MoviePipeline::ButtonOffset)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(SComboButton)
					.ToolTipText(LOCTEXT("QueueManagementButton_Tooltip", "Export the current queue to an asset, or load a previously saved queue."))
					.ContentPadding(MoviePipeline::ButtonPadding)
					.ComboButtonStyle(FMovieRenderPipelineStyle::Get(), "ComboButton")
					.OnGetMenuContent(this, &SMoviePipelineQueuePanel::OnGenerateSavedQueuesMenu)
					.ForegroundColor(FSlateColor::UseForeground())
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(0, 1, 4, 0)
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("AssetEditor.SaveAsset"))
						]

						+ SHorizontalBox::Slot()
						.Padding(0, 1, 0, 0)
						[
							SNew(STextBlock)
							.Text(this, &SMoviePipelineQueuePanel::GetQueueMenuButtonText)
						]
					]
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Horizontal)
		]

		// Main queue body
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(QueueDetailsSplitter, SSplitter)
			.Orientation(EOrientation::Orient_Horizontal)
			.OnSplitterFinishedResizing(this, &SMoviePipelineQueuePanel::OnDetailsSplitterFinishedResizing)
			+ SSplitter::Slot()
			.Value(TAttribute<float>::CreateLambda([this](){ return 1.f - DetailsSplitterValue; }))
			[
				PipelineQueueEditorWidget.ToSharedRef()
			]
			+ SSplitter::Slot()
			.Value(TAttribute<float>::CreateLambda([this](){ return DetailsSplitterValue; }))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(1.f, 1.0f))
				.Content()
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex(this, &SMoviePipelineQueuePanel::GetDetailsViewWidgetIndex)
					.IsEnabled(this, &SMoviePipelineQueuePanel::IsDetailsViewEnabled)
					+ SWidgetSwitcher::Slot()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(6.0f, 2.0f, 2.0f, 0.0f)
						[
							JobDetailsPanelWidget->GetNameAreaWidget().ToSharedRef()
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							JobDetailsPanelWidget->GetFilterAreaWidget().ToSharedRef()
						]
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							JobDetailsPanelWidget.ToSharedRef()
						]
					]
					+ SWidgetSwitcher::Slot()
					.Padding(2.0f, 24.0f, 2.0f, 2.0f)
					[
						SNew(SBox)
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.Content()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("NoJobSelected", "Select a job to view details."))
						]
					]
				]
			]
		]

	];
}

TSharedRef<SWidget> SMoviePipelineQueuePanel::MakeToolbar()
{
	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetStyle(&FMovieRenderPipelineStyle::Get(), "MovieRenderPipeline.ToolBar");
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	// Section 1: Save / Find
	ToolbarBuilder.BeginSection("SaveFind");
	{
		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::OnSaveButtonClicked)),
			NAME_None,
			LOCTEXT("SaveQueueButton_Text", "Save"),
			LOCTEXT("SaveQueueButton_Tooltip", "Save the current queue. If the queue has not been saved before, prompts for a save location."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset"));

		ToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::OnFindInContentBrowserClicked),
				FCanExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::IsFindInContentBrowserEnabled)),
			NAME_None,
			LOCTEXT("FindInContentBrowser_Text", "Browse"),
			LOCTEXT("FindInContentBrowser_Tooltip", "Select the job's level sequence in the Content Browser."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BrowseContent"));

	}
	ToolbarBuilder.EndSection();

	// Section 2: Add / Render
	ToolbarBuilder.BeginSection("AddRender");
	{
		const FToolBarStyle& ToolBarStyle = ToolbarBuilder.GetStyleSet()->GetWidgetStyle<FToolBarStyle>(ToolbarBuilder.GetStyleName());

		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.VAlign(VAlign_Fill)
			.Padding(ToolBarStyle.ButtonPadding)
			[
				SNew(SPositiveActionButton)
				.OnGetMenuContent_Lambda([this]() -> TSharedRef<SWidget>
				{
					return PipelineQueueEditorWidget->OnGenerateNewJobFromAssetMenu();
				})
				.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
				.Text(LOCTEXT("AddNewJob_Text", "Add"))
			]);

		ToolbarBuilder.AddSeparator();

		ToolbarBuilder.SetLabelVisibility(EVisibility::Visible);
		ToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::OnRenderClicked),
				FCanExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::IsRenderEnabled)),
			NAME_None,
			TAttribute<FText>::CreateSP(this, &SMoviePipelineQueuePanel::GetRenderButtonLabel),
			LOCTEXT("RenderQueue_Tooltip", "Render the current queue using the selected render mode."),
			TAttribute<FSlateIcon>::CreateLambda([this]
			{
				return FSlateIcon(
					FMovieRenderPipelineStyle::Get().GetStyleSetName(),
					ShouldRenderLocal()
						? "MovieRenderPipeline.QuickRender.Icon.MovieRenderQueueMode"
						: "MovieRenderPipeline.Icon.RenderRemote");
			}));
		ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SMoviePipelineQueuePanel::OnGenerateRenderMenu),
			LOCTEXT("RenderOptions_Text", "Options"),
			LOCTEXT("RenderOptions_Tooltip", "Change the active render mode."),
			FSlateIcon(),
			/*bSimpleComboBox*/ true);

	}
	ToolbarBuilder.EndSection();

	// Section 3: After Render
	ToolbarBuilder.BeginSection("AfterRender");
	{
		ToolbarBuilder.AddWidget(
			SNew(SMoviePipelineAfterRenderWidget));
	}
	ToolbarBuilder.EndSection();

	return ToolbarBuilder.MakeWidget();
}

void SMoviePipelineQueuePanel::OnDetailsSplitterFinishedResizing()
{
	if (!QueueDetailsSplitter.IsValid())
	{
		return;
	}

	// SlotAt(1) is the right (details) slot. SSplitter normalizes the pair, so we only persist one value.
	DetailsSplitterValue = QueueDetailsSplitter->SlotAt(1).GetSizeValue();
	GConfig->SetFloat(GQueueSettingsSection, GDetailsSplitterValueKey, DetailsSplitterValue, GEditorPerProjectIni);
}

void SMoviePipelineQueuePanel::OnRenderLocalRequested()
{
	UE::MovieRenderPipelineEditor::Private::PerformLocalRender();
}

bool SMoviePipelineQueuePanel::IsRenderLocalEnabled() const
{
	const bool bConfigWindowIsOpen = WeakEditorWindow.IsValid();
	return UE::MovieRenderPipelineEditor::Private::CanPerformLocalRender() && !bConfigWindowIsOpen;
}

void SMoviePipelineQueuePanel::OnRenderRemoteRequested()
{
	UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);

	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	TSubclassOf<UMoviePipelineExecutorBase> ExecutorClass = ProjectSettings->DefaultRemoteExecutor.TryLoadClass<UMoviePipelineExecutorBase>();

	// OnRenderRemoteRequested should only get called if IsRenderRemoteEnabled() returns true, meaning there's a valid class.
	check(ExecutorClass != nullptr);

	Subsystem->RenderQueueWithExecutor(ExecutorClass);

	constexpr bool bIsLocal = false;
	FMoviePipelineTelemetry::SendRendersRequestedTelemetry(bIsLocal, Subsystem->GetQueue()->GetJobs());
}

bool SMoviePipelineQueuePanel::IsRenderRemoteEnabled() const
{
	UMoviePipelineQueueSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>();
	check(Subsystem);

	const UMovieRenderPipelineProjectSettings* ProjectSettings = GetDefault<UMovieRenderPipelineProjectSettings>();
	const bool bHasExecutor = ProjectSettings->DefaultRemoteExecutor.TryLoadClass<UMoviePipelineExecutorBase>() != nullptr;
	const bool bNotRendering = !Subsystem->IsRendering();
	const bool bConfigWindowIsOpen = WeakEditorWindow.IsValid();

	bool bAtLeastOneJobAvailable = false;
	for (const UMoviePipelineExecutorJob* Job : Subsystem->GetQueue()->GetJobs())
	{
		if (!Job->IsConsumed() && Job->IsEnabled())
		{
			bAtLeastOneJobAvailable = true;
			break;
		}
	}

	return bHasExecutor && bNotRendering && bAtLeastOneJobAvailable && !bConfigWindowIsOpen;
}

bool SMoviePipelineQueuePanel::IsRenderEnabled() const
{
	return IsRenderLocalEnabled() || IsRenderRemoteEnabled();
}

TSharedRef<SWidget> SMoviePipelineQueuePanel::OnGenerateRenderMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RenderLocal_Text", "Render (Local)"),
		LOCTEXT("RenderLocal_Tooltip", "Renders the current queue in the current process using Play in Editor."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::OnRenderModeMenuEntryChosen, /*bIsLocal=*/true),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::IsRenderLocalEnabled),
			FIsActionChecked::CreateSP(this, &SMoviePipelineQueuePanel::ShouldRenderLocal)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RenderRemote_Text", "Render (Remote)"),
		LOCTEXT("RenderRemote_Tooltip", "Renders the current queue in a separate process."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::OnRenderModeMenuEntryChosen, /*bIsLocal=*/false),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::IsRenderRemoteEnabled),
			FIsActionChecked::CreateLambda([this](){ return !ShouldRenderLocal(); })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	return MenuBuilder.MakeWidget();
}

void SMoviePipelineQueuePanel::OnRenderModeMenuEntryChosen(bool bIsLocal)
{
	GConfig->SetBool(GQueueSettingsSection, GLastQueueRenderWasLocalKey, bIsLocal, GEditorPerProjectIni);
}

void SMoviePipelineQueuePanel::OnRenderClicked()
{
	if (ShouldRenderLocal())
	{
		OnRenderLocalRequested();
	}
	else
	{
		OnRenderRemoteRequested();
	}
}

FText SMoviePipelineQueuePanel::GetRenderButtonLabel() const
{
	return ShouldRenderLocal()
		? LOCTEXT("RenderLocal_Text", "Render (Local)")
		: LOCTEXT("RenderRemote_Text", "Render (Remote)");
}

bool SMoviePipelineQueuePanel::ShouldRenderLocal() const
{
	bool bLastWasLocal = true;
	GConfig->GetBool(GQueueSettingsSection, GLastQueueRenderWasLocalKey, bLastWasLocal, GEditorPerProjectIni);

	// If the saved mode is currently disabled (e.g. remote executor unavailable), fall back to the available one.
	if (bLastWasLocal && !IsRenderLocalEnabled() && IsRenderRemoteEnabled())
	{
		return false;
	}
	if (!bLastWasLocal && !IsRenderRemoteEnabled() && IsRenderLocalEnabled())
	{
		return true;
	}

	return bLastWasLocal;
}

void SMoviePipelineQueuePanel::OnJobPresetChosen(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot)
{
	UMovieRenderPipelineProjectSettings* ProjectSettings = GetMutableDefault<UMovieRenderPipelineProjectSettings>();
	if (!InShot.IsValid() && InJob.IsValid())
	{
		ProjectSettings->LastPresetOrigin = InJob->GetPresetOrigin();
	}
	ProjectSettings->SaveConfig();
}

void SMoviePipelineQueuePanel::OnEditJobConfigRequested(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot)
{
	if (!InJob.IsValid())
	{
		return;
	}

	// Only allow one config editor open at a time
	if (WeakEditorWindow.IsValid())
	{
		FWidgetPath ExistingWindowPath;
		if (FSlateApplication::Get().FindPathToWidget(WeakEditorWindow.Pin().ToSharedRef(), ExistingWindowPath, EVisibility::All))
		{
			WeakEditorWindow.Pin()->BringToFront();
			FSlateApplication::Get().SetAllUserFocus(ExistingWindowPath, EFocusCause::SetDirectly);
		}
		return;
	}

	// Determine whether to open a graph editor or the legacy config panel
	UMovieGraphConfig* GraphToEdit = nullptr;
	if (InShot.IsValid() && InShot->IsUsingGraphConfiguration())
	{
		GraphToEdit = InShot->GetGraphPreset();
		if (!GraphToEdit)
		{
			GraphToEdit = UE::MovieRenderPipelineEditor::Private::GenerateNewShotSubgraph(InJob.Get(), InShot.Get());
			if (!GraphToEdit)
			{
				return;
			}
		}
	}
	else if (InJob.IsValid() && InJob->IsUsingGraphConfiguration())
	{
		GraphToEdit = InJob->GetGraphPreset();
	}

	if (GraphToEdit)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(GraphToEdit);
		return;
	}

	// Legacy config panel path
	TSubclassOf<UMoviePipelineConfigBase> ConfigType;
	UMoviePipelineConfigBase* BasePreset = nullptr;
	UMoviePipelineConfigBase* BaseConfig = nullptr;
	if (InShot.IsValid())
	{
		ConfigType = UMoviePipelineShotConfig::StaticClass();
		BasePreset = InShot->GetShotOverridePresetOrigin();
		BaseConfig = InShot->GetShotOverrideConfiguration();
	}
	else
	{
		ConfigType = UMoviePipelinePrimaryConfig::StaticClass();
		BasePreset = InJob->GetPresetOrigin();
		BaseConfig = InJob->GetConfiguration();
	}

	TSharedRef<SWindow> EditorWindow =
		SNew(SWindow)
		.ClientSize(FVector2D(700, 600));

	TSharedRef<SMoviePipelineConfigPanel> ConfigEditorPanel =
		SNew(SMoviePipelineConfigPanel, ConfigType)
		.Job(InJob)
		.Shot(InShot)
		.OnConfigurationModified(this, &SMoviePipelineQueuePanel::OnConfigUpdatedForJob)
		.OnConfigurationSetToPreset(this, &SMoviePipelineQueuePanel::OnConfigUpdatedForJobToPreset)
		.BasePreset(BasePreset)
		.BaseConfig(BaseConfig);

	EditorWindow->SetContent(ConfigEditorPanel);

	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(EditorWindow, ParentWindow.ToSharedRef());
	}

	WeakEditorWindow = EditorWindow;
}

void SMoviePipelineQueuePanel::OnConfigWindowClosed()
{
	if (WeakEditorWindow.IsValid())
	{
		WeakEditorWindow.Pin()->RequestDestroyWindow();
	}
}

void SMoviePipelineQueuePanel::OnConfigUpdatedForJob(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot, UMoviePipelineConfigBase* InConfig)
{
	if (InJob.IsValid())
	{
		if (InShot.IsValid())
		{
			if (UMoviePipelineShotConfig* ShotConfig = Cast<UMoviePipelineShotConfig>(InConfig))
			{
				InShot->SetShotOverrideConfiguration(ShotConfig);
			}
		}
		else
		{
			if (UMoviePipelinePrimaryConfig* PrimaryConfig = Cast<UMoviePipelinePrimaryConfig>(InConfig))
			{
				InJob->SetConfiguration(PrimaryConfig);
			}
		}
	}

	OnConfigWindowClosed();
}

void SMoviePipelineQueuePanel::OnConfigUpdatedForJobToPreset(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot, UMoviePipelineConfigBase* InConfig)
{
	if (InJob.IsValid())
	{
		if (InShot.IsValid())
		{
			if (UMoviePipelineShotConfig* ShotConfig = Cast<UMoviePipelineShotConfig>(InConfig))
			{
				InShot->SetShotOverridePresetOrigin(ShotConfig);
			}
		}
		else
		{
			if (UMoviePipelinePrimaryConfig* PrimaryConfig = Cast<UMoviePipelinePrimaryConfig>(InConfig))
			{
				InJob->SetPresetOrigin(PrimaryConfig);
			}
		}
	}

	OnJobPresetChosen(InJob, InShot);
	OnConfigWindowClosed();
}

void SMoviePipelineQueuePanel::OnSelectionChanged(const TArray<UMoviePipelineExecutorJob*>& InSelectedJobs, const TArray<UMoviePipelineExecutorShot*>& InSelectedShots)
{
	// Prefer showing shot details when a shot is selected, otherwise show the job
	TArray<UObject*> ObjectsToDisplay;
	if (!InSelectedShots.IsEmpty())
	{
		ObjectsToDisplay.Append(InSelectedShots);
	}
	else
	{
		ObjectsToDisplay.Append(InSelectedJobs);
	}

	JobDetailsPanelWidget->SetObjects(ObjectsToDisplay);
	NumSelectedJobs = InSelectedJobs.Num();
}

int32 SMoviePipelineQueuePanel::GetDetailsViewWidgetIndex() const
{
	return NumSelectedJobs == 0;
}

bool SMoviePipelineQueuePanel::IsDetailsViewEnabled() const
{
	TArray<TWeakObjectPtr<UObject>> OutObjects = JobDetailsPanelWidget->GetSelectedObjects();

	for (const TWeakObjectPtr<UObject>& Object : OutObjects)
	{
		const UMoviePipelineExecutorJob* Job = Cast<UMoviePipelineExecutorJob>(Object);
		if (Job && Job->IsConsumed())
		{
			return false;
		}
	}

	return true;
}

TSharedRef<SWidget> SMoviePipelineQueuePanel::OnGenerateSavedQueuesMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	// Display the currently loaded queue name (or "None") as a disabled header entry
	static const FText NoLoadedQueueText = LOCTEXT("NoLoadedQueue", "None");
	static const FText LoadedQueueFormatText = LOCTEXT("CurrentQueueFormat", "Current Queue: {0}");
	const FString QueueName = GetQueueOriginName();
	const FText LoadedQueueName = FText::Format(
		LoadedQueueFormatText, !QueueName.IsEmpty() ? FText::FromString(QueueName) : NoLoadedQueueText);

	MenuBuilder.AddMenuEntry(
		LoadedQueueName,
		LoadedQueueName,
		FSlateIcon(),
		FUIAction(
			FExecuteAction(),
			FCanExecuteAction::CreateLambda([]() { return false; })),
		NAME_None,
		EUserInterfaceActionType::None
	);

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SaveQueue_Text", "Save Queue"),
		LOCTEXT("SaveQueue_Tip", "Save the current configuration in its existing preset which can be shared between multiple jobs, or imported later as the base of a new configuration."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAsset"),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::OnSaveAsset),
			FCanExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::IsQueueDirty)
		)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SaveAsQueue_Text", "Save Queue As"),
		LOCTEXT("SaveAsQueue_Tip", "Save the current configuration as a new preset that can be shared between multiple jobs, or imported later as the base of a new configuration."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAssetAs"),
		FUIAction(FExecuteAction::CreateSP(this, &SMoviePipelineQueuePanel::OnSaveAsAsset))
	);

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bSortByPathInColumnView = false;

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoQueueAssets_Warning", "No Queues Found");
		AssetPickerConfig.Filter.ClassPaths.Add(UMoviePipelineQueue::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SMoviePipelineQueuePanel::OnImportSavedQueueAsset);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("LoadQueue_MenuSection", "Import Queue"));
	{
		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.MinDesiredWidth(400.f)
			.MinDesiredHeight(400.f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FText SMoviePipelineQueuePanel::GetQueueMenuButtonText() const
{
	FText QueueName = FText::FromString(GetQueueOriginName());
	if (QueueName.IsEmpty())
	{
		QueueName = LOCTEXT("QueueSaveMenuUnsavedConfig_Text", "Unsaved Queue");
	}
	return FText::Format(FText::FromString(TEXT("{0}{1}")),
		QueueName,
		FText::FromString(IsQueueDirty() ? TEXT(" *") : TEXT("")));
}

void SMoviePipelineQueuePanel::OnSaveAsset()
{
	UE::MovieRenderPipelineEditor::Private::SaveQueue();
}

void SMoviePipelineQueuePanel::OnSaveAsAsset()
{
	UE::MovieRenderPipelineEditor::Private::SaveQueueAs();
}

void SMoviePipelineQueuePanel::OnImportSavedQueueAsset(const FAssetData& InPresetAsset) const
{
	FSlateApplication::Get().DismissAllMenus();
	UE::MovieRenderPipelineEditor::Private::ImportSavedQueueAsset(InPresetAsset);
}

bool SMoviePipelineQueuePanel::IsQueueDirty() const
{
	return UE::MovieRenderPipelineEditor::Private::IsQueueDirty();
}

FString SMoviePipelineQueuePanel::GetQueueOriginName() const
{
	return UE::MovieRenderPipelineEditor::Private::GetQueueOriginName();
}

void SMoviePipelineQueuePanel::OnSaveButtonClicked()
{
	// SaveQueue() already does Save As when there is no queue origin
	UE::MovieRenderPipelineEditor::Private::SaveQueue();
}

void SMoviePipelineQueuePanel::OnFindInContentBrowserClicked()
{
	for (UMoviePipelineExecutorJob* Job : PipelineQueueEditorWidget->GetSelectedJobs())
	{
		if (UObject* SequenceAsset = Job->Sequence.TryLoad())
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToAssets(TArray<UObject*>({ SequenceAsset }));
			return;
		}
	}
}

bool SMoviePipelineQueuePanel::IsFindInContentBrowserEnabled() const
{
	for (const UMoviePipelineExecutorJob* Job : PipelineQueueEditorWidget->GetSelectedJobs())
	{
		if (Job->Sequence.IsValid())
		{
			return true;
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE // SMoviePipelineQueuePanel
