// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UMoviePipelineConfigBase;
class UMoviePipelineQueue;
class SMoviePipelineQueueEditor;
class SWindow;
class UMoviePipelineExecutorJob;
class UMoviePipelineExecutorShot;
class IDetailsView;
struct FAssetData;

/**
 * Provides the UI for managing render jobs: adding/removing sequences, editing
 * graph/basic/legacy configs, saving/loading queue presets, and dispatching
 * local or remote renders.
 */
class SMoviePipelineQueuePanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMoviePipelineQueuePanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Kicks off a local (in-process) render of the queue. */
	void OnRenderLocalRequested();
	bool IsRenderLocalEnabled() const;

	/** Kicks off a remote render of the queue via the configured executor. */
	void OnRenderRemoteRequested();
	bool IsRenderRemoteEnabled() const;

	/** Returns true if any render option (local or remote) is currently available. */
	bool IsRenderEnabled() const;

	/** Builds the render mode dropdown menu (local/remote). */
	TSharedRef<SWidget> OnGenerateRenderMenu();

	/** Invokes the effective render mode shown on the primary render button. */
	void OnRenderClicked();

	/** Records the chosen render mode. */
	void OnRenderModeMenuEntryChosen(bool bIsLocal);

	/** Returns the last-chosen render mode for display on the Render button. */
	FText GetRenderButtonLabel() const;

	/** Returns whether a render (via the Render button) should perform a local render (vs. a remote render). */
	bool ShouldRenderLocal() const;

	/** Opens the config editor (graph or legacy) for the given job/shot. */
	void OnEditJobConfigRequested(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot);

	/** Called when an existing preset is chosen for a job or shot. */
	void OnJobPresetChosen(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot);

	/** Applies a modified config back to the owning job/shot. */
	void OnConfigUpdatedForJob(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot, UMoviePipelineConfigBase* InConfig);

	/** Applies a preset-based config back to the owning job/shot. */
	void OnConfigUpdatedForJobToPreset(TWeakObjectPtr<UMoviePipelineExecutorJob> InJob, TWeakObjectPtr<UMoviePipelineExecutorShot> InShot, UMoviePipelineConfigBase* InConfig);

	/** Destroys the currently open config editor window. */
	void OnConfigWindowClosed();

	/** Responds to job/shot selection changes in the queue editor. */
	void OnSelectionChanged(const TArray<UMoviePipelineExecutorJob*>& InSelectedJobs, const TArray<UMoviePipelineExecutorShot*>& InSelectedShots);

	/** Returns the widget-switcher index: 0 for details view, 1 for "no selection" placeholder. */
	int32 GetDetailsViewWidgetIndex() const;
	bool IsDetailsViewEnabled() const;

	/** Builds the save/load queue dropdown menu. */
	TSharedRef<SWidget> OnGenerateSavedQueuesMenu();
	FText GetQueueMenuButtonText() const;

	/** Saves the queue (Save As if unsaved, Save if already saved). */
	void OnSaveButtonClicked();

	/** Selects the selected job's level sequence in the Content Browser. */
	void OnFindInContentBrowserClicked();

	/** Returns true when a job with a valid sequence is selected. */
	bool IsFindInContentBrowserEnabled() const;

	/** Builds the main toolbar using FToolBarBuilder. */
	TSharedRef<SWidget> MakeToolbar();

	/** Persists the details splitter slot value to GEditorPerProjectIni when the user finishes dragging. */
	void OnDetailsSplitterFinishedResizing();

	/** Thin wrappers that delegate to utils and handle Slate-specific concerns (e.g. menu dismissal). */
	void OnSaveAsset();
	void OnSaveAsAsset();
	void OnImportSavedQueueAsset(const FAssetData& InPresetAsset) const;
	bool IsQueueDirty() const;
	FString GetQueueOriginName() const;

private:
	/** The main queue editor list widget. */
	TSharedPtr<SMoviePipelineQueueEditor> PipelineQueueEditorWidget;

	/** Details panel showing properties for the selected job(s) or shot(s). */
	TSharedPtr<IDetailsView> JobDetailsPanelWidget;

	/** The config-editor child window, if one is currently open. */
	TWeakPtr<SWindow> WeakEditorWindow;

	/** Cached count of selected jobs, used to drive the details-view widget switcher. */
	int32 NumSelectedJobs = 0;

	/** Splitter that divides the queue tree (left) and the job details panel (right). */
	TSharedPtr<class SSplitter> QueueDetailsSplitter;

	/** Fraction (0..1) of horizontal space allocated to the details panel. Usually sourced from GEditorPerProjectIni. */
	float DetailsSplitterValue = 0.4f;
};
