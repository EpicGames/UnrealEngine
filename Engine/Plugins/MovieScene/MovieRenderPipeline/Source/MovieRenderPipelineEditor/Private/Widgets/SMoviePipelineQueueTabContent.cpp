// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMoviePipelineQueueTabContent.h"
#include "SMoviePipelineQueuePanel.h"

// Analytics
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"

#define LOCTEXT_NAMESPACE "SMoviePipelineQueueTabContent"

void SMoviePipelineQueueTabContent::Construct(const FArguments& InArgs)
{
	ChildSlot
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(WeakPanel, SMoviePipelineQueuePanel)
		];

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MoviePipeline.QueueTabOpened"));
	}
}



#undef LOCTEXT_NAMESPACE // SMoviePipelineQueueTabContent
