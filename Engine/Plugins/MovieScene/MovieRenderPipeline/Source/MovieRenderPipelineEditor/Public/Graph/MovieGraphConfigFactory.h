// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "MovieGraphConfigFactory.generated.h"

#define UE_API MOVIERENDERPIPELINEEDITOR_API

class UMovieGraphConfig;

UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphConfigFactory : public UFactory
{
    GENERATED_BODY()
public:
	UE_API UMovieGraphConfigFactory();
	
	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ End UFactory Interface

private:
	/**
	 * If the factory was created with an InitialSubgraphAsset, adds a subgraph node pointing to InitialSubgraphAsset to the target graph, and
	 * connects it up to the Inputs and Outputs nodes.
	 */
	UE_API void AddSubgraphNodeToGraph(UMovieGraphConfig* InTargetGraph) const;

public:
	/**
	 * If set, creates an empty graph with one subgraph node (where the subgraph points to this graph). If not set, the factory will create a graph
	 * from the template like normal.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphConfig> InitialSubgraphAsset = nullptr;
};

#undef UE_API
