// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"

#include "Graph/MovieGraphQuickRenderSettings.h"
#include "LevelSequence.h"
#include "MoviePipelinePIEExecutor.h"
#include "MoviePipelineQueue.h"

#include "MovieGraphQuickRender.generated.h"

#define UE_API MOVIERENDERPIPELINEEDITOR_API

class FEditorViewportClient;

/**
 * Provides the ability to perform a "Quick Render". A Quick Render is a render which requires no conventional Movie Render Queue setup, like
 * creating a queue, adding a job(s) to it, specifying the level sequence to use, providing a custom configuration or graph, etc. Quick Renders are
 * designed to get you a render as fast as possible while providing minimal configuration input, for use in things like approving animation.
 */
UCLASS(MinimalAPI)
class UMovieGraphQuickRenderSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	/* Begins a quick render using the provided mode and settings. */
	UFUNCTION(BlueprintCallable, Category="Quick Render")
	UE_API void BeginQuickRender(const EMovieGraphQuickRenderMode InQuickRenderMode, const UMovieGraphQuickRenderModeSettings* InQuickRenderSettings);

	/**
	 * Plays the last render that Quick Render generated, using the settings specified in Editor Preferences. If no render has been
	 * generated in this editor session yet, this does nothing.
	 */
	UFUNCTION(BlueprintCallable, Category="Quick Render")
	UE_API void PlayLastRender();

	/** Determines if the last render can be played (ie, whether a render has been performed in the current editor session or not). */
	UFUNCTION(BlueprintCallable, Category="Quick Render")
	UE_API bool CanPlayLastRender();

	/** 
	 * Opens the output directory that Quick Render will save media into. This is sourced from the graph that is used for rendering, which is why
	 * settings need to be provided.
	 */
	UFUNCTION(BlueprintCallable, Category="Quick Render")
	UE_API void OpenOutputDirectory(const UMovieGraphQuickRenderModeSettings* InQuickRenderSettings);

private:
	/** Data that was cached prior to PIE starting. */
	struct FCachedPrePieData
	{
		/**
		 * Level sequence actors which had their AutoPlay setting turned ON prior to PIE (this setting needs to be turned OFF during PIE). This
		 * setting will be restored to ON after PIE ends.
		 */
		TArray<TWeakObjectPtr<class ALevelSequenceActor>> ModifiedLevelSequenceActors;
		
		/** The cameras that were selected. When PIE starts, the Outliner selection is cleared out, so it needs to be cached prior to PIE. */
		TArray<TWeakObjectPtr<ACameraActor>> SelectedCameras;

		/** Actors that have been temporarily flagged as RF_NonPIEDuplicateTransient. This enables them to be copied to the PIE world when they typically would not. */
		TArray<TWeakObjectPtr<AActor>> TemporarilyNonTransientActors;

		/** Components that have been temporarily flagged as RF_NonPIEDuplicateTransient. This enables them to be copied to the PIE world when they typically would not. */
		TArray<TWeakObjectPtr<UActorComponent>> TemporarilyNonTransientComponents;

		/** The actor that the viewport is locked to (also referred to as the pilot camera). Nullptr if no actor lock is active. */
		TStrongObjectPtr<AActor> ViewportActorLock = nullptr;

		/** Cached world transform of the viewport actor lock (pilot camera). */
		FTransform ViewportActorLockWorldTransform = FTransform::Identity;

		/** Whether the viewport actor lock world transform should actually be used. */
		bool bUseViewportActorLockWorldTransform = false;

		/**
		 * If the viewport is locked to an actor (see ViewportActorLock), this is the specific camera component that it is locked to. May be
		 * nullptr if the viewport is locked to a non-camera actor (like a light).
		 */
		TStrongObjectPtr<UCameraComponent> ViewportActorLockCameraComponent = nullptr;

		/** Cached viewport camera properties for the non-piloted camera path. These are captured at BeginQuickRender time so the viewport
		 *  does not need to be active/visible when the PIE world is set up later. */
		struct FViewportCameraData
		{
			FVector Location = FVector::ZeroVector;
			FRotator Rotation = FRotator::ZeroRotator;
			float FOV = 90.f;
			float AspectRatio = 1.78f;
			float OrthoWidth = 0.f;
			bool bIsPerspective = true;
			bool bIsValid = false;
		} ViewportCamera;
	};
	
	/**
	 * Gets the graph that should be used for the quick render. This will be a duplicate of either the user-specified graph or the default quick render graph.
	 * OutOriginalToDupeMap will provide a mapping of the pre-duplicate -> duplicate graphs (may contain multiple graphs if subgraphs are involved).
	 */
	UE_API UMovieGraphConfig* GetQuickRenderGraph(UMovieGraphConfig* InUserSpecifiedGraph, TMap<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>>& OutOriginalToDupeMap);

	/** Generates a new queue for use with Quick Render. Adds a job to the queue, specifies the graph, and all other setup required to get the queue operational. */
	UE_API bool GenerateAndPopulateQueue(TObjectPtr<UMoviePipelineQueue>& OutQueue, TObjectPtr<UMoviePipelineExecutorJob>& OutJob, TObjectPtr<const UMovieGraphQuickRenderModeSettings> InQuickRenderSettings);

	/** Performs the required post-render clean-up, mostly clearing out temporary data members. */
	UE_API void PerformPostRenderCleanup();

	/** Applies all necessary updates to the duplicated graph to prepare it for a quick render (adjusting start/end frames, etc). */
	UE_API void ApplyQuickRenderUpdatesToDuplicateGraph(const UWorld* InEditorWorld);

	/** Helper for ApplyQuickRenderUpdatesToDuplicateGraph() that applies actor editor visibility (by injecting a visibility modifier into the graph). */
	UE_API void ApplyQuickRenderUpdatesToDuplicateGraph_ApplyEditorVisibility(const UWorld* InEditorWorld);

	/** Helper for ApplyQuickRenderUpdatesToDuplicateGraph() that applies editor-only actor visibility (by injecting a visibility modifier into the graph). */
	UE_API void ApplyQuickRenderUpdatesToDuplicateGraph_ApplyEditorOnlyActorVisibility(const UWorld* InEditorWorld);

	/** Helper for ApplyQuickRenderUpdatesToDuplicateGraph() that applies viewport OCIO (by injecting overrides to file output and renderer nodes into the graph). */
	UE_API void ApplyQuickRenderUpdatesToDuplicateGraph_ApplyOcio(const FOpenColorIODisplayConfiguration* InOcioConfiguration);

	/** Helper for ApplyQuickRenderUpdatesToDuplicateGraph() that applies viewport scalability if the graph didn't specify an explicit scalability setting. */
	UE_API void ApplyQuickRenderUpdatesToDuplicateGraph_Scalability();

	/**
	 * Adds a new collection + visibility modifier to the graph used for Quick Render. The operation name should be unique. The modifier can be
	 * specified to either hide or show the actors in the collection, and can optionally process editor-only actors. Returns the collection query,
	 * which can then be updated to include the actors that should be affected.
	 */
	template <typename QueryType>
	QueryType* AddNewCollectionWithVisibilityModifier(const FString& InOperationName, const bool bModifierShouldHide, const bool bProcessEditorOnlyActors = false);

	/** Injects a new node in the furthest-downstream position within the specified branch in the duplicated graph. */
	UE_API UMovieGraphNode* InjectNodeIntoBranch(const TSubclassOf<UMovieGraphNode> NodeType, const FName& InBranchName) const;

	/** Determines if the prerequisites for the given mode are met. */
	UE_API bool AreModePrerequisitesMet() const;

	/** Caches any necessary data before PIE starts. Stored in the CachedPrePieData member variable. */
	UE_API void CachePrePieData(UWorld* InEditorWorld);

	/** Handles restoring data after PIE ends. */
	UE_API void RestorePreRenderState();

	/**
	 * Returns the current level sequence active in Sequencer, or sets up a new level sequence for rendering (depending on the mode that
	 * Quick Render is using).
	 */
	UE_API ULevelSequence* SetUpRenderingLevelSequence() const;

	/**
	 * Returns a new level sequence used for utility purposes (like overriding the camera in use, or changing level
	 * visibility). It will be injected into the rendering level sequence as a subsequence.
	 */
	UE_API ULevelSequence* SetUpUtilityLevelSequence() const;

	/**
	 * Populates both the rendering and utility level sequences, and assigns them to the relevant members. Returns true if everything could be set
	 * up, otherwise false.
	 */
	UE_API bool SetUpAllLevelSequences();

	/** Gets the first world found of the specified type. */
	UE_API UWorld* GetWorldOfType(const EWorldType::Type WorldType) const;

	/**
	 * Gets the explicitly-set playback range of the level sequence being rendered (set either by a selection in Sequencer, or custom start/end
	 * frames). If using the unaltered playback range of the level sequence, this will return an empty range.
	 */
	UE_API TRange<FFrameNumber> GetPlaybackRange();

	/**
	 * Converts a frame number in the focused level sequence to the equivalent in the root level sequence. If the focused and root level sequences
	 * are the same, this just returns the provided frame number. The "focused" level sequence might be a subsequence, for example.
	 */
	UE_API FFrameNumber ConvertSubSequenceFrameToRootFrame(const int32 FrameNum) const;

	/** Does any setup needed in the PIE world prior to the render starting. */
	UE_API void PerformPreRenderSetup(UWorld* InEditorWorld);

	/** Determines if the given viewport look flag is currently active. */
	UE_API bool IsViewportLookFlagActive(const EMovieGraphQuickRenderViewportLookFlags ViewportLookFlag) const;

private:
	/** The temporary queue that is used by quick render. */
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineQueue> TemporaryQueue = nullptr;

	/** The temporary executor that drives the quick render. */
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelinePIEExecutor> TemporaryExecutor = nullptr;

	/** The temporary graph that is used by quick render. Usually a duplicate of a graph created outside of quick render, but will be modified by quick render. */
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphConfig> TemporaryGraph = nullptr;

	/** The temporary evaluated graph that is generated before a render starts; some setup processes need to inspect it. */
	UPROPERTY(Transient)
	TObjectPtr<UMovieGraphEvaluatedConfig> TemporaryEvaluatedGraph = nullptr;

	/** Maps a graph to its duplicate (which is done during render setup). Includes subgraphs. */
	UPROPERTY(Transient)
	TMap<TObjectPtr<UMovieGraphConfig>, TObjectPtr<UMovieGraphConfig>> OriginalGraphToDupeMap;

	/**
	 * The level sequence that is used for rendering (ie, provided directly to MRG to render from). Sometimes this will be the current level sequence
	 * in Sequencer, and other times it may be a temporary level sequence solely for use in Quick Render. It depends on the mode that is being used.
	 */
	UPROPERTY(Transient)
	TObjectPtr<ULevelSequence> RenderingLevelSequence = nullptr;
	
	/**
     * The level sequence that is being used by Quick Render for utility purposes. This is not directly rendered from, but may be included in another
     * level sequence to manipulate it (for example). For most modes, this will remain unused.
     */
    UPROPERTY(Transient)
    TObjectPtr<ULevelSequence> UtilityLevelSequence = nullptr;

	/**
	 * The mode that Quick Render is actively using in a render. This may be different from the mode that's set in the INI file, which indicates the
	 * mode that the UI is using (which could differ if Quick Render is triggered with scripting, for example). See QuickRenderModeSettings for mode
	 * configuration settings.
	 */
	UPROPERTY(Transient)
	EMovieGraphQuickRenderMode QuickRenderMode = EMovieGraphQuickRenderMode::CurrentViewport;

	/**
	 * The mode settings Quick Render was initialized with. These are cached here to grab a reference to them before the render begins so the
	 * executor callbacks still have access to them later.
	 */
	UPROPERTY(Transient)
	TObjectPtr<const UMovieGraphQuickRenderModeSettings> QuickRenderModeSettings = nullptr;

	/** Any data that needed to be cached prior to starting PIE. */
	FCachedPrePieData CachedPrePieData;
};

#undef UE_API
