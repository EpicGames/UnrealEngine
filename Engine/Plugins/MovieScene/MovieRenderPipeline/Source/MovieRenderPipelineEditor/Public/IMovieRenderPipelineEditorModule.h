// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

/**
 * Public module interface for the Movie Render Pipeline module
 */
class IMovieRenderPipelineEditorModule : public IModuleInterface
{
public:
	/** The tab name for the movie render queue editor tab */
	static FName MoviePipelineQueueTabName;

	/** The default label for the movie render queue editor tab. */
	static FText MoviePipelineQueueTabLabel;

	/** The tab name for the movie render config editor tab */
	static FName MoviePipelineConfigEditorTabName;

	/** The default label for the movie render config editor tab. */
	static FText MoviePipelineConfigEditorTabLabel;
};
