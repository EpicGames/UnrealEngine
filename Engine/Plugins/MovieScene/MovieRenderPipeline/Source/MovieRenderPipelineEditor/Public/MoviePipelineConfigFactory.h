// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "MoviePipelineConfigFactory.generated.h"

#define UE_API MOVIERENDERPIPELINEEDITOR_API

// Forward Declare
UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelinePrimaryConfigFactory : public UFactory
{
    GENERATED_BODY()
public:
	UE_API UMoviePipelinePrimaryConfigFactory();
	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual uint32 GetMenuCategories() const override;
	//~ End UFactory Interface
};

// Forward Declare
UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineShotConfigFactory : public UFactory
{
    GENERATED_BODY()
public:
	UE_API UMoviePipelineShotConfigFactory();
	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual uint32 GetMenuCategories() const override;
	//~ End UFactory Interface
};

#undef UE_API
