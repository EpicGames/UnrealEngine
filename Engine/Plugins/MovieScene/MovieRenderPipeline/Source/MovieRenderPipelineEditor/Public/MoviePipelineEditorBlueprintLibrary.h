// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"
#include "Internationalization/Text.h"
#include "Containers/UnrealString.h"
#include "MoviePipelinePrimaryConfig.h"

#include "MoviePipelineEditorBlueprintLibrary.generated.h"

#define UE_API MOVIERENDERPIPELINEEDITOR_API

// Forward Declare
class UMovieGraphConfig;
class UMoviePipelinePrimaryConfig;
class UMoviePipelineExecutorJob;
class ULevelSequence;

UCLASS(MinimalAPI, meta=(ScriptName="MoviePipelineEditorLibrary"))
class UMoviePipelineEditorBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API bool ExportConfigToAsset(const UMoviePipelinePrimaryConfig* InConfig, const FString& InPackagePath, const FString& InFileName, const bool bInSaveAsset, UMoviePipelinePrimaryConfig*& OutAsset, FText& OutErrorReason);

	/** Checks to see if any of the Jobs try to point to maps that wouldn't be valid on a remote render (ie: unsaved maps) */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	static UE_API bool IsMapValidForRemoteRender(const TArray<UMoviePipelineExecutorJob*>& InJobs);

	/** Pop a dialog box that specifies that they cannot render due to never saved map. Only shows OK button. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API void WarnUserOfUnsavedMap();

	/** Take the specified Queue, duplicate it and write it to disk in the ../Saved/MovieRenderPipeline/ folder. Returns the duplicated queue. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API UMoviePipelineQueue* SaveQueueToManifestFile(UMoviePipelineQueue* InPipelineQueue, FString& OutManifestFilePath);

	/** Loads the specified manifest file and converts it into an FString to be embedded with HTTP REST requests. Use in combination with SaveQueueToManifestFile. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API FString ConvertManifestFileToString(const FString& InManifestFilePath);

	/** Create a job from a level sequence. Sets the map as the currently editor world, the author, the sequence and the job name as the sequence name on the new job. Returns the newly created job. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API UMoviePipelineExecutorJob* CreateJobFromSequence(UMoviePipelineQueue* InPipelineQueue, const ULevelSequence* InSequence);

	/** Applies the user's default configuration type to the given job. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API void ApplyDefaultConfigurationTypeToJob(UMoviePipelineExecutorJob* InJob);

	/** Assigns the project's default graph configuration to the given job, and switches it to graph configuration mode. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline", meta = (AdvancedDisplay="bCreateTransaction"))
	static UE_API void AssignDefaultGraphPresetToJob(UMoviePipelineExecutorJob* InJob, const bool bCreateTransaction = true);

	/**
	 * Ensure the job has the settings specified by the project settings added. Idempotent per-job:
	 * once defaults have been applied, subsequent calls are no-ops so that settings the user has
	 * deliberately removed won't be resurrected. Pass bForce=true to re-apply after an explicit
	 * configuration reset (e.g. SetConfiguration to a fresh empty config).
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API void EnsureJobHasDefaultSettings(UMoviePipelineExecutorJob* InJob, bool bForce = false);

	/** Returns display string for output directory for this job. Does not resolve the full path from tokens. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API bool GetDisplayOutputPathFromJob(UMoviePipelineExecutorJob* InJob, FString& OutOutputPath);

	/**
	 * Resolves as much of the output directory for this job into a usable directory path as possible. Cannot
	 * resolve anything that relies on shot name, frame numbers, etc. Returns an empty string upon failure.
	 * @see TryResolveOutputDirectoryFromJob() for the same function which also provides the failure reason
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API FString ResolveOutputDirectoryFromJob(UMoviePipelineExecutorJob* InJob);

	/**
	 * The same as ResolveOutputDirectoryFromJob: returns the resolved output directory, but reports an error string via
	 * OutError if resolution failed (e.g. graph traversal could not be completed). Returns an empty string on failure.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API FString TryResolveOutputDirectoryFromJob(UMoviePipelineExecutorJob* InJob, FText& OutError);

	/**
	 * Generates a UMovieGraphConfig from the job's Basic config.
	 * Prompts the user to choose a save location via a Save Asset dialog, saves the asset to disk, then switches the
	 * job to Graph configuration mode using the newly saved asset.
	 *
	 * @param InJob  The job to convert. Must be using Basic configuration mode.
	 * @return true if the graph was saved and the job was switched; false if the job is not in Basic mode,
	 *         graph generation or asset creation failed, the user canceled, or the save failed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	static UE_API bool SaveBasicConfigAsGraphConfig(UMoviePipelineExecutorJob* InJob);
};

#undef UE_API
