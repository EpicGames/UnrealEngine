// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineLinearExecutor.h"
#include "MoviePipelineNewProcessExecutor.generated.h"

#define UE_API MOVIERENDERPIPELINEEDITOR_API

/**
* This is the implementation responsible for executing the rendering of
* multiple movie pipelines on the local machine in an external process.
* This simply handles launching and managing the external processes and 
* acts as a proxy to them where possible. This internally uses the
* UMoviePipelineInProcessExecutor on the launched instances.
*/
UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineNewProcessExecutor : public UMoviePipelineExecutorBase
{
	GENERATED_BODY()

	// UMoviePipelineExecutorBase Interface
	UE_API virtual void Execute_Implementation(UMoviePipelineQueue* InPipelineQueue) override;
	virtual bool IsRendering_Implementation() const override { return ProcessHandle.IsValid(); }

	// Canceling current job is equivalent to canceling all jobs for this executor
	virtual void CancelCurrentJob_Implementation() override { CancelAllJobs_Implementation(); }
	UE_API virtual void CancelAllJobs_Implementation() override;
	// ~UMoviePipelineExecutorBase Interface

protected:
	UE_API void CheckForProcessFinished();

protected:
	/** A handle to the currently running process (if any) for the active job. */
	FProcHandle ProcessHandle;

};

#undef UE_API
