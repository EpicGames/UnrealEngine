// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineLinearExecutor.h"
#include "Logging/MessageLog.h"
#include "Misc/OutputDevice.h"
#include "MoviePipeline.h"
#include "MoviePipelinePIEExecutor.generated.h"

#define UE_API MOVIERENDERPIPELINEEDITOR_API

class UMoviePipeline;

// These are deprecated, see OnIndividualShotWorkFinishedDelegate which contains both the job and the data actually rendered.
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMoviePipelineIndividualJobFinishedNative, UMoviePipelineExecutorJob* /*FinishedJob*/, bool /*bSuccess*/);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMoviePipelineIndividualJobFinished, UMoviePipelineExecutorJob*, FinishedJob, bool, bSuccess);

// These are called right before UMoviePipeline::Initialize(). They only contain the job because no work has been done yet.
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMoviePipelineIndividualJobStartedNative, UMoviePipelineExecutorJob* /*JobToStart*/);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMoviePipelineIndividualJobStarted, UMoviePipelineExecutorJob*, StartedJob);

/**
* This is the implementation responsible for executing the rendering of
* multiple movie pipelines in the currently running Editor process. This
* involves launching a Play in Editor session for each Movie Pipeline to
* process.
*/
UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelinePIEExecutor : public UMoviePipelineLinearExecutorBase
{
	GENERATED_BODY()
	
public:
	UE_API UMoviePipelinePIEExecutor();

public:
	/** Deprecated. Use OnIndividualJobWorkFinished instead. */
	UE_DEPRECATED(4.27, "Use OnIndividualJobWorkFinished() instead.")
	FOnMoviePipelineIndividualJobFinishedNative& OnIndividualJobFinished()
	{
		return OnIndividualJobFinishedDelegateNative;
	}

	/** Native C++ event to listen to for when an individual job has been finished. */
	FMoviePipelineWorkFinishedNative& OnIndividualJobWorkFinished()
	{
		return OnIndividualJobWorkFinishedDelegateNative;
	}

	/** Native C++ event to listen to for when an individual shot has been finished. Only called if the UMoviePipeline is set up correctly, see its headers for details. This usually means setting bFlushDiskWritesPerShot to true in the UMoviePipelineOutputSetting for each job before rendering. */
	FMoviePipelineWorkFinishedNative& OnIndividualShotWorkFinished()
	{
		return OnIndividualShotWorkFinishedDelegateNative;
	}

	/** Called right before the specified job is started. This is your last chance to modify the Job Properties before things are initialized from it. */
	FOnMoviePipelineIndividualJobStartedNative& OnIndividualJobStarted()
	{
		return OnIndividualJobStartedDelegateNative;
	}

	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	void SetInitializationTime(const FDateTime& InInitializationTime) { CustomInitializationTime = InInitializationTime; }

	/** Should it render without any UI elements showing up (such as the rendering progress window)? */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	void SetIsRenderingOffscreen(const bool bInRenderOffscreen) { bRenderOffscreen = bInRenderOffscreen; }

	/** Will it render without any UI elements showing up (such as the rendering progress window)? */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	bool IsRenderingOffscreen() const { return bRenderOffscreen; }

	/**
	 * Set whether the executor should be allowed to use unsaved levels when rendering (by default, this is not permitted). There are some very
	 * specific circumstances where this may be OK. Do not set this unless you're certain it will not cause issues in your use case.
	 */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline")
	UE_API void SetAllowUsingUnsavedLevels(const bool bShouldAllow);

protected:
	//~ Begin UMoviePipelineLinearExecutorBase interface
	UE_API virtual void Start(const UMoviePipelineExecutorJob* InJob) override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void OnExecutorFinishedImpl() override;
	//~ End UMoviePipelineLinearExecutorBase interface

	//~ Begin UMoviePipelineExecutorBase interface
	virtual bool GetSupportsPostRenderActions_Implementation() override;
	//~ End UMoviePipelineExecutorBase interface
	
	/**
	* This should be called after PIE has been shut down for an individual job and it is generally safer to
	* make modifications to the editor world.
	*/
	UE_API void OnIndividualJobFinishedImpl(FMoviePipelineOutputData InOutputData);
	/**
	* This is called after PIE starts up but before UMoviePIpeline::Initialize() is called.
	*/
	UE_API void OnIndividualJobStartedImpl(UMoviePipelineExecutorJob* InJob);
private:
	/** Called when PIE finishes booting up and it is safe for us to spawn an object into that world. */
	UE_API void OnPIEStartupFinished(bool);

	/** If they're using delayed initialization, this is called each frame to process the countdown until start, also updates Window Title each frame. */
	UE_API void OnTick();

	/** Called before PIE tears down the world during shutdown. Used to detect cancel-via-escape/stop PIE. */
	UE_API void OnPIEEnded(bool);
	/** Called when the instance of the pipeline in the PIE world has finished. */
	UE_API void OnPIEMoviePipelineFinished(FMoviePipelineOutputData InOutputData);
	UE_API void OnJobShotFinished(FMoviePipelineOutputData InOutputData);

	/** Called a short period of time after OnPIEMoviePipelineFinished to allow Editor the time to fully close PIE before we make a new request. */
	UE_API void DelayedFinishNotification();
private:
	bool bRenderOffscreen;
	/** If using delayed initialization, how many frames are left before we call Initialize. Will be -1 if not actively counting down. */
	int32 RemainingInitializationFrames;
	bool bPreviousUseFixedTimeStep;
	double PreviousFixedTimeStepDelta;
	TWeakPtr<class SWindow> WeakCustomWindow;
	TOptional<FDateTime> CustomInitializationTime;
	FMoviePipelineOutputData CachedOutputDataParams;

	/** Whether unsaved levels can be rendered from. */
	bool bAllowUnsavedLevels;

	/** Output data generated from all jobs in the queue (one entry in the array for each job). May be a mix of graph and preset output data. */
	TArray<FMoviePipelineOutputData> AllJobOutputData;

	/** The resolved output directories for all jobs that have been rendered. */
	TArray<FString> ResolvedOutputDirectories;

	/** Deprecated. use OnIndividualJobWorkFinishedDelegate instead. */
	UE_DEPRECATED(4.27, "Use OnIndividualJobWorkFinishedDelegate instead.")
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FOnMoviePipelineIndividualJobFinished OnIndividualJobFinishedDelegate;

	FOnMoviePipelineIndividualJobFinishedNative OnIndividualJobFinishedDelegateNative;


	/** Called after each job is finished in the queue. Params struct contains an output of all files written. */
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FMoviePipelineWorkFinished OnIndividualJobWorkFinishedDelegate;

	/** 
	* Called after each shot is finished for a particular render. Params struct contains an output of files written for this shot. 
	* Only called if the UMoviePipeline is set up correctly, requires a flag in the output setting to be set. 
	*/
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FMoviePipelineWorkFinished OnIndividualShotWorkFinishedDelegate;

	/**
	* Called right before this job is used to initialize a UMoviePipeline.
	*/
	UPROPERTY(BlueprintAssignable, Category = "Movie Render Pipeline")
	FOnMoviePipelineIndividualJobStarted OnIndividualJobStartedDelegate;

	FMoviePipelineWorkFinishedNative OnIndividualJobWorkFinishedDelegateNative;
	FMoviePipelineWorkFinishedNative OnIndividualShotWorkFinishedDelegateNative;
	FOnMoviePipelineIndividualJobStartedNative OnIndividualJobStartedDelegateNative;

	class FValidationMessageGatherer : public FOutputDevice
	{
	public:

		FValidationMessageGatherer();

		void StartGathering()
		{
			FString PageName = FString("High Quality Media Export: ") + FDateTime::Now().ToString();
			ExecutorLog->NewPage(FText::FromString(PageName));
			GLog->AddOutputDevice(this);
		}

		void StopGathering()
		{
			GLog->RemoveOutputDevice(this);
		}

		virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override;

		void OpenLog()
		{
			ExecutorLog->Open();
		}

	private:
		TUniquePtr<FMessageLog> ExecutorLog;
		const static TArray<FString> AllowList;
	};

	FValidationMessageGatherer ValidationMessageGatherer;
};

#undef UE_API
