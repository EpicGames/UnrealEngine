// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EditorSubsystem.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineExecutor.h"
#include "MoviePipelineQueueSubsystem.generated.h"

#define UE_API MOVIERENDERPIPELINEEDITOR_API

DECLARE_MULTICAST_DELEGATE(FOnMoviePipelineQueueLoaded)

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineQueueSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()
	
public:

	UMoviePipelineQueueSubsystem()
	{
		CurrentQueue = CreateDefaultSubobject<UMoviePipelineQueue>("EditorMoviePipelineQueue");
	}
	
	/** Returns the queue of Movie Pipelines that need to be rendered. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelineQueue* GetQueue() const
	{
		return CurrentQueue;
	}

	/**
	 * Loads a new queue by copying it into the queue subsystem's current transient queue (the one returned by GetQueue()).
	 * 
	 * If bPromptOnReplacingDirtyQueue is true and the current queue has been modified since being loaded, a dialog will prompt the
	 * user if they want to discard their changes. If this dialog is rejected, or there was an error loading the queue, returns
	 * false, else returns true. Note that bPromptOnReplacingDirtyQueue is effectively ignored if the application is in
	 * "unattended" mode because the dialog is auto-accepted.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UE_API bool LoadQueue(UMoviePipelineQueue* QueueToLoad, const bool bPromptOnReplacingDirtyQueue = true);

	/**
	 * Returns true if the current queue has been modified since it was loaded, else returns false. Note that the empty
	 * queue that is in use upon the initial load of MRQ is not considered dirty.
	 */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UE_API bool IsQueueDirty() const;

	/** Returns the active executor (if there is one). This can be used to subscribe to events on an already in-progress render. May be null. */
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline")
	UMoviePipelineExecutorBase* GetActiveExecutor() const
	{
		return ActiveExecutor;
	}

	/** 
	* Starts processing the current queue with the supplied executor class. This starts an async process which
	* may or may not run in a separate process (or on separate machines), determined by the executor implementation.
	* The executor should report progress for jobs depending on the implementation.
	*
	* @param InExecutorType	A subclass of UMoviePipelineExecutorBase. An instance of this class is created and started.
	* @return A pointer to the instance of the class created. This instance will be kept alive by the Queue Subsystem
			  until it has finished (or been canceled). Register for progress reports and various callbacks on this instance.
	*/
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "InExecutorType"), Category = "Movie Render Pipeline|Rendering")
	UE_API UMoviePipelineExecutorBase* RenderQueueWithExecutor(TSubclassOf<UMoviePipelineExecutorBase> InExecutorType);

	/** Like RenderQueueWithExecutor(), but a specific queue instance is used rather than the current queue. */
	UFUNCTION(BlueprintCallable, meta = (DeterminesOutputType = "InExecutorType"), Category = "Movie Render Pipeline|Rendering")
	UE_API UMoviePipelineExecutorBase* RenderQueueInstanceWithExecutor(UMoviePipelineQueue* InQueue, TSubclassOf<UMoviePipelineExecutorBase> InExecutorType);

	/**
	* Starts processing the current queue with the supplied executor. This starts an async process which
	* may or may not run in a separate process (or on separate machines), determined by the executor implementation.
	* The executor should report progress for jobs depending on the implementation.
	*
	* @param InExecutor	Instance of a subclass of UMoviePipelineExecutorBase.
	*/
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Rendering")
	UE_API void RenderQueueWithExecutorInstance(UMoviePipelineExecutorBase* InExecutor);

	/** Like RenderQueueWithExecutorInstance(), but a specific queue instance is used rather than the current queue. */
	UFUNCTION(BlueprintCallable, Category = "Movie Render Pipeline|Rendering")
	UE_API void RenderQueueInstanceWithExecutorInstance(UMoviePipelineQueue* InQueue, UMoviePipelineExecutorBase* InExecutor);

	/**
	* Returns true if there is an active executor working on producing a movie. This could be actively rendering frames,
	* or working on post processing (finalizing file writes, etc.). Use GetActiveExecutor() and query it directly for
	* more information, progress updates, etc.
	*/
	UFUNCTION(BlueprintPure, Category = "Movie Render Pipeline|Rendering")
	bool IsRendering() const
	{
		return ActiveExecutor ? ActiveExecutor->IsRendering() : false;
	}

public:
	/** Called when a queue is loaded into the system. */
	FOnMoviePipelineQueueLoaded OnQueueLoaded;

private:
	/** Called when the executor is finished so that we can release it and stop reporting IsRendering() == true. */
	UE_API void OnExecutorFinished(UMoviePipelineExecutorBase* InPipelineExecutor, bool bSuccess);
	UE_API void OnSequencerContextBinding(bool& bAllowBinding);

private:
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineExecutorBase> ActiveExecutor;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<UMoviePipelineQueue> CurrentQueue;
};

#undef UE_API
