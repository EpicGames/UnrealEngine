// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "MoviePipelineExecutor.h"
#include "MoviePipeline.h"
#include "Engine/EngineTypes.h"
#include "MovieRenderPipelineSettings.generated.h"

class UMovieGraphConfig;
class UMoviePipelineExecutorBase;
class UMoviePipeline;
class UMoviePipelinePrimaryConfig;

/**
 * Universal Movie Render Pipeline settings that apply to the whole project.
 */
UCLASS(BlueprintType, config=Editor, defaultconfig, MinimalAPI)
class UMovieRenderPipelineProjectSettings : public UObject
{
public:
	GENERATED_BODY()
	
	MOVIERENDERPIPELINEEDITOR_API UMovieRenderPipelineProjectSettings();
	
	/**
	* This allows you to implement your own Pipeline to handle timing and rendering of a movie. Changing
	* this will allow you to re-use the existing UI/Executors while providing your own logic for producing
	* a single render.
	*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, NoClear, meta = (MetaClass = "/Script/MovieRenderPipelineCore.MoviePipelineBase"), Category="Movie Render Pipeline", DisplayName = "Default Configuration Pipeline")
	FSoftClassPath DefaultPipeline;

	/**
	* Which directory should we try to save presets in by default?
	*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Movie Render Pipeline", DisplayName = "Preset Save Location")
	FDirectoryPath PresetSaveDir;


	/**
	* What was the last configuration preset the user used? Can be null.
	*/
	UPROPERTY(Transient, BlueprintReadWrite, Category = "Movie Render Pipeline")
	TSoftObjectPtr<UMoviePipelinePrimaryConfig> LastPresetOrigin;
	
	/**
	* When the user uses the UI to request we render a movie locally, which implementation should we use
	* to execute the queue of things they want rendered. This allows you to implement your own executor 
	* which does different logic. See UMoviePipelineExecutorBase for more information. This is used for
	* the Render button on the UI.
	*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, meta = (MetaClass = "/Script/MovieRenderPipelineCore.MoviePipelineExecutorBase"), Category="Movie Render Pipeline")
	FSoftClassPath DefaultLocalExecutor;

	/**
	* When the user uses the UI to request we render a movie remotely, which implementation should we use
	* to execute the queue of things they want rendered. This allows you to implement your own executor
	* which does different logic. See UMoviePipelineExecutorBase for more information. This is used for
	* the Render Remotely button on the UI.
	*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, meta = (MetaClass = "/Script/MovieRenderPipelineCore.MoviePipelineExecutorBase"), Category = "Movie Render Pipeline")
	FSoftClassPath DefaultRemoteExecutor;
	
	/**
	* Which Job class should we create by default when adding a job? This allows you to make custom jobs
	* that will have editable properties in the UI for special handling with your executor. This can be
	* made dynamic if you add jobs to the queue programatically instead of through the UI.
	*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, meta = (MetaClass = "/Script/MovieRenderPipelineCore.MoviePipelineExecutorJob"), Category="Movie Render Pipeline")
	FSoftClassPath DefaultExecutorJob;

	/** The graph that newly-created graph assets will be based off of. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category="Movie Render Pipeline")
	TSoftObjectPtr<UMovieGraphConfig> DefaultGraph;

	// TODO: Delete this once there's a full Settings dialog for Quick Render
	/** The graph that quick renders will use. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category="Movie Render Pipeline")
	TSoftObjectPtr<UMovieGraphConfig> DefaultQuickRenderGraph;

	/**
	* The settings specified here will automatically be added to a Movie Pipeline Primary Configuration when using the UI. 
	* This does not apply to scripting and does not apply to runtime. It is only a convenience function so that when a job is
	* created, it can be pre-filled with some settings to make the render functional out of the gate. It can also be
	* used to automatically add your own setting to jobs.
	*
	* This only applies to jobs created via the UI. If you do not use the UI (ie: Scripting/Python) you will need to
	* add settings by hand for each job you create. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Movie Render Pipeline", meta = (MetaClass = "/Script/MovieRenderPipelineCore.MoviePipelineSetting"), DisplayName = "Default Job Settings Classes")
	TArray<FSoftClassPath> DefaultClasses;

public:
	/** Gets the path to the default graph asset supplied by MRQ. */
	static FSoftObjectPath GetDefaultGraphPath();

	/** Gets the path to the default quick render graph asset supplied by MRQ. */
	static FSoftObjectPath GetDefaultQuickRenderGraphPath();
};