// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakInterfacePtr.h"
#include "Widgets/SCompoundWidget.h"

class IMovieGraphModifierNodeInterface;

/**
 * A widget which displays the header for modifier collections in the details panel.
 */
class SMovieGraphCollectionsHeaderWidget final : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnCollectionPicked, FName);

	SLATE_BEGIN_ARGS(SMovieGraphCollectionsHeaderWidget) { }
		/** The modifier node that is associated with the collections. */
		SLATE_ATTRIBUTE(TWeakInterfacePtr<IMovieGraphModifierNodeInterface>, WeakModifierInterface)

		/** Called when a collection is picked in the list. */
		SLATE_EVENT(FOnCollectionPicked, OnCollectionPicked);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** The modifier node associated with this widget (if any). */
	TWeakInterfacePtr<IMovieGraphModifierNodeInterface> WeakModifierInterface;

	/** Called when a collection is picked in the list. */
	FOnCollectionPicked OnCollectionPicked;
};