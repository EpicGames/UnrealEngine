// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphSharedWidgets.h"
#include "Graph/Nodes/MovieGraphModifierNode.h"

class UMovieGraphCollectionNode;
class UMovieGraphConfig;

/** Discovers collections that are pickable from a specific graph, and presents them in a list. */
class SMovieGraphCollectionPicker final : public SMovieGraphSimplePicker<FName>
{
public:
	DECLARE_DELEGATE_OneParam(FOnCollectionPicked, FName);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnFilter, FName);

	SLATE_BEGIN_ARGS(SMovieGraphCollectionPicker)
		{}
		/** The graph to begin discovering collections from. */
		SLATE_ATTRIBUTE(UMovieGraphConfig*, Graph)

		/** Called when a collection is picked in the list. */
		SLATE_EVENT(FOnCollectionPicked, OnCollectionPicked);

		/** Optional filter that can prevent discovered collections from showing up in the list. */
		SLATE_EVENT(FOnFilter, OnFilter);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Discovers all available collections, and refreshes the data the list is displaying. */
	void UpdateDataSource();

private:
	/** The current graph being viewed. The data source will be populated from this graph. */
	UMovieGraphConfig* CurrentGraph = nullptr;

	/** Called when a collection is picked in the list. */
	FOnCollectionPicked OnCollectionPicked;

	/** Optional filter that can prevent discovered collections from showing up in the list. */
	FOnFilter OnFilter;
};