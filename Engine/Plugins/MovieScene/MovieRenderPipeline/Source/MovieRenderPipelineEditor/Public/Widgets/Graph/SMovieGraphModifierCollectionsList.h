// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphSharedWidgets.h"
#include "UObject/WeakInterfacePtr.h"

class IMovieGraphModifierNodeInterface;

/** Lists out the collections that have been chosen for a specific modifier node. */
class SMovieGraphModifierCollectionsList final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMovieGraphModifierCollectionsList) { }
		/** The modifier node that is associated with the collections. */
		SLATE_ATTRIBUTE(TWeakInterfacePtr<IMovieGraphModifierNodeInterface>, WeakModifierInterface)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Refreshes the widget and its internal data source. */
	void Refresh();

private:
	static FText GetCollectionRowText(const FName CollectionName);

	/** Refreshes the list's data source to reflect the data model. */
	void RefreshListDataSource();

private:
	/** The data source for the list view. Modifiers don't expose a direct pointer to the array of collections, which is required by the list view. */
	TArray<FName> ListDataSource;
	
	/** The modifier node associated with this widget (if any). */
	TWeakInterfacePtr<IMovieGraphModifierNodeInterface> WeakModifierInterface;

	/** Displays the collections which have been chosen. */
	TSharedPtr<SMovieGraphSimpleList<FName>> CollectionsList;
};