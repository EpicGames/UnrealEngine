// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MovieRenderPipelineMP4Encoder : ModuleRules
	{
		public MovieRenderPipelineMP4Encoder(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(				
				new string[] {
					"MovieRenderPipelineCore",
					"MovieRenderPipelineSettings",
					"OpenColorIO",
					"SlateCore",
                });

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
                    "CoreUObject",
                    "Engine",
                }
            );
			
			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PublicDelayLoadDLLs.Add("mfreadwrite.dll");
				PublicDelayLoadDLLs.Add("mfplat.dll");
				PublicDelayLoadDLLs.Add("mfuuid.dll");

				PublicSystemLibraries.Add("mfreadwrite.lib");
				PublicSystemLibraries.Add("mfplat.lib");
				PublicSystemLibraries.Add("mfuuid.lib");

				PrivateIncludePaths.Add("MovieRenderPipelineMP4Encoder/Private/Windows/");
			}
			else if(Target.Platform.IsInGroup(UnrealPlatformGroup.Linux))
			{
				PrivateIncludePaths.Add("MovieRenderPipelineMP4Encoder/Private/GenericPlatform/");
			}
			else if(Target.Platform.IsInGroup(UnrealPlatformGroup.Apple))
			{
				PrivateIncludePaths.Add("MovieRenderPipelineMP4Encoder/Private/GenericPlatform/");
			}
        }
    }
}
