// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "MoviePipelineMP4EncoderCommon.h"
#include "MovieRenderPipelineCoreModule.h"

/**
* Dummy implementation for unsupported platforms which returns failure if used, to avoid
* doing rendering work and then having no output. This should have a matching public API
* to the implementation in the other platform folders.
*/
class FMoviePipelineMP4Encoder
{
public:
	FMoviePipelineMP4Encoder(const FMoviePipelineMP4EncoderOptions& InOptions)
	: Options(InOptions)
	{}
	
	~FMoviePipelineMP4Encoder() = default;

	bool Initialize()
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "MP4 Encoder is unsupported on this platform. Please remove it from your configuration.")
		return false;
	}
	void Finalize() {}
	bool WriteFrame(const uint8* InFrameData) { return false; }
	bool WriteAudioSample(const TArrayView<int16>& InAudioSamples) { return false; }
	const FMoviePipelineMP4EncoderOptions& GetOptions() const { return Options; }
	bool IsInitialized() const { return false; }

private:
	/** Configure and initialize the output file */
	bool InitializeEncoder();

private:
	FMoviePipelineMP4EncoderOptions Options;
};
