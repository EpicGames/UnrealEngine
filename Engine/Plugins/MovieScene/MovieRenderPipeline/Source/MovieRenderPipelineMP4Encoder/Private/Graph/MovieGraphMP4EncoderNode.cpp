// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphMP4EncoderNode.h"

#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphOCIOHelper.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphBurnInNode.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "ImageWriteTask.h"
#include "MoviePipelineImageQuantization.h"
#include "MoviePipelineTelemetry.h"
#include "MovieRenderPipelineCoreModule.h"
#include "SampleBuffer.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphMP4EncoderNode)


UMovieGraphMP4EncoderNode::UMovieGraphMP4EncoderNode()
{
}

#if WITH_EDITOR
FText UMovieGraphMP4EncoderNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText NodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_MP4", "H.264 MP4");
	return NodeName;
}

FText UMovieGraphMP4EncoderNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "WMFNode_Category", "Output Type");
}

FText UMovieGraphMP4EncoderNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "MP4_Keywords", "mp4 h264 h265 windows mpeg mov movie video");
	return Keywords;
}

FLinearColor UMovieGraphMP4EncoderNode::GetNodeTitleColor() const
{
	static const FLinearColor NodeColor = FLinearColor(0.047f, 0.654f, 0.537f);
	return NodeColor;
}

FSlateIcon UMovieGraphMP4EncoderNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon Icon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.RenderMovieFile");

	OutColor = FLinearColor::White;
	return Icon;
}

FText UMovieGraphMP4EncoderNode::GetBasicConfigShortDisplayName() const
{
#if PLATFORM_WINDOWS
	return NSLOCTEXT("MovieGraphNodes", "BasicConfigShortName_MP4", "MP4");
#else
	// The MP4 encoder is only implemented on Windows (Media Foundation); the GenericPlatform
	// stub used on Linux/Mac fails at Initialize(). Hide the Basic config on these platforms for now.
	return FText::GetEmpty();
#endif
}
#endif // WITH_EDITOR

TUniquePtr<MovieRenderGraph::IVideoCodecWriter> UMovieGraphMP4EncoderNode::Initialize_GameThread(const FMovieGraphVideoNodeInitializationContext& InInitializationContext)
{
	bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	UMovieGraphGlobalOutputSettingNode* OutputSetting =
		InInitializationContext.EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);

	bIncludeCDOs = false;
	const UMovieGraphMP4EncoderNode* EvaluatedNode = Cast<UMovieGraphMP4EncoderNode>(
		InInitializationContext.EvaluatedConfig->GetSettingForBranch(GetClass(), FName(InInitializationContext.PassData->Key.RootBranchName), bIncludeCDOs, bExactMatch));
	checkf(EvaluatedNode, TEXT("MP4 Encoder node could not be found in the graph in branch [%s]."), *InInitializationContext.PassData->Key.RootBranchName.ToString());
	
	const FFrameRate SourceFrameRate = InInitializationContext.Pipeline->GetDataSourceInstance()->GetDisplayRate();
	const FFrameRate EffectiveFrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputSetting, SourceFrameRate);
	
	FMoviePipelineMP4EncoderOptions Options;
	Options.OutputFilename = InInitializationContext.FileName;
	Options.Width = InInitializationContext.Resolution.X;
	Options.Height = InInitializationContext.Resolution.Y;
	Options.FrameRate = EffectiveFrameRate;
	Options.CommonMeanBitRate = FMath::RoundToInt(EvaluatedNode->AverageBitrateInMbps*1024*1024);
	Options.CommonQualityVsSpeed = 100;
	Options.CommonConstantRateFactor = EvaluatedNode->ConstantRateFactor;
	Options.EncodingRateControl = EvaluatedNode->EncodingRateControl;
	Options.bIncludeAudio = EvaluatedNode->bIncludeAudio;
	
	// Optional properties that can be overridden by scripting if needed; not exposed to the UI currently
	if (EvaluatedNode->bOverride_MaxBitrateInMbps)
	{
		Options.CommonMaxBitRate = FMath::RoundToInt(EvaluatedNode->MaxBitrateInMbps*1024*1024);
	}
	if (EvaluatedNode->bOverride_EncodingProfile)
	{
		Options.EncodingProfile = EvaluatedNode->EncodingProfile;
	}
	if (EvaluatedNode->bOverride_EncodingLevel)
	{
		Options.EncodingLevel = EvaluatedNode->EncodingLevel;
	}
	
	TUniquePtr<FMP4CodecWriter> NewWriter = MakeUnique<FMP4CodecWriter>();
	NewWriter->Writer = MakeUnique<FMoviePipelineMP4Encoder>(Options);

	// If OCIO is enabled, don't do additional color conversion.
	NewWriter->bSkipColorConversions = (EvaluatedNode->bOverride_OCIOConfiguration && EvaluatedNode->OCIOConfiguration.bIsEnabled && InInitializationContext.bAllowOCIO);
	CachedPipeline = InInitializationContext.Pipeline;
	
	return NewWriter;
}

bool UMovieGraphMP4EncoderNode::Initialize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	const FMP4CodecWriter* CodecWriter = static_cast<FMP4CodecWriter*>(InWriter);
	if(!CodecWriter->Writer->Initialize())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to initialize Movie Pipeline MP4 writer. An encoder that supports the render resolution and requested MP4 encode options was not found. Try again with a different resolution and/or encoder settings.");
		return false;
	}
	
	return true;
}

void UMovieGraphMP4EncoderNode::WriteFrame_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<FMovieGraphPassData>&& InCompositePasses, TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FString& InBranchName)
{
	FMP4CodecWriter* CodecWriter = static_cast<FMP4CodecWriter*>(InWriter);
	if (!CodecWriter->Writer)
	{
		return;
	}

	bool bIncludeCDOs = false;
	constexpr bool bExactMatch = true;
	const UMovieGraphMP4EncoderNode* EvaluatedNode = Cast<UMovieGraphMP4EncoderNode>(
		InEvaluatedConfig->GetSettingForBranch(GetClass(), FName(InBranchName), bIncludeCDOs, bExactMatch));
	checkf(EvaluatedNode, TEXT("MP4 Encoder node could not be found in the graph in branch [%s]."), *InBranchName);

	bIncludeCDOs = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSettingNode =
		InEvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);
	
	const UE::MovieGraph::FMovieGraphSampleState* GraphPayload = InPixelData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
	
	TArray<FPixelPreProcessor> PixelPreProcessors;

	// Run OCIO before quantizing. For highest accuracy, OCIO should run on the non-quantized pixel data.
#if WITH_OCIO
	if (FMovieGraphOCIOHelper::GenerateOcioPixelPreProcessor(GraphPayload, CachedPipeline.Get(), InEvaluatedConfig, EvaluatedNode->OCIOConfiguration, EvaluatedNode->OCIOContext, PixelPreProcessors))
	{
		PixelPreProcessors.Pop()(InPixelData);
	}
#endif

	constexpr int32 TargetBitDepth = 8;
	const bool bConvertToSrgb = !CodecWriter->bSkipColorConversions;
	const TUniquePtr<FImagePixelData> QuantizedPixelData = UE::MoviePipeline::QuantizeImagePixelDataToBitDepth(InPixelData, TargetBitDepth, nullptr, bConvertToSrgb);

	// Do a quick composite of renders/burn-ins.
	for (const FMovieGraphPassData& CompositePass : InCompositePasses)
	{
		// We don't need to copy the data here (even though it's being passed to a async system) because we already made a unique copy of the
		// burn in/widget data when we decided to composite it.
		switch (QuantizedPixelData->GetType())
		{
		case EImagePixelType::Color:
			PixelPreProcessors.Add(TAsyncCompositeImage<FColor>(CompositePass.Value->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float16:
			PixelPreProcessors.Add(TAsyncCompositeImage<FFloat16Color>(CompositePass.Value->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float32:
			PixelPreProcessors.Add(TAsyncCompositeImage<FLinearColor>(CompositePass.Value->MoveImageDataToNew()));
			break;
		}
	}

	// This is done on the current thread for simplicity but the composite itself is parallelized.
	FImagePixelData* PixelData = QuantizedPixelData.Get();
	for (const FPixelPreProcessor& PreProcessor : PixelPreProcessors)
	{
		// PreProcessors are assumed to be valid.
		PreProcessor(PixelData);
	}

	const void* Data = nullptr;
	int64 DataSize;
	QuantizedPixelData->GetRawData(Data, DataSize);

	// WriteFrame expects Rec 709 8-bit data.
	CodecWriter->Writer->WriteFrame((uint8*)Data);
}

void UMovieGraphMP4EncoderNode::BeginFinalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	// If the writer was not initialized, don't try to finalize anything.
	FMP4CodecWriter* CodecWriter = static_cast<FMP4CodecWriter*>(InWriter);
	if (!CodecWriter->Writer)
	{
		return;
	}

	if (!CodecWriter->Writer->IsInitialized())
	{
		return;
	}
	
	// Nothing to do here if audio isn't being generated. The "invalid shot index" warning below is legitimate *if audio is being rendered*, but if
	// no audio is being rendered (eg, with -nosound) then we don't want the warning to show up.
	if (!FApp::CanEverRenderAudio())
	{
		return;
	}

	const MoviePipeline::FAudioState& AudioData = CachedPipeline->GetAudioRendererInstance()->GetAudioState();

	MovieRenderGraph::Audio::WriteShotAudioSamples(
		AudioData,
		CodecWriter->LightweightSourceData,
		CodecWriter->Writer->GetOptions().FrameRate,
		[&](TArrayView<int16> InSamples)
		{
			CodecWriter->Writer->WriteAudioSample(InSamples);
		});
}

void UMovieGraphMP4EncoderNode::Finalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	// Write to disk
	const FMP4CodecWriter* CodecWriter = static_cast<FMP4CodecWriter*>(InWriter);
	if(!CodecWriter->Writer)
	{
		return;
	}
	CodecWriter->Writer->Finalize();
}

const TCHAR* UMovieGraphMP4EncoderNode::GetFilenameExtension() const
{
	return TEXT("mp4");
}

bool UMovieGraphMP4EncoderNode::IsAudioSupported() const
{
	return true;
}

void UMovieGraphMP4EncoderNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesMP4 = true;
	InTelemetry->bMovieEncodesAudio |= bIncludeAudio;
}

void UMovieGraphMP4EncoderNode::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	if (bEnableBurnIn)
	{
		UMovieGraphOutputBurnInNode* BurnInNode = NewObject<UMovieGraphOutputBurnInNode>(InEvaluatedConfig);
		BurnInNode->bOverride_OutputName = true;
		BurnInNode->bOverride_FileNameFormat = true;
		BurnInNode->bOverride_OutputRestriction = true;
		BurnInNode->bOverride_LayerNameFormat = true;
		BurnInNode->bOverride_BurnInClass = true;
		BurnInNode->bOverride_bCompositeOntoFinalImage = true;
		BurnInNode->OutputName = TEXT("MP4");
		BurnInNode->FileNameFormat = BurnInFileNameFormat;
		BurnInNode->OutputRestriction = FSoftClassPath(GetClass());
		BurnInNode->LayerNameFormat = TEXT("");	// Unused
		BurnInNode->BurnInClass = BurnInClass;
		BurnInNode->bCompositeOntoFinalImage = bCompositeOntoFinalImage;

		OutInjectedNodes.Add(BurnInNode);
	}
}