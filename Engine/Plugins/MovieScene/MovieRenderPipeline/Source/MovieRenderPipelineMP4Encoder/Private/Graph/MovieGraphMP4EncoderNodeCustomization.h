// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Graph/MovieGraphMP4EncoderNode.h"

#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "MovieRenderPipelineMP4Encoder"

/** Customize how the MP4 Encoder node appears in the details panel. */
class FMovieGraphMP4EncoderNodeNodeCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMovieGraphMP4EncoderNodeNodeCustomization>();
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CustomizeDetails(*DetailBuilder);
	}
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		for (const TWeakObjectPtr<UMovieGraphMP4EncoderNode>& WeakMP4Node : DetailBuilder.GetObjectsOfTypeBeingCustomized<UMovieGraphMP4EncoderNode>())
		{
			const TAttribute<EVisibility> AverageBitrateVisibleAttr = TAttribute<EVisibility>::Create([WeakMP4Node]()
			{
				if (const TStrongObjectPtr<UMovieGraphMP4EncoderNode> MP4NodePin = WeakMP4Node.Pin())
				{
					return (MP4NodePin->EncodingRateControl == EMoviePipelineMP4EncodeRateControlMode::VariableBitRate) ? EVisibility::Visible : EVisibility::Collapsed;
				}
				
				return EVisibility::Collapsed;
			});

			const TAttribute<EVisibility> ConstantRateFactorVisibleAttr = TAttribute<EVisibility>::Create([WeakMP4Node]()
			{
				if (const TStrongObjectPtr<UMovieGraphMP4EncoderNode> MP4NodePin = WeakMP4Node.Pin())
				{
					return (MP4NodePin->EncodingRateControl == EMoviePipelineMP4EncodeRateControlMode::Quality) ? EVisibility::Visible : EVisibility::Collapsed;
				}
				
				return EVisibility::Collapsed;
			});

			const TSharedRef<IPropertyHandle> AverageBitrateHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphMP4EncoderNode, AverageBitrateInMbps));
			DetailBuilder.EditDefaultProperty(AverageBitrateHandle)
				->Visibility(AverageBitrateVisibleAttr);

			const TSharedRef<IPropertyHandle> ConstantRateFactorHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieGraphMP4EncoderNode, ConstantRateFactor));
			DetailBuilder.EditDefaultProperty(ConstantRateFactorHandle)
				->Visibility(ConstantRateFactorVisibleAttr);
		}
	}
	//~ End IDetailCustomization interface
};

#undef LOCTEXT_NAMESPACE

#endif	// WITH_EDITOR