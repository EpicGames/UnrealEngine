// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "MoviePipelineMP4EncoderCommon.generated.h"

// This file contains enums/structures that are common to both the Engine/Editor (for use in the UI) and for platform specific implementations of encoders.

/**
* Which encoding profile should be used for encoding? A higher profile usually means better quality for a given bitrate,
* but may not play back on older hardware.
*/
UENUM(BlueprintType)
enum class EMoviePipelineMP4EncodeProfile : uint8
{
	Baseline,
	Main,
	High,
};

/**
* A higher level generally results in a higher quality for a given bitrate, but
* a higher level requires newer encoders and decoders. Auto will let the encoder
* choose an appropriate one given the other parameters and is generally the best
* choice short of external needs.
*/
UENUM(BlueprintType)
enum class EMoviePipelineMP4EncodeLevel : uint8
{
	/** Let the encoder choose the best level based on other parameters. */
	Auto = 0,
	Level1 = 10,
	Level1_b = 11,
	Level1_1 = 11,
	Level1_2 = 12,
	Level1_3 = 13,
	Level2 = 20,
	Level2_1 = 21,
	Level2_2 = 22,
	Level3 = 30,
	Level3_1 = 31,
	Level3_2 = 32,
	Level4 = 40,
	Level4_1 = 41,
	Level4_2 = 42,
	Level5 = 50,
	Level5_1 = 51,
	Level5_2 = 52,
};

UENUM(BlueprintType)
enum class EMoviePipelineMP4EncodeRateControlMode : uint8
{
	/** Automatically chooses a bit rate to target the given quality. Valid ranges are 16-51. */
	ConstantQP UMETA(Hidden),

	/** Automatically chooses a bit rate to target the given quality. */
	Quality,

	/** Attempts to achieve a given mean bitrate for every frame. Can result in higher bitrates than necessary on simple frames, and lower bitrates than required on complex frames. Unconstrained and single-pass. */
	VariableBitRate,

	/** Uses a variable bitrate that attempts to achieve a given mean bitrate, but can use a higher bitrate (with no max) on a given frame if needed. Value is in bytes per second.*/
	VariableBitRate_Constrained UMETA(Hidden),

	/** Uses a variable bitrate that attempts to achieve a given mean bitrate, but can specify a maximum bitrate at which point quality will drop if needed. Value is in bytes per second.*/
	ConstantBitRate UMETA(Hidden),
}; 

struct FMoviePipelineMP4EncoderOptions
{
	FMoviePipelineMP4EncoderOptions()
		: OutputFilename()
		, Width(0)
		, Height(0)
		, FrameRate(30, 1)
		, bIncludeAudio(true)
		, AudioChannelCount(2)
		, AudioSampleRate(48000)
		, AudioAverageBitRate(24000) // 192 kbps (bits)
		, CommonMeanBitRate(12 * 1024 * 1024) // 12 mbps (bits)
		, CommonMaxBitRate(16 * 1024 * 1024) // 16 mbps (bits)
		, CommonQualityVsSpeed(100) // 0-33 Low Complexity, 34-66 Medium, 67-100 High (Higher = Slower but Better Quality)
		, CommonConstantRateFactor(18) // 16-51, Higher is Worse
		, EncodingProfile(EMoviePipelineMP4EncodeProfile::High)
		, EncodingLevel(EMoviePipelineMP4EncodeLevel::Auto)
		, EncodingRateControl(EMoviePipelineMP4EncodeRateControlMode::VariableBitRate)
	{}

	/** The absolute path on disk to try and save the video file to. */
	FString OutputFilename;

	/** The width of the video file. */
	uint32 Width;

	/** The height of the video file. */
	uint32 Height;

	/** Frame Rate of the output video. */
	FFrameRate FrameRate;
	
	/** If false, then audio tracks will not be written and calls to WriteAudioSample will be ignored. */
	bool bIncludeAudio;

	/** Number of audio channels for the audio track. */
	uint32 AudioChannelCount;

	/** Number of samples per second (ie: 48'000) for the audio track. */
	uint32 AudioSampleRate;

	/** Average bitrate for audio track. (in bytes-per-second). Only supported values are 12'000, 16'000, 20'000, and 24'000 */
	uint32 AudioAverageBitRate;

	/** Average bytes per second for ConstantBitRate, VariableBitRate_Constrained, VariableBitRate_Unconstrained rate control modes. */
	uint32 CommonMeanBitRate;
	/** Maximum bytes per second for VariableBitRate_Constrained */
	uint32 CommonMaxBitRate;
	/* Quality vs. Speed tradeoff during encode. 0 is faster but worse encode, 100 is slower but higher quality. */
	uint32 CommonQualityVsSpeed;
	/* If using Constant Quality, what is the CRF value? 16-51 is the valid range. */
	uint32 CommonConstantRateFactor;

	/** Which profile should be used when encoding? */
	EMoviePipelineMP4EncodeProfile EncodingProfile;

	/** What level should the profile use? */
	EMoviePipelineMP4EncodeLevel EncodingLevel;

	/** Which encoding rate control method should be used? */
	EMoviePipelineMP4EncodeRateControlMode EncodingRateControl;
};