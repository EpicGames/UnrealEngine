// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineMP4EncoderOutput.h"
#include "ImagePixelData.h"
#include "MoviePipeline.h"
#include "MoviePipelineImageQuantization.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineTelemetry.h"
#include "MovieRenderPipelineCoreModule.h"
#include "SampleBuffer.h"

// For logs
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineMP4EncoderOutput)

TUniquePtr<MovieRenderPipeline::IVideoCodecWriter> UMoviePipelineMP4EncoderOutput::Initialize_GameThread(const FString& InFileName, FIntPoint InResolution, EImagePixelType InPixelType, ERGBFormat InPixelFormat, uint8 InBitDepth, uint8 InNumChannels)
{
	const UMoviePipelineOutputSetting* OutputSettings = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	if (!OutputSettings)
	{
		return nullptr; 
	} 
	 
	FMoviePipelineMP4EncoderOptions Options;
	Options.OutputFilename = InFileName;
	Options.Width = InResolution.X;
	Options.Height = InResolution.Y;
	Options.FrameRate = GetPipeline()->GetPipelinePrimaryConfig()->GetEffectiveFrameRate(GetPipeline()->GetTargetSequence());
	Options.CommonMaxBitRate = FMath::RoundToInt(MaxBitrateInMbps*1024*1024);
	Options.CommonMeanBitRate = FMath::RoundToInt(AverageBitrateInMbps*1024*1024);
	Options.CommonQualityVsSpeed = 100;
	Options.CommonConstantRateFactor = ConstantRateFactor;
	Options.EncodingProfile = EncodingProfile;
	Options.EncodingLevel = EncodingLevel;
	Options.EncodingRateControl = EncodingRateControl;
	Options.bIncludeAudio = bIncludeAudio;
	
	TUniquePtr<FMoviePipelineMP4Encoder> Writer = MakeUnique<FMoviePipelineMP4Encoder>(Options);
	
	TUniquePtr<FMP4CodecWriter> OutWriter = MakeUnique<FMP4CodecWriter>();
	OutWriter->Writer = MoveTemp(Writer);
	OutWriter->FileName = InFileName;
	
	return OutWriter;
}

bool UMoviePipelineMP4EncoderOutput::Initialize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter)
{
	FMP4CodecWriter* CodecWriter = static_cast<FMP4CodecWriter*>(InWriter);
	if(!CodecWriter->Writer->Initialize())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to initialize Movie Pipeline MP4 writer. An encoder that supports the render resolution and requested MP4 encode options was not found. Try again with a different resolution and/or encoder settings.");
		return false;
	}

	return true;
}

void UMoviePipelineMP4EncoderOutput::WriteFrame_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<MoviePipeline::FCompositePassInfo>&& InCompositePasses)
{
	FMP4CodecWriter* CodecWriter = static_cast<FMP4CodecWriter*>(InWriter);
	
	// Quantize our 16 bit float data to 8 bit.
	const bool bConvertTosRGB = InWriter->bConvertToSrgb;
	constexpr int32 TargetBitDepth = 8;
	const TUniquePtr<FImagePixelData> QuantizedPixelData = UE::MoviePipeline::QuantizeImagePixelDataToBitDepth(InPixelData, TargetBitDepth, nullptr, bConvertTosRGB);

	// Do a quick composite of renders/burn-ins.
	TArray<FPixelPreProcessor> PixelPreProcessors;
	for (const MoviePipeline::FCompositePassInfo& CompositePass : InCompositePasses)
	{
		// We don't need to copy the data here (even though it's being passed to a async system) because we already made a unique copy of the
		// burn in/widget data when we decided to composite it.
		switch (QuantizedPixelData->GetType())
		{
		case EImagePixelType::Color:
			PixelPreProcessors.Add(TAsyncCompositeImage<FColor>(CompositePass.PixelData->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float16:
			PixelPreProcessors.Add(TAsyncCompositeImage<FFloat16Color>(CompositePass.PixelData->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float32:
			PixelPreProcessors.Add(TAsyncCompositeImage<FLinearColor>(CompositePass.PixelData->MoveImageDataToNew()));
			break;
		}
	}

	// This is done on the main thread for simplicity but the composite itself is parallelized.
	FImagePixelData* PixelData = QuantizedPixelData.Get();
	for (const FPixelPreProcessor& PreProcessor : PixelPreProcessors)
	{
		// PreProcessors are assumed to be valid.
		PreProcessor(PixelData);
	}

	const void* Data = nullptr;
	int64 DataSize;
	QuantizedPixelData->GetRawData(Data, DataSize);

	// WriteFrame expects Rec 709 8-bit data.
	CodecWriter->Writer->WriteFrame((uint8*)Data);
}

void UMoviePipelineMP4EncoderOutput::BeginFinalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter)
{
	const MoviePipeline::FAudioState& AudioData = GetPipeline()->GetAudioState();
	FMP4CodecWriter* CodecWriter = static_cast<FMP4CodecWriter*>(InWriter);

	// If the writer was not initialized, don't try to finalize anything.
	if (!CodecWriter->Writer)
	{
		return;
	}

	if (!CodecWriter->Writer->IsInitialized())
	{
		return;
	}

	// Nothing to do here if audio isn't being generated. The "invalid shot index" warning below is legitimate *if audio is being rendered*, but if
	// no audio is being rendered (eg, with -nosound) then we don't want the warning to show up.
	if (!FApp::CanEverRenderAudio())
	{
		return;
	}

	for (const TPair<int32, MovieRenderPipeline::IVideoCodecWriter::FLightweightSourceData>& SourceData : CodecWriter->LightweightSourceData)
	{
		if (!AudioData.FinishedSegments.IsValidIndex(SourceData.Key))
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Invalid shot index was requested for audio data, skipping audio writes.");
			continue;
		}

		// Look up the audio segment for this shot
		const MoviePipeline::FAudioState::FAudioSegment& AudioSegment = AudioData.FinishedSegments[SourceData.Key];

		// Audio data isn't very sample accurate at this point, so we may have generated slightly more (or less) audio than we expect for
		// the number of frames, so we're actually going to trim down the view of data we provide to match the number of frames rendered,
		// to avoid any excess audio after the end of the video.
		const int32 NumFrames = SourceData.Value.SubmittedFrameCount;

		// ToDo: This is possibly dropping fractions of a sample (ie: 1/48,000th) if the audio sample rate can't be evenly divisible by
		// the frame rate.
		const int32 SamplesPerFrame = AudioSegment.SampleRate * CodecWriter->Writer->GetOptions().FrameRate.AsInterval();
		int32 ExpectedSampleCount = NumFrames * SamplesPerFrame * AudioSegment.NumChannels;

		ExpectedSampleCount = FMath::Min(ExpectedSampleCount, AudioSegment.SegmentData.Num());
		Audio::TSampleBuffer<int16> SampleBuffer = Audio::TSampleBuffer<int16>(AudioSegment.SegmentData.GetData(), ExpectedSampleCount, AudioSegment.NumChannels, AudioSegment.SampleRate);

		const TArrayView<int16> SampleData = SampleBuffer.GetArrayView();
		CodecWriter->Writer->WriteAudioSample(SampleData);
	}
}

void UMoviePipelineMP4EncoderOutput::Finalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter)
{
	// Commit this to disk.
	FMP4CodecWriter* CodecWriter = static_cast<FMP4CodecWriter*>(InWriter);
	if(!CodecWriter->Writer)
	{
		return;
	}

	CodecWriter->Writer->Finalize();
}

void UMoviePipelineMP4EncoderOutput::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesMP4 = true;
	InTelemetry->bMovieEncodesAudio |= bIncludeAudio;
}
