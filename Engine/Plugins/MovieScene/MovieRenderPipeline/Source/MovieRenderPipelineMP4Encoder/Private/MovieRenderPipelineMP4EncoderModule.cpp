// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "Graph/MovieGraphMP4EncoderNode.h"
#include "Graph/MovieGraphMP4EncoderNodeCustomization.h"

class FMovieRenderPipelineMP4EncoderModule : public IModuleInterface
{
public:
	/*~ Begin IModuleInterface interface */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	/*~ End IModuleInterface interface */

private:
	/** StaticClass() isn't safe to call during ShutdownModule(), so cache the names of registered classes here. */
	TArray<FName> ClassesToUnregisterOnShutdown;
};

IMPLEMENT_MODULE(FMovieRenderPipelineMP4EncoderModule, MovieRenderPipelineMP4Encoder)

void FMovieRenderPipelineMP4EncoderModule::StartupModule()
{
#if WITH_EDITOR
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	ClassesToUnregisterOnShutdown.Add(UMovieGraphMP4EncoderNode::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout(ClassesToUnregisterOnShutdown.Last(), FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphMP4EncoderNodeNodeCustomization::MakeInstance));
#endif	// WITH_EDITOR
}

void FMovieRenderPipelineMP4EncoderModule::ShutdownModule()
{
#if WITH_EDITOR
	// Unregister details customizations
	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (const FName& ClassToUnregister : ClassesToUnregisterOnShutdown)
		{
			PropertyModule->UnregisterCustomClassLayout(ClassToUnregister);
		}
	}
#endif	// WITH_EDITOR
}