// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineMP4Encoder.h"

#include "MovieRenderPipelineCoreModule.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/WindowsPlatformMisc.h"
#include "Windows/WindowsHWrapper.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <Windows.h>
#include <mfapi.h>
#include <mfidl.h>
#include <Mfreadwrite.h>
#include <mferror.h>
#include <Codecapi.h>
#include <strmif.h>
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

namespace
{
	/** Gets the path of the DLL that the ICodecAPI comes from. */
	FString GetCodecApiDllPath(ICodecAPI* InCodecAPI)
	{
		static const FString UnknownDLL(TEXT("Unknown"));

		if (!InCodecAPI)
		{
			return UnknownDLL;
		}

		// This is really funky pointer trickery. According to Microsoft, "A COM interface pointer is a pointer to a structure that consists of just
		// a vtable". The first entry in the vtable is the address to the QueryInterface (which will originate from the DLL of the ICodecAPI
		// implementation that is being used).
		void** VtableAddress = *reinterpret_cast<void***>(InCodecAPI);
		const void* QueryInterfaceAddress = VtableAddress[0];

		// Get memory information about the page at the query interface's address
		MEMORY_BASIC_INFORMATION MemoryInformation;
		if (VirtualQueryEx(GetCurrentProcess(), QueryInterfaceAddress, &MemoryInformation, sizeof(MemoryInformation)))
		{
			// Get the path of the module handle
			WCHAR DllPath[MAX_PATH] = {};
			if (GetModuleFileNameW(static_cast<HMODULE>(MemoryInformation.AllocationBase), DllPath, MAX_PATH))
			{
				return FString(DllPath);
			}
		}

		return UnknownDLL;
	}
}

FMoviePipelineMP4Encoder::FMoviePipelineMP4Encoder(const FMoviePipelineMP4EncoderOptions& InOptions)
	: Options(InOptions)
	, bInitialized(false)
	, bFinalized(false)
	, NumVideoSamplesWritten(0)
	, NumAudioSamplesWritten(0)
	, SinkWriter(nullptr)
	, VideoStreamIndex(0)
	, AudioStreamIndex(0)
{

}

FMoviePipelineMP4Encoder::~FMoviePipelineMP4Encoder()
{
	// Insure Finalize is called so that we release the COM library if we were ever initialized.
	Finalize();
}

bool FMoviePipelineMP4Encoder::Initialize()
{
	// Initialize us for single-threaded communication with the library.
	if (!FWindowsPlatformMisc::CoInitialize())
	{
		UE_LOGF(LogTemp, Error, "Failed to initialize COM library.");
		return false;
	}

	// Initialize the Microsoft Media Foundation
	HRESULT Result = MFStartup(MF_VERSION);
	if (!SUCCEEDED(Result))
	{
		UE_LOGF(LogTemp, Error, "Failed to initialize Microsoft Media Foundation.");
		return false;
	}

	bool bResult = InitializeEncoder();
	if (!bResult)
	{
		UE_LOGF(LogTemp, Error, "Failed to initialize Sink Writer.");
		return false;
	}

	bInitialized = true;
	return true;
}

void FMoviePipelineMP4Encoder::Finalize()
{
	if (bFinalized || !bInitialized)
	{
		return;
	}

	if (SinkWriter)
	{
		HRESULT Result = SinkWriter->Finalize();
		if (!SUCCEEDED(Result))
		{
			UE_LOGF(LogTemp, Error, "Failed to finalize Sink Writer.");
		}

		SinkWriter->Release();
		SinkWriter = nullptr;
	}

	HRESULT Result = MFShutdown();
	if (!SUCCEEDED(Result))
	{
		UE_LOGF(LogTemp, Error, "Failed to shut down Microsoft Media Foundation.");
	}

	// Release the COM library
	FWindowsPlatformMisc::CoUninitialize();

	bFinalized = true;
}


bool FMoviePipelineMP4Encoder::WriteFrame(const uint8* InFrameData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WMFVideoEncoder_WriteFrame);
	if (!ensureMsgf(bInitialized || bFinalized, TEXT("WriteFrame should not be called if not initialized or after finalize! Initialized: %d Finalized: %d"), bInitialized, bFinalized))
	{
		return false;
	}

	IMFSample* Sample = nullptr;
	IMFMediaBuffer* Buffer = nullptr;
	HRESULT Result;

	const LONG SourceStride = Options.Width * 4; // 4 bytes per pixel
	const LONG BufferSize = SourceStride * Options.Height;
	{
		Result = MFCreateMemoryBuffer(BufferSize, &Buffer);

		if (!SUCCEEDED(Result))
		{
			UE_LOGF(LogTemp, Error, "Failed to allocate Media Foundation Buffer to write frame. Width: %d Height: %d Size: %d", Options.Width, Options.Height, BufferSize);
			return false;
		}
	}

	// Lock the buffer and copy our incoming frame data into the Media Foundation buffer.
	BYTE* DestinationData = nullptr;
	{
		Result = Buffer->Lock(&DestinationData, NULL, NULL);
	}

	// We can't early out on fail since we need to release the resources in the event of failure.
	if (SUCCEEDED(Result))
	{
		Result = MFCopyImage(DestinationData, SourceStride, (BYTE*)InFrameData, SourceStride, SourceStride, Options.Height);

		// The below snippet would let you read in data that is from bottom-to-top, instead of top-to-bottom. Depending on
		// the parameters, the hardware and software encoder have produced different ordering results, see comment on
		// MF_MT_VIDEO_NOMINAL_RANGE below.
		/*Result = MFCopyImage(DestinationData,
			SourceStride,
			((BYTE*) InFrameData) + (Options.Height-1)*SourceStride,
			-SourceStride,
			SourceStride,
			Options.Height); // Not Flipped*/
	}

	if (SUCCEEDED(Result))
	{
		Result = Buffer->Unlock();
	}

	if (SUCCEEDED(Result))
	{
		// Specify how much of the data in the buffer is valid.
		Result = Buffer->SetCurrentLength(BufferSize);
	}

	// Create a new sample
	if (SUCCEEDED(Result))
	{
		Result = MFCreateSample(&Sample);
	}
	
	if (SUCCEEDED(Result) && Sample)
	{
		Result = Sample->AddBuffer(Buffer);
	}

	// Duration & Timestamp for this frame.
	if (SUCCEEDED(Result) && Sample)
	{
		// In 100 nano-second units.
		uint64 FrameDuration = (10 * 1000 * 1000) * Options.FrameRate.AsInterval();
		Result = Sample->SetSampleTime(FrameDuration * NumVideoSamplesWritten);
		NumVideoSamplesWritten++;
	}

	if (SUCCEEDED(Result) && Sample)
	{
		// In 100 nano-second units.
		uint64 FrameDuration = (10 * 1000 * 1000) * Options.FrameRate.AsInterval();
		Result = Sample->SetSampleDuration(FrameDuration);
	}

	// Send the sample to the Sink Writer
	if (SUCCEEDED(Result) && Sample)
	{
		Result = SinkWriter->WriteSample(VideoStreamIndex, Sample);
	}

	if (Sample)
	{
		Sample->Release();
	}
	if (Buffer)
	{
		Buffer->Release();
	}

	return Result == S_OK;
}

bool FMoviePipelineMP4Encoder::WriteAudioSample(const TArrayView<int16>& InAudioSamples)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MRQ_WMFSink_WriteAudioSample);

	// If we aren't including audio, then we just say it was a success so that callers don't get confused.
	if (!Options.bIncludeAudio)
	{
		return true;
	}

	// If we were not initialized, we can't write the sample.
	if (!ensureMsgf(bInitialized || bFinalized, TEXT("WriteAudioSample should not be called if not initialized or after finalize! Initialized: %d Finalized: %d"), bInitialized, bFinalized))
	{
		return false;
	}

	// Figure out how many frames worth of audio are being provided by the TArrayView.
	int32 SampleCountPerChannel = InAudioSamples.Num() / Options.AudioChannelCount;
	int32 SamplesPerFrame = Options.AudioSampleRate * Options.FrameRate.AsInterval();
	int32 NumFramesOfData = SampleCountPerChannel / SamplesPerFrame;

	HRESULT Result = S_OK;

	for (int32 SampleIndex = 0; SampleIndex < NumFramesOfData; SampleIndex++)
	{
		IMFSample* Sample = nullptr;
		IMFMediaBuffer* Buffer = nullptr;

		const LONG BufferSize = SamplesPerFrame * sizeof(int16) * Options.AudioChannelCount;
		Result = MFCreateMemoryBuffer(BufferSize, &Buffer);

		if (!SUCCEEDED(Result))
		{
			UE_LOGF(LogTemp, Error, "Failed to allocate Media Foundation Buffer to write audio sample. Buffer Size: %d", BufferSize);
			return false;
		}

		// Lock the buffer and copy our incoming frame data into the Media Foundation buffer.
		BYTE* DestinationData = nullptr;
		Result = Buffer->Lock(&DestinationData, NULL, NULL);

		if (SUCCEEDED(Result))
		{
			// Figure out the offset into the source data.
			int16* SourceOffset = (int16*)InAudioSamples.GetData() + (SampleIndex * (SamplesPerFrame * Options.AudioChannelCount));

			// MFCopyImage seems to really just be a generic SSE optimized copy function. 
			Result = MFCopyImage(DestinationData,
				BufferSize,
				(BYTE*)SourceOffset,
				BufferSize,
				BufferSize,
				1);
		}

		Buffer->Unlock();

		if (SUCCEEDED(Result))
		{
			// Specify how much of the data in the buffer is valid.
			Result = Buffer->SetCurrentLength(BufferSize);
		}

		// Create a new sample
		if (SUCCEEDED(Result))
		{
			Result = MFCreateSample(&Sample);
		}

		if (SUCCEEDED(Result) && Sample)
		{
			Result = Sample->AddBuffer(Buffer);
		}

		// Duration & Timestamp for this frame.
		if (SUCCEEDED(Result) && Sample)
		{
			// The duration matches the duration of one frame.
			uint64 FrameDuration = (10 * 1000 * 1000) * Options.FrameRate.AsInterval();
			Result = Sample->SetSampleDuration(FrameDuration);
		}

		if (SUCCEEDED(Result) && Sample)
		{
			uint64 FrameDuration = (10 * 1000 * 1000) * Options.FrameRate.AsInterval();
			Result = Sample->SetSampleTime(NumAudioSamplesWritten * FrameDuration);

			// We track this in the file and not just this for loop because WriteAudioSample can get called
			// repeatedly in the event that this video rendered multiple separate shots.
			NumAudioSamplesWritten++;
		}

		// Send the sample to the Sink Writer
		if (SUCCEEDED(Result) && Sample)
		{
			Result = SinkWriter->WriteSample(AudioStreamIndex, Sample);
		}

		if (Sample)
		{
			Sample->Release();
		}
		if (Buffer)
		{
			Buffer->Release();
		}

		// Early out if we were unable to write a sample for some reason,
		// we should have freed any buffers allocated during this loop.
		if (!SUCCEEDED(Result))
		{
			return false;
		}
	}


	return Result == S_OK;
}

static uint32_t GetEncodingProfile(const EMoviePipelineMP4EncodeProfile InProfile)
{
	switch (InProfile)
	{
	case EMoviePipelineMP4EncodeProfile::Baseline: return eAVEncH264VProfile_Base;
	case EMoviePipelineMP4EncodeProfile::Main: return eAVEncH264VProfile_Main;
	case EMoviePipelineMP4EncodeProfile::High: return eAVEncH264VProfile_High;
	default:
		check(false);
	}
	return 0;
}

static uint32_t GetEncodingRateControl(const EMoviePipelineMP4EncodeRateControlMode InProfile)
{
	switch (InProfile)
	{
	case EMoviePipelineMP4EncodeRateControlMode::ConstantBitRate: return eAVEncCommonRateControlMode_CBR;
	case EMoviePipelineMP4EncodeRateControlMode::VariableBitRate_Constrained: return eAVEncCommonRateControlMode_PeakConstrainedVBR;
	case EMoviePipelineMP4EncodeRateControlMode::VariableBitRate: return eAVEncCommonRateControlMode_UnconstrainedVBR;
	case EMoviePipelineMP4EncodeRateControlMode::Quality: return eAVEncCommonRateControlMode_Quality;
	case EMoviePipelineMP4EncodeRateControlMode::ConstantQP: return eAVEncCommonRateControlMode_PeakConstrainedVBR;
	default:
		check(false);
	}
	return 0;
}

static HRESULT CreateVideoMediaOutputStream(IMFMediaType** OutVideoMediaTypeOutput, const FMoviePipelineMP4EncoderOptions& InOptions)
{
	HRESULT Result = S_OK;
	if (SUCCEEDED(Result))
	{
		Result = MFCreateMediaType(OutVideoMediaTypeOutput);
	}

	if (*OutVideoMediaTypeOutput == nullptr)
	{
		return Result;
	}

	// Set the major media type for the stream.
 	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video);
	}

	// Set the sub-type for the stream
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetGUID(MF_MT_SUBTYPE, MFVideoFormat_H264);
	}

	// Interlace Mode
	if (SUCCEEDED(Result))
	{
		// No support for interleaving.
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_INTERLACE_MODE, MFVideoInterlace_Progressive);
	}

	// Specify the Width/Height of the Output
	if (SUCCEEDED(Result))
	{
		Result = MFSetAttributeSize((*OutVideoMediaTypeOutput), MF_MT_FRAME_SIZE, InOptions.Width, InOptions.Height);
	}

	// Frame Rate
	if (SUCCEEDED(Result))
	{
		Result = MFSetAttributeRatio((*OutVideoMediaTypeOutput), MF_MT_FRAME_RATE, InOptions.FrameRate.Numerator, InOptions.FrameRate.Denominator);
	}

	// Pixel Aspect Ratio
	if (SUCCEEDED(Result))
	{
		// Always square pixels.
		Result = MFSetAttributeRatio((*OutVideoMediaTypeOutput), MF_MT_PIXEL_ASPECT_RATIO, 1, 1);
	}

	// Color Primaries
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_VIDEO_PRIMARIES, MFVideoPrimaries_BT709);
	}

	// Color Transfer Function
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_TRANSFER_FUNCTION, MFVideoTransFunc_sRGB);
	}

	// Color YUV Matrix
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_YUV_MATRIX, MFVideoTransferMatrix_BT709);
	}

	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_VIDEO_NOMINAL_RANGE, MFNominalRange_16_235);
	}

	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(CODECAPI_AVEncCommonQualityVsSpeed, InOptions.CommonQualityVsSpeed);
	}

	// Note: Some parameters can be set on the stream before stream initialization, others need to be set as part of the stream
	// initialization.
	// These are set for H264 encoding and may be different for other formats in the future.
	{
		// There's a number of additional properties supported by the encoder, mostly
		// related to giving more control over bitrate, GOP, Profiles, etc. Ommitted
		// for now due to defaults generally being acceptable.
		// See: https://learn.microsoft.com/en-us/windows/win32/medfound/h-264-video-encoder
		if (SUCCEEDED(Result))
		{
			Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_MPEG2_PROFILE, GetEncodingProfile(InOptions.EncodingProfile));
		}
		if (SUCCEEDED(Result))
		{
			EMoviePipelineMP4EncodeLevel Level = InOptions.EncodingLevel;
			uint32_t LevelValue = Level == EMoviePipelineMP4EncodeLevel::Auto ? -1 : (uint32_t)Level;
			Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_MPEG2_LEVEL, LevelValue);
		}

		if (SUCCEEDED(Result))
		{
			Result = (*OutVideoMediaTypeOutput)->SetUINT32(CODECAPI_AVEncH264CABACEnable, true);
		}

		if (SUCCEEDED(Result))
		{
			Result = (*OutVideoMediaTypeOutput)->SetUINT32(CODECAPI_AVEncCommonRateControlMode, GetEncodingRateControl(InOptions.EncodingRateControl));
		}

		if (SUCCEEDED(Result))
		{
			Result = (*OutVideoMediaTypeOutput)->SetUINT32(CODECAPI_AVEncMPVDefaultBPictureCount, 2);
		}
	}

	return Result;
}

static HRESULT CreateAudioMediaOutputStream(IMFMediaType** OutVideoMediaTypeOutput, const FMoviePipelineMP4EncoderOptions& InOptions)
{
	HRESULT Result = S_OK;
	if (SUCCEEDED(Result))
	{
		Result = MFCreateMediaType(OutVideoMediaTypeOutput);
	}

	if (*OutVideoMediaTypeOutput == nullptr)
	{
		return Result;
	}

	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio);
	}

	// MP4 requires MP3 (or AAC)
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetGUID(MF_MT_SUBTYPE, MFAudioFormat_AAC);
	}

	// Bits per Channel
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, 16);
	}

	// Samples per Second
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_AUDIO_SAMPLES_PER_SECOND, InOptions.AudioSampleRate);
	}

	// Channel Count
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_AUDIO_NUM_CHANNELS, InOptions.AudioChannelCount);
	}

	// Average bytes per second
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeOutput)->SetUINT32(MF_MT_AUDIO_AVG_BYTES_PER_SECOND, InOptions.AudioAverageBitRate);
	}

	return Result;
}

static HRESULT CreateVideoMediaTypeIn(IMFMediaType** OutVideoMediaTypeIn, const FMoviePipelineMP4EncoderOptions& InOptions)
{
	HRESULT Result = S_OK;

	// Create Video and Audio media types.
	if (SUCCEEDED(Result))
	{
		Result = MFCreateMediaType(OutVideoMediaTypeIn);
	}

	if (*OutVideoMediaTypeIn == nullptr)
	{
		return Result;
	}

	// Set the major media type for the stream.
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeIn)->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video);
	}

	// Set the sub-type for the stream
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeIn)->SetGUID(MF_MT_SUBTYPE, MFVideoFormat_ARGB32);
	}

	// Interlace Mode
	if (SUCCEEDED(Result))
	{
		// No support for interleaving.
		Result = (*OutVideoMediaTypeIn)->SetUINT32(MF_MT_INTERLACE_MODE, MFVideoInterlace_Progressive);
	}

	// Specify the Width/Height of the Input
	if (SUCCEEDED(Result))
	{
		Result = MFSetAttributeSize((*OutVideoMediaTypeIn), MF_MT_FRAME_SIZE, InOptions.Width, InOptions.Height);
	}

	// Frame Rate
	if (SUCCEEDED(Result))
	{
		Result = MFSetAttributeRatio((*OutVideoMediaTypeIn), MF_MT_FRAME_RATE, InOptions.FrameRate.Numerator, InOptions.FrameRate.Denominator);
	}

	// Pixel Aspect Ratio
	if (SUCCEEDED(Result))
	{
		// Always square pixels.
		Result = MFSetAttributeRatio((*OutVideoMediaTypeIn), MF_MT_PIXEL_ASPECT_RATIO, 1, 1);
	}

	// Input frames have dependencies on previous samples
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeIn)->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, true);
	}

	// Setting an explicit (positive) stride and a sample size is important to ensure all encoders process frames top-down, otherwise some encoders
	// may process frames upside-down.
	const uint32 Stride = InOptions.Width * 4;	// 4 bytes per pixel
	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeIn)->SetUINT32(MF_MT_DEFAULT_STRIDE, Stride);
	}

	if (SUCCEEDED(Result))
	{
		Result = (*OutVideoMediaTypeIn)->SetUINT32(MF_MT_SAMPLE_SIZE, Stride * InOptions.Height);
	}

	return Result;
}

static HRESULT CreateAudioMediaTypeIn(IMFMediaType** OutAudioMediaTypeIn, const FMoviePipelineMP4EncoderOptions& InOptions)
{
	HRESULT Result = S_OK;
	if (SUCCEEDED(Result))
	{
		Result = MFCreateMediaType(OutAudioMediaTypeIn);
	}

	if (*OutAudioMediaTypeIn == nullptr)
	{
		return Result;
	}

	if (SUCCEEDED(Result))
	{
		Result = (*OutAudioMediaTypeIn)->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio);
	}

	if (SUCCEEDED(Result))
	{
		// MP4 with MP3 requires PCM.
		Result = (*OutAudioMediaTypeIn)->SetGUID(MF_MT_SUBTYPE, MFAudioFormat_PCM);
	}

	const int32 BytesPerChannel = 2;

	// Bits Per Sample 
	if (SUCCEEDED(Result))
	{
		// PCM requires signed 16 bit integer. 
		Result = (*OutAudioMediaTypeIn)->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, BytesPerChannel * 8);
	}

	// Input Samples per Second
	if (SUCCEEDED(Result))
	{
		Result = (*OutAudioMediaTypeIn)->SetUINT32(MF_MT_AUDIO_SAMPLES_PER_SECOND, InOptions.AudioSampleRate);
	}

	// Input Channel Count
	if (SUCCEEDED(Result))
	{
		Result = (*OutAudioMediaTypeIn)->SetUINT32(MF_MT_AUDIO_NUM_CHANNELS, InOptions.AudioChannelCount);
	}

	// PCM audio has no dependencies on previous samples
	if (SUCCEEDED(Result))
	{
		Result = (*OutAudioMediaTypeIn)->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, true);
	}

	// PCM block alignment is numChannels*bytesPerChannel
	if (SUCCEEDED(Result))
	{
		Result = (*OutAudioMediaTypeIn)->SetUINT32(MF_MT_AUDIO_BLOCK_ALIGNMENT, InOptions.AudioChannelCount * BytesPerChannel);
	}

	// Average byte-rate is just block alignment (above) * samples per second
	if (SUCCEEDED(Result))
	{
		Result = (*OutAudioMediaTypeIn)->SetUINT32(MF_MT_AUDIO_AVG_BYTES_PER_SECOND, (InOptions.AudioChannelCount * BytesPerChannel) * InOptions.AudioSampleRate);
	}

	return Result;
}

static HRESULT GetVideoStreamEncoderAttributes(IMFAttributes** OutEncoderAttributes, const FMoviePipelineMP4EncoderOptions& InOptions)
{
	HRESULT Result = MFCreateAttributes(OutEncoderAttributes, 1);

	if (*OutEncoderAttributes == nullptr)
	{
		return Result;
	}

	// These are set for H264 encoding and may be different for other formats in the future.
	{
		if (InOptions.EncodingRateControl == EMoviePipelineMP4EncodeRateControlMode::ConstantBitRate)
		{
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonMeanBitRate, InOptions.CommonMeanBitRate);
			}

			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonBufferSize, InOptions.CommonMeanBitRate);
			}

		}
		else if (InOptions.EncodingRateControl == EMoviePipelineMP4EncodeRateControlMode::VariableBitRate_Constrained)
		{
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonBufferSize, InOptions.CommonMeanBitRate);

			}
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonMaxBitRate, InOptions.CommonMaxBitRate);
			}
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonMeanBitRate, InOptions.CommonMeanBitRate);
			}
		}
		else if (InOptions.EncodingRateControl == EMoviePipelineMP4EncodeRateControlMode::VariableBitRate)
		{
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonMaxBitRate, InOptions.CommonMaxBitRate);
			}
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonMeanBitRate, InOptions.CommonMeanBitRate);
			}
		}
		else if (InOptions.EncodingRateControl == EMoviePipelineMP4EncodeRateControlMode::Quality)
		{
			// This seems to make very similar results to ConstantQP despite being different code.
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonRateControlMode, eAVEncCommonRateControlMode_Quality);
			}
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT64(CODECAPI_AVEncVideoEncodeQP, InOptions.CommonConstantRateFactor);
			}
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonQuality, 0);
			}
		}
		else if (InOptions.EncodingRateControl == EMoviePipelineMP4EncodeRateControlMode::ConstantQP)
		{
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncCommonQuality, InOptions.CommonConstantRateFactor);
			}
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncVideoMinQP, InOptions.CommonConstantRateFactor);
			}
			if (SUCCEEDED(Result))
			{
				Result = (*OutEncoderAttributes)->SetUINT32(CODECAPI_AVEncVideoMaxQP, InOptions.CommonConstantRateFactor);
			}
		}
	}

	return Result;
}

bool FMoviePipelineMP4Encoder::InitializeEncoder()
{
	SinkWriter = nullptr;

	// Create a Video and Audio Stream
	IMFMediaType* VideoMediaTypeOut = nullptr;
	IMFMediaType* VideoMediaTypeIn = nullptr;

	IMFMediaType* AudioMediaTypeOut = nullptr;
	IMFMediaType* AudioMediaTypeIn = nullptr;

	IMFAttributes* ConfigAttributes = nullptr;
	HRESULT Result = MFCreateAttributes(&ConfigAttributes, 1);

	// Avoid any early-outs so that we can still release allocations in the event of an error.
	if (SUCCEEDED(Result))
	{
		// Disable low-latency as we don't need this to be realtime which may affect quality.
		ConfigAttributes->SetUINT32(CODECAPI_AVLowLatencyMode, false);

		// Use hardware transforms if available. 
		ConfigAttributes->SetUINT32(MF_READWRITE_ENABLE_HARDWARE_TRANSFORMS, true);

		// Disable throttling in WriteFrame, trading system memory for overall runtime performance.
		ConfigAttributes->SetUINT32(MF_SINK_WRITER_DISABLE_THROTTLING, true);
	}

	if (SUCCEEDED(Result))
	{
		// Attempt to create the output file.
		Result = MFCreateSinkWriterFromURL(*Options.OutputFilename, /*Byte Stream Dest*/ NULL, ConfigAttributes, &SinkWriter);
	}
	if (SUCCEEDED(Result))
	{
		Result = CreateVideoMediaOutputStream(&VideoMediaTypeOut, Options);
	}

	// Video Stream
	if (SUCCEEDED(Result) && SinkWriter)
	{
		Result = SinkWriter->AddStream(VideoMediaTypeOut, (DWORD*)&VideoStreamIndex);
	}

	if (Options.bIncludeAudio)
	{
		if (SUCCEEDED(Result))
		{
			Result = CreateAudioMediaOutputStream(&AudioMediaTypeOut, Options);
		}

		// Audio Stream
		if (SUCCEEDED(Result) && SinkWriter)
		{
			Result = SinkWriter->AddStream(AudioMediaTypeOut, (DWORD*)&AudioStreamIndex);
		}

		if (SUCCEEDED(Result))
		{
			Result = CreateAudioMediaTypeIn(&AudioMediaTypeIn, Options);
		}
	}

	if (SUCCEEDED(Result))
	{
		Result = CreateVideoMediaTypeIn(&VideoMediaTypeIn, Options);
	}

	if (SUCCEEDED(Result) && SinkWriter)
	{
		IMFAttributes* pEncAttrs = nullptr;
		Result = GetVideoStreamEncoderAttributes(&pEncAttrs, Options);
		if (SUCCEEDED(Result))
		{
			Result = SinkWriter->SetInputMediaType(VideoStreamIndex, VideoMediaTypeIn, pEncAttrs);
		}

		if (pEncAttrs)
		{
			pEncAttrs->Release();
		}
	}

	if (Options.bIncludeAudio)
	{
		if (SUCCEEDED(Result) && SinkWriter)
		{
			Result = SinkWriter->SetInputMediaType(AudioStreamIndex, AudioMediaTypeIn, NULL);
		}
	}

	// Log the path of the DLL the encoder is using for debug purposes
	if (SUCCEEDED(Result) && SinkWriter)
	{
		ICodecAPI* CodecApi = nullptr;
		Result = SinkWriter->GetServiceForStream(VideoStreamIndex, GUID_NULL, __uuidof(ICodecAPI), (LPVOID*)&CodecApi);
		
		// The best we can do is provide the DLL path. Ideally we could use IMFSinkWriterEx to get the friendly name attribute for the active
		// encoder, but IMFSinkWriterEx is not available with the currently defined WINVER (needs to be >= 0x0602).
		const FString CodecApiDllPath = GetCodecApiDllPath(CodecApi);
		UE_LOGF(LogMovieRenderPipeline, Display, "Using the following encoder for the MP4 encode: %ls", *CodecApiDllPath);
	}

	// The sink writer can now accept data.
	if (SUCCEEDED(Result) && SinkWriter)
	{
		SinkWriter->BeginWriting();
	}

	if (SUCCEEDED(Result) && SinkWriter)
	{
		SinkWriter->AddRef();
	}

	// Release temporary resources.
	if (SinkWriter)
	{
		SinkWriter->Release();
	}

	if (VideoMediaTypeOut)
	{
		VideoMediaTypeOut->Release();
	}
	if (VideoMediaTypeIn)
	{
		VideoMediaTypeIn->Release();
	}
	if (AudioMediaTypeOut)
	{
		AudioMediaTypeOut->Release();
	}
	if (AudioMediaTypeIn)
	{
		AudioMediaTypeIn->Release();
	}

	return Result == S_OK;
}
