// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "MoviePipelineMP4EncoderCommon.h"

/**
* Takes in 8-bit RGBA frames which have had an sRGB gamut applied, and outputs a YUV 4:2:0 encoded video.
* Targeted at non-realtime applications (ie: movie rendering) and not realtime encoding. Experimental.
* 
* This should have a matching public API to the implementation in the other platform folders.
*/
class FMoviePipelineMP4Encoder
{
public:
	FMoviePipelineMP4Encoder(const FMoviePipelineMP4EncoderOptions& InOptions);
	~FMoviePipelineMP4Encoder();

	/** Call to initialize the Sink Writer. This must be done before attempting to write data to it. */
	bool Initialize();

	/** Finalize the video file and finish writing it to disk. Called by the destructor if not automatically called. */
	void Finalize();

	/** Appends a new frame onto the output file. */
	bool WriteFrame(const uint8* InFrameData);
	/** Appends a new audio sample onto the audio stream. */
	bool WriteAudioSample(const TArrayView<int16>& InAudioSamples);

	const FMoviePipelineMP4EncoderOptions& GetOptions() const { return Options; }

	bool IsInitialized() const { return bInitialized; }

private:
	/** Configure and initialize the output file */
	bool InitializeEncoder();

private:
	/** Input/Output options this writer was initialized with. */
	FMoviePipelineMP4EncoderOptions Options;

	/** Has Initialize been successfully called? */
	bool bInitialized;

	/** Has Finalize been called? */
	bool bFinalized;

	/** How many video samples (frames) have we written so far? */
	uint64 NumVideoSamplesWritten;

	/** How many audio samples (frames) have we written so far? */
	uint64 NumAudioSamplesWritten;

	/** The sink writer we are writing samples to. */
	struct IMFSinkWriter* SinkWriter;

	/** Stream index for video within the Sink Writer */
	uint32 VideoStreamIndex;
	/** Stream index for audio within the Sink Writer */
	uint32 AudioStreamIndex;
};