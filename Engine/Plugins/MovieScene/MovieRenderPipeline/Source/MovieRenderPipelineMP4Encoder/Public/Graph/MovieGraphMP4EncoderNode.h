// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineMP4Encoder.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/Nodes/MovieGraphVideoOutputNode.h"

#include "MovieGraphMP4EncoderNode.generated.h"

#define UE_API MOVIERENDERPIPELINEMP4ENCODER_API

/** A node which can output H264 mp4 files. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphMP4EncoderNode final : public UMovieGraphVideoOutputNode, public IMovieGraphEvaluationNodeInjector
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphMP4EncoderNode();

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual FText GetBasicConfigShortDisplayName() const override;
#endif

protected:
	// UMovieGraphVideoOutputNode Interface
	UE_API virtual TUniquePtr<MovieRenderGraph::IVideoCodecWriter> Initialize_GameThread(const FMovieGraphVideoNodeInitializationContext& InInitializationContext) override;
	UE_API virtual bool Initialize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) override;
	UE_API virtual void WriteFrame_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<FMovieGraphPassData>&& InCompositePasses, TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FString& InBranchName) override;
	UE_API virtual void BeginFinalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) override;
	UE_API virtual void Finalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) override;
	UE_API virtual const TCHAR* GetFilenameExtension() const override;
	UE_API virtual bool IsAudioSupported() const override;
	// ~UMovieGraphVideoOutputNode Interface

	// UMovieGraphSettingNode Interface
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	// ~UMovieGraphSettingNode Interface

	// IMovieGraphEvaluationNodeInjector Interface
	virtual void InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes) override;
	// ~IMovieGraphEvaluationNodeInjector Interface

protected:
	struct FMP4CodecWriter : public MovieRenderGraph::IVideoCodecWriter
	{
		bool bSkipColorConversions = false;
		TUniquePtr<FMoviePipelineMP4Encoder> Writer;
	};
	
	/** The pipeline that is running this node. */
	TWeakObjectPtr<UMovieGraphPipeline> CachedPipeline;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_EncodingRateControl : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AverageBitrateInMbps : 1;
	
	UPROPERTY(BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_MaxBitrateInMbps : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ConstantRateFactor : 1;

	UPROPERTY(BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_EncodingProfile : 1;
	
	UPROPERTY(BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_EncodingLevel : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bIncludeAudio : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OCIOConfiguration : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OCIOContext : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableBurnIn : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInClass : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCompositeOntoFinalImage : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInFileNameFormat : 1;
		
	/**
	* Specifies the bitrate control method used by the encoder. Quality lets the user target a given quality without concern
	* to the filesize, while Variable targets an average bit rate per frame.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "H.264", meta=(EditCondition="bOverride_EncodingRateControl"))
	EMoviePipelineMP4EncodeRateControlMode EncodingRateControl = EMoviePipelineMP4EncodeRateControlMode::VariableBitRate;
	
	/**
	* What is the average bitrate the encoder should target per second? Value is in Megabits per Second,
	* so a value of 8 will become 8192Kbps (kilobits per second). Higher resolutions and higher framerates
	* need higher bitrates, reasonable starting values are 8 for 1080p30, 45 for 4k. 
	* 
	* Only applies to encoding modes not related to Quality.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "H.264", meta=(ClampMin=0.1, UIMin=0.1, UIMax=50, EditCondition="bOverride_AverageBitrateInMbps"))
	float AverageBitrateInMbps = 8.f;
	
	/**
	* When using VariableBitRate_Constrained, what is the maximum bitrate that the encoder can briefly use for
	* more complex scenes, while still trying to maintain the average set in AverageBitrateInMbps. In theory the
	* maximum should be twice the average, but often in practice a smaller difference of 3-6Mbps is sufficient.
	*
	* Only applies to constrained max bit rate.
	*
	* Not exposed to the UI because it is expected most users do not need to change this, but is still available to be scripted.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "H.264", meta = (ClampMin = 0.1, UIMin = 0.1, UIMax = 50, EditCondition = "bOverride_MaxBitrateInMbps"))
	float MaxBitrateInMbps = 16.f;

	/**
	* What is the Constant Rate Factor (CRF) when targeting a specific quality. Values of 17-18 are generally considered
	* perceptually lossless, while higher values produce smaller files with lower quality results. There is an absolute
	* maximum value range of [16-51], where 16 is the highest quality possible and 51 is the lowest quality. This scale
	* is logarithmic, so small changes can result in large differences in quality, and filesize cost. 
	*
	* Only applies to Quality encoding rate control mode.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "H.264", meta=(ClampMin=16, UIMin=16, ClampMax=51, UIMax=51, EditCondition="bOverride_ConstantRateFactor"))
	int32 ConstantRateFactor = 20;

	/**
	 * A higher profile generally results in a better quality video for the same bitrate, but may not be supported for playback on old devices.
	 *
	 * Not exposed to the UI because it is expected most users do not need to change this, but is still available to be scripted.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "H.264", meta=(AdvancedDisplay, EditCondition="bOverride_EncodingProfile"))
	EMoviePipelineMP4EncodeProfile EncodingProfile = EMoviePipelineMP4EncodeProfile::High;

	/** 
	* A higher encode level generally results in a better quality video for the same bitrate, but may not be supported for playback on old devices.
	* Auto will let the encoder automatically select the best level given the resolution, profile, and bitrate.
	*
	* Not exposed to the UI because it is expected most users do not need to change this, but is still available to be scripted.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "H.264", meta=(AdvancedDisplay, EditCondition="bOverride_EncodingLevel"))
	EMoviePipelineMP4EncodeLevel EncodingLevel = EMoviePipelineMP4EncodeLevel::Auto;

	/** If true, audio will be included in the video file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "H.264", meta = (EditCondition = "bOverride_bIncludeAudio"))
	bool bIncludeAudio = true;
	
	/**
	* OCIO configuration/transform settings.
	*
	* Note: There are differences from the previous implementation in MRQ given that we are now doing CPU-side processing.
	* 1) This feature only works on desktop platforms when the OpenColorIO library is available.
	* 2) Users are now responsible for setting the renderer output space to Final Color (HDR) in Linear Working Color Space (SCS_FinalColorHDR) by
	*    disabling the Tone Curve setting on the renderer node.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", DisplayName="OCIO Configuration", meta = (DisplayAfter = "FileNameFormat", EditCondition = "bOverride_OCIOConfiguration"))
	FOpenColorIODisplayConfiguration OCIOConfiguration;

	/**
	* OCIO context of key-value string pairs, typically used to apply shot-specific looks (such as a CDL color correction, or a 1D grade LUT).
	* 
	* Notes:
	* 1) If a configuration asset base context was set, it remains active but can be overridden here with new key-values.
	* 2) Format tokens such as {shot_name} are supported and will get resolved before submission.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", DisplayName = "OCIO Context", meta = (DisplayAfter = "OCIOConfiguration", EditCondition = "bOverride_OCIOContext"))
	TMap<FString, FString> OCIOContext;

	/** If true, this output node will also generate a burn-in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bEnableBurnIn"))
	bool bEnableBurnIn = false;

	/** The widget that the burn-in should use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Burn In", meta=(MetaClass="/Script/MovieRenderPipelineCore.MovieGraphBurnInWidget", EditCondition="bOverride_BurnInClass"))
	FSoftClassPath BurnInClass = FSoftClassPath(TEXT("/MovieRenderPipeline/Blueprints/Graph/DefaultGraphBurnIn.DefaultGraphBurnIn_C"));
	
	/** If true, the burn-in that's generated will be composited onto this output. Otherwise, the burn-in will be written to a different file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bCompositeOntoFinalImage"))
	bool bCompositeOntoFinalImage = true;

	/** The file name format used for writing out the burn-in video (if not composited). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_BurnInFileNameFormat", MrgTokenAutocomplete))
	FString BurnInFileNameFormat = TEXT("{sequence_name}.{layer_name}.{renderer_name}");
};

#undef UE_API
