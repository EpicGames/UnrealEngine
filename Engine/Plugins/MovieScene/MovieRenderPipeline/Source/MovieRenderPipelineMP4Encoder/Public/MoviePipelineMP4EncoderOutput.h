// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineVideoOutputBase.h"
#include "MoviePipelineMP4Encoder.h"
#include "MoviePipelineMP4EncoderOutput.generated.h"

#define UE_API MOVIERENDERPIPELINEMP4ENCODER_API

// Forward Declare
class FMoviePipelineMP4Encoder;

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineMP4EncoderOutput final : public UMoviePipelineVideoOutputBase
{
	GENERATED_BODY()

		UMoviePipelineMP4EncoderOutput()
		: UMoviePipelineVideoOutputBase()
	{
	}

protected:
	// UMoviePipelineVideoOutputBase Interface
	UE_API virtual TUniquePtr<MovieRenderPipeline::IVideoCodecWriter> Initialize_GameThread(const FString& InFileName, FIntPoint InResolution, EImagePixelType InPixelType, ERGBFormat InPixelFormat, uint8 InBitDepth, uint8 InNumChannels) override;
	UE_API virtual bool Initialize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter) override;
	UE_API virtual void WriteFrame_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<MoviePipeline::FCompositePassInfo>&& InCompositePasses) override;
	UE_API virtual void BeginFinalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter);
	UE_API virtual void Finalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter);
	virtual const TCHAR* GetFilenameExtension() const override { return TEXT("mp4"); }
	virtual bool IsAudioSupported() const override { return true; }
	// ~UMoviePipelineVideoOutputBase Interface
	
	// UMoviePipelineSetting Interface
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	// ~UMoviePipelineSetting Interface

	// UMoviePipelineOutputBase Interface
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "MP4EncoderNode_DisplayName", "H.264 MP4 [8bit]"); }
#endif
	// ~UMoviePipelineOutputBase Interface

public:
	/**
	* Specifies the bitrate control method used by the encoder. Quality lets the user target a given quality without concern
	* to the filesize, while Average/Constant modes allow you to suggest desired sizes, though the resulting file may still
	* end up larger or smaller than specified.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	EMoviePipelineMP4EncodeRateControlMode EncodingRateControl = EMoviePipelineMP4EncodeRateControlMode::Quality;
	
	/**
	* What is the average bitrate the encoder should target per second? Value is in Megabits per Second,
	* so a value of 8 will become 8192Kbps (kilobits per second). Higher resolutions and higher framerates
	* need higher bitrates, reasonable starting values are 8 for 1080p30, 45 for 4k. Only applies to
	* encoding modes not related to Quality.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta=(ClampMin=0.1, UIMin=0.1, UIMax=50, EditCondition="EncodingRateControl == EMoviePipelineMP4EncodeRateControlMode::VariableBitRate", EditConditionHides))
	float AverageBitrateInMbps = 8.f;
	
	/**
	* When using VariableBitRate_Constrained, what is the maximum bitrate that the encoder can briefly use for
	* more complex scenes, while still trying to maintain the average set in AverageBitrateInMbps. In theory the
	* maximum should be twice the average, but often in practice a smaller difference of 3-6Mbps is sufficient.
	*
	* Not exposed to the UI because it is expected most users do not need to change this, but is still available to be scripted.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "Settings", meta = (ClampMin = 0.1, UIMin = 0.1, UIMax = 50))
	float MaxBitrateInMbps = 16.f;

	/**
	* What is the Constant Rate Factor (CRF) when targeting a specific quality. Values of 17-18 are generally considered
	* perceptually lossless, while higher values produce smaller files with lower quality results. There is an absolute
	* maximum value range of [16-51], where 16 is the highest quality possible and 51 is the lowest quality. This scale
	* is logarithmic, so small changes can result in large differences in quality, and filesize cost.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta=(ClampMin=16, UIMin=16, ClampMax=51, UIMax=51, EditCondition="EncodingRateControl == EMoviePipelineMP4EncodeRateControlMode::Quality", EditConditionHides))
	int32 ConstantRateFactor = 20;

	/**
	 * A higher profile generally results in a better quality video for the same bitrate, but may not be supported for playback on old devices.
	 *
	 * Not exposed to the UI because it is expected most users do not need to change this, but is still available to be scripted.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Settings", meta=(AdvancedDisplay))
	EMoviePipelineMP4EncodeProfile EncodingProfile = EMoviePipelineMP4EncodeProfile::High;

	/** 
	* A higher encode level generally results in a better quality video for the same bitrate, but may not be supported for playback on old devices.
	* Auto will let the encoder automatically select the best level given the resolution, profile, and bitrate.
	*
	* Not exposed to the UI because it is expected most users do not need to change this, but is still available to be scripted.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "Settings", meta=(AdvancedDisplay))
	EMoviePipelineMP4EncodeLevel EncodingLevel = EMoviePipelineMP4EncodeLevel::Auto;

	/** If true, audio will be included in the video file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bIncludeAudio = true;

protected:
	struct FMP4CodecWriter : public MovieRenderPipeline::IVideoCodecWriter
	{
		TUniquePtr<FMoviePipelineMP4Encoder> Writer;
	};
};

#undef UE_API
