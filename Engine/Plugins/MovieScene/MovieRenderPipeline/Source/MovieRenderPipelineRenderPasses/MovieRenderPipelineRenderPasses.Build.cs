// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MovieRenderPipelineRenderPasses : ModuleRules
{
	public MovieRenderPipelineRenderPasses(ReadOnlyTargetRules Target) : base(Target)
	{
		bEnableExceptions = true;
		bDisableAutoRTFMInstrumentation = true; // AutoRTFM cannot be used with exceptions

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"ImageWriteQueue",
                "SignalProcessing", // Needed for wave writer.
				"AudioMixer",
				"Imath",
				"UEOpenExr", // Needed for multilayer EXRs
				"UEOpenExrRTTI", // Needed for EXR metadata
				"ImageWrapper",				
				"CinematicCamera", // For metadata
				"MovieRenderPipelineSettings", // For settings
				"MovieScene",
				"Slate",
				"SlateCore",
				"ImageCore",
				"OpenColorIOWrapper",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"MovieRenderPipelineCore",
				"Renderer",
				"RenderCore",
				"RHI",
				"OpenColorIO",
				"ActorLayerUtilities", // For Layering
			}
        );

		// Required for UEOpenExr
		AddEngineThirdPartyPrivateStaticDependencies(Target, "zlib");
	}
}
