// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DeferredLayerWarmUpHelpers.h"

#include "Animation/MeshDeformerInstance.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphTimeStepData.h"
#include "Graph/Nodes/MovieGraphRenderLayerNode.h"
#include "Graph/Nodes/MovieGraphWarmUpSettingNode.h"
#include "Graph/Renderers/MovieGraphImagePassBase.h"

namespace UE::MovieGraph::Private
{
	namespace
	{
		void StoreAndSetCvar(IConsoleVariable*& OutCVar, int32& OutPreviousValue, const TCHAR* InName, const int32 InNewValue)
		{
			OutCVar = IConsoleManager::Get().FindConsoleVariable(InName);
			if (OutCVar)
			{
				OutPreviousValue = OutCVar->GetInt();
				OutCVar->SetWithCurrentPriority(InNewValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
			}
		}

		void RestoreCvar(IConsoleVariable* InCVar, int32 InPreviousValue)
		{
			if (InCVar)
			{
				InCVar->SetWithCurrentPriority(InPreviousValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
			}
		}

		// This cvar is likely not needed in practice. However, the functionality that skips skeletal meshes with deformers was added late-release,
		// and thus this is here as an escape hatch if needed. Doesn't need to persist to subsequent releases unless it proves useful.
		static TAutoConsoleVariable<bool> CVarMoviePipelineLayerWarmUpSkipMeshDeformerMotionVectorClear(
			TEXT("MoviePipeline.LayerWarmUp.SkipMeshDeformerMotionVectorClear"),
			true,
			TEXT("Skips USkinnedMeshComponent::ClearMotionVector() during layer warm-ups for mesh deformers that write deformed geometry buffers.\n")
			TEXT(" 0 - Clear motion vectors on all skinned mesh components.\n")
			TEXT(" 1 - Skip mesh-deformer components that write positions or tangents."),
			ECVF_Default);

		/** Resolves the LOD that will be used for render data queries on the component. */
		int32 ResolveSkinnedMeshLOD(const USkinnedMeshComponent* InSkinnedMeshComponent)
		{
			if (!InSkinnedMeshComponent || InSkinnedMeshComponent->GetNumLODs() <= 0)
			{
				return INDEX_NONE;
			}

			const int32 MinLODIndex = InSkinnedMeshComponent->ComputeMinLOD();
			const int32 MaxLODIndex = InSkinnedMeshComponent->GetNumLODs() - 1;
			return FMath::Clamp(InSkinnedMeshComponent->GetPredictedLODLevel(), MinLODIndex, MaxLODIndex);
		}

		/**
		 * Returns true when the component's resolved LOD uses a mesh deformer that writes deformed geometry.
		 * Deformers that only provide auxiliary data are still safe to process through the standard skeletal path.
		 */
		bool HasGeometryOutputMeshDeformerForResolvedLOD(const USkinnedMeshComponent* InSkinnedMeshComponent, const int32 InLODIndex)
		{
			if (!InSkinnedMeshComponent || InLODIndex == INDEX_NONE)
			{
				return false;
			}

			const UMeshDeformerInstance* MeshDeformerInstance = InSkinnedMeshComponent->GetMeshDeformerInstanceForLOD(InLODIndex);
			if (!MeshDeformerInstance)
			{
				return false;
			}

			const EMeshDeformerOutputBuffer OutputBuffers = MeshDeformerInstance->GetOutputBuffers();
			return EnumHasAnyFlags(OutputBuffers, EMeshDeformerOutputBuffer::SkinnedMeshPosition | EMeshDeformerOutputBuffer::SkinnedMeshTangents);
		}

		/**
		 * Returns whether layer warm-up should call ClearMotionVector() on this component.
		 * Standard skinned meshes need the clear to prevent stale previous-bone data from leaking across render layers.
		 * Mesh deformers that write positions/tangents are skipped because their final geometry comes from separately
		 * managed deformer output buffers; forcing skeletal bone-buffer revision aliasing during same-engine-frame
		 * warm-ups can leave those deformed outputs invalid for the render.
		 */
		bool ShouldClearSkinnedMotionVector(const USkinnedMeshComponent* InSkinnedMeshComponent)
		{
			if (!InSkinnedMeshComponent)
			{
				return false;
			}

			const int32 LODIndex = ResolveSkinnedMeshLOD(InSkinnedMeshComponent);
			const bool bHasGeometryOutputMeshDeformerForLOD = HasGeometryOutputMeshDeformerForResolvedLOD(InSkinnedMeshComponent, LODIndex);

			if (bHasGeometryOutputMeshDeformerForLOD && CVarMoviePipelineLayerWarmUpSkipMeshDeformerMotionVectorClear.GetValueOnGameThread())
			{
				return false;
			}

			return true;
		}
	}

	FLumenCvarCache::~FLumenCvarCache()
	{
		// Safety net: if StoreAndOverride was called but Restore was never invoked (eg, early return)
		// restore now so cvars don't stay pinned to aggressive values.
		Restore();
	}

	void FLumenCvarCache::StoreAndOverride()
	{
		StoreAndSetCvar(RadiosityUpdateFactor.CVar, RadiosityUpdateFactor.PreviousValue, TEXT("r.LumenScene.Radiosity.UpdateFactor"), 1);
		StoreAndSetCvar(SurfaceCacheCardCaptureFactor.CVar, SurfaceCacheCardCaptureFactor.PreviousValue, TEXT("r.LumenScene.SurfaceCache.CardCaptureFactor"), 1);
		StoreAndSetCvar(SurfaceCacheFeedback.CVar, SurfaceCacheFeedback.PreviousValue, TEXT("r.LumenScene.SurfaceCache.Feedback"), 0);
		StoreAndSetCvar(SurfaceCacheRecaptureEveryFrame.CVar, SurfaceCacheRecaptureEveryFrame.PreviousValue, TEXT("r.LumenScene.SurfaceCache.RecaptureEveryFrame"), 1);
		bHasOverriddenState = true;
	}

	void FLumenCvarCache::RelaxForConvergence() const
	{
		// Note: RadiosityUpdateFactor is deliberately left at 1 (set by StoreAndOverride).
		// It stays aggressive through the remainder of warm-up and the actual output render so the
		// radiosity feedback loop accumulates bounces as fast as possible. Restore() resets it.

		// Stop forcing a full recapture every frame so the surface cache can build upon previous results.
		if (SurfaceCacheRecaptureEveryFrame.CVar)
		{
			SurfaceCacheRecaptureEveryFrame.CVar->SetWithCurrentPriority(0, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
		}

		// Re-enable surface cache feedback so Lumen prioritizes capturing the most-visible cards
		// rather than updating them uniformly.
		RestoreCvar(SurfaceCacheFeedback.CVar, SurfaceCacheFeedback.PreviousValue);

		// Restore CardCaptureFactor so surface cache tile captures are paced at the user's configured
		// rate rather than the warm-up-phase aggressive budget. RecaptureEveryFrame=0 above is what
		// actually re-enables atlas accumulation; CardCaptureFactor only controls how many tiles are
		// captured per frame when RecaptureEveryFrame is off.
		RestoreCvar(SurfaceCacheCardCaptureFactor.CVar, SurfaceCacheCardCaptureFactor.PreviousValue);
	}

	void FLumenCvarCache::Restore()
	{
		if (!bHasOverriddenState)
		{
			return;
		}

		RestoreCvar(RadiosityUpdateFactor.CVar, RadiosityUpdateFactor.PreviousValue);
		RestoreCvar(SurfaceCacheCardCaptureFactor.CVar, SurfaceCacheCardCaptureFactor.PreviousValue);
		RestoreCvar(SurfaceCacheFeedback.CVar, SurfaceCacheFeedback.PreviousValue);
		RestoreCvar(SurfaceCacheRecaptureEveryFrame.CVar, SurfaceCacheRecaptureEveryFrame.PreviousValue);

		bHasOverriddenState = false;
	}

	int32 ResolveEffectiveLayerWarmUpFrames(const UMovieGraphEvaluatedConfig* InEvaluatedConfig, const FName InBranchName)
	{
		if (!InEvaluatedConfig)
		{
			return 0;
		}

		constexpr bool bIncludeCDOs = true;
		const UMovieGraphWarmUpSettingNode* WarmUpNode =
			InEvaluatedConfig->GetSettingForBranch<UMovieGraphWarmUpSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs);
		int32 EffectiveFrames = WarmUpNode ? WarmUpNode->LayerWarmUpFrames : 0;

		const UMovieGraphRenderLayerNode* RenderLayerNode =
			InEvaluatedConfig->GetSettingForBranch<UMovieGraphRenderLayerNode>(InBranchName, bIncludeCDOs);
		if (RenderLayerNode && RenderLayerNode->bOverride_LayerWarmUpFrames)
		{
			EffectiveFrames = RenderLayerNode->LayerWarmUpFrames;
		}

		return FMath::Max(EffectiveFrames, 0);
	}

	void PerformLayerWarmUps(
		const UWorld* InWorld,
		const int32 InNumLayerWarmUpFrames,
		FMovieGraphTimeStepData& InOutWarmUpTimeStepData,
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		UE::MovieGraph::Rendering::FMovieGraphImagePassBase& InPassInstance,
		FLumenCvarCache& InOutLumenCvarCache)
	{
		if (InNumLayerWarmUpFrames <= 0 || !InWorld)
		{
			return;
		}

		// Skeletal meshes use a separate double-buffered bone transform system for velocity, independent of
		// FSceneVelocityData (which bCameraCut=true on warm-up frame 0 handles for all other primitive types).
		// If an actor was invisible in a previous layer, its "previous bones" buffer holds stale data from the
		// last time it was rendered, potentially many frames ago in a different layer context. On warm-up frame 1
		// (the first frame where the velocity pass runs with bCameraCut=false), the renderer computes bone velocity
		// as delta(stale-previous-bones -> warm-up-1-pose), producing a large spurious motion vector. This corrupts
		// TSR/TAA history during the warm-up convergence window and can bleed into the actual output frame.
		// ClearMotionVector() resets the bone revision counter so the GPU binds the current bone buffer to both
		// the current and previous slots, yielding zero velocity for the next render and breaking the stale-delta chain.
		// Mesh deformers that write positions/tangents are skipped because their final geometry comes from separately
		// managed deformer output buffers; forcing skeletal bone-buffer motion-vector clears during same-engine-frame
		// layer warm-ups can invalidate that path.
		//
		// Note that clearing motion vectors here can result in no motion blur for the skeletal mesh. Temporal sampling
		// may be required to restore motion blur if layer warm-ups are used.
		TInlineComponentArray<USkinnedMeshComponent*> SkinnedMeshes;
		for (TActorIterator<AActor> ActorIt(InWorld); ActorIt; ++ActorIt)
		{
			ActorIt->GetComponents(SkinnedMeshes);

			for (USkinnedMeshComponent* SkinnedMeshComponent : SkinnedMeshes)
			{
				if (ShouldClearSkinnedMotionVector(SkinnedMeshComponent))
				{
					SkinnedMeshComponent->ClearMotionVector();
				}
			}
		}

		// Render N warm-up frames for this render layer.
		bool bDidRelaxLumenForConvergence = false;
		for (int32 WarmUpIndex = 0; WarmUpIndex < InNumLayerWarmUpFrames; ++WarmUpIndex)
		{
			// Relax Lumen exactly once, immediately after the first warm-up frame's render submits, so
			// subsequent warm-ups see the relaxed cvars.
			// Note: Calling this *before* Render() is important, but unfortunately it's unclear why. If this is
			// called *after* Render(), gated by WarmUpIndex == 0, Lumen flickers badly.
			if (WarmUpIndex == 1)
			{
				bDidRelaxLumenForConvergence = true;
				InOutLumenCvarCache.RelaxForConvergence();
			}

			InOutWarmUpTimeStepData.bIsCameraCut = (WarmUpIndex == 0);
			InPassInstance.Render(InFrameTraversalContext, InOutWarmUpTimeStepData);
			InOutWarmUpTimeStepData.RenderedFrameNumber += 1;
		}

		// If InNumLayerWarmUpFrames == 1, RelaxForConvergence() won't be called in the loop above. Call it
		// here if it wasn't called yet.
		if (!bDidRelaxLumenForConvergence)
		{
			InOutLumenCvarCache.RelaxForConvergence();
		}
	}
}
