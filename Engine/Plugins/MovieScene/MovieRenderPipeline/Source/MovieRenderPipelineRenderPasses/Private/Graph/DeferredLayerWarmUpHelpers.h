// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "UObject/NameTypes.h"

class UMovieGraphEvaluatedConfig;
class UWorld;
struct FMovieGraphTimeStepData;
struct FMovieGraphTraversalContext;

namespace UE::MovieGraph::Rendering
{
	struct FMovieGraphImagePassBase;
}

namespace UE::MovieGraph::Private
{
	/**
	 * Manages Lumen surface cache cvars during layer warm-ups. On the first warm-up frame the surface
	 * cache is forced to fully recapture (RecaptureEveryFrame=1, Feedback=0, CardCaptureFactor=1). On
	 * subsequent warm-up frames and the actual render, those three settings are relaxed via
	 * RelaxForConvergence() so Lumen can converge. RadiosityUpdateFactor stays overridden at 1 for the
	 * entire warm-up + render sequence (restored only by Restore()) to maximize radiosity accumulation.
	 *
	 * RAII: the destructor calls Restore() as a safety net if the cvars were overridden but never explicitly
	 * restored. Restore() is idempotent.
	 */
	struct FLumenCvarCache
	{
		FLumenCvarCache() = default;
		~FLumenCvarCache();

		FLumenCvarCache(const FLumenCvarCache&) = delete;
		FLumenCvarCache& operator=(const FLumenCvarCache&) = delete;

		/** Store current cvar values and apply aggressive settings for the first warm-up frame. */
		void StoreAndOverride();

		/**
		 * Relax the first-frame-only settings so Lumen can converge through temporal accumulation.
		 * Intended to be called EXACTLY ONCE per layer warm-up sequence, immediately after the first
		 * layer warm-up render. Calling it more often is not idempotent in practice: every
		 * IConsoleVariable::Set() invokes OnChanged() which enqueues an RT shadow-propagation
		 * command and flags all registered cvar sinks regardless of value equality, which disrupts
		 * Lumen's temporal convergence on subsequent warm-up frames.
		 *
		 * RadiosityUpdateFactor remains overridden through the final render (Restore() resets it).
		 */
		void RelaxForConvergence() const;

		/** Restore all cvars to their original values. Safe to call multiple times; only acts if StoreAndOverride was called. */
		void Restore();

	private:
		struct FCachedCvar
		{
			IConsoleVariable* CVar = nullptr;
			int32 PreviousValue = 0;
		};

		FCachedCvar RadiosityUpdateFactor;
		FCachedCvar SurfaceCacheCardCaptureFactor;
		FCachedCvar SurfaceCacheFeedback;
		FCachedCvar SurfaceCacheRecaptureEveryFrame;
		bool bHasOverriddenState = false;
	};

	/**
	 * Computes the number of layer warm-up frames to run for a given layer. The Warm Up Settings node
	 * provides the global value; a Render Layer node on the given branch can optionally override it.
	 */
	int32 ResolveEffectiveLayerWarmUpFrames(const UMovieGraphEvaluatedConfig* InEvaluatedConfig, FName InBranchName);

	/**
	 * Perform N layer warm-up frames before the layer's final render. The first layer warm-up frame is flagged
	 * as a camera cut to suppress velocity/motion-blur artifacts from stale previous-frame transforms.
	 * Skeletal mesh bone velocity buffers are cleared before warm-ups begin to prevent cross-layer leakage;
	 * mesh deformers that write deformed geometry buffers are skipped by default because those buffers are
	 * managed separately from the skeletal bone-buffer path.
	 * The Lumen cvar cache is relaxed after the first warm-up frame so Lumen can converge through temporal
	 * accumulation on subsequent frames and on the actual render.
	 */
	void PerformLayerWarmUps(
		const UWorld* InWorld,
		int32 InNumLayerWarmUpFrames,
		FMovieGraphTimeStepData& InOutWarmUpTimeStepData,
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		UE::MovieGraph::Rendering::FMovieGraphImagePassBase& InPassInstance,
		FLumenCvarCache& InOutLumenCvarCache);
}
