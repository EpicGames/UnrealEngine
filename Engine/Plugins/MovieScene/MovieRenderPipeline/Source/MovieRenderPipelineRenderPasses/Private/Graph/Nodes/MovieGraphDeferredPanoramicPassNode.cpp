// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphDeferredPanoramicPassNode.h"

#include "Graph/DeferredLayerWarmUpHelpers.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/Renderers/MovieGraphDeferredPanoramicPass.h"
#include "Graph/Renderers/MovieGraphImagePassBase.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphDeferredPanoramicPassNode)

UMovieGraphDeferredPanoramicNode::UMovieGraphDeferredPanoramicNode()
	: NumHorizontalSteps(8)
	, NumVerticalSteps(3)
	, bFollowCameraOrientation(true)
	, bAllocateHistoryPerPane(false)
	, bPageToSystemMemory(false)
	, SpatialSampleCount(1)
	, AntiAliasingMethod(EAntiAliasingMethod::AAM_TSR)
	, Filter(EMoviePipelinePanoramicFilterType::Bilinear)
	, bWriteAllSamples(false)
	, bDisableToneCurve(false)
	, bAllowOCIO(true)
	, ViewModeIndex(VMI_Lit)
{
	RendererName = TEXT("DeferredPanoramic");
}

// Out-of-line so the TMap<..., TUniquePtr<FLumenCvarCache>> destructor can see the complete type.
UMovieGraphDeferredPanoramicNode::~UMovieGraphDeferredPanoramicNode() = default;

void UMovieGraphDeferredPanoramicNode::RenderImpl(
	const FMovieGraphTraversalContext& InFrameTraversalContext,
	const FMovieGraphTimeStepData& InTimeData)
{
	// Shared across every PreLayerRender call in this loop so RenderedFrameNumber accumulates
	// monotonically across all layers' warm-up renders.
	CurrentWarmUpTimeStepData = InTimeData;
	CurrentWarmUpTimeStepData.bDiscardOutput = true;
	CurrentWarmUpTimeStepData.bRequiresAccumulator = false;

	Super::RenderImpl(InFrameTraversalContext, InTimeData);
}

void UMovieGraphDeferredPanoramicNode::PreLayerRender(
	const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
	const FMovieGraphTraversalContext& InFrameTraversalContext,
	const FMovieGraphTimeStepData& InTimeData)
{
	// Shot warm-up frames shouldn't run layer warm-ups (it's unnecessary).
	const bool bIsShotWarmUpFrame = InTimeData.ShotOutputFrameNumber < 0;
	if (bIsShotWarmUpFrame)
	{
		return;
	}

	const int32 EffectiveLayerWarmUpFrames = UE::MovieGraph::Private::ResolveEffectiveLayerWarmUpFrames(
		InFrameTraversalContext.Time.EvaluatedConfig, InInstance->GetBranchName());

	TUniquePtr<UE::MovieGraph::Private::FLumenCvarCache> LumenCvarCache = MakeUnique<UE::MovieGraph::Private::FLumenCvarCache>();

	if (EffectiveLayerWarmUpFrames > 0)
	{
		// The cvars must remain active through both the warm-up frames AND the actual render frame
		// so Lumen's converged state is what the final render captures. PostLayerRender restores them.
		LumenCvarCache->StoreAndOverride();

		// CurrentWarmUpTimeStepData is a member accumulated across every PreLayerRender call within
		// this RenderImpl invocation. See RenderImpl for why the monotonic RenderedFrameNumber matters.
		UE::MovieGraph::Private::PerformLayerWarmUps(
			InInstance->GetRenderer()->GetWorld(),
			EffectiveLayerWarmUpFrames,
			CurrentWarmUpTimeStepData,
			InFrameTraversalContext,
			*InInstance,
			*LumenCvarCache);
	}

	ActiveLayerCaches.Add(InInstance.Get(), MoveTemp(LumenCvarCache));
}

void UMovieGraphDeferredPanoramicNode::PostLayerRender(
	const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
	const FMovieGraphTraversalContext& /*InFrameTraversalContext*/,
	const FMovieGraphTimeStepData& /*InTimeData*/)
{
	if (TUniquePtr<UE::MovieGraph::Private::FLumenCvarCache>* Cache = ActiveLayerCaches.Find(InInstance.Get()))
	{
		(*Cache)->Restore();
		ActiveLayerCaches.Remove(InInstance.Get());
	}
}

void UMovieGraphDeferredPanoramicNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	FString MetadataPrefix = UE::MoviePipeline::GetMetadataPrefixPath(InRenderDataIdentifier);

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("pano_count_h"), FString::FromInt(NumHorizontalSteps));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/paneCountH"), *MetadataPrefix), FString::FromInt(NumHorizontalSteps));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("pano_count_v"), FString::FromInt(NumVerticalSteps));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/paneCountV"), *MetadataPrefix), FString::FromInt(NumVerticalSteps));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("pano_follow_orientation"), FString::FromInt(bFollowCameraOrientation));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/followOrientation"), *MetadataPrefix), FString::FromInt(bFollowCameraOrientation));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("aaMethod"), UEnum::GetValueAsString(AntiAliasingMethod));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/aaMethod"), *MetadataPrefix), UEnum::GetValueAsString(AntiAliasingMethod));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("pano_filter"), UEnum::GetValueAsString(Filter));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/filter"), *MetadataPrefix), UEnum::GetValueAsString(Filter));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("disable_tonecurve"), FString::FromInt(bDisableToneCurve));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/disableTonecurve"), *MetadataPrefix), FString::FromInt(bDisableToneCurve));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("history_per_pane"), FString::FromInt(bAllocateHistoryPerPane));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/historyPerPane"), *MetadataPrefix), FString::FromInt(bAllocateHistoryPerPane));
}

FEngineShowFlags UMovieGraphDeferredPanoramicNode::GetShowFlags() const
{
	FEngineShowFlags Flags = Super::GetShowFlags();
	Flags.SetVignette(false);
	Flags.SetSceneColorFringe(false);
	Flags.SetPhysicalMaterialMasks(false);
	Flags.SetDepthOfField(false);

	return Flags;
}

#if WITH_EDITOR
FText UMovieGraphDeferredPanoramicNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("MovieRenderGraph", "DeferredPanoramicNodeTitle", "Panoramic Deferred Renderer");		
}

FSlateIcon UMovieGraphDeferredPanoramicNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon DeferredRendererIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.SizeMap");
	OutColor = FLinearColor::White;
	
	return DeferredRendererIcon;
}
#endif	// WITH_EDITOR

void UMovieGraphDeferredPanoramicNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	Super::UpdateTelemetry(InTelemetry);

	InTelemetry->bUsesPanoramic = true;
	InTelemetry->AntiAliasingType = UEnum::GetValueAsString(UE::MovieRenderPipeline::GetEffectiveAntiAliasingMethod(bOverride_AntiAliasingMethod, AntiAliasingMethod));
}

TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> UMovieGraphDeferredPanoramicNode::CreateInstance() const
{
	return MakeUnique<UE::MovieGraph::Rendering::FMovieGraphDeferredPanoramicPass>();
}

EViewModeIndex UMovieGraphDeferredPanoramicNode::GetViewModeIndex() const
{
	return ViewModeIndex;
}

bool UMovieGraphDeferredPanoramicNode::GetWriteAllSamples() const
{
	return bWriteAllSamples;
}

int32 UMovieGraphDeferredPanoramicNode::GetNumSpatialSamples() const
{
	return SpatialSampleCount;
}

bool UMovieGraphDeferredPanoramicNode::GetDisableToneCurve() const
{
	return bDisableToneCurve;
}

bool UMovieGraphDeferredPanoramicNode::GetAllowOCIO() const
{
	return bAllowOCIO;
}

EAntiAliasingMethod UMovieGraphDeferredPanoramicNode::GetAntiAliasingMethod() const
{
	return AntiAliasingMethod;
}
