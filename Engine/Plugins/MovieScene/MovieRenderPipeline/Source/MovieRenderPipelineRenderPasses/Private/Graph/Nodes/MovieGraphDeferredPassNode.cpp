// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphDeferredPassNode.h"

#include "Graph/DeferredLayerWarmUpHelpers.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/Renderers/MovieGraphDeferredPass.h"
#include "Graph/Renderers/MovieGraphImagePassBase.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphDeferredPassNode)

FCriticalSection UMovieGraphDeferredRenderPassNode::PassInstanceFactoriesCS;
TArray<UMovieGraphDeferredRenderPassNode::FNamedPassFactory> UMovieGraphDeferredRenderPassNode::PassInstanceFactories;

void UMovieGraphDeferredRenderPassNode::RegisterPassInstanceFactory(FName InName, FPassInstanceFactory InFactory)
{
	FScopeLock Lock(&PassInstanceFactoriesCS);

	// Replace existing factory with the same name, or add new
	for (FNamedPassFactory& Existing : PassInstanceFactories)
	{
		if (Existing.Name == InName)
		{
			Existing.Factory = MoveTemp(InFactory);
			return;
		}
	}
	PassInstanceFactories.Add({ InName, MoveTemp(InFactory) });
}

void UMovieGraphDeferredRenderPassNode::UnregisterPassInstanceFactory(FName InName)
{
	FScopeLock Lock(&PassInstanceFactoriesCS);

	PassInstanceFactories.RemoveAll([InName](const FNamedPassFactory& Entry)
	{
		return Entry.Name == InName;
	});
}

void UMovieGraphDeferredRenderPassNode::SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData)
{
	for (const FMovieGraphRenderPassLayerData& LayerData : InSetupData.Layers)
	{
		TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> RendererInstance;
		FName WinningFactory;

		// Query registered factories — first non-null result wins
		{
			FScopeLock Lock(&PassInstanceFactoriesCS);

			for (const FNamedPassFactory& Entry : PassInstanceFactories)
			{
				if (!Entry.Factory)
				{
					continue;
				}

				FPassInstanceFactoryContext Context{LayerData, InSetupData.Renderer.Get(), InSetupData.EvaluatedConfig};
				TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> Override =
					Entry.Factory(Context);

				if (!Override)
				{
					continue;
				}

				if (RendererInstance)
				{
					// Conflict: multiple factories want to override this layer
					UE_LOGF(LogMovieRenderPipeline, Warning,
						"Deferred pass factory conflict for layer '%ls': "
							 "'%ls' was already selected, ignoring '%ls'. "
							 "Multiple plugins are trying to override the deferred renderer for the same layer.",
						*LayerData.LayerName, *WinningFactory.ToString(), *Entry.Name.ToString());
				}
				else
				{
					RendererInstance = MoveTemp(Override);
					WinningFactory = Entry.Name;
				}
			}
		}

		// Fall back to default deferred pass
		if (!RendererInstance)
		{
			RendererInstance = CreateInstance();
		}

		if (RendererInstance)
		{
			RendererInstance->Setup(InSetupData.Renderer, this, LayerData);
			CurrentInstances.Add(MoveTemp(RendererInstance));
		}
	}
}

void UMovieGraphDeferredRenderPassNode::RenderImpl(
	const FMovieGraphTraversalContext& InFrameTraversalContext,
	const FMovieGraphTimeStepData& InTimeData)
{
	// Shared across every PreLayerRender call in this loop so RenderedFrameNumber accumulates
	// monotonically across all layers' warm-up renders.
	CurrentWarmUpTimeStepData = InTimeData;
	CurrentWarmUpTimeStepData.bDiscardOutput = true;
	CurrentWarmUpTimeStepData.bRequiresAccumulator = false;

	Super::RenderImpl(InFrameTraversalContext, InTimeData);
}

void UMovieGraphDeferredRenderPassNode::PreLayerRender(
	const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
	const FMovieGraphTraversalContext& InFrameTraversalContext,
	const FMovieGraphTimeStepData& InTimeData)
{
	// Shot warm-up frames shouldn't run layer warm-ups (it's unnecessary).
	const bool bIsShotWarmUpFrame = InTimeData.ShotOutputFrameNumber < 0;
	if (bIsShotWarmUpFrame)
	{
		return;
	}

	const int32 EffectiveLayerWarmUpFrames = UE::MovieGraph::Private::ResolveEffectiveLayerWarmUpFrames(
		InFrameTraversalContext.Time.EvaluatedConfig, InInstance->GetBranchName());

	TUniquePtr<UE::MovieGraph::Private::FLumenCvarCache> LumenCvarCache = MakeUnique<UE::MovieGraph::Private::FLumenCvarCache>();

	if (EffectiveLayerWarmUpFrames > 0)
	{
		// The cvars must remain active through both the warm-up frames AND the actual render frame
		// so Lumen's converged state is what the final render captures. PostLayerRender restores them.
		LumenCvarCache->StoreAndOverride();

		// CurrentWarmUpTimeStepData is a member accumulated across every PreLayerRender call within
		// this RenderImpl invocation. See RenderImpl for why the monotonic RenderedFrameNumber matters.
		UE::MovieGraph::Private::PerformLayerWarmUps(
			InInstance->GetRenderer()->GetWorld(),
			EffectiveLayerWarmUpFrames,
			CurrentWarmUpTimeStepData,
			InFrameTraversalContext,
			*InInstance,
			*LumenCvarCache);
	}

	ActiveLayerCaches.Add(InInstance.Get(), MoveTemp(LumenCvarCache));
}

void UMovieGraphDeferredRenderPassNode::PostLayerRender(
	const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
	const FMovieGraphTraversalContext& /*InFrameTraversalContext*/,
	const FMovieGraphTimeStepData& /*InTimeData*/)
{
	if (TUniquePtr<UE::MovieGraph::Private::FLumenCvarCache>* Cache = ActiveLayerCaches.Find(InInstance.Get()))
	{
		(*Cache)->Restore();
		ActiveLayerCaches.Remove(InInstance.Get());
	}
}

TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> UMovieGraphDeferredRenderPassNode::CreateInstance() const
{
	return MakeUnique<UE::MovieGraph::Rendering::FMovieGraphDeferredPass>();
}

UMovieGraphDeferredRenderPassNode::UMovieGraphDeferredRenderPassNode()
	: SpatialSampleCount(1)
	, AntiAliasingMethod(EAntiAliasingMethod::AAM_TSR)
	, bWriteAllSamples(false)
	, bDisableToneCurve(false)
	, bAllowOCIO(true)
	, ViewModeIndex(VMI_Lit)
	, bIncludeBeautyRenderInOutput(true)
	, PPMFileNameFormat(TEXT("{sequence_name}.{layer_name}.{renderer_sub_name}.{frame_number}"))
	, bEnableHighResolutionTiling(false)
	, TileCount(1)
	, OverlapPercentage(0.f)
	, bAllocateHistoryPerTile(false)
	, bPageToSystemMemory(false)
{
	RendererName = TEXT("Deferred");

	// To help user knowledge we pre-seed the additional post processing materials with an array of potentially common passes.
	TArray<FString> DefaultPostProcessMaterials;
	DefaultPostProcessMaterials.Add(DefaultDepthAsset);
	DefaultPostProcessMaterials.Add(DefaultMotionVectorsAsset);

	for (FString& MaterialPath : DefaultPostProcessMaterials)
	{
		FMoviePipelinePostProcessPass& NewPass = AdditionalPostProcessMaterials.AddDefaulted_GetRef();
		NewPass.Name = (MaterialPath == DefaultDepthAsset) ? FString(TEXT("depth")) : FString(TEXT("motion"));
		NewPass.Material = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(MaterialPath));
		NewPass.bEnabled = false;
		NewPass.bHighPrecisionOutput = MaterialPath.Equals(DefaultDepthAsset);
		NewPass.bUseLosslessCompression = true;
	}
}

// Out-of-line so the TMap<..., TUniquePtr<FLumenCvarCache>> destructor can see the complete type.
UMovieGraphDeferredRenderPassNode::~UMovieGraphDeferredRenderPassNode() = default;

void UMovieGraphDeferredRenderPassNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	FString MetadataPrefix = UE::MoviePipeline::GetMetadataPrefixPath(InRenderDataIdentifier);

	// We intentionally skip some settings for not being very meaningful to output, ie: bAllowOCIO, bPageToSystemMemory, bWriteAllSamples
	OutMergedFormatArgs.FilenameArguments.Add(TEXT("ss_count"), FString::FromInt(SpatialSampleCount));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/spatialSampleCount"), *MetadataPrefix), FString::FromInt(SpatialSampleCount));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("disable_tonecurve"), FString::FromInt(bDisableToneCurve));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/disableTonecurve"), *MetadataPrefix), FString::FromInt(bDisableToneCurve));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("overlap_percentage"), FString::SanitizeFloat(OverlapPercentage));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/highres/overlapPercentage"), *MetadataPrefix), FString::SanitizeFloat(OverlapPercentage));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("history_per_tile"), FString::FromInt(bAllocateHistoryPerTile));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/highres/historyPerTile"), *MetadataPrefix), FString::FromInt(bAllocateHistoryPerTile));
	
	OutMergedFormatArgs.FilenameArguments.Add(TEXT("aaMethod"), UEnum::GetValueAsString(AntiAliasingMethod));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/aaMethod"), *MetadataPrefix), UEnum::GetValueAsString(AntiAliasingMethod));
}

void UMovieGraphDeferredRenderPassNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	Super::UpdateTelemetry(InTelemetry);

	InTelemetry->bUsesDeferred = true;
	InTelemetry->bUsesPPMs |= Algo::AnyOf(AdditionalPostProcessMaterials, [](const FMoviePipelinePostProcessPass& Pass) { return Pass.bEnabled; });
	InTelemetry->SpatialSampleCount = FMath::Max(InTelemetry->SpatialSampleCount, SpatialSampleCount);
	InTelemetry->AntiAliasingType = UEnum::GetValueAsString(UE::MovieRenderPipeline::GetEffectiveAntiAliasingMethod(bOverride_AntiAliasingMethod, AntiAliasingMethod));
	InTelemetry->HighResTileCount = FMath::Max(InTelemetry->HighResTileCount, TileCount);
	InTelemetry->HighResOverlap = FMath::Max(InTelemetry->HighResOverlap, OverlapPercentage);
	InTelemetry->bUsesPageToSystemMemory |= bPageToSystemMemory;
}

void UMovieGraphDeferredRenderPassNode::ResolveTokenContainingProperties(TFunction<void(FString&)>& ResolveFunc, const FMovieGraphTokenResolveContext& InContext)
{
	Super::ResolveTokenContainingProperties(ResolveFunc, InContext);

	for (FMoviePipelinePostProcessPass& PostProcessPass : AdditionalPostProcessMaterials)
	{
		ResolveFunc(PostProcessPass.Name);
	}
}

#if WITH_EDITOR
FText UMovieGraphDeferredRenderPassNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("MovieGraphNodes", "DeferredRenderPassGraphNode_Description", "Deferred Renderer");
}

FSlateIcon UMovieGraphDeferredRenderPassNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon DeferredRendererIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelViewport.HighResScreenshot");
	
	OutColor = FLinearColor::White;
	return DeferredRendererIcon;
}
#endif

EViewModeIndex UMovieGraphDeferredRenderPassNode::GetViewModeIndex() const
{
	return ViewModeIndex;
}

bool UMovieGraphDeferredRenderPassNode::GetWriteBeautyPassToDisk() const
{
	return bIncludeBeautyRenderInOutput;
}

bool UMovieGraphDeferredRenderPassNode::GetWriteAllSamples() const
{
	return bWriteAllSamples;
}

TArray<FMoviePipelinePostProcessPass> UMovieGraphDeferredRenderPassNode::GetAdditionalPostProcessMaterials() const
{
	return AdditionalPostProcessMaterials;
}

FString UMovieGraphDeferredRenderPassNode::GetPPMFileNameFormatString() const
{
	return PPMFileNameFormat;
}

int32 UMovieGraphDeferredRenderPassNode::GetNumSpatialSamples() const
{
	return SpatialSampleCount;
}

bool UMovieGraphDeferredRenderPassNode::GetDisableToneCurve() const
{
	return bDisableToneCurve;
}

bool UMovieGraphDeferredRenderPassNode::GetAllowOCIO() const
{
	return bAllowOCIO;
}

EAntiAliasingMethod UMovieGraphDeferredRenderPassNode::GetAntiAliasingMethod() const
{
	return AntiAliasingMethod;
}
