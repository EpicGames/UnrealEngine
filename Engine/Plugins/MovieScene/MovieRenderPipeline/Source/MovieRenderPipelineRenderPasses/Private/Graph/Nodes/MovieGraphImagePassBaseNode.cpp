// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphImagePassBaseNode.h"

#include "Graph/MovieGraphRenderLayerSubsystem.h" 
#include "MoviePipelineQueue.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphImagePassBaseNode)


static TAutoConsoleVariable<bool> CVarMoviePipelineFlushRenderingPerLayerHack(
	TEXT("MoviePipeline.FlushLayersDebug"),
	false,
	TEXT("This is a debug cvar intended to help diagnose issues in the MoviePipeline multi-layer rendering mode\n")
	TEXT("that flushes any outstanding rendering commands for each layer. This can have a large performance impact,\n")
	TEXT("as it effectively removes any parallelism between the Game Thread and Render Thread!\n")
	TEXT(" 0 - Don't flush (default)\n")
	TEXT(" 1 - Flush (but negatively impacts performance)"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarMoviePipelineNumWarmUpFramesWithSpatialSampling(
	TEXT("MovieRenderPipeline.NumWarmUpFramesWithSpatialSampling"),
	5,
	TEXT("The number of warm-up frames that should have spatial sampling applied to them.\n"),
	ECVF_Default);

FString UMovieGraphImagePassBaseNode::DefaultDepthAsset = TEXT("/MovieRenderPipeline/Materials/MovieRenderQueue_WorldDepth.MovieRenderQueue_WorldDepth");
FString UMovieGraphImagePassBaseNode::DefaultMotionVectorsAsset = TEXT("/MovieRenderPipeline/Materials/MovieRenderQueue_MotionVectors.MovieRenderQueue_MotionVectors");

UMovieGraphImagePassBaseNode::UMovieGraphImagePassBaseNode()
{
	RendererName = TEXT("UnnamedImageRenderPass");
	ShowFlags = CreateDefaultSubobject<UMovieGraphShowFlags>(TEXT("ShowFlags"));
}

void UMovieGraphImagePassBaseNode::SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData)
{
	// To make the implementation simpler, we make one instance of FMovieGraphDeferredRenderPas
	// per camera, and per render layer. These objects can pull from common pools to share state,
	// which gives us a better overview of how many resources are being used by MRQ.
	for (const FMovieGraphRenderPassLayerData& LayerData : InSetupData.Layers)
	{
		// Each node can change which renderer instance actually gets created.
		TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> RendererInstance = CreateInstance();
		if(RendererInstance)
		{
			RendererInstance->Setup(InSetupData.Renderer, this, LayerData);
			CurrentInstances.Add(MoveTemp(RendererInstance));
		}
	}
}

void UMovieGraphImagePassBaseNode::TeardownImpl()
{
	// We don't need to flush the rendering commands as we assume the MovieGraph
	// Renderer has already done that once, so all data for all passes should
	// have been submitted to the GPU (and subsequently read back) by now.
	for (TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& Instance : CurrentInstances)
	{
		Instance->Teardown();
	}
	CurrentInstances.Reset();
}

void UMovieGraphImagePassBaseNode::RenderImpl(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData)
{
	for (const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& Instance : CurrentInstances)
	{
		UMovieGraphRenderLayerSubsystem* LayerSubsystem =
			Instance->GetRenderer()->GetWorld()->GetSubsystem<UMovieGraphRenderLayerSubsystem>();
		
		// Apply all modifiers in the evaluated graph
		if (LayerSubsystem)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MRQ::DeferredRender::ActivateRenderLayer);
			LayerSubsystem->SetActiveRenderLayerByName(Instance->GetBranchName());
		}
		
		// This is a hack to provide debug tooling for rendering issues caused by
		// multiple layers. Flushing can ensure that all render proxies/etc. are
		// recreated which can be used to see if it fixes various issues, but
		// this comes at a big performance cost as it effectively removes GT/RT parallelism.
		if (CVarMoviePipelineFlushRenderingPerLayerHack.GetValueOnGameThread())
		{
			FlushRenderingCommands();
		}

		PreLayerRender(Instance, InFrameTraversalContext, InTimeData);
		Instance->Render(InFrameTraversalContext, InTimeData);
		PostLayerRender(Instance, InFrameTraversalContext, InTimeData);

		// Revert all modifiers
		if (LayerSubsystem)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MRQ::DeferredRender::RevertRenderLayer);
			LayerSubsystem->ClearActiveRenderLayer();
		}

		if (CVarMoviePipelineFlushRenderingPerLayerHack.GetValueOnGameThread())
		{
			FlushRenderingCommands();
		}
	}
}

void UMovieGraphImagePassBaseNode::GatherOutputPassesImpl(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const
{
	for (const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& Instance : CurrentInstances)
	{
		Instance->GatherOutputPasses(InConfig, OutExpectedPasses);
	}
}

void UMovieGraphImagePassBaseNode::GatherRenderCameraSourcesImpl(TArray<TSharedRef<FMovieGraphRenderCameraSource>>& InOutSources) const
{
	// Note: InOutSources may already contain sources from other nodes; AddUnique() avoids duplicates.
	
	for (const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InstanceIt : CurrentInstances)
	{
		const TSharedPtr<FMovieGraphRenderCameraSource> CameraSource = InstanceIt.IsValid()
			? InstanceIt->GetRenderCameraSource() : nullptr;

		if (CameraSource)
		{
			InOutSources.AddUnique(CameraSource.ToSharedRef());
		}
	}
}

void UMovieGraphImagePassBaseNode::UpdateRenderPassTelemetryImpl(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const
{
	for (const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InstanceIt : CurrentInstances)
	{
		if (InstanceIt.IsValid())
		{
			InstanceIt->UpdateRenderPassTelemetry(InOutTelemetry);
		}
	}
}

void UMovieGraphImagePassBaseNode::GetPreviewBannerMessagesImpl(const FMovieGraphRenderDataIdentifier& InIdentifier, TArray<FMovieGraphPreviewBannerMessage>& OutMessages) const
{
	// Forward to the pass instance that produced this preview tile. The renderer creates one instance per
	// (branch, camera), so branch + camera uniquely identify the owning instance.
	for (const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InstanceIt : CurrentInstances)
	{
		if (InstanceIt.IsValid()
			&& InstanceIt->GetBranchName() == InIdentifier.RootBranchName
			&& InstanceIt->GetCameraName() == InIdentifier.CameraName)
		{
			InstanceIt->GetPreviewBannerMessages(OutMessages);
		}
	}
}

EViewModeIndex UMovieGraphImagePassBaseNode::GetViewModeIndex() const
{
	return VMI_Lit;
}

FEngineShowFlags UMovieGraphImagePassBaseNode::GetShowFlags() const
{
	return ShowFlags->GetShowFlags();
}

TArray<uint32> UMovieGraphImagePassBaseNode::GetDefaultShowFlags() const
{
	return TArray<uint32>();
}

int32 UMovieGraphImagePassBaseNode::GetNumSpatialSamplesDuringWarmUp(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) const
{
	if (InFrameTraversalContext.Shot)
	{
		if (InFrameTraversalContext.Shot->ShotInfo.NumEngineWarmUpFramesRemaining > CVarMoviePipelineNumWarmUpFramesWithSpatialSampling.GetValueOnGameThread())
		{
			return 1;
		}
	}
	
	return GetNumSpatialSamples();
}

void UMovieGraphImagePassBaseNode::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
	
	UMovieGraphImagePassBaseNode* This = CastChecked<UMovieGraphImagePassBaseNode>(InThis);
	for (TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& Instance : This->CurrentInstances)
	{
		Instance->AddReferencedObjects(Collector);
	}
}
