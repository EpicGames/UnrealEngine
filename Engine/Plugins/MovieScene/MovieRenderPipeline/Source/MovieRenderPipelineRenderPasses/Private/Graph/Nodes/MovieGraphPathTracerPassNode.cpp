// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/MovieGraphPathTracerPassNode.h"

#include "Graph/Renderers/MovieGraphPathTracerPass.h"
#include "Engine/EngineBaseTypes.h"
#include "MoviePipelineTelemetry.h"
#include "RenderUtils.h"
#include "ShowFlags.h"
#include "PathTracingDenoiser.h"
#include "MoviePipelineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphPathTracerPassNode)

TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> UMovieGraphPathTracerRenderPassNode::CreateInstance() const
{
	return MakeUnique<UE::MovieGraph::Rendering::FMovieGraphPathTracerPass>();
}

UMovieGraphPathTracerRenderPassNode::UMovieGraphPathTracerRenderPassNode()
	: SpatialSampleCount(1)
	, SeedOffset(0)
	, bEnableReferenceMotionBlur(true)
	, bEnableDenoiser(true)
	, DenoiserType(EMovieGraphPathTracerDenoiserType::Spatial)
	, FrameCount(2)
	, bWriteAllSamples(false)
	, bDisableToneCurve(false)
	, bAllowOCIO(true)
	, bIncludeBeautyRenderInOutput(true)
	, PPMFileNameFormat(TEXT("{sequence_name}.{layer_name}.{renderer_sub_name}.{frame_number}"))
	, TileCount(1)
{
	RendererName = TEXT("PathTraced");

	ShowFlags->ApplyDefaultShowFlagValue(VMI_PathTracing, true);
	// TODO: Showflag for SetMotionBlur()?

	// To help user knowledge we pre-seed the additional post processing materials with an array of potentially common passes.
	TArray<FString> DefaultPostProcessMaterials;
	DefaultPostProcessMaterials.Add(DefaultDepthAsset);
	DefaultPostProcessMaterials.Add(DefaultMotionVectorsAsset);

	for (FString& MaterialPath : DefaultPostProcessMaterials)
	{
		FMoviePipelinePostProcessPass& NewPass = AdditionalPostProcessMaterials.AddDefaulted_GetRef();
		NewPass.Name = (MaterialPath == DefaultDepthAsset) ? FString(TEXT("depth")) : FString(TEXT("motion"));
		NewPass.Material = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(MaterialPath));
		NewPass.bEnabled = false;
		NewPass.bHighPrecisionOutput = MaterialPath.Equals(DefaultDepthAsset);
		NewPass.bUseLosslessCompression = true;
	}
}

void UMovieGraphPathTracerRenderPassNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	FString MetadataPrefix = UE::MoviePipeline::GetMetadataPrefixPath(InRenderDataIdentifier);
	
	OutMergedFormatArgs.FilenameArguments.Add(TEXT("ss_count"), FString::FromInt(SpatialSampleCount));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/spatialSampleCount"), *MetadataPrefix), FString::FromInt(SpatialSampleCount));
	
	OutMergedFormatArgs.FilenameArguments.Add(TEXT("path_tracer_seed_offset"), FString::FromInt(SeedOffset));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/seedOffset"), *MetadataPrefix), FString::FromInt(SeedOffset));
	
	OutMergedFormatArgs.FilenameArguments.Add(TEXT("disable_tonecurve"), FString::FromInt(bDisableToneCurve));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/disableTonecurve"), *MetadataPrefix), FString::FromInt(bDisableToneCurve));
}

#if WITH_EDITOR
FText UMovieGraphPathTracerRenderPassNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("MovieGraphNodes", "PathTracedRenderPassGraphNode_Description", "Path Traced Renderer");
}

FSlateIcon UMovieGraphPathTracerRenderPassNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon PathTracerRendererIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.PathTracingMode");
	
    OutColor = FLinearColor::White;
    return PathTracerRendererIcon;
}
#endif

void UMovieGraphPathTracerRenderPassNode::SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData)
{
	Super::SetupImpl(InSetupData);

	// Hide the progress display during the render
	if (IConsoleVariable* ProgressDisplayCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PathTracing.ProgressDisplay")))
	{
		bOriginalProgressDisplayCvarValue = ProgressDisplayCvar->GetBool();
		ProgressDisplayCvar->SetWithCurrentPriority(false, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}

	// Different Path Traced renders can't currently use different NFOR settings, so we take the largest setting from all branches.
	int32 MaxFrameCount = 0;
	int32 MaxDenoiserType = 0;
	FString WarningLayerName = TEXT("None");

	for (const FMovieGraphRenderPassLayerData& LayerData : InSetupData.Layers)
	{
		UMovieGraphRenderPassNode* RenderPassNode = LayerData.RenderPassNode.Get();
		UMovieGraphPathTracerRenderPassNode* PathTracerNode = CastChecked<UMovieGraphPathTracerRenderPassNode>(RenderPassNode);
		if (PathTracerNode)
		{
			MaxFrameCount = FMath::Max(MaxFrameCount, PathTracerNode->GetCoolingDownFrameCount());
			MaxDenoiserType = FMath::Max(MaxDenoiserType, static_cast<int32>(PathTracerNode->DenoiserType));

			// Record the last temporal layer
			if (PathTracerNode->DenoiserType == EMovieGraphPathTracerDenoiserType::Temporal)
			{
				WarningLayerName = LayerData.LayerName;
			}
		}
	}

	// Reset the max frame count to 0 if spatial denoiser is in use.
	// We need this reset because here we use the max of all nodes.
	const EMovieGraphPathTracerDenoiserType EffectiveDenoiserType = static_cast<EMovieGraphPathTracerDenoiserType>(MaxDenoiserType);

	if (EffectiveDenoiserType == EMovieGraphPathTracerDenoiserType::Spatial)
	{
		MaxFrameCount = 0;
	}

	if (IConsoleVariable* FrameCountCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.NFOR.FrameCount")))
	{
		OriginalFrameCountCvarValue = FrameCountCvar->GetInt();
		FrameCountCvar->SetWithCurrentPriority(MaxFrameCount, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}

	if (IConsoleVariable* DenoiserTypeCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PathTracing.SpatialDenoiser.Type")))
	{
		OriginalDenoiserType = DenoiserTypeCvar->GetInt();
		DenoiserTypeCvar->SetWithCurrentPriority(static_cast<int32>(MaxDenoiserType), NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}

	//If no temporal denoiser is enabled, provide warning if we use temporal denoiser.
	if (!HasTemporalDenoiser())
	{
		bool bShouldPerformTemporalDenoising = EMovieGraphPathTracerDenoiserType::Temporal == EffectiveDenoiserType;

		if (bShouldPerformTemporalDenoising)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "The Path Traced Renderer node of Layer `%ls` enables temporal denoising but no temporal denoiser plugin is enabled." 
														 "Fallback to the first available spatial denoiser. Please enable at least one denoiser plugin "
														 "with temporal denoising capability to fully apply the node setting.", *WarningLayerName);
		}
	}

	bool bSupportsPathTracing = false;
	if (IsRayTracingEnabled())
	{
		if (const IConsoleVariable* PathTracingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PathTracing")))
		{
			bSupportsPathTracing = PathTracingCVar->GetInt() != 0;
		}
	}

	// Warn if the path tracer is being used, but it's not enabled in the Rendering settings. The path tracer won't work otherwise.
	if (!bSupportsPathTracing)
	{
		// TODO: Ideally this is called in a general-purpose validation step instead, but that framework does not exist yet.
		UE_LOGF(LogMovieRenderPipeline, Warning, "An active Path Traced Renderer node was found, but path tracing support is not enabled. To get "
													 "renders with path tracing, enable 'Support Hardware Ray Tracing' and 'Path Tracing' in the "
													 "project's Rendering settings.");
	}
}

void UMovieGraphPathTracerRenderPassNode::TeardownImpl()
{
	Super::TeardownImpl();

	// Restore the original setting for the progress display
	if (IConsoleVariable* ProgressDisplayCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PathTracing.ProgressDisplay")))
	{
		ProgressDisplayCvar->SetWithCurrentPriority(bOriginalProgressDisplayCvarValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}

	if (IConsoleVariable* FrameCountCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.NFOR.FrameCount")))
	{
		FrameCountCvar->SetWithCurrentPriority(OriginalFrameCountCvarValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}

	if (IConsoleVariable* DenoiserTypeCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PathTracing.SpatialDenoiser.Type")))
	{
		DenoiserTypeCvar->SetWithCurrentPriority(OriginalDenoiserType, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
	}
}

int32 UMovieGraphPathTracerRenderPassNode::GetCoolingDownFrameCount() const
{
	return ((DenoiserType == EMovieGraphPathTracerDenoiserType::Temporal)
		&& GetAllowDenoiser()
		&& HasTemporalDenoiser()) ? FrameCount : 0;
}

EViewModeIndex UMovieGraphPathTracerRenderPassNode::GetViewModeIndex() const
{
	return VMI_PathTracing;
}

bool UMovieGraphPathTracerRenderPassNode::GetWriteAllSamples() const
{
	return bWriteAllSamples;
}

TArray<FMoviePipelinePostProcessPass> UMovieGraphPathTracerRenderPassNode::GetAdditionalPostProcessMaterials() const
{
	return AdditionalPostProcessMaterials;
}

FString UMovieGraphPathTracerRenderPassNode::GetPPMFileNameFormatString() const
{
	return PPMFileNameFormat;
}

int32 UMovieGraphPathTracerRenderPassNode::GetNumSpatialSamples() const
{
	return SpatialSampleCount;
}

int32 UMovieGraphPathTracerRenderPassNode::GetSeedOffset() const
{
	return SeedOffset;
}

int32 UMovieGraphPathTracerRenderPassNode::GetNumSpatialSamplesDuringWarmUp(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) const
{
	// Path Tracer doesn't have an image history like the deferred renderer, so it doesn't need
	// to run all the spatial samples.
	return 1;
}

bool UMovieGraphPathTracerRenderPassNode::GetDisableToneCurve() const
{
	return bDisableToneCurve;
}

bool UMovieGraphPathTracerRenderPassNode::GetAllowOCIO() const
{
	return bAllowOCIO;
}

bool UMovieGraphPathTracerRenderPassNode::GetAllowDenoiser() const
{
	return bEnableDenoiser;
}

bool UMovieGraphPathTracerRenderPassNode::GetWriteBeautyPassToDisk() const
{
	return bIncludeBeautyRenderInOutput;
}

FEngineShowFlags UMovieGraphPathTracerRenderPassNode::GetShowFlags() const
{
	FEngineShowFlags OutShowFlags = Super::GetShowFlags();
	OutShowFlags.SetPathTracing(true);
	OutShowFlags.SetMotionBlur(!bEnableReferenceMotionBlur);

	return OutShowFlags;
}

bool UMovieGraphPathTracerRenderPassNode::GetEnableHighResolutionTiling() const
{
	return bEnableHighResolutionTiling;
}

FIntPoint UMovieGraphPathTracerRenderPassNode::GetTileCount() const
{
	return bEnableHighResolutionTiling ? FIntPoint(TileCount, TileCount) : FIntPoint(1, 1);
}

float UMovieGraphPathTracerRenderPassNode::GetTileOverlapPercentage() const
{
	return bEnableHighResolutionTiling ? OverlapPercentage : 0;
}

bool UMovieGraphPathTracerRenderPassNode::GetEnablePageToSystemMemory() const
{
	return bPageToSystemMemory;
}

bool UMovieGraphPathTracerRenderPassNode::GetEnableHistoryPerTile() const
{
	return bAllocateHistoryPerTile;
}

void UMovieGraphPathTracerRenderPassNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	Super::UpdateTelemetry(InTelemetry);

	InTelemetry->bUsesPathTracer = true;
	InTelemetry->bUsesPPMs |= Algo::AnyOf(AdditionalPostProcessMaterials, [](const FMoviePipelinePostProcessPass& Pass) { return Pass.bEnabled; });
	InTelemetry->SpatialSampleCount = FMath::Max(InTelemetry->SpatialSampleCount, SpatialSampleCount);
}

void UMovieGraphPathTracerRenderPassNode::ResolveTokenContainingProperties(TFunction<void(FString&)>& ResolveFunc, const FMovieGraphTokenResolveContext& InContext)
{
	Super::ResolveTokenContainingProperties(ResolveFunc, InContext);
	
	for (FMoviePipelinePostProcessPass& PostProcessPass : AdditionalPostProcessMaterials)
	{
		ResolveFunc(PostProcessPass.Name);
	}
}
