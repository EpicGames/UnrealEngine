// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Renderers/MovieGraphDeferredPanoramicPass.h"

#include "Engine/TextureRenderTarget2D.h"
#include "EngineModule.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphApplyViewportLookNode.h"
#include "Graph/Nodes/MovieGraphCameraNode.h"
#include "Graph/Nodes/MovieGraphDeferredPanoramicPassNode.h"
#include "MoviePipelinePanoramicBlenderBase.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderOverlappedImage.h"
#include "SceneManagement.h"
#include "TextureResource.h"

namespace UE::MovieGraph::Rendering
{
	struct FMovieGraphPanoSampleState : public FMovieGraphSampleState
	{
		virtual TSharedRef<FMovieGraphSampleState> Copy() const override
		{
			return MakeShared<FMovieGraphPanoSampleState>(*this);
		}

		UE::MoviePipeline::FPanoramicPane Pane;
	};

	static_assert(std::is_base_of<FMovieGraphSampleState, FMovieGraphPanoSampleState>::value,
		"PanoSampleState must inherit from FMovieGraphSampleState due to shared basecode that does static casting.");

	class FMovieGraphPanoramicBlender : public UE::MovieGraph::IMovieGraphOutputMerger
	{
	public:
		FMovieGraphPanoramicBlender(TSharedPtr<UE::MovieGraph::IMovieGraphOutputMerger> InOutputMerger, const FIntPoint InOutputResolution)
		{
			OutputMerger = InOutputMerger;
			OutputResolution = InOutputResolution;
		}

		static FMovieGraphOutputMergerFrame MovieGraphDummyOutputFrame;
		static TQueue<FMovieGraphOutputMergerFrame> MovieGraphDummyOutputQueue;

		virtual FMovieGraphOutputMergerFrame& AllocateNewOutputFrame_GameThread(const int32 InRenderedFrameNumber) override
		{
			// Unsupported, the main Output Builder should be the one tracking this.
			// Since these are references we have to return something, so we return some dummy data.
			check(0);
			return MovieGraphDummyOutputFrame;
		}

		virtual FMovieGraphOutputMergerFrame& GetOutputFrame_GameThread(const int32 InRenderedFrameNumber) override
		{
			// Unsupported, the main Output Builder should be the one tracking this.
			check(0);
			return MovieGraphDummyOutputFrame;
		}
		virtual void OnCompleteRenderPassDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData)
		{

			// Copy the payload at the start, because it will get destroyed when the pixel data is moved to a task, which then
			// uses it and discards it before this function finishes.
			TSharedRef<FMovieGraphPanoSampleState> DataPayload = StaticCastSharedRef<FMovieGraphPanoSampleState>(InData->GetPayload<FMovieGraphPanoSampleState>()->Copy());
			
			// This function is called every time a sample comes in from the GPU (after being accumulated) and needs to handle
			// multiple samples from multiple frames being in flight at once. First step is to search to see if we're already
			// working on an output frame for this sample.
			FPoolEntry* TargetBlender = nullptr;
			{
				// Do a quick lock while we're iterating/adding to the PendingData array so a second sample
				// doesn't come in mid iteration.
				FScopeLock ScopeLock(&GlobalQueueDataMutex);

				for (TUniquePtr<FPoolEntry>& Item : PendingData)
				{
					if (Item->OutputFrameNumber == DataPayload->TraversalContext.Time.OutputFrameNumber && Item->bActive)
					{
						TargetBlender = Item.Get();
						break;
					}
				}

				if (!TargetBlender)
				{
					// UE_LOGF(LogMovieRenderPipeline, Log, "Starting a new Output Frame in the Panoramic Blender for frame: %d", DataPayload->SampleState.OutputState.OutputFrameNumber);

					// If we didn't find a blender already working on this frame, we'll try to re-use a previously allocated blender.
					for (TUniquePtr<FPoolEntry>& Item : PendingData)
					{
						if (!Item->bActive)
						{
							TargetBlender = Item.Get();
						}
					}

					// If we still don't have a target blender, then this is a new one and we need to allocate an entry.
					if (!TargetBlender)
					{
						TUniquePtr<FPoolEntry> NewEntry = MakeUnique<FPoolEntry>();
						int32 NewIndex = PendingData.Add(MoveTemp(NewEntry));
						TargetBlender = PendingData[NewIndex].Get();
					}

					check(TargetBlender);

					// If we were already working on this frame, the first for loop through the pending data would have found it.
					// So we know that if we get here, that we need to initialize whatever blender we ended up with.
					TargetBlender->OutputFrameNumber = DataPayload->TraversalContext.Time.OutputFrameNumber;
					TargetBlender->bActive = true;
					TargetBlender->NumCompletedAccumulations = 0;
					TargetBlender->Blender.Initialize(OutputResolution);
				}
			}

			// This can get called later (due to blending being async) so only capture by value.
			auto OnDebugSampleAvailable = [
			DataPayloadCopy = DataPayload->Copy(),
			WeakOutputMerger = OutputMerger](FLinearColor* Data, FIntPoint Resolution)
				{
					TSharedRef<FMovieGraphPanoSampleState> PayloadAsPano = StaticCastSharedRef<FMovieGraphPanoSampleState>(DataPayloadCopy);
					if(!PayloadAsPano->bWriteSampleToDisk)
					{
						return;
					}

					if (PayloadAsPano->Pane.EyeIndex >= 0)
					{
						PayloadAsPano->Debug_OverrideFilename = FString::Printf(TEXT("/%s_PaneX_%d_PaneY_%dEye_%d-Blended.%d"),
							*PayloadAsPano->TraversalContext.RenderDataIdentifier.LayerName, PayloadAsPano->Pane.HorizontalStepIndex,
							PayloadAsPano->Pane.VerticalStepIndex, PayloadAsPano->Pane.EyeIndex, PayloadAsPano->TraversalContext.Time.OutputFrameNumber);
					}
					else
					{
						PayloadAsPano->Debug_OverrideFilename = FString::Printf(TEXT("/%s_PaneX_%d_PaneY_%d-Blended.%d"),
							*PayloadAsPano->TraversalContext.RenderDataIdentifier.LayerName, PayloadAsPano->Pane.HorizontalStepIndex,
							PayloadAsPano->Pane.VerticalStepIndex, PayloadAsPano->TraversalContext.Time.OutputFrameNumber);
					}
					
					// We have to copy the memory because the blender is going to re-use it.
					TArray64<FLinearColor> BlendDataCopy = TArray64<FLinearColor>(Data, Resolution.X * Resolution.Y);
					TUniquePtr<TImagePixelData<FLinearColor>> FinalPixelData = MakeUnique<TImagePixelData<FLinearColor>>(Resolution, MoveTemp(BlendDataCopy), PayloadAsPano->Copy());
					
					if (ensure(WeakOutputMerger.IsValid()))
					{
						WeakOutputMerger.Pin()->OnSingleSampleDataAvailable_AnyThread(MoveTemp(FinalPixelData));
					}
				};

			// Now that we know which blender we're trying to accumulate to, we can just send the data to it directly. We're already
			// on a task thread, and the blending process supports multiple task threads working on blending at the same time.
			TargetBlender->Blender.BlendSample_AnyThread(MoveTemp(InData), DataPayload->Pane, OnDebugSampleAvailable);
			
			// Checking to see if this is the last sample is slightly complicated, because we can have multiple threads in this function at the same time.
			// Inside the blender, it only lets one thread increment the sample count at the same time, but that means when we look at it, we need to
			// go through the same lock so that we don't have two threads (in this function) read the value, and both decide they're the last sample.
			
			{
				// We put this behind our lock just so that we don't have two threads get the same value one after the other and still decide
				// they're the last.
				FScopeLock ScopeLock(&GlobalQueueDataMutex);
				TargetBlender->NumCompletedAccumulations++;
			
				int32 NumCompletedAccumulations = TargetBlender->NumCompletedAccumulations;
				const bool bLastSample = NumCompletedAccumulations == DataPayload->Pane.NumHorizontalSteps * DataPayload->Pane.NumVerticalSteps;
			
				if (bLastSample)
				{
					// BlendSample_AnyThread returns immediately and we'll increment it as completed, so if this is the last sample,
					// we'll wait for the outstanding work to finish.
					{
						TargetBlender->Blender.TaskConcurrencyLimiter->Wait();
					}

					if (ensure(OutputMerger.IsValid()))
					{
						TUniquePtr<TImagePixelData<FLinearColor> > FinalPixelData = MakeUnique<TImagePixelData<FLinearColor>>(OutputResolution, DataPayload->Copy());
						TargetBlender->Blender.FetchFinalPixelDataLinearColor(FinalPixelData->Pixels);
			
						OutputMerger.Pin()->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
					}
			
					// Release the pool item so future frames can use it.
					TargetBlender->bActive = false;
				}
			}
		}
		virtual void OnSingleSampleDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) override
		{
			// This is used for debug output, just pass it straight through.
			ensure(OutputMerger.IsValid());
			OutputMerger.Pin()->OnSingleSampleDataAvailable_AnyThread(MoveTemp(InData));
		}
		virtual void AbandonOutstandingWork() override
		{
			// Not implemented
			check(0);
		}
		virtual int32 GetNumOutstandingFrames() const
		{
			// Not implemented (this function isn't called anywhere right now)
			return 0;
		}
		virtual TQueue<FMovieGraphOutputMergerFrame>& GetFinishedFrames()
		{
			// Not implemented
			check(0);
			return MovieGraphDummyOutputQueue;
		}

	private:
		struct FPoolEntry
		{
			UE::MoviePipeline::FMoviePipelinePanoramicBlenderBase Blender;
			bool bActive;
			int32 OutputFrameNumber;
			std::atomic<int32> NumCompletedAccumulations;
		};

		// Pool entries are allocated as pointers on the heap so that if the array is resized while a thread is
		// working on a previous frame, it doesn't have the memory moved out from underneath it.
		TArray<TUniquePtr<FPoolEntry>> PendingData;

		FCriticalSection GlobalQueueDataMutex;

		FIntPoint OutputResolution;

		TWeakPtr<UE::MovieGraph::IMovieGraphOutputMerger> OutputMerger;
	};

	FMovieGraphOutputMergerFrame FMovieGraphPanoramicBlender::MovieGraphDummyOutputFrame;
	TQueue<FMovieGraphOutputMergerFrame> FMovieGraphPanoramicBlender::MovieGraphDummyOutputQueue;

FMovieGraphDeferredPanoramicPass::FMovieGraphDeferredPanoramicPass()
{
}

static void GetFieldOfView(float& OutHorizontal, float& OutVertical)
{
	// Hard-coded for the moment as we don't support stereo or allowing users to override the pane FOV
	OutHorizontal = 90.f;
	OutVertical = 90.f;
}

static FIntPoint GetPaneResolution(const FIntPoint& InSize)
{
	// We calculate a different resolution than the final output resolution.
	float HorizontalFoV;
	float VerticalFoV;
	GetFieldOfView(HorizontalFoV, VerticalFoV);

	// Horizontal FoV is a proportion of the global horizontal resolution
	// ToDo: We might have to check which is higher, if numVerticalPanes > numHorizontalPanes this math might be backwards.
	float HorizontalRes = (HorizontalFoV / 360.0f) * InSize.X;
	float Intermediate = FMath::Tan(FMath::DegreesToRadians(VerticalFoV) * 0.5f) / FMath::Tan(FMath::DegreesToRadians(HorizontalFoV) * 0.5f);
	float VerticalRes = HorizontalRes * Intermediate;

	return FIntPoint(FMath::CeilToInt(HorizontalRes), FMath::CeilToInt(VerticalRes));
}

void FMovieGraphDeferredPanoramicPass::Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer)
{
	FMovieGraphImagePassBase::Setup(InRenderer, InRenderPassNode, InLayer);

	RenderDataIdentifier.RootBranchName = InLayer.BranchName;
	RenderDataIdentifier.LayerName = InLayer.LayerName;
	RenderDataIdentifier.RendererName = InLayer.RenderPassNode->GetRendererName();
	RenderDataIdentifier.SubResourceName = InLayer.RenderPassNode->GetRendererSubName();
	RenderDataIdentifier.CameraName = InLayer.CameraName;
	LayerData = InLayer;

	UMovieGraphDeferredPanoramicNode* ParentNode = Cast<UMovieGraphDeferredPanoramicNode>(InLayer.RenderPassNode);
	int32 NumHorizontalSteps = FMath::Max(0, ParentNode->NumHorizontalSteps);
	int32 NumVerticalSteps = FMath::Max(0, ParentNode->NumVerticalSteps);
	int32 NumPanoramicPanes = NumHorizontalSteps * NumVerticalSteps;

	if (ParentNode->bAllocateHistoryPerPane)
	{
		if (ParentNode->bPageToSystemMemory)
		{
			SystemMemoryMirror = FSceneViewStateInterface::SystemMemoryMirrorAllocate();
		}

		PaneViewStates.SetNum(NumPanoramicPanes);

		const UE::MovieGraph::DefaultRenderer::FCameraInfo& CameraInfo = GetRenderCameraInfo();

		// ToDo: This doesn't take into account blended post-process values from the world, but we don't have a way to do the blending without having a FSceneView which doesn't exist until render time.
		const bool bUsesAutoExposure = CameraInfo.ViewInfo.PostProcessSettings.bOverride_AutoExposureMethod && CameraInfo.ViewInfo.PostProcessSettings.AutoExposureMethod != EAutoExposureMethod::AEM_Manual;

		if (bUsesAutoExposure)
		{
			AutoExposureViewStates.SetNum(CubeFace_MAX);

			// ShareOrigin must be called before Allocate.  ShareOrigin is necessary for Lumen to work with 6 way cube split screen
			// (causes Lumen scene data to be shared for all views, and overrides Lumen's regular 2 view split screen limitation).
			for (int32 Index = 1; Index < CubeFace_MAX; Index++)
			{
				AutoExposureViewStates[Index].ShareOrigin(&AutoExposureViewStates[0]);
			}

			for (int32 Index = 0; Index < CubeFace_MAX; Index++)
			{
				AutoExposureViewStates[Index].Allocate(InRenderer->GetWorld()->GetFeatureLevel());
			}
		}

		// Now that we've stopped allocating View States, we can Allocate them all.
		for (int32 Index = 0; Index < PaneViewStates.Num(); Index++)
		{
			PaneViewStates[Index].Allocate(InRenderer->GetWorld()->GetFeatureLevel());
		}

	}
}

void FMovieGraphDeferredPanoramicPass::Teardown()
{
	for (int32 Index = 0; Index < PaneViewStates.Num(); Index++)
	{
		FSceneViewStateInterface* Ref = PaneViewStates[Index].GetReference();
		if (Ref)
		{
			Ref->ClearMIDPool();
		}
		PaneViewStates[Index].Destroy();
	}
	PaneViewStates.Reset();

	if (AutoExposureViewStates.Num() > 0)
	{
		check(AutoExposureViewStates.Num() == CubeFace_MAX)
			for (int32 Index = 0; Index < CubeFace_MAX; Index++)
			{
				FSceneViewStateInterface* Ref = AutoExposureViewStates[Index].GetReference();
				if (Ref)
				{
					Ref->ClearMIDPool();
				}
			}

		// View states using FSceneViewStateReference::ShareOrigin need to be destroyed before their target, so remove last 5 elements first
		AutoExposureViewStates.RemoveAt(1, CubeFace_MAX - 1);
		AutoExposureViewStates.Reset();
	}

	FMovieGraphImagePassBase::Teardown();
}

void FMovieGraphDeferredPanoramicPass::AddReferencedObjects(FReferenceCollector& Collector)
{
	FMovieGraphImagePassBase::AddReferencedObjects(Collector);

	for (int32 Index = 0; Index < PaneViewStates.Num(); Index++)
	{
		FSceneViewStateInterface* Ref = PaneViewStates[Index].GetReference();
		if (Ref)
		{
			Ref->AddReferencedObjects(Collector);
		}
	}

	for (int32 Index = 0; Index < AutoExposureViewStates.Num(); Index++)
	{
		FSceneViewStateInterface* Ref = AutoExposureViewStates[Index].GetReference();
		if (Ref)
		{
			Ref->AddReferencedObjects(Collector);
		}
	}
}

void FMovieGraphDeferredPanoramicPass::GatherOutputPasses(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const
{
	OutExpectedPasses.Add(RenderDataIdentifier);
}


UMovieGraphImagePassBaseNode* FMovieGraphDeferredPanoramicPass::GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const
{
	constexpr bool bIncludeCDOs = true;
	UMovieGraphDeferredPanoramicNode* ParentNode = InConfig->GetSettingForBranch<UMovieGraphDeferredPanoramicNode>(GetBranchName(), bIncludeCDOs);
	if (!ensureMsgf(ParentNode, TEXT("DeferredPanoramicPass should not exist without parent node in graph.")))
	{
		return nullptr;
	}

	return ParentNode;
}

bool FMovieGraphDeferredPanoramicPass::ShouldDiscardOutput(const TSharedRef<FSceneViewFamilyContext>& InFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return false;
	}

	if (UMovieGraphPipeline* Pipeline = GraphRenderer->GetOwningGraph())
	{
		// The deferred renderer should attempt to discard anything that isn't from the rendering state, as we don't need any data from
		// the warm-up or cool-down phases of the shot.
		return Pipeline->GetActiveShotList()[Pipeline->GetCurrentShotIndex()]->ShotInfo.State != EMovieRenderShotState::Rendering;
	}
	return false;
}

FSceneViewStateInterface* FMovieGraphDeferredPanoramicPass::GetSceneViewState(UMovieGraphDeferredPanoramicNode* ParentNodeThisFrame, int32_t PaneX, int32_t PaneY)
{
	check(ParentNodeThisFrame);

	// If history per pane isn't supported then we only allocated one scene view at [0,0]
	FIntPoint PaneIndex = ParentNodeThisFrame->GetEnableHistoryPerTile() ? FIntPoint(PaneX, PaneY) : FIntPoint(0, 0);
	uint32 PaneIndex1D = (PaneIndex.Y * ParentNodeThisFrame->NumHorizontalSteps) + PaneIndex.X;
	
	// If they don't have history-per-pane enabled, we don't allocate any view states.
	if (PaneViewStates.Num() > 0)
	{
		// This function can't be const because GetReference() isn't const
		return PaneViewStates[PaneIndex1D].GetReference();
	}

	return nullptr;
}

FName FMovieGraphDeferredPanoramicPass::GetBranchName() const
{
	return LayerData.BranchName;
}

FString FMovieGraphDeferredPanoramicPass::GetCameraName() const
{
	return LayerData.CameraName;
}

int32 FMovieGraphDeferredPanoramicPass::GetRenderCameraIndex() const
{
	return LayerData.CameraIndex;
}

void FMovieGraphDeferredPanoramicPass::Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData)
{
	FMovieGraphImagePassBase::Render(InFrameTraversalContext, InTimeData);

	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return;
	}

	// Get the original desired output reoslution, which will then be modified to fit the correct ratio required for panoramic images (2:1)
	FIntPoint DesiredOutputResolution = UMovieGraphBlueprintLibrary::GetDesiredOutputResolution(InTimeData.EvaluatedConfig, 0.f);
	const FIntPoint PaneResolution = GetPaneResolution(DesiredOutputResolution);

	// Each tile has its own Temporal/Spatial accumulation buffer, and then when all of the samples have finished rendering,
	// instead of passing them to the Movie Graph Output Builder, we pass them to this one, which performs the blending,
	// and then forwards it onto the regular output builder for writing to disk.
	if (!PanoramicOutputBlender)
	{
		// This has to wait until the first call to Render to be initialized, because we need the output resolution, but that isn't
		// available during the Setup function.
		PanoramicOutputBlender = MakeShared<FMovieGraphPanoramicBlender>(GraphRenderer->GetOwningGraph()->GetOutputMerger(), DesiredOutputResolution);
	}

	UMovieGraphDeferredPanoramicNode* ParentNodeThisFrame = CastChecked<UMovieGraphDeferredPanoramicNode>(GetParentNode(InTimeData.EvaluatedConfig));

	// The outer rendering system takes care of allocating one of these per camera rendered,
	// but we need to know if we're rendering all cameras to pick up the correct post-process settings.
	const bool bIncludeCDOs = false;
	bool bRenderAllCameras = false;
	const UMovieGraphCameraSettingNode* CameraNode = InTimeData.EvaluatedConfig->GetSettingForBranch<UMovieGraphCameraSettingNode>(LayerData.BranchName, bIncludeCDOs);
	if (CameraNode)
	{
		bRenderAllCameras = CameraNode->bRenderAllCameras;
	}

	// We can only write rendered frames to disk right now (warm-up/cool-down indexes aren't propagated so files overwrite each other).
	const bool bWriteAllSamples = ParentNodeThisFrame->GetWriteAllSamples() && InFrameTraversalContext.Shot->ShotInfo.State == EMovieRenderShotState::Rendering;
	const bool bIsRenderingState =
		InFrameTraversalContext.Shot->ShotInfo.State == EMovieRenderShotState::Rendering ||
		InFrameTraversalContext.Shot->ShotInfo.State == EMovieRenderShotState::CoolingDown;

	int32 NumSpatialSamples = FMath::Max(1, bIsRenderingState ? ParentNodeThisFrame->GetNumSpatialSamples() : ParentNodeThisFrame->GetNumSpatialSamplesDuringWarmUp(InFrameTraversalContext, InTimeData));
	const ESceneCaptureSource SceneCaptureSource = ParentNodeThisFrame->GetDisableToneCurve() ? ESceneCaptureSource::SCS_FinalColorHDR : ESceneCaptureSource::SCS_FinalToneCurveHDR;
	const EAntiAliasingMethod AntiAliasingMethod = UE::MovieRenderPipeline::GetEffectiveAntiAliasingMethod(ParentNodeThisFrame->GetOverrideAntiAliasing(), ParentNodeThisFrame->GetAntiAliasingMethod());

	if (!bHasPrintedRenderingInfo)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Set-up Deferred Panoramic Renderer: %ls Layer: %ls OutputRes: [%d, %d] PaneRes: [%d, %d] PaneCount: [%d, %d] bPageToSystemMemory: %d bAutoExposurePass: %d",
			*RenderDataIdentifier.RendererName, *RenderDataIdentifier.LayerName,
			DesiredOutputResolution.X, DesiredOutputResolution.Y,
			PaneResolution.X, PaneResolution.Y, ParentNodeThisFrame->NumHorizontalSteps, ParentNodeThisFrame->NumVerticalSteps,
			ParentNodeThisFrame->GetEnablePageToSystemMemory(), AutoExposureViewStates.Num() > 0);

		bHasPrintedRenderingInfo = true;
	}

	// ToDo: Auto Exposure Pass first

	DefaultRenderer::FRenderTargetInitParams RenderTargetInitParams = GetRenderTargetInitParams(InTimeData, PaneResolution);

	FIntPoint PaneCount = FIntPoint(ParentNodeThisFrame->NumHorizontalSteps, ParentNodeThisFrame->NumVerticalSteps);

	for (int32 PaneY = 0; PaneY < PaneCount.Y; PaneY++)
	{
		for (int32 PaneX = 0; PaneX < PaneCount.X; PaneX++)
		{
			for (int32 SpatialIndex = 0; SpatialIndex < NumSpatialSamples; SpatialIndex++)
			{
				UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = GetRenderCameraInfo();

				UE::MoviePipeline::FPanoramicPane Data;
				
				// Fill out pano-pane specific information needed for the blending pass later
				{
					Data.OriginalCameraLocation = CameraInfo.ViewInfo.Location;
					Data.OriginalCameraRotation = CameraInfo.ViewInfo.Rotation;
				
					FTransform PrevTransform = CameraInfo.ViewInfo.PreviousViewTransform.Get(FTransform(CameraInfo.ViewInfo.Rotation, CameraInfo.ViewInfo.Location, FVector::OneVector));
					Data.PrevOriginalCameraLocation = PrevTransform.GetLocation();
					Data.PrevOriginalCameraRotation = FRotator(PrevTransform.GetRotation());

					constexpr int32 StereoIndex = -1;
					Data.EyeIndex = StereoIndex;
					Data.VerticalStepIndex = PaneY;
					Data.HorizontalStepIndex = PaneX;
					Data.NumHorizontalSteps = PaneCount.X;
					Data.NumVerticalSteps = PaneCount.Y;
					Data.EyeSeparation = 0.f;
					Data.EyeConvergenceDistance = 0.f;
					Data.bUseLocalRotation = ParentNodeThisFrame->bFollowCameraOrientation;
					Data.Resolution = PaneResolution;
					Data.FilterType = ParentNodeThisFrame->Filter;

					// The calculations above are for the main camera, now transform this pane's information to be specific to the current pane.
					bool bInPrevPos = false;
					UE::MoviePipeline::Panoramic::GetCameraOrientationForStereo(Data.CameraLocation, Data.CameraRotation, Data.CameraLocalRotation, Data, StereoIndex, bInPrevPos);

					bInPrevPos = true;
					FRotator DummyPrevLocalRot;
					UE::MoviePipeline::Panoramic::GetCameraOrientationForStereo(Data.PrevCameraLocation, Data.PrevCameraRotation, DummyPrevLocalRot, Data, StereoIndex, bInPrevPos);

					GetFieldOfView(Data.HorizontalFieldOfView, Data.VerticalFieldOfView);
				}

				UTextureRenderTarget2D* RenderTarget = GraphRenderer->GetOrCreateViewRenderTarget(RenderTargetInitParams, RenderDataIdentifier);
				FRenderTarget* RenderTargetResource = RenderTarget->GameThread_GetRenderTargetResource();
				check(RenderTargetResource);

				// World should be paused for every spatial sample except the last one, so that
				// the view doesn't update histories until the end, allowing us to render the same
				// scene multiple times.
				const bool bWorldIsPaused = !(SpatialIndex == (NumSpatialSamples - 1));
				const int32 FrameIndex = InTimeData.RenderedFrameNumber * ((InTimeData.TemporalSampleCount * NumSpatialSamples) + (InTimeData.TemporalSampleIndex * NumSpatialSamples)) + SpatialIndex;

				// We need to do this check before we start seeing if we need anti-aliasing samples so that when it falls back
				// to no AA, it still does the right thing and produces AA if they have spatial/temporal samples.
				EAntiAliasingMethod EffectiveAntiAliasingMethod = AntiAliasingMethod;
				const bool bRequiresHistory = IsTemporalAccumulationBasedMethod(EffectiveAntiAliasingMethod);
				if (!ParentNodeThisFrame->bAllocateHistoryPerPane && bRequiresHistory)
				{
					if (!bHasPrintedWarnings)
					{
						UE_LOGF(LogMovieRenderPipeline, Warning, "Panoramic Renders do not support TAA/TSR without enabling bAllocateHistoryPerPane! Forcing AntiAliasing off.");
						bHasPrintedWarnings = true;
					}

					EffectiveAntiAliasingMethod = EAntiAliasingMethod::AAM_None;
				}

				// We only allow a spatial jitter if we have more than one sample
				FVector2f SpatialShiftAmount = FVector2f(0.f, 0.f);
				const bool bAntiAliasingAllowsJitter = EffectiveAntiAliasingMethod == EAntiAliasingMethod::AAM_None;
				const bool bSampleCountsAllowsJitter = NumSpatialSamples > 1 || InTimeData.TemporalSampleCount > 1;
				if (bAntiAliasingAllowsJitter && bSampleCountsAllowsJitter)
				{
					const int32 NumSamplesPerOutputFrame = NumSpatialSamples * InTimeData.TemporalSampleCount;
					SpatialShiftAmount = UE::MoviePipeline::GetSubPixelJitter(FrameIndex, NumSamplesPerOutputFrame);
				}

				CameraInfo.bAllowCameraAspectRatio = false;
				CameraInfo.TilingParams.TileSize = PaneResolution;
				CameraInfo.TilingParams.OverlapPad = FIntPoint(0, 0); // ?
				CameraInfo.TilingParams.TileCount = FIntPoint(1, 1);
				CameraInfo.TilingParams.TileIndexes = FIntPoint(0, 0);
				CameraInfo.SamplingParams.TemporalSampleIndex = InTimeData.TemporalSampleIndex;
				CameraInfo.SamplingParams.TemporalSampleCount = InTimeData.TemporalSampleCount;
				CameraInfo.SamplingParams.SpatialSampleIndex = SpatialIndex;
				CameraInfo.SamplingParams.SpatialSampleCount = NumSpatialSamples;
				CameraInfo.SamplingParams.SeedOffset = 0;
				CameraInfo.ProjectionMatrixJitterAmount = FVector2D((SpatialShiftAmount.X) * 2.0f / (float)PaneResolution.X, SpatialShiftAmount.Y * -2.0f / (float)PaneResolution.Y);
				CameraInfo.bUseCameraManagerPostProcess = !bRenderAllCameras;
				CameraInfo.ViewInfo.ClearOverscan();
				
				// We override some of the information coming from the engine camera
				{
					CameraInfo.ViewInfo.Location = Data.CameraLocation;
					CameraInfo.ViewInfo.Rotation = Data.CameraRotation;
					CameraInfo.ViewInfo.PreviousViewTransform = FTransform(Data.PrevCameraRotation, Data.PrevCameraLocation, FVector::OneVector);
					CameraInfo.ViewInfo.bConstrainAspectRatio = false;
				}

				// The Scene View Family must be constructed first as the FSceneView needs it to be constructed
				UE::MovieGraph::Rendering::FViewFamilyInitData ViewFamilyInitData;
				ViewFamilyInitData.RenderTarget = RenderTargetResource;
				ViewFamilyInitData.World = GraphRenderer->GetWorld();
				ViewFamilyInitData.TimeData = InTimeData;
				ViewFamilyInitData.SceneCaptureSource = SceneCaptureSource;
				ViewFamilyInitData.bWorldIsPaused = bWorldIsPaused;
				ViewFamilyInitData.FrameIndex = FrameIndex;
				ViewFamilyInitData.AntiAliasingMethod = EffectiveAntiAliasingMethod;
				ViewFamilyInitData.ShowFlags = ParentNodeThisFrame->GetShowFlags();
				ViewFamilyInitData.ViewModeIndex = ParentNodeThisFrame->GetViewModeIndex();
				ViewFamilyInitData.ProjectionMode = CameraInfo.ViewInfo.ProjectionMode;

				FSceneViewStateInterface* ViewState = GetSceneViewState(ParentNodeThisFrame, PaneX, PaneY);
				if (ViewState && ParentNodeThisFrame->bAllocateHistoryPerPane && ParentNodeThisFrame->bPageToSystemMemory)
				{
					// If paging to system memory, restore the data needed for this particular Scene View History,
					// Transfering from CPU->GPU
					ViewState->SystemMemoryMirrorRestore(SystemMemoryMirror.Get());
				}

				TSharedRef<FSceneViewFamilyContext> ViewFamily = CreateSceneViewFamily(ViewFamilyInitData);
				FSceneViewInitOptions SceneViewInitOptions = CreateViewInitOptions(CameraInfo, ViewFamily.ToSharedPtr().Get(), ViewState);

				CalculateProjectionMatrix(CameraInfo, SceneViewInitOptions, PaneResolution, PaneResolution);

				float MinZ = GNearClippingPlane;
				const float MaxZ = MinZ;
				// Avoid zero ViewFOV's which cause divide by zero's in projection matrix
				const float MatrixFOV = FMath::Max(0.001f, Data.HorizontalFieldOfView) * (float)PI / 360.0f;
				// ToDo: I think this is a FMath::DegreesToRadians, easier to read that way than PI/360
				Data.NearClippingPlane = MinZ;

				float XAxisMultiplier = 1.f;
				float YAxisMultiplier = 1.f;
				SceneViewInitOptions.ProjectionMatrix = FReversedZPerspectiveMatrix(
					MatrixFOV,
					MatrixFOV,
					XAxisMultiplier,
					YAxisMultiplier,
					MinZ,
					MaxZ
				);

				// Construct a View to go within this family.
				FSceneView* NewView = CreateSceneView(SceneViewInitOptions, ViewFamily, CameraInfo);

				// Viewport-look mode may need to apply additional customizations to the view
#if WITH_EDITOR
				constexpr bool bExactMatch = true;
				if (const UMovieGraphApplyViewportLookNode* ViewportLookNode = InTimeData.EvaluatedConfig->GetSettingForBranch<UMovieGraphApplyViewportLookNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch))
				{
					ViewportLookNode->UpdateSceneView(NewView);
				}
#endif	// WITH_EDITOR

				// Then apply Movie Render Queue specific overrides to the ViewFamily, and then to the SceneView.
				ApplyMovieGraphOverridesToViewFamily(ViewFamily, ViewFamilyInitData);
				ApplyMovieGraphOverridesToSceneView(ViewFamily, ViewFamilyInitData, CameraInfo);

				FHitProxyConsumer* HitProxyConsumer = nullptr;
				const float DPIScale = 1.0f;
				FCanvas Canvas = FCanvas(RenderTargetResource, HitProxyConsumer, GraphRenderer->GetWorld(), GraphRenderer->GetWorld()->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, DPIScale);

				// Construct the sample state that reflects the current render sample
				FMovieGraphPanoSampleState SampleState;
				{
					// Take our per-frame Traversal Context and update it with context specific to this sample.
					FMovieGraphTraversalContext UpdatedTraversalContext = InFrameTraversalContext;
					UpdatedTraversalContext.Time = InTimeData;
					UpdatedTraversalContext.Time.SpatialSampleIndex = SpatialIndex;
					UpdatedTraversalContext.Time.SpatialSampleCount = NumSpatialSamples;
					UpdatedTraversalContext.RenderDataIdentifier = RenderDataIdentifier;

					SampleState.TraversalContext = MoveTemp(UpdatedTraversalContext);
					SampleState.OverscannedResolution = DesiredOutputResolution;
					SampleState.UnpaddedTileSize = PaneResolution;
					SampleState.BackbufferResolution = PaneResolution;
					SampleState.AccumulatorResolution = PaneResolution;
					SampleState.bWriteSampleToDisk = bWriteAllSamples;
					SampleState.bRequiresAccumulator = InTimeData.bRequiresAccumulator || (NumSpatialSamples > 1);
					SampleState.bFetchFromAccumulator = InTimeData.bIsLastTemporalSampleForFrame && (SpatialIndex == (NumSpatialSamples - 1));
					SampleState.OverlappedPad = FIntPoint(0, 0);
					SampleState.OverlappedOffset = FIntPoint(0, 0);
					SampleState.OverlappedSubpixelShift = FVector2D(0.5f - SpatialShiftAmount.X, 0.5f - SpatialShiftAmount.Y);
					SampleState.OverscanFraction = 0.f;
					SampleState.CropRectangle = FIntRect(0, 0, DesiredOutputResolution.X, DesiredOutputResolution.Y); // ToDo: Output resolution will be forced to a 2:1 ratio but this currently respects what the user put in.
					SampleState.bAllowOCIO = ParentNodeThisFrame->GetAllowOCIO();
					SampleState.bForceLosslessCompression = ParentNodeThisFrame->GetForceLosslessCompression();
					SampleState.bAllowsCompositing = ParentNodeThisFrame->GetAllowsCompositing();
					SampleState.bIsBeautyPass = true;
					SampleState.SceneCaptureSource = SceneCaptureSource;
					SampleState.CompositingSortOrder = 10;
					SampleState.RenderLayerIndex = LayerData.LayerIndex;
					SampleState.Pane = Data;

					// The TileX and TileY are hard-coded to zero to match the MoviePipeline version which did support tiles.
					SampleState.Debug_OverrideFilename = FString::Printf(TEXT("/%s_SS_%d_TS_%d_TileX_0_TileY_0_PaneX_%d_PaneY_%d.%d"),
						*SampleState.TraversalContext.RenderDataIdentifier.LayerName, SpatialIndex, InTimeData.TemporalSampleIndex,
						PaneX, PaneY, InTimeData.OutputFrameNumber);
				}

				ApplyMovieGraphOverridesToSampleState(SampleState);

				// If this was just to contribute to the history buffer, no need to go any further. Never discard if we're writing individual samples, though.
				bool bDiscardOutput = (InTimeData.bDiscardOutput || ShouldDiscardOutput(ViewFamily, CameraInfo)) && !SampleState.bWriteSampleToDisk;

				// Submit the renderer to be rendered
				GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.ToSharedPtr().Get());

				ENQUEUE_RENDER_COMMAND(TransitionTextureSRVState)(
					[RenderTargetResource](FRHICommandListImmediate& RHICmdList) mutable
					{
						// Transition our render target from a render target view to a shader resource view to allow the UMG preview material to read from this Render Target.
						RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVGraphicsPixel));
					});

				// After submission, if we're paging to system memory, mark the resources for download into system memory.
				if (ViewState && ParentNodeThisFrame->bAllocateHistoryPerPane && ParentNodeThisFrame->bPageToSystemMemory)
				{
					ViewState->SystemMemoryMirrorBackup(SystemMemoryMirror.Get());
				}

				// Data may not be something we actually want to read back and write to disk after rendering.
				if (bDiscardOutput)
				{
					continue;
				}

				PostRendererSubmission(SampleState, RenderTargetInitParams, Canvas, CameraInfo);
			}
		}
	}
}



TSharedRef<::MoviePipeline::IMoviePipelineAccumulationArgs> FMovieGraphDeferredPanoramicPass::GetOrCreateAccumulator(const TObjectPtr<UMovieGraphDefaultRenderer> InGraphRenderer, const FMovieGraphSampleState& InSampleState) const
{
	const FMoviePipelineAccumulatorPoolPtr SampleAccumulatorPool = InGraphRenderer->GetOrCreateAccumulatorPool<FImageOverlappedAccumulator>();

	// Because this is a virtual function and we need to get data out of the polymorphic SampleState 
	TSharedRef<FMovieGraphPanoSampleState> PanoSampleState = StaticCastSharedRef<FMovieGraphPanoSampleState>(InSampleState.Copy());
	// Generate a unique PassIdentifier for this Panoramic Pane, to ensure each pane gets its own accumulator.
	FMovieGraphRenderDataIdentifier RenderDataIdentifierCopy = RenderDataIdentifier;
	RenderDataIdentifierCopy.SubResourceName = FString::Printf(TEXT("%s_x%d_y%d"), *RenderDataIdentifier.SubResourceName, PanoSampleState->Pane.HorizontalStepIndex, PanoSampleState->Pane.VerticalStepIndex);


	const DefaultRenderer::FSurfaceAccumulatorPool::FInstancePtr AccumulatorInstance = SampleAccumulatorPool->GetAccumulatorInstance_GameThread<FImageOverlappedAccumulator>(InSampleState.TraversalContext.Time.OutputFrameNumber, RenderDataIdentifierCopy);
	TSharedRef<FMovieGraphRenderDataAccumulationArgs> AccumulationArgs = MakeShared<FMovieGraphRenderDataAccumulationArgs>();
	AccumulationArgs->OutputMerger = PanoramicOutputBlender;
	AccumulationArgs->ImageAccumulator = StaticCastSharedPtr<FImageOverlappedAccumulator>(AccumulatorInstance->Accumulator);
	AccumulationArgs->AccumulatorInstance = AccumulatorInstance;

	return AccumulationArgs;
}

UE::MovieGraph::Rendering::FMovieGraphImagePassBase::FAccumulatorSampleFunc FMovieGraphDeferredPanoramicPass::GetAccumulateSampleFunction() const
{
	return FMovieGraphImagePassBase::GetAccumulateSampleFunction();
}
} // UE::MovieGraph::Rendering
