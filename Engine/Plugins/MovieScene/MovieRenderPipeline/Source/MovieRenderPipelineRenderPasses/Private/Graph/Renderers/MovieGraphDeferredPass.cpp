// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Renderers/MovieGraphDeferredPass.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "Graph/Nodes/MovieGraphApplyViewportLookNode.h"
#include "Graph/Nodes/MovieGraphCameraNode.h"
#include "Graph/Nodes/MovieGraphDeferredPassNode.h"
#include "MoviePipelineSurfaceReader.h"
#include "MovieRenderOverlappedImage.h"

#include "CanvasTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineModule.h"
#include "LegacyScreenPercentageDriver.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "OpenColorIODisplayExtension.h"
#include "PostProcess/PostProcessInputs.h"
#include "SceneManagement.h"
#include "SceneRenderTargetParameters.h"
#include "SceneView.h"
#include "Tasks/Task.h"
#include "TextureResource.h"

#include "MoviePipelineBlueprintLibrary.h" // For overscan calculations
#include "MoviePipelineUtils.h" // For Sub-Pixel Jitter
#include "MovieRenderPipelineDataTypes.h" // For the 1D Weight table for accumulation

// Shader parameter struct used to force depth buffer decompression via RDG resource transition.
// On AMD GPUs, the depth buffer may remain HTile-compressed when Movie Render Graph disables
// intermediate post-process passes (DOF, MotionBlur, TAA) that would normally trigger decompression.
// Adding an empty RDG pass with this struct as SRV forces the necessary decompression barrier.
BEGIN_SHADER_PARAMETER_STRUCT(FBufferVisDepthDecompressParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, SceneDepth)
END_SHADER_PARAMETER_STRUCT()

// Scene view extension that forces depth buffer decompression before post-processing begins.
// This ensures buffer visualization post-process materials (WorldDepth, etc.) can correctly
// read SceneDepth on all GPU vendors.
class FMovieGraphDepthDecompressExtension : public ISceneViewExtension
{
public:
	virtual ~FMovieGraphDepthDecompressExtension() = default;

	virtual void PrePostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPostProcessingInputs& Inputs) override
	{
		if (Inputs.SceneTextures)
		{
			if (const FRDGTextureRef SceneDepthTexture = (*Inputs.SceneTextures)->SceneDepthTexture)
			{
				FBufferVisDepthDecompressParameters* DecompressParams = GraphBuilder.AllocParameters<FBufferVisDepthDecompressParameters>();
				DecompressParams->SceneDepth = SceneDepthTexture;
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("MRG_DecompressDepthForBufferVisualization"),
					DecompressParams,
					ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
					[](FRDGAsyncTask, FRHICommandList&) {}
				);
			}
		}
	}
};

namespace UE::MovieGraph::Rendering
{
void FMovieGraphDeferredPass::WritePerViewMetadata(
	TMap<FString, FString>& InOutMetadata,
	const FSceneView& InView,
	const FSceneViewInitOptions& InViewInitOptions,
	const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo,
	const FMovieGraphRenderDataIdentifier& InIdentifier)
{
	const FString Prefix = UE::MoviePipeline::GetMetadataPrefixPath(InIdentifier);

	InOutMetadata.Add(FString::Printf(TEXT("%s/matrix/worldToCamera"), *Prefix), InViewInitOptions.ViewRotationMatrix.ToString());
	InOutMetadata.Add(FString::Printf(TEXT("%s/matrix/worldToUEClip"), *Prefix), InViewInitOptions.ProjectionMatrix.ToString());

	UE::MoviePipeline::GetMetadataFromCameraLocRot(
		InCameraInfo.ViewInfo.Location, InCameraInfo.ViewInfo.Rotation,
		InCameraInfo.ViewInfo.PreviousViewTransform.Get(FTransform::Identity).GetLocation(),
		InCameraInfo.ViewInfo.PreviousViewTransform.Get(FTransform::Identity).GetRotation().Rotator(),
		Prefix, InOutMetadata);

	// TODO: If an actor had multiple cine camera components this could be wrong, but Sequencer also can't disambiguate between multiple camera components on a single actor.
	UE::MoviePipeline::GetMetadataFromCineCamera(
		InCameraInfo.ViewActor ? InCameraInfo.ViewActor->GetComponentByClass<UCineCameraComponent>() : nullptr,
		Prefix, InOutMetadata);

	// Per-view blended camera metadata (post-process settings reflect the final blended result)
	InOutMetadata.Add(FString::Printf(TEXT("%s/fstop"), *Prefix),           FString::SanitizeFloat(InView.FinalPostProcessSettings.DepthOfFieldFstop));
	InOutMetadata.Add(FString::Printf(TEXT("%s/fov"), *Prefix),             FString::SanitizeFloat(InViewInitOptions.FOV));
	InOutMetadata.Add(FString::Printf(TEXT("%s/focalDistance"), *Prefix),   FString::SanitizeFloat(InView.FinalPostProcessSettings.DepthOfFieldFocalDistance));
	InOutMetadata.Add(FString::Printf(TEXT("%s/sensorWidth"), *Prefix),     FString::SanitizeFloat(InView.FinalPostProcessSettings.DepthOfFieldSensorWidth));
	InOutMetadata.Add(FString::Printf(TEXT("%s/overscanPercent"), *Prefix), FString::SanitizeFloat(InCameraInfo.ViewInfo.GetOverscan()));
}

int32 FMovieGraphDeferredPass::GetRenderCameraIndex() const
{
	return LayerData.CameraIndex;
}

FSceneViewStateInterface* FMovieGraphDeferredPass::GetSceneViewState(UMovieGraphImagePassBaseNode* ParentNodeThisFrame, int32_t TileX, int32_t TileY)
{
	check(ParentNodeThisFrame);

	// If history per tile isn't supported then we only allocated one scene view at [0,0]
	FIntPoint TileIndex = ParentNodeThisFrame->GetEnableHistoryPerTile() ? FIntPoint(TileX, TileY) : FIntPoint(0, 0);

	// This function can't be const because GetReference() isn't const
	return SceneViewStates[TileIndex].GetReference();
}

FMovieGraphDeferredPass::FResolutionAndCameraInfo FMovieGraphDeferredPass::GetResolutionAndCameraInfo(UMovieGraphEvaluatedConfig* InEvaluatedConfig) const
{
	FResolutionAndCameraInfo Info;

	const UMovieGraphImagePassBaseNode* ParentNodeThisFrame = GetParentNode(InEvaluatedConfig);
	if (!ParentNodeThisFrame)
	{
		// GetParentNode() will log a warning; no need to log another warning here.
		return Info;
	}

	constexpr bool bIncludeCDOs = false;
	const UMovieGraphCameraSettingNode* CameraNode = InEvaluatedConfig->GetSettingForBranch<UMovieGraphCameraSettingNode>(LayerData.BranchName, bIncludeCDOs);

	Info.CameraInfo = GetRenderCameraInfo();

	// Get the overscan fraction and if all cameras should be rendered
	bool bOverscanFromMRGOverride = false;
	{
		// By default, use the overscan fraction specified directly on the camera actor
		Info.OverscanFraction = GetRenderCameraOverscan();

		// The Camera Settings node in the graph may specify an overscan override
		if (CameraNode)
		{
			Info.bRenderAllCameras = CameraNode->bRenderAllCameras;

			if (CameraNode->bOverride_OverscanPercentage)
			{
				Info.OverscanFraction = FMath::Clamp(CameraNode->OverscanPercentage / 100.f, 0.f, 1.f);
				bOverscanFromMRGOverride = true;
			}
		}
	}

	Info.CameraAspectRatio = (Info.CameraInfo.bAllowCameraAspectRatio && Info.CameraInfo.ViewInfo.bConstrainAspectRatio)
		? Info.CameraInfo.ViewInfo.AspectRatio
		: 0.f;

	// Camera asked for overscan-without-crop (UCameraComponent::bCropOverscan == false), expecting a downstream
	// pass like lens distortion to consume the extra pixels. Don't enlarge the accumulator and don't center-cut
	// at output because the rendered overscanned frame is the intended output.
	const bool bCameraRequestsNoCrop =
		!bOverscanFromMRGOverride
		&& !FMath::IsNearlyZero(Info.OverscanFraction)
		&& FMath::IsNearlyEqual(Info.CameraInfo.ViewInfo.CropFraction, 1.0f);

	const float OverscanFractionForAccumulator = bCameraRequestsNoCrop ? 0.0f : Info.OverscanFraction;
	Info.AccumulatorResolution = UMovieGraphBlueprintLibrary::GetOverscannedResolution(InEvaluatedConfig, OverscanFractionForAccumulator, Info.CameraAspectRatio);

	if (bCameraRequestsNoCrop)
	{
		Info.AccumulatorResolutionCropRect = FIntRect(FIntPoint::ZeroValue, Info.AccumulatorResolution);
	}
	else
	{
		// Center cut applied at write time: non-EXR formats crop to this rect, but EXR uses it as the display window.
		Info.AccumulatorResolutionCropRect = UMovieGraphBlueprintLibrary::GetOverscanCropRectangle(InEvaluatedConfig, Info.OverscanFraction, Info.CameraAspectRatio);
	}

	// If a custom camera source is active, let it override the accumulator resolution, crop rect,
	// and overscan fraction computed above with per-camera values.
	const TSharedPtr<const FMovieGraphRenderCameraSource> CameraSource = GetRenderCameraSource();
	if (CameraSource && CameraSource->GetCameraOverscannedResolution(GetRenderCameraIndex(),
		Info.AccumulatorResolution, Info.AccumulatorResolutionCropRect, Info.OverscanFraction))
	{
		// A custom camera source provides multiple cameras, so bRenderAllCameras must be
		// set to override the default single-camera path and render all of them.
		Info.bRenderAllCameras = true;
	}

	// This is what the actual renders would be at, without overlap.
	const FIntPoint TileCount = ParentNodeThisFrame->GetTileCount();
	Info.BackbufferResolution = FIntPoint(
		FMath::CeilToInt(static_cast<float>(Info.AccumulatorResolution.X) / static_cast<float>(TileCount.X)),
		FMath::CeilToInt(static_cast<float>(Info.AccumulatorResolution.Y) / static_cast<float>(TileCount.Y))
	);

	// Tile size is the size of a tile before any overlap padding.
	Info.TileSize = Info.BackbufferResolution;

	// User deals with overlap as 0-100, we need it as 0-1
	const float TileOverlapFraction = ParentNodeThisFrame->GetTileOverlapPercentage() / 100.f;

	// We now apply the overlap ratio (which is effectively overscan) to the backbuffer, but we use twice the user-specified overscan. This is because
	// in High Res Tiling it's "overlap" percentage, so 10% overlap implies 10% on each side, while for Overscan, 10% overscan implies 10% bigger.
	Info.BackbufferResolution = UE::MoviePipeline::ScaleResolutionByOverscan(TileOverlapFraction * 2, Info.BackbufferResolution);

	return Info;
}

void FMovieGraphDeferredPass::Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer)
{
	FMovieGraphImagePassBase::Setup(InRenderer, InRenderPassNode, InLayer);
	
	LayerData = InLayer;

	RenderDataIdentifier.RootBranchName = LayerData.BranchName;
	RenderDataIdentifier.LayerName = LayerData.LayerName;
	RenderDataIdentifier.RendererName = InLayer.RenderPassNode->GetRendererName();
	RenderDataIdentifier.SubResourceName = InLayer.RenderPassNode->GetRendererSubName();
	RenderDataIdentifier.CameraName =  InLayer.CameraName;

	UMovieGraphImagePassBaseNode* ParentNode = Cast< UMovieGraphImagePassBaseNode>(LayerData.RenderPassNode);
	check(ParentNode);

	FIntPoint TileCountWithHistory = ParentNode->GetTileCount();
	int32 NumSceneViewStateReferencesAllocated = 0;
	// If we don't need a view state for each tile, then we only allocate one.
	if (!ParentNode->GetEnableHistoryPerTile())
	{
		TileCountWithHistory = FIntPoint(1, 1);
	}
	for (int32 TileX = 0; TileX < TileCountWithHistory.X; TileX++)
	{
		for (int32 TileY = 0; TileY < TileCountWithHistory.Y; TileY++)
		{
			SceneViewStates.Add(FIntPoint(TileX, TileY));
			NumSceneViewStateReferencesAllocated++;
		}
	}

	// Continue to allocate this in case derived classes used it. The allocation
	// doesn't cost that much (until a scene view is rendered with it) so we don't
	// pay the overhead in our default implementations here.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SceneViewState.Allocate(InRenderer->GetWorld()->GetFeatureLevel());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (ParentNode->GetEnableHistoryPerTile())
	{
		if (ParentNode->GetEnablePageToSystemMemory())
		{
			SystemMemoryMirror = FSceneViewStateInterface::SystemMemoryMirrorAllocate();
		}

		// ToDo: This doesn't take into account blended post-process values from the world, but we don't have a way to do the blending without having a FSceneView which doesn't exist until render time.
		const UE::MovieGraph::DefaultRenderer::FCameraInfo& CameraInfo = GetRenderCameraInfo();
		const bool bUsesAutoExposure = CameraInfo.ViewInfo.PostProcessSettings.bOverride_AutoExposureMethod && CameraInfo.ViewInfo.PostProcessSettings.AutoExposureMethod != EAutoExposureMethod::AEM_Manual;

		const bool bHighRes = ParentNode->GetEnableHighResolutionTiling();
		const bool bHasTiles = (TileCountWithHistory.X * TileCountWithHistory.Y) > 1;

		// Auto Exposure passes are only needed if:
		// A) The user has requested them
		// B) High Resolution Tiling is enabled
		// C) The tile count needed by high resolution is > 1.
		bHasAutoExposurePass = bUsesAutoExposure && bHighRes && bHasTiles;
		if (bHasAutoExposurePass)
		{
			// Auto Exposure passes are always at -1, -1
			SceneViewStates.Add(FIntPoint(-1, -1));
			NumSceneViewStateReferencesAllocated++;
		}
	}

	for (TPair<FIntPoint, FSceneViewStateReference>& Pair : SceneViewStates)
	{
		// Once we call Allocate, FSceneViewStateReference is no longer trivially relocatable, so we
		// have to wait until all of the states are added before initializing them.
		Pair.Value.Allocate(InRenderer->GetWorld()->GetFeatureLevel());
	}

	// The InRenderPassNode is not initialized with user's config. Use InLayer to 
	// initialize the frames to delay for post submission.
	UMovieGraphRenderPassNode* RenderPassNode = InLayer.RenderPassNode.Get();
	FramesToDelayPostSubmission = RenderPassNode ? RenderPassNode->GetCoolingDownFrameCount() : 0;
	RemainingCooldownReadbackFrames = FramesToDelayPostSubmission;

	// The user may request that the beauty pass isn't written to disk (sometimes used if only PPM passes are desired).
	bWriteBeautyPassToDisk = ParentNode->GetWriteBeautyPassToDisk();

	UE_LOGF(LogMovieRenderPipeline, Log, "Initialized Renderer: %ls Layer: %ls TileCount[%d, %d]/[%d, %d] AutoExposurePass: %d HighRes: %d PageToSystem: %d TotalSceneViewStateReferences: %d",
		*RenderDataIdentifier.RendererName, *RenderDataIdentifier.LayerName, ParentNode->GetTileCount().X, ParentNode->GetTileCount().Y, TileCountWithHistory.X, TileCountWithHistory.Y,
		bHasAutoExposurePass, ParentNode->GetEnableHighResolutionTiling(), ParentNode->GetEnablePageToSystemMemory(), NumSceneViewStateReferencesAllocated);
}

void FMovieGraphDeferredPass::Teardown()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSceneViewStateInterface* SceneViewRef = SceneViewState.GetReference();
	if (SceneViewRef)
	{
		SceneViewRef->ClearMIDPool();
	}
	SceneViewState.Destroy();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (TPair<FIntPoint, FSceneViewStateReference>& ViewState : SceneViewStates)
	{
		FSceneViewStateInterface* SceneViewInterface = ViewState.Value.GetReference();
		if (SceneViewInterface)
		{
			SceneViewInterface->ClearMIDPool();
		}
		ViewState.Value.Destroy();
	}
}

void FMovieGraphDeferredPass::GatherOutputPasses(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const
{
	FMovieGraphImagePassBase::GatherOutputPasses(InConfig,OutExpectedPasses);
	
	// Add our pre-calculated identifier if we're writing out the beauty pass.
	if (bWriteBeautyPassToDisk)
	{
		OutExpectedPasses.Add(RenderDataIdentifier);
	}

	if (const UMovieGraphImagePassBaseNode* ParentNode = GetParentNode(InConfig))
	{
		for (const FMoviePipelinePostProcessPass& AdditionalPass : ParentNode->GetAdditionalPostProcessMaterials())
		{
			if (AdditionalPass.bEnabled)
			{
				if (const UMaterialInterface* Material = AdditionalPass.Material.LoadSynchronous())
				{
					FMovieGraphRenderDataIdentifier Identifier = RenderDataIdentifier;
					Identifier.SubResourceName = AdditionalPass.Name.IsEmpty() ? Material->GetName() : AdditionalPass.Name;
					OutExpectedPasses.Add(Identifier);
				}
			}
		}
	}
}

void FMovieGraphDeferredPass::AddReferencedObjects(FReferenceCollector& Collector)
{
	FMovieGraphImagePassBase::AddReferencedObjects(Collector);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSceneViewStateInterface* SceneViewRef = SceneViewState.GetReference();
	if (SceneViewRef)
	{
		SceneViewRef->AddReferencedObjects(Collector);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (TPair<FIntPoint, FSceneViewStateReference>& ViewState : SceneViewStates)
	{
		FSceneViewStateInterface* SceneViewInterface = ViewState.Value.GetReference();
		if (SceneViewInterface)
		{
			SceneViewInterface->AddReferencedObjects(Collector);
		}
	}
}

FName FMovieGraphDeferredPass::GetBranchName() const
{
	return LayerData.BranchName;
}

FString FMovieGraphDeferredPass::GetCameraName() const
{
	return LayerData.CameraName;
}

bool FMovieGraphDeferredPass::ShouldDiscardOutput(const TSharedRef<FSceneViewFamilyContext>& InFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return false;
	}

	if (UMovieGraphPipeline* Pipeline = GraphRenderer->GetOwningGraph())
	{
		// The deferred renderer should attempt to discard anything that isn't from the rendering state, as we don't need any data from
		// the warm-up or cool-down phases of the shot.
		return Pipeline->GetActiveShotList()[Pipeline->GetCurrentShotIndex()]->ShotInfo.State != EMovieRenderShotState::Rendering;
	}
	return false;
}


UMovieGraphImagePassBaseNode* FMovieGraphDeferredPass::GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const
{
	// This is a bit of a workaround for the fact that the pass doesn't have a strong pointer to the node it's supposed to be associated with,
	// since that instance changes every frame. So instead we have a virtual function here so the node can look it up by type, and then we can
	// call a bunch of virtual functions on the right instance to fetch values.
	const bool bIncludeCDOs = true;
	UMovieGraphDeferredRenderPassNode* ParentNode = InConfig->GetSettingForBranch<UMovieGraphDeferredRenderPassNode>(GetBranchName(), bIncludeCDOs);
	if (!ensureMsgf(ParentNode, TEXT("DeferredPass should not exist without parent node in graph.")))
	{
		return nullptr;
	}

	return ParentNode;
}

FMovieGraphDeferredPass::FCreateViewResult::FCreateViewResult(const TSharedRef<FSceneViewFamilyContext> InViewFamily)
	: ViewFamily(InViewFamily)
{
}

bool FMovieGraphDeferredPass::HasRenderResourceParametersChanged(const FIntPoint& AccumulatorResolution, const FIntPoint& BackbufferResolution) const
{
	return (PrevAccumulatorResolution != AccumulatorResolution) || (PrevBackbufferResolution != BackbufferResolution);
}

FMovieGraphDeferredPass::FCreateViewResult FMovieGraphDeferredPass::CreateConfiguredView(
	UE::MovieGraph::DefaultRenderer::FCameraInfo& InOutCameraInfo,
	const FViewFamilyInitData& InViewFamilyInitData,
	const FIntPoint& InBackbufferResolution,
	const FIntPoint& InAccumulatorResolution,
	UMovieGraphImagePassBaseNode* InParentNode,
	const int32 TileX,
	const int32 TileY)
{
	FSceneViewStateInterface* ViewState = GetSceneViewState(InParentNode, TileX, TileY);

	// Optional system memory mirroring. Skipped when bAutoRestoreViewStateMirror is false so derived passes
	// can drive restore/backup as needed.
	if (bAutoRestoreViewStateMirror && ViewState && InParentNode->GetEnableHistoryPerTile() && InParentNode->GetEnablePageToSystemMemory())
	{
		ViewState->SystemMemoryMirrorRestore(SystemMemoryMirror.Get());
	}

	const TSharedRef<FSceneViewFamilyContext> ViewFamily = CreateSceneViewFamily(InViewFamilyInitData);

	FSceneViewInitOptions SceneViewInitOptions = CreateViewInitOptions(InOutCameraInfo, ViewFamily.ToSharedPtr().Get(), ViewState);

	CalculateProjectionMatrix(InOutCameraInfo, SceneViewInitOptions, InBackbufferResolution, InAccumulatorResolution);

	// Modify the perspective matrix to do an off center projection, with overlap for high-res tiling
	const bool bOrthographic = InOutCameraInfo.ViewInfo.ProjectionMode == ECameraProjectionMode::Type::Orthographic;
	ModifyProjectionMatrixForTiling(InOutCameraInfo.TilingParams, bOrthographic, SceneViewInitOptions.ProjectionMatrix, InOutCameraInfo.DoFSensorScale);

	// Scale the DoF sensor scale to counteract overscan, otherwise the size of Bokeh changes when you have Overscan enabled.
	// This needs to come after we modify it for Tiling.
	InOutCameraInfo.DoFSensorScale *= 1.0 + InOutCameraInfo.ViewInfo.GetOverscan();

	// Construct a View to go within this family.
	FSceneView* NewView = CreateSceneView(SceneViewInitOptions, ViewFamily, InOutCameraInfo);

	// Object Occlusion/Histories — force reset when tiling to avoid queries from other tiles
	{
		const FIntPoint TileCount = InOutCameraInfo.TilingParams.TileCount;
		if (TileCount.X > 1 || TileCount.Y > 1)
		{
			NewView->bForceCameraVisibilityReset = true;
		}
	}

	// Viewport-look mode may need to apply additional customizations to the view
#if WITH_EDITOR
	constexpr bool bIncludeCDOs = false;
	constexpr bool bExactMatch = true;
	if (const UMovieGraphApplyViewportLookNode* ViewportLookNode = InViewFamilyInitData.TimeData.EvaluatedConfig->GetSettingForBranch<UMovieGraphApplyViewportLookNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch))
	{
		ViewportLookNode->UpdateSceneView(NewView);
	}
#endif	// WITH_EDITOR

	// Then apply Movie Render Queue specific overrides to the ViewFamily, and then to the SceneView.
	ApplyMovieGraphOverridesToViewFamily(ViewFamily, InViewFamilyInitData);

	ApplyMovieGraphOverridesToSceneView(ViewFamily, InViewFamilyInitData, InOutCameraInfo);

	FCreateViewResult Result(ViewFamily);
	Result.View = NewView;
	Result.ViewFamilyInitData = InViewFamilyInitData;
	Result.SceneViewInitOptions = SceneViewInitOptions;

	return Result;
}

void FMovieGraphDeferredPass::Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData)
{
	// ToDo: InFrameTraversalContext includes a copy of TimeData, but may be the one cached at the first temporal sample,
	// maybe we can combine, maybe we can't?
	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return;
	}

	UMovieGraphImagePassBaseNode* ParentNodeThisFrame = GetParentNode(InTimeData.EvaluatedConfig);
	
	// We can only write rendered frames to disk right now (warm-up/cool-down indexes aren't propagated so files overwrite each other).
	const bool bWriteAllSamples = ParentNodeThisFrame->GetWriteAllSamples() && InFrameTraversalContext.Shot->ShotInfo.State == EMovieRenderShotState::Rendering;
	const bool bIsRenderingState = InFrameTraversalContext.Shot->ShotInfo.State == EMovieRenderShotState::Rendering ||
									InFrameTraversalContext.Shot->ShotInfo.State == EMovieRenderShotState::CoolingDown;
	int32 NumSpatialSamples = FMath::Max(1, bIsRenderingState ? ParentNodeThisFrame->GetNumSpatialSamples() : ParentNodeThisFrame->GetNumSpatialSamplesDuringWarmUp(InFrameTraversalContext, InTimeData));
	const int32 SeedOffset = ParentNodeThisFrame->GetSeedOffset();
	const FIntPoint TileCount = ParentNodeThisFrame->GetTileCount();
	const float TileOverlapFraction = ParentNodeThisFrame->GetTileOverlapPercentage() / 100.f; // User deals with overlap as 0-100, we want it as 0-1

	const ESceneCaptureSource SceneCaptureSource = ParentNodeThisFrame->GetDisableToneCurve() ? ESceneCaptureSource::SCS_FinalColorHDR : ESceneCaptureSource::SCS_FinalToneCurveHDR;
	const EAntiAliasingMethod AntiAliasingMethod = UE::MovieRenderPipeline::GetEffectiveAntiAliasingMethod(ParentNodeThisFrame->GetOverrideAntiAliasing(), ParentNodeThisFrame->GetAntiAliasingMethod());

	const FResolutionAndCameraInfo ResolutionAndCameraInfo = GetResolutionAndCameraInfo(InTimeData.EvaluatedConfig.Get());
	const FIntPoint AccumulatorResolution = ResolutionAndCameraInfo.AccumulatorResolution;
	const FIntPoint BackbufferResolution = ResolutionAndCameraInfo.BackbufferResolution;
	if (HasRenderResourceParametersChanged(AccumulatorResolution, BackbufferResolution))
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Set-up Renderer: %ls Layer: %ls AccumulatorResolution: [%d, %d] BackbufferResolution: [%d, %d]",
			*RenderDataIdentifier.RendererName, *RenderDataIdentifier.LayerName, AccumulatorResolution.X, AccumulatorResolution.Y, BackbufferResolution.X, BackbufferResolution.Y);

		PrevAccumulatorResolution = AccumulatorResolution;
		PrevBackbufferResolution = BackbufferResolution;
	}

	DefaultRenderer::FRenderTargetInitParams RenderTargetInitParams = GetRenderTargetInitParams(InTimeData, BackbufferResolution);

	for (int32 TileY = 0; TileY < TileCount.Y; TileY++)
	{
		for (int32 TileX = 0; TileX < TileCount.X; TileX++)
		{
			// Now we can actually construct our ViewFamily, SceneView, and submit it for Rendering + Readback
			for (int32 SpatialIndex = 0; SpatialIndex < NumSpatialSamples; SpatialIndex++)
			{
				UTextureRenderTarget2D* RenderTarget = GraphRenderer->GetOrCreateViewRenderTarget(RenderTargetInitParams, RenderDataIdentifier);
				FRenderTarget* RenderTargetResource = RenderTarget->GameThread_GetRenderTargetResource();
				check(RenderTargetResource);

				// World should be paused for every spatial sample except the last one, so that
				// the view doesn't update histories until the end, allowing us to render the same
				// scene multiple times.
				const bool bIsLastTile = FIntPoint(TileX, TileY) == FIntPoint(TileCount.X - 1, TileCount.Y - 1);
				const bool bHasTiles = (TileCount.X * TileCount.Y) > 1;
				const bool bWorldIsPaused = !(SpatialIndex == (NumSpatialSamples - 1));
				const int32 FrameIndex = InTimeData.RenderedFrameNumber * ((InTimeData.TemporalSampleCount * NumSpatialSamples) + (InTimeData.TemporalSampleIndex * NumSpatialSamples)) + SpatialIndex;

				// We only allow a spatial jitter if we have more than one sample
				FVector2f SpatialShiftAmount = FVector2f(0.f, 0.f);
				const bool bAntiAliasingAllowsJitter = AntiAliasingMethod == EAntiAliasingMethod::AAM_None;
				const bool bSampleCountsAllowsJitter = NumSpatialSamples > 1 || InTimeData.TemporalSampleCount > 1;
				if (bAntiAliasingAllowsJitter && bSampleCountsAllowsJitter)
				{
					const int32 NumSamplesPerOutputFrame = NumSpatialSamples * InTimeData.TemporalSampleCount;
					SpatialShiftAmount = UE::MoviePipeline::GetSubPixelJitter(FrameIndex, NumSamplesPerOutputFrame);
				}

				// These are the parameters of our camera 
				UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = GetRenderCameraInfo();

				FIntPoint TileSize = ResolutionAndCameraInfo.TileSize;
				FIntPoint OverlappedPad = FIntPoint(FMath::CeilToInt(TileSize.X * TileOverlapFraction), FMath::CeilToInt(TileSize.Y * TileOverlapFraction));

				CameraInfo.bAllowCameraAspectRatio = true;
				CameraInfo.TilingParams.TileSize = TileSize;
				CameraInfo.TilingParams.OverlapPad = OverlappedPad; // ?
				CameraInfo.TilingParams.TileCount = TileCount;
				CameraInfo.TilingParams.TileIndexes = FIntPoint(TileX, TileY);
				CameraInfo.SamplingParams.TemporalSampleIndex = InTimeData.TemporalSampleIndex;
				CameraInfo.SamplingParams.TemporalSampleCount = InTimeData.TemporalSampleCount;
				CameraInfo.SamplingParams.SpatialSampleIndex = SpatialIndex;
				CameraInfo.SamplingParams.SpatialSampleCount = NumSpatialSamples;
				CameraInfo.SamplingParams.SeedOffset = SeedOffset;
				FVector2D DefaultJitter = FVector2D(
					SpatialShiftAmount.X * 2.0f / static_cast<float>(BackbufferResolution.X),
					SpatialShiftAmount.Y * -2.0f / static_cast<float>(BackbufferResolution.Y));
				CameraInfo.ProjectionMatrixJitterAmount = ComputeProjectionMatrixJitter(DefaultJitter, BackbufferResolution);
				CameraInfo.bUseCameraManagerPostProcess = !ResolutionAndCameraInfo.bRenderAllCameras;

				// Make sure our View Info is updated with the correct values from global camera overscan, this needs to happen
				// before we start tinkering with the matrices for tiling.
				const FIntPoint OutputResolution = UMovieGraphBlueprintLibrary::GetDesiredOutputResolution(InTimeData.EvaluatedConfig.Get(), ResolutionAndCameraInfo.CameraAspectRatio);
				ReapplyOverscanPreservingEngineScaling(CameraInfo, ResolutionAndCameraInfo.OverscanFraction, ResolutionAndCameraInfo.AccumulatorResolution, OutputResolution);

				// For this particular tile, what is the offset into the output image
				FIntPoint OverlappedOffset = FIntPoint(CameraInfo.TilingParams.TileIndexes.X * TileSize.X - OverlappedPad.X, CameraInfo.TilingParams.TileIndexes.Y * TileSize.Y - OverlappedPad.Y);

				// Move the final render by this much in the accumulator to counteract the offset put into the view matrix.
				// Note that when bAllowSpatialJitter is false, SpatialShiftX/Y will always be zero.
				FVector2D OverlappedSubpixelShift = FVector2D(0.5f - SpatialShiftAmount.X, 0.5f - SpatialShiftAmount.Y);

				// The Scene View Family must be constructed first as the FSceneView needs it to be constructed
				UE::MovieGraph::Rendering::FViewFamilyInitData ViewFamilyInitData;
				ViewFamilyInitData.RenderTarget = RenderTargetResource;
				ViewFamilyInitData.World = GraphRenderer->GetWorld();
				ViewFamilyInitData.TimeData = InTimeData;
				ViewFamilyInitData.SceneCaptureSource = SceneCaptureSource;
				ViewFamilyInitData.bWorldIsPaused = bWorldIsPaused;
				ViewFamilyInitData.FrameIndex = FrameIndex;
				ViewFamilyInitData.AntiAliasingMethod = AntiAliasingMethod;
				ViewFamilyInitData.ShowFlags = ParentNodeThisFrame->GetShowFlags();
				ViewFamilyInitData.ViewModeIndex = ParentNodeThisFrame->GetViewModeIndex();
				ViewFamilyInitData.ProjectionMode = CameraInfo.ViewInfo.ProjectionMode;
				ViewFamilyInitData.bIsCameraCut = InTimeData.bIsCameraCut;

				FCreateViewResult ViewResult = CreateConfiguredView(CameraInfo, ViewFamilyInitData, BackbufferResolution, AccumulatorResolution, ParentNodeThisFrame, TileX, TileY);
				TSharedRef<FSceneViewFamilyContext> ViewFamily = ViewResult.ViewFamily;
				FSceneView* NewView = ViewResult.View;

				FHitProxyConsumer* HitProxyConsumer = nullptr;
				const float DPIScale = 1.0f;
				FCanvas Canvas = FCanvas(RenderTargetResource, HitProxyConsumer, GraphRenderer->GetWorld(), GraphRenderer->GetWorld()->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, DPIScale);
		
				// Construct the sample state that reflects the current render sample
				UE::MovieGraph::FMovieGraphSampleState SampleState;
				{
					// Take our per-frame Traversal Context and update it with context specific to this sample.
					FMovieGraphTraversalContext UpdatedTraversalContext = InFrameTraversalContext;
					UpdatedTraversalContext.Time = InTimeData;
					UpdatedTraversalContext.Time.SpatialSampleIndex = SpatialIndex;
					UpdatedTraversalContext.Time.SpatialSampleCount = NumSpatialSamples;
					UpdatedTraversalContext.RenderDataIdentifier = RenderDataIdentifier;

					SampleState.TraversalContext = MoveTemp(UpdatedTraversalContext);
					SampleState.OverscannedResolution = AccumulatorResolution; // not sure this is correct
					SampleState.UnpaddedTileSize = TileSize; // not sure this is correct
					SampleState.BackbufferResolution = BackbufferResolution;
					SampleState.AccumulatorResolution = AccumulatorResolution;
					SampleState.bWriteSampleToDisk = bWriteAllSamples;
					SampleState.bRequiresAccumulator = InTimeData.bRequiresAccumulator || (NumSpatialSamples > 1) || bHasTiles;
					SampleState.bFetchFromAccumulator = InTimeData.bIsLastTemporalSampleForFrame && (SpatialIndex == (NumSpatialSamples - 1)) && bIsLastTile;
					SampleState.OverlappedPad = OverlappedPad;
					SampleState.OverlappedOffset = OverlappedOffset;
					SampleState.OverlappedSubpixelShift = OverlappedSubpixelShift;
					SampleState.OverscanFraction = CameraInfo.ViewInfo.GetOverscan();
					SampleState.CropRectangle = ResolutionAndCameraInfo.AccumulatorResolutionCropRect;
					SampleState.bAllowOCIO = ParentNodeThisFrame->GetAllowOCIO();
					SampleState.bForceLosslessCompression = ParentNodeThisFrame->GetForceLosslessCompression();
					SampleState.bAllowsCompositing = ParentNodeThisFrame->GetAllowsCompositing();
					SampleState.bIsBeautyPass = true;
					SampleState.SceneCaptureSource = SceneCaptureSource;
					SampleState.CompositingSortOrder = 10;
					SampleState.RenderLayerIndex = LayerData.LayerIndex;
				}

				ApplyMovieGraphOverridesToSampleState(SampleState);

				WritePerViewMetadata(SampleState.AdditionalFileMetadata, *NewView, ViewResult.SceneViewInitOptions, CameraInfo, RenderDataIdentifier);

				// If this was just to contribute to the history buffer, no need to go any further. Never discard if we're writing individual samples, though.
				bool bRenderPass = (!InTimeData.bDiscardOutput && !ShouldDiscardOutput(ViewFamily, CameraInfo)) || SampleState.bWriteSampleToDisk;

				if (UMovieGraphImagePassBaseNode* ParentNode = GetParentNode(InFrameTraversalContext.Time.EvaluatedConfig))
				{
					TSet<UMaterialInterface*> HighPrecisionMaterials;

					for (const FMoviePipelinePostProcessPass& PostProcessPass : ParentNode->GetAdditionalPostProcessMaterials())
					{
						UMaterialInterface* Material = PostProcessPass.Material.LoadSynchronous();
				
						if (!PostProcessPass.bEnabled || !Material)
						{
							continue;
						}

						// If we're not going to keep the output for the main pass then we don't want to create forwarding endpoints,
						// as they'll read back data for the discarded main pass results and then try to pass them on.
						if (!bRenderPass)
						{
							continue;
						}
				
						auto BufferPipe = MakeShared<FImagePixelPipe, ESPMode::ThreadSafe>();
				
						NewView->FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Add(Material);
				
						if (PostProcessPass.bHighPrecisionOutput)
						{
							HighPrecisionMaterials.Add(Material);
							BufferPipe->bIsExpecting32BitPixelData = true;
						}
				
						FMovieGraphRenderDataIdentifier Identifier = RenderDataIdentifier;
						Identifier.SubResourceName = PostProcessPass.Name.IsEmpty() ? Material->GetName() : PostProcessPass.Name;
				
						FMovieGraphSampleState PassSampleState = SampleState;
						PassSampleState.TraversalContext.RenderDataIdentifier = Identifier;

						// Additional Post Process materials should not have things composited onto them (like burn-ins)
						// nor should they have OCIO applied (as they're going to be data buffers like depth).
						PassSampleState.bAllowsCompositing = false;
						PassSampleState.bAllowOCIO = false;

						// Some Post Process materials may need lossless compression in order to preserve the data properly.
						PassSampleState.bForceLosslessCompression = PostProcessPass.bUseLosslessCompression;

						// PPM passes are never the beauty pass.
						PassSampleState.bIsBeautyPass = false;

						// Post process materials that don't explicitly output alpha will inherit the render
						// target's clear color (alpha=0), producing a fully transparent image. Detect this
						// case so the accumulator/output path can force alpha to 1.0.
						UMaterial* BaseMaterial = Material->GetMaterial();
						PassSampleState.bForceOpaqueAlpha = !(BaseMaterial && BaseMaterial->IsPostProcessMaterialOutputingAlpha());

						// PPM passes may override the file name format that they use.
						PassSampleState.FilenameFormatOverride = ParentNode->GetPPMFileNameFormatString();

						// Give a lower priority to materials so they show up after the main pass in multi-layer exrs.
						PassSampleState.CompositingSortOrder = SampleState.CompositingSortOrder + 1;
						BufferPipe->AddEndpoint(MakeForwardingEndpoint(PassSampleState, InTimeData));

						NewView->FinalPostProcessSettings.BufferVisualizationPipes.Add(Material->GetFName(), BufferPipe);
					}
				}

				int32 NumValidMaterials = NewView->FinalPostProcessSettings.BufferVisualizationPipes.Num();
				NewView->FinalPostProcessSettings.bBufferVisualizationDumpRequired = NumValidMaterials > 0;
				NewView->FinalPostProcessSettings.bOverride_PathTracingEnableDenoiser = true;

				// The denoiser is disabled during warm-up frames.
				NewView->FinalPostProcessSettings.PathTracingEnableDenoiser = bIsRenderingState && ParentNodeThisFrame->GetAllowDenoiser();

				// When buffer visualization PPMs are active, register a view extension that forces depth
				// buffer decompression before post-processing. This prevents AMD HTile corruption when
				// PPMs read SceneDepth as SRV after intermediate passes that normally trigger decompression
				// (DOF, MotionBlur, TAA) have been skipped.
				if (NumValidMaterials > 0)
				{
					ViewFamily->ViewExtensions.Add(MakeShareable(new FMovieGraphDepthDecompressExtension()));
				}

				// Submit the renderer to be rendered
				GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.ToSharedPtr().Get());

				ENQUEUE_RENDER_COMMAND(TransitionTextureSRVState)(
					[RenderTargetResource](FRHICommandListImmediate& RHICmdList) mutable
					{
						// Transition our render target from a render target view to a shader resource view to allow the UMG preview material to read from this Render Target.
						RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVGraphicsPixel));
					});

				// After submission, if we're paging to system memory, mark the resources for download into system memory.
				if (ParentNodeThisFrame->GetEnableHistoryPerTile() && ParentNodeThisFrame->GetEnablePageToSystemMemory())
				{
					if (FSceneViewStateInterface* ViewState = GetSceneViewState(ParentNodeThisFrame, TileX, TileY))
					{
						ViewState->SystemMemoryMirrorBackup(SystemMemoryMirror.Get());
					}
				}

				// Data may not be something we actually want to read back and write to disk after rendering.
				const bool bDiscardOutput = !bRenderPass || !bWriteBeautyPassToDisk;
				if (bDiscardOutput)
				{
					continue;
				}

				// Example Assumptions: 2 frame denoise temporal denoise with 8 temporal sub-samples. 
				// If you're using Cooldown Frames, we can get into a scenario where due to the particular render pass settings,
				// you don't need all the cooldown frames. If you're using Path Tracer's "Use Reference Motion Blur" then the 
				// above Discard is true for everything but the last sample. This means that we needed 8 Cool Down _samples_ to 
				// produce the two output frames (matching the 2 frame delay in the PT denoiser).
				// But if Use Reference Motion Blur is off, then the first two samples of the cool-down are enough to finish flushing
				// the PT denoiser, and the remaining 14 end up confusing the system because it gets data it doesn't think it should.
				if(InFrameTraversalContext.Shot->ShotInfo.State == EMovieRenderShotState::CoolingDown)
				{
					// When we're cooling down, we track the number of times we actually
					// go to do a readback (ie: pass the above bShouldDiscardOutput check)
					// and once we reach the number needed to actually flush the PT denoiser
					// we stop submitting.
					if (RemainingCooldownReadbackFrames == 0)
					{
						continue;
					}
					RemainingCooldownReadbackFrames--;
				}
		
				// Post-submission is a little bit complicated to allow supporting temporal denoisers in the Path Tracer.
				// When using the denoiser with a sample frame count of 2, for a frame to be produced it must look 2 frames
				// backwards, and 2 frames forwards, ie: to denoise Frame 5, we need to have rendered 3, 4, 5, 6, and 7.
				// The complication for this is that when we request a render and then immediately schedule a readback, when the
				// readback is completed the image will be the denoised result from a previous frame. ie: If on frame 5 you schedule
				// the readback, the result that will be copied to the CPU is the data from Frame 3.
				//
				// To resolve these issues, we capture the PostRendererSubmission and place it in a FIFO queue, and then when we schedule
				// a readback, we provide the old captured state, ie: on Frame 5 we provide Frame 3's data, and that will line up with the
				// image data actually generated on Frame 3 (which is what is returned by the GPU). A slight complication to this is that
				// PostRendererSubmission can no longer depend on any member state (since that would be using old data in combination with new),
				// but the one exception to this is that the current FCanvas is provided since it's only a wrapper for drawing letterboxing anyways.
				//
				// Under non-path-tracer temporal denoiser cases, this should effectively work out to be a no-op, the queue will dequeue immediately.

				FMovieGraphPostRendererSubmissionParams Params;
				Params.SampleState = SampleState;
				Params.RenderTargetInitParams = RenderTargetInitParams;
				Params.CameraInfo = CameraInfo;

				// Push the current frame into our FIFO queue
				SubmissionQueue.Enqueue(Params);

				// When we first start rendering we don't want to schedule a readback (as there isn't actually finished data to read back)
				// so we skip the first few frames. When we get to the end of a shot, we'll be in a cool-down period where we render extra
				// frames to allow finishing the denoising on the previous "real" frames. Those frames can't have discard output set on them,
				// otherwise we won't actually read back the end of the "real" frames. This means the queue will be left with some extra data
				// in it (for the cool-down frames which were calculated and submitted but never themselves get read back) but that's okay.
				if (FramesToDelayPostSubmission == 0)
				{
					// Now we schedule a readback using the oldest data.
					FMovieGraphPostRendererSubmissionParams PostParamsToUse;
					if (SubmissionQueue.Dequeue(PostParamsToUse))
					{
						// It's okay that we use the current FCanvas here as it's just a vessel to draw letterboxing based on state captured by the params.
						PostRendererSubmission(PostParamsToUse.SampleState, PostParamsToUse.RenderTargetInitParams, Canvas, PostParamsToUse.CameraInfo);
					}
					else
					{
						UE_LOGF(LogMovieRenderPipeline, Error, "De-queue post-submission parameters failed. Attempted to send a frame to post-render submission, but no frames were available in the FIFO queue.");
					}
				}
				else
				{
					FramesToDelayPostSubmission--;
				}
			}
		}
	}
}

void FMovieGraphDeferredPass::PostRendererSubmission(
	const UE::MovieGraph::FMovieGraphSampleState& InSampleState,
	const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams, FCanvas& InCanvas, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return;
	}

	// Draw letterboxing
	// ToDo: Multi-camera support
	if(InCameraInfo.ViewInfo.bConstrainAspectRatio)
	{
		const FMinimalViewInfo CameraCache = InCameraInfo.ViewInfo;
		
		// Taking overscan into account.
        const FIntPoint FullOutputSize = InSampleState.AccumulatorResolution;
	
		const float OutputSizeAspectRatio = FullOutputSize.X / (float)FullOutputSize.Y;
		const float CameraAspectRatio = InCameraInfo.bAllowCameraAspectRatio ? CameraCache.AspectRatio : OutputSizeAspectRatio;
	
		// Note: We use RoundToInt to match the unscaled/constrained view rectangle calculated in FViewport::CalculateViewExtents
		const FIntPoint ConstrainedFullSize = CameraAspectRatio > OutputSizeAspectRatio ?
			FIntPoint(FullOutputSize.X, FMath::RoundToInt((double)FullOutputSize.X / (double)CameraAspectRatio)) :
			FIntPoint(FMath::RoundToInt(CameraAspectRatio * FullOutputSize.Y), FullOutputSize.Y);
	
		const FIntPoint TileViewMin = InSampleState.OverlappedOffset;
		const FIntPoint TileViewMax = TileViewMin + InSampleState.BackbufferResolution;
	
		// Camera ratio constrained rect, clipped by the tile rect
        FIntPoint ConstrainedViewMin = (FullOutputSize - ConstrainedFullSize) / 2;
		FIntPoint ConstrainedViewMax = ConstrainedViewMin + ConstrainedFullSize;
		ConstrainedViewMin = FIntPoint(FMath::Clamp(ConstrainedViewMin.X, TileViewMin.X, TileViewMax.X),
			FMath::Clamp(ConstrainedViewMin.Y, TileViewMin.Y, TileViewMax.Y));
		ConstrainedViewMax = FIntPoint(FMath::Clamp(ConstrainedViewMax.X, TileViewMin.X, TileViewMax.X),
			FMath::Clamp(ConstrainedViewMax.Y, TileViewMin.Y, TileViewMax.Y));
	
		// Difference between the clipped constrained rect and the tile rect
		const FIntPoint OffsetMin = ConstrainedViewMin - TileViewMin;
		const FIntPoint OffsetMax = TileViewMax - ConstrainedViewMax;
	
		// Clear left
		if (OffsetMin.X > 0)
		{
			InCanvas.DrawTile(0, 0, OffsetMin.X, InSampleState.BackbufferResolution.Y,
				0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::Black, nullptr, false);
		}
		// Clear right
		if (OffsetMax.X > 0)
		{
			InCanvas.DrawTile(InSampleState.BackbufferResolution.X - OffsetMax.X, 0, InSampleState.BackbufferResolution.X, InSampleState.BackbufferResolution.Y,
				0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::Black, nullptr, false);
		}
		// Clear top
		if (OffsetMin.Y > 0)
		{
			InCanvas.DrawTile(0, 0, InSampleState.BackbufferResolution.X, OffsetMin.Y,
				0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::Black, nullptr, false);
		}
		// Clear bottom
		if (OffsetMax.Y > 0)
		{
			InCanvas.DrawTile(0, InSampleState.BackbufferResolution.Y - OffsetMax.Y, InSampleState.BackbufferResolution.X, InSampleState.BackbufferResolution.Y,
				0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::Black, nullptr, false);
		}
	
		InCanvas.Flush_GameThread(true);
	}
	
	FMovieGraphImagePassBase::PostRendererSubmission(InSampleState, InRenderTargetInitParams, InCanvas, InCameraInfo);
}

FVector2D FMovieGraphDeferredPass::ComputeProjectionMatrixJitter(const FVector2D& InDefaultJitter, const FIntPoint& InBackbufferResolution) const
{
	return InDefaultJitter;
}

} // namespace UE::MovieGraph::Rendering
