// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Renderers/MovieGraphImagePassBase.h"

#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphApplyViewportLookNode.h"
#include "Graph/Nodes/MovieGraphImagePassBaseNode.h"
#include "SceneView.h"
#include "MoviePipelineUtils.h"
#include "LegacyScreenPercentageDriver.h"
#include "MoviePipelineSurfaceReader.h"
#include "CanvasTypes.h"
#include "MovieRenderOverlappedImage.h"
#include "Engine/TextureRenderTarget.h"
#include "Engine/RendererSettings.h"
#include "UnrealClient.h"
#include "SceneViewExtensionContext.h"
#include "SceneViewExtension.h"
#include "Engine/Engine.h"
#include "ImageUtils.h"

#if WITH_EDITOR
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#endif


namespace UE::MovieGraph::Rendering
{

void FMovieGraphImagePassBase::Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer)
{
	// This is a pointer to the UMovieGraphPipeline's renderer which is valid throughout the entire render.
	WeakGraphRenderer = InRenderer;

	// Fetch the camera source from InLayer.RenderPassNode.
	if (ensureMsgf(InLayer.RenderPassNode.IsValid(), TEXT("FMovieGraphImagePassBase::Setup called with an invalid RenderPassNode.")))
	{
		RenderCameraSource = InLayer.RenderPassNode.Get()->GetRenderCameraSource();
	}
}

void FMovieGraphImagePassBase::Teardown()
{
	// Release our shared reference to the camera source. The source stays alive until every pass
	// instance for this node has torn down and dropped its reference.
	RenderCameraSource.Reset();
}

void FMovieGraphImagePassBase::UpdateRenderPassTelemetry(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const
{
}

void FMovieGraphImagePassBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (RenderCameraSource.IsValid())
	{
		RenderCameraSource->AddReferencedObjects(Collector);
	}
}

UE::MovieGraph::DefaultRenderer::FCameraInfo FMovieGraphImagePassBase::GetRenderCameraInfo() const
{
	const TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return UE::MovieGraph::DefaultRenderer::FCameraInfo();
	}

	return GraphRenderer->GetCameraInfo(GetRenderCameraIndex(), GetRenderCameraSource().Get());
}

float FMovieGraphImagePassBase::GetRenderCameraOverscan() const
{
	const TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return 0.f;
	}

	return GraphRenderer->GetCameraOverscan(GetRenderCameraIndex(), GetRenderCameraSource().Get());
}

FSceneViewInitOptions FMovieGraphImagePassBase::CreateViewInitOptions(const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo, FSceneViewFamilyContext* InViewFamily, FSceneViewStateReference& InViewStateRef) const
{
	return CreateViewInitOptions(InCameraInfo, InViewFamily, InViewStateRef.GetReference());
}

FSceneViewInitOptions FMovieGraphImagePassBase::CreateViewInitOptions(const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo, FSceneViewFamilyContext* InViewFamily, FSceneViewStateInterface* InViewStateInterface) const
{
	check(InViewFamily);

	FIntPoint RenderResolution = InViewFamily->RenderTarget->GetSizeXY();

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = InViewFamily;
	ViewInitOptions.ViewOrigin = InCameraInfo.ViewInfo.Location;
	ViewInitOptions.ViewLocation = InCameraInfo.ViewInfo.Location;
	ViewInitOptions.ViewRotation = InCameraInfo.ViewInfo.Rotation;
	ViewInitOptions.SetViewRectangle(FIntRect(FIntPoint(0, 0), RenderResolution));
	ViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(InCameraInfo.ViewInfo.Rotation);
	ViewInitOptions.ViewActor = InCameraInfo.ViewActor;
	
	// Rotate the view 90 degrees to match the rest of the engine.
	ViewInitOptions.ViewRotationMatrix = ViewInitOptions.ViewRotationMatrix * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	ViewInitOptions.SceneViewStateInterface = InViewStateInterface;
	ViewInitOptions.FOV = InCameraInfo.ViewInfo.FOV;
	ViewInitOptions.DesiredFOV = InCameraInfo.ViewInfo.FOV;

	// Propagate the overscan resolution fraction.
	ViewInitOptions.OverscanResolutionFraction = InCameraInfo.ViewInfo.OverscanResolutionFraction;

	return ViewInitOptions;
}

void FMovieGraphImagePassBase::ReapplyOverscanPreservingEngineScaling(
	UE::MovieGraph::DefaultRenderer::FCameraInfo& InOutCameraInfo,
	float InOverscanMagnitude,
	const FIntPoint& InAccumulatorResolution,
	const FIntPoint& InOutputResolution)
{
	// MRG is offline rendering, so we always quality-preserve via engine-side view-rect scaling — except when
	// MRG has already enlarged the accumulator (legacy "render bigger, crop back" or a custom source), where
	// scaling on top would over-render. UCameraComponent::bScaleResolutionWithOverscan is intentionally not
	// honored: it's a real-time GPU-saving knob and shouldn't affect offline output quality.
	const bool bAccumulatorIsEnlarged = InAccumulatorResolution.X > InOutputResolution.X
	                                 || InAccumulatorResolution.Y > InOutputResolution.Y;
	const float OverscanResolutionFraction = bAccumulatorIsEnlarged ? 1.0f : 1.0f + InOverscanMagnitude;

	InOutCameraInfo.ViewInfo.ClearOverscan();
	InOutCameraInfo.ViewInfo.ApplyOverscan(InOverscanMagnitude);

	InOutCameraInfo.ViewInfo.OverscanResolutionFraction = OverscanResolutionFraction;
}

FSceneView* FMovieGraphImagePassBase::CreateSceneView(const FSceneViewInitOptions& InInitOptions, TSharedRef<FSceneViewFamilyContext> InViewFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	// Create our view based on the Init Options
	FSceneView* View = new FSceneView(InInitOptions);
	InViewFamily->Views.Add(View);

	View->StartFinalPostprocessSettings(InInitOptions.ViewLocation);
	ApplyCameraManagerPostProcessBlends(View, InCameraInfo.ViewInfo, InCameraInfo.bUseCameraManagerPostProcess);

	// Scaling sensor size inversely with the the projection matrix [0][0] should physically
	// cause the circle of confusion to be unchanged.
	View->FinalPostProcessSettings.DepthOfFieldSensorWidth *= InCameraInfo.DoFSensorScale;
	// Modify the 'center' of the lens to be offset for high-res tiling, helps some effects (vignette) etc. still work.
	View->LensPrincipalPointOffsetScale = CalculatePrinciplePointOffsetForTiling(InCameraInfo.TilingParams);
	View->EndFinalPostprocessSettings(InInitOptions);

	UE::MovieRenderPipeline::UpdateSceneViewForShowFlags(View);

	for (int ViewExt = 0; ViewExt < InViewFamily->ViewExtensions.Num(); ViewExt++)
	{
		InViewFamily->ViewExtensions[ViewExt]->SetupView(*InViewFamily, *View);
	}

	return View;
}

DefaultRenderer::FRenderTargetInitParams FMovieGraphImagePassBase::GetRenderTargetInitParams(const FMovieGraphTimeStepData& InTimeData, const FIntPoint& InResolution)
{
	DefaultRenderer::FRenderTargetInitParams InitParams;

	InitParams.Size = InResolution;

	// OCIO: Since this is a manually created Render target we don't need Gamma to be applied.
	// We use this render target to render to via a display extension that utilizes Display Gamma
	// which has a default value of 2.2 (DefaultDisplayGamma), therefore we need to set Gamma on this render target to 2.2 to cancel out any unwanted effects.
	InitParams.TargetGamma = UTextureRenderTarget::GetDefaultDisplayGamma();
	InitParams.PixelFormat = PF_FloatRGBA;

	return InitParams;
}

void FMovieGraphImagePassBase::ApplyCameraManagerPostProcessBlends(FSceneView* InView, const FMinimalViewInfo& InViewInfo, bool bUseCameraManagerPostProcess) const
{
	check(InView);

	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return;
	}

	if (bUseCameraManagerPostProcess)
	{
		APlayerController* LocalPlayerController = GraphRenderer->GetWorld()->GetFirstPlayerController();
		// CameraAnim override
		if (LocalPlayerController->PlayerCameraManager)
		{
			TArray<FPostProcessSettings> const* CameraAnimPPSettings;
			TArray<float> const* CameraAnimPPBlendWeights;
			LocalPlayerController->PlayerCameraManager->GetCachedPostProcessBlends(CameraAnimPPSettings, CameraAnimPPBlendWeights);

			if (LocalPlayerController->PlayerCameraManager->bEnableFading)
			{
				InView->OverlayColor = LocalPlayerController->PlayerCameraManager->FadeColor;
				InView->OverlayColor.A = FMath::Clamp(LocalPlayerController->PlayerCameraManager->FadeAmount, 0.f, 1.f);
			}

			if (LocalPlayerController->PlayerCameraManager->bEnableColorScaling)
			{
				FVector ColorScale = LocalPlayerController->PlayerCameraManager->ColorScale;
				InView->ColorScale = FLinearColor(ColorScale.X, ColorScale.Y, ColorScale.Z);
			}

			FMinimalViewInfo ViewInfo = LocalPlayerController->PlayerCameraManager->GetCameraCacheView();
			for (int32 PPIdx = 0; PPIdx < CameraAnimPPBlendWeights->Num(); ++PPIdx)
			{
				InView->OverridePostProcessSettings((*CameraAnimPPSettings)[PPIdx], (*CameraAnimPPBlendWeights)[PPIdx]);
			}

			InView->OverridePostProcessSettings(ViewInfo.PostProcessSettings, ViewInfo.PostProcessBlendWeight);
		}
	}
	else
	{
		UE::MoviePipeline::DoPostProcessBlend(InViewInfo.Location, GraphRenderer->GetWorld(), InViewInfo, InView);
	}
}

TSharedRef<FSceneViewFamilyContext> FMovieGraphImagePassBase::CreateSceneViewFamily(const FViewFamilyInitData& InInitData) const
{
	EViewModeIndex ViewModeIndex = InInitData.ViewModeIndex;
	FEngineShowFlags ShowFlags = InInitData.ShowFlags;

	// Viewport rendering mode may need to customize the show flags, view mode index, and exposure
	bool bIsViewportLook = false;
	TOptional<FExposureSettings> ExposureSettings;
#if WITH_EDITOR
	if (InInitData.TimeData.EvaluatedConfig)
	{
		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = true;
		if (const UMovieGraphApplyViewportLookNode* ViewportLookNode = InInitData.TimeData.EvaluatedConfig->GetSettingForBranch<UMovieGraphApplyViewportLookNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch))
		{
			EViewModeIndex ViewportViewModeIndex;
			FEngineShowFlags ViewportShowFlags(ESFIM_Editor);
			bIsViewportLook = true;
			
			if (ViewportLookNode->GetViewportInfo(ViewportShowFlags, ViewportViewModeIndex))
			{
				if (ViewportLookNode->bOverride_bViewMode && ViewportLookNode->bViewMode)
				{
					ViewModeIndex = ViewportViewModeIndex;
				}

				if (ViewportLookNode->bOverride_bShowFlags && ViewportLookNode->bShowFlags)
				{
					ShowFlags = ViewportShowFlags;
				}
			}

			if (const FLevelEditorViewportClient* ViewportClient = UMovieGraphApplyViewportLookNode::GetViewportClient())
			{
				ExposureSettings = ViewportClient->ExposureSettings;
			}
		}
	}
#endif	// WITH_EDITOR

	const bool bIsPerspective = InInitData.ProjectionMode == ECameraProjectionMode::Type::Perspective;

	// Allow the Engine Showflag system to override our engine showflags, based on our view mode index.
	// This is required for certain debug view modes (to have matching show flags set for rendering).
	ApplyViewMode(/*In*/ ViewModeIndex, bIsPerspective, /*InOut*/ShowFlags);

	// And then we have to let another system override them again (based on cvars, etc.)
	EngineShowFlagOverride(bIsViewportLook ? ESFIM_Editor : ESFIM_Game, ViewModeIndex, ShowFlags, false);

	TSharedRef<FSceneViewFamilyContext> OutViewFamily = MakeShared<FSceneViewFamilyContext>(FSceneViewFamily::ConstructionValues(
		InInitData.RenderTarget,
		InInitData.World->Scene,
		ShowFlags)
		.SetTime(FGameTime::CreateUndilated(InInitData.TimeData.WorldSeconds, InInitData.TimeData.FrameDeltaTime))
		.SetRealtimeUpdate(true));

	if (ExposureSettings.IsSet())
	{
		OutViewFamily->ExposureSettings = ExposureSettings.GetValue();
	}

	// Need to add the engine-wide view extensions, as rendering code may depend on them (ie: landscapes)
	OutViewFamily->ViewExtensions.Append(GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(InInitData.World->Scene)));

	for (FSceneViewExtensionRef& ViewExt : OutViewFamily->ViewExtensions)
	{
		ViewExt->SetupViewFamily(*OutViewFamily);
	}

	// Support scene captures with the "bMainViewFamily" flag set
	OutViewFamily->bIsMainViewFamily = true;

	return OutViewFamily;
}

void FMovieGraphImagePassBase::ApplyMovieGraphOverridesToViewFamily(TSharedRef<FSceneViewFamilyContext> InOutFamily, const FViewFamilyInitData& InInitData) const
{
	// Used to specify if the Tone Curve is being applied or not to our Linear Output data
	InOutFamily->SceneCaptureSource = InInitData.SceneCaptureSource;
	InOutFamily->bWorldIsPaused = InInitData.bWorldIsPaused;
	InOutFamily->bOverrideVirtualTextureThrottle = true;
	
	// We need to check if this is the first FSceneView being submitted to the renderer module, and set some flags on the ViewFamily for ensuring some
	// parts of the renderer only get updated once per frame. Kept as an if/else statement to avoid the confusion with setting all of these values to 
	// some permutation of !/!!bHasRenderedFirstViewThisFrame.
	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (GraphRenderer)
	{
		if (!GraphRenderer->GetHasRenderedFirstViewThisFrame())
		{
			// Update our renderer
			GraphRenderer->SetHasRenderedFirstViewThisFrame(true);

			InOutFamily->bIsFirstViewInMultipleViewFamily = true;
			InOutFamily->bAdditionalViewFamily = false;
		}
		else
		{
			InOutFamily->bIsFirstViewInMultipleViewFamily = false;
			InOutFamily->bAdditionalViewFamily = true;
		}
	}
	
	// Skip the whole pass if they don't want motion blur.
	if (FMath::IsNearlyZero(InInitData.TimeData.MotionBlurFraction))
	{
		InOutFamily->EngineShowFlags.SetMotionBlur(false);
	}
	

	// ToDo: Let settings modify the ScreenPercentageInterface so third party screen percentages are supported.

	// If UMoviePipelineViewFamilySetting never set a Screen percentage interface we fallback to default.
	if (InOutFamily->GetScreenPercentageInterface() == nullptr)
	{
		InOutFamily->SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(*InOutFamily, FLegacyScreenPercentageDriver::GetCVarResolutionFraction()));
	}
}

void FMovieGraphImagePassBase::ApplyMovieGraphOverridesToSceneView(TSharedRef<FSceneViewFamilyContext> InOutFamily, const FViewFamilyInitData& InInitData, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	// Override the view's FrameIndex to be based on our progress through the sequence. This greatly increases
	// determinism with things like TAA.
	FSceneView* View = const_cast<FSceneView*>(InOutFamily->Views[0]);
	View->OverrideFrameIndexValue = InInitData.FrameIndex;
	View->OverrideOutputFrameIndexValue = InInitData.TimeData.OutputFrameNumber;

	// Each shot should initialize a scene history from scratch so there should be no need to do an extra camera cut flag.
	// However, layer warm-ups may need to perform a camera cut.
	View->bCameraCut = InInitData.bIsCameraCut; 
	View->AntiAliasingMethod = InInitData.AntiAliasingMethod;
	View->bIsOfflineRender = true;

	// Override the Motion Blur settings since these are controlled by the movie pipeline.
	{
		// We need to inversly scale the target FPS by time dilation to counteract slowmo. If scaling isn't applied then motion blur length
		// stays the same length despite the smaller delta time and the blur ends up too long.
		View->FinalPostProcessSettings.MotionBlurTargetFPS = FMath::RoundToInt(InInitData.TimeData.FrameRate.AsDecimal() / FMath::Max(SMALL_NUMBER, InInitData.TimeData.WorldTimeDilation));
		View->FinalPostProcessSettings.MotionBlurAmount = InInitData.TimeData.MotionBlurFraction;
		View->FinalPostProcessSettings.MotionBlurMax = 100.f;
		View->FinalPostProcessSettings.bOverride_MotionBlurAmount = true;
		View->FinalPostProcessSettings.bOverride_MotionBlurTargetFPS = true;
		View->FinalPostProcessSettings.bOverride_MotionBlurMax = true;

	}

	// Warn the user for invalid setting combinations / enforce hardware limitations
	// Locked Exposure
	const bool bAutoExposureAllowed = true; // IsAutoExposureAllowed(InOutSampleState);
	{
		// If the rendering pass doesn't allow autoexposure and they dont' have manual exposure set up, warn.
		if (!bAutoExposureAllowed && (View->FinalPostProcessSettings.AutoExposureMethod != EAutoExposureMethod::AEM_Manual))
		{
			// Skip warning if the project setting is disabled though, as exposure will be forced off in the renderer anyways.
			const URendererSettings* RenderSettings = GetDefault<URendererSettings>();
			if (RenderSettings->bDefaultFeatureAutoExposure != false)
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "Camera Auto Exposure Method not supported by one or more render passes. Change the Auto Exposure Method to Manual!");
				View->FinalPostProcessSettings.AutoExposureMethod = EAutoExposureMethod::AEM_Manual;
			}
		}
	}

	{
		bool bMethodWasUnsupported = false;
		if (View->AntiAliasingMethod == AAM_TemporalAA && !SupportsGen4TAA(View->GetShaderPlatform()))
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "TAA was requested but this hardware does not support it.");
			bMethodWasUnsupported = true;
		}
		else if (View->AntiAliasingMethod == AAM_TSR && !SupportsTSR(View->GetShaderPlatform()))
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "TSR was requested but this hardware does not support it.");
			bMethodWasUnsupported = true;
		}

		if (bMethodWasUnsupported)
		{
			View->AntiAliasingMethod = AAM_None;
		}
	}

	// Anti Aliasing
	{
		// If we're not using TAA, TSR, or Path Tracing we will apply the View Matrix projection jitter. Normally TAA sets this
		// inside FSceneRenderer::PreVisibilityFrameSetup. Path Tracing does its own anti-aliasing internally.
		bool bApplyProjectionJitter = 
			   !InOutFamily->EngineShowFlags.PathTracing
			&& !IsTemporalAccumulationBasedMethod(View->AntiAliasingMethod);
		if (bApplyProjectionJitter)
		{
			View->ViewMatrices.HackAddTemporalAAProjectionJitter(InCameraInfo.ProjectionMatrixJitterAmount);
		}
	}
}

void FMovieGraphImagePassBase::CalculateProjectionMatrix(UE::MovieGraph::DefaultRenderer::FCameraInfo& InOutCameraInfo, FSceneViewProjectionData& InOutProjectionData, const FIntPoint InBackbufferResolution, const FIntPoint InAccumulatorResolution) const
{
	// TileSize should respect the actual backbuffer size being used by the render.
	float ViewRectWidth = InBackbufferResolution.X;
	float ViewRectHeight = InBackbufferResolution.Y;
	FIntRect ViewRect = FIntRect(FIntPoint(0, 0), InBackbufferResolution);

	const float DestAspectRatio = ViewRectWidth / ViewRectHeight;
	const float CameraAspectRatio = InOutCameraInfo.bAllowCameraAspectRatio ? InOutCameraInfo.ViewInfo.AspectRatio : DestAspectRatio;
	
	const int TotalTileCount = InOutCameraInfo.TilingParams.TileCount.X * InOutCameraInfo.TilingParams.TileCount.Y;

	// If they're using high-resolution tiling we can't support letterboxing (as the blended areas we would render with
	// would have been cropped via letterboxing), so to handle this scenario we disable aspect ratio constraints and then
	// manually rescale the view (if needed) to mimick the effect of letterboxing.
	TEnumAsByte<EAspectRatioAxisConstraint> AspectRatioAxisConstraint = InOutCameraInfo.ViewInfo.AspectRatioAxisConstraint.Get(EAspectRatioAxisConstraint::AspectRatio_MaintainXFOV);
	if (TotalTileCount > 1 && InOutCameraInfo.ViewInfo.bConstrainAspectRatio)
	{
		if (CameraAspectRatio < DestAspectRatio)
		{
			AspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainYFOV;
			InOutCameraInfo.ViewInfo.OrthoWidth *= (DestAspectRatio / CameraAspectRatio);

			// Off-center camera projections are calculated based on constrained aspect ratios, but those are disabled
			// when using high-resolution tiling. This means that we need to scale the offset projection as well.
			// 
			// To calculate the required size change, we can look at an Aspect Ratio of 0.5 inside a square output, 
			// ie: the rendered area is 1000 x 2000 for an output that is 2000x2000 (this is 0.5 of 1.0). With an
			// off-center projection, an offset of 1.0 on X originally only moved by 500 pixels (1000x0.5), but with the aspect
			// ratio constraint disabled, it now applies to the full output image (2000x0.5) resulting in a move that is twice as big.
			// 
			// To resolve this, we scale the offset by the CameraAspectRatio / DestAspectRatio, which is 0.5 / 1.0 for this example,
			// meaning we multiply the user-intended offset (1.0) by 0.5, resulting in the originally desired 500px offset.
			const double Ratio = CameraAspectRatio / DestAspectRatio; // ex: Ratio = 0.5 / 1
			InOutCameraInfo.ViewInfo.OffCenterProjectionOffset.X *= Ratio;
		}
		else if (CameraAspectRatio > DestAspectRatio)
		{
			// Don't rescale the width and keep it X-constrained.
			AspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainXFOV;

			// Like above, off-center projections need to be rescaled too.
			const double Ratio = DestAspectRatio / CameraAspectRatio;
			InOutCameraInfo.ViewInfo.OffCenterProjectionOffset.Y *= Ratio;
		}
		InOutCameraInfo.ViewInfo.bConstrainAspectRatio = false;
	}

	FIntRect ViewExtents = FViewport::CalculateViewExtents(InOutCameraInfo.ViewInfo.AspectRatio, DestAspectRatio, ViewRect, InAccumulatorResolution);


	

	// This function updates data in both the FMinimalViewInfo and the ProjectionData
	FMinimalViewInfo::CalculateProjectionMatrixGivenViewRectangle(InOutCameraInfo.ViewInfo, AspectRatioAxisConstraint, ViewExtents, InOutProjectionData);

	// Give the camera source a chance to apply final adjustments to the computed projection data.
	if (const TSharedPtr<const FMovieGraphRenderCameraSource> CameraSource = GetRenderCameraSource())
	{
		CameraSource->SetupCameraViewProjectionData(GetRenderCameraIndex(), InOutProjectionData);
	}
}

FVector4f FMovieGraphImagePassBase::CalculatePrinciplePointOffsetForTiling(const UE::MovieGraph::DefaultRenderer::FMovieGraphTilingParams& InTilingParams) const 
{
	// We need our final view parameters to be in the space of [-1,1], including all the tiles.
	// Starting with a single tile, the middle of the tile in offset screen space is:
	FVector2f TilePrincipalPointOffset;

	TilePrincipalPointOffset.X = (float(InTilingParams.TileIndexes.X) + 0.5f - (0.5f * float(InTilingParams.TileCount.X))) * 2.0f;
	TilePrincipalPointOffset.Y = (float(InTilingParams.TileIndexes.Y) + 0.5f - (0.5f * float(InTilingParams.TileCount.Y))) * 2.0f;

	// For the tile size ratio, we have to multiply by (1.0 + overlap) and then divide by tile num
	FVector2D OverlapScale;
	OverlapScale.X = (1.0f + float(2 * InTilingParams.OverlapPad.X) / float(InTilingParams.TileSize.X));
	OverlapScale.Y = (1.0f + float(2 * InTilingParams.OverlapPad.Y) / float(InTilingParams.TileSize.Y));

	TilePrincipalPointOffset.X /= OverlapScale.X;
	TilePrincipalPointOffset.Y /= OverlapScale.Y;

	FVector2D TilePrincipalPointScale;
	TilePrincipalPointScale.X = OverlapScale.X / float(InTilingParams.TileCount.X);
	TilePrincipalPointScale.Y = OverlapScale.Y / float(InTilingParams.TileCount.Y);

	TilePrincipalPointOffset.X *= TilePrincipalPointScale.X;
	TilePrincipalPointOffset.Y *= TilePrincipalPointScale.Y;

	return FVector4f(TilePrincipalPointOffset.X, -TilePrincipalPointOffset.Y, TilePrincipalPointScale.X, TilePrincipalPointScale.Y);
}

void FMovieGraphImagePassBase::ModifyProjectionMatrixForTiling(const UE::MovieGraph::DefaultRenderer::FMovieGraphTilingParams& InTilingParams, const bool bInOrthographic, FMatrix& InOutProjectionMatrix, float& OutDoFSensorScale) const
{
	float PadRatioX = 1.0f;
	float PadRatioY = 1.0f;

	if (InTilingParams.OverlapPad.X > 0 && InTilingParams.OverlapPad.Y > 0)
	{
		PadRatioX = float(InTilingParams.OverlapPad.X * 2 + InTilingParams.TileSize.X) / float(InTilingParams.TileSize.X);
		PadRatioY = float(InTilingParams.OverlapPad.Y * 2 + InTilingParams.TileSize.Y) / float(InTilingParams.TileSize.Y);
	}

	float ScaleX = PadRatioX / float(InTilingParams.TileCount.X);
	float ScaleY = PadRatioY / float(InTilingParams.TileCount.Y);

	InOutProjectionMatrix.M[0][0] /= ScaleX;
	InOutProjectionMatrix.M[1][1] /= ScaleY;
	OutDoFSensorScale = ScaleX;

	// this offset would be correct with no pad
	float OffsetX = -((float(InTilingParams.TileIndexes.X) + 0.5f - float(InTilingParams.TileCount.X) / 2.0f) * 2.0f);
	float OffsetY = ((float(InTilingParams.TileIndexes.Y) + 0.5f - float(InTilingParams.TileCount.Y) / 2.0f) * 2.0f);

	if (bInOrthographic)
	{
		// Scale the off-center projection matrix too so that it's appropriately sized down for each tile.
		InOutProjectionMatrix.M[3][0] /= ScaleX;
		InOutProjectionMatrix.M[3][1] /= ScaleY;
		InOutProjectionMatrix.M[3][0] += OffsetX / PadRatioX;
		InOutProjectionMatrix.M[3][1] += OffsetY / PadRatioY;
	}
	else
	{
		// Scale the off-center projection matrix too so that it's appropriately sized down for each tile.
		InOutProjectionMatrix.M[2][0] /= ScaleX;
		InOutProjectionMatrix.M[2][1] /= ScaleY;
		// Then offset it for this particular tile.
		InOutProjectionMatrix.M[2][0] += OffsetX / PadRatioX;
		InOutProjectionMatrix.M[2][1] += OffsetY / PadRatioY;
	}
}


void FMovieGraphImagePassBase::PostRendererSubmission(
	const UE::MovieGraph::FMovieGraphSampleState& InSampleState,
	const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams, FCanvas& InCanvas, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	// We have a pool of accumulators - we multi-thread the accumulation on the task graph, and for each frame,
	// the task has the previous samples as pre-reqs to keep the accumulation in order. However, each accumulator
	// can only work on one frame at a time, so we create a pool of them to work concurrently. This needs a limit
	// as large accumulations (16k) can take a lot of system RAM.
	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return;
	}

	FMoviePipelineSurfaceQueuePtr LocalSurfaceQueue = GraphRenderer->GetOrCreateSurfaceQueue(InRenderTargetInitParams);
	LocalSurfaceQueue->BlockUntilAnyAvailable();

	TSharedRef<FMovieGraphRenderDataAccumulationArgs> AccumulationArgs =
		StaticCastSharedRef<FMovieGraphRenderDataAccumulationArgs>(GetOrCreateAccumulator(GraphRenderer, InSampleState));

	// Panoramic renders need sample state to be polymorphic, so we don't want to capture the parameter by value.
	TSharedRef<UE::MovieGraph::FMovieGraphSampleState> SampleStateCopy = InSampleState.Copy();

	auto OnSurfaceReadbackFinished = [this, SampleStateCopy, AccumulationArgs](TUniquePtr<FImagePixelData>&& InPixelData)
	{
		FAccumulatorSampleFunc AccumulateFunction = GetAccumulateSampleFunction();
		
		const UE::Tasks::TTask<void> Task = UE::Tasks::Launch(UE_SOURCE_LOCATION, [PixelData = MoveTemp(InPixelData), SampleStateCopy, AccumulationArgs, AccumulateFunction]() mutable
		{
			// Enqueue a encode for this frame onto our worker thread.
			AccumulateFunction(MoveTemp(PixelData), SampleStateCopy, AccumulationArgs);

			// We have to defer clearing the accumulator until after sample accumulation has finished
			if (SampleStateCopy->bFetchFromAccumulator)
			{
				// Final sample has now been executed, free the accumulator for reuse.
				AccumulationArgs->AccumulatorInstance->SetIsActive(false);
			}
		}, AccumulationArgs->AccumulatorInstance->TaskPrereq);

		// Make the next accumulation task that uses this accumulator use the task we just created as a pre-req.
		AccumulationArgs->AccumulatorInstance->TaskPrereq = Task;

		// Because we're run on a separate thread, we need to check validity differently. The standard
		// TWeakObjectPtr will report non-valid during GC (even if the object it's pointing to isn't being
		// GC'd).
		const bool bEvenIfPendingKill = false;
		const bool bThreadsafeTest = true;
		const bool bValid = this->WeakGraphRenderer.IsValid(bEvenIfPendingKill, bThreadsafeTest);
		if (ensureMsgf(bValid, TEXT("Renderer was garbage collected while outstanding tasks existed, outstanding tasks were not flushed properly during shutdown!")))
		{
			// The regular Get() will fail during GC so we use the above check to see if it's valid
			// before ignoring it and directly getting the object.
			this->WeakGraphRenderer.GetEvenIfUnreachable()->AddOutstandingRenderTask_AnyThread(Task);
		}
	};

	FRenderTarget* RenderTarget = InCanvas.GetRenderTarget();

	ENQUEUE_RENDER_COMMAND(CanvasRenderTargetResolveCommand)(
		[LocalSurfaceQueue, OnSurfaceReadbackFinished, RenderTarget](FRHICommandListImmediate& RHICmdList) mutable
		{
			// The legacy surface reader takes the payload just so it can shuffle it into our callback, but we can just include the data
			// directly in the callback, so this is just a dummy payload.
			TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> FramePayload = MakeShared<FImagePixelDataPayload, ESPMode::ThreadSafe>();
			LocalSurfaceQueue->OnRenderTargetReady_RenderThread(RenderTarget->GetRenderTargetTexture(), FramePayload, MoveTemp(OnSurfaceReadbackFinished));
		});
}

TFunction<void(TUniquePtr<FImagePixelData>&&)> FMovieGraphImagePassBase::MakeForwardingEndpoint(
	const FMovieGraphSampleState& InSampleState, const FMovieGraphTimeStepData& InTimeData)
{
	// We have a pool of accumulators - we multi-thread the accumulation on the task graph, and for each frame,
	// the task has the previous samples as pre-reqs to keep the accumulation in order. However, each accumulator
	// can only work on one frame at a time, so we create a pool of them to work concurrently. This needs a limit
	// as large accumulations (16k) can take a lot of system RAM.
	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return nullptr;
	}
		
	// TODO: Is this correct? The prior code is *slightly* different.
	TSharedRef<FMovieGraphRenderDataAccumulationArgs> AccumulationArgs =
		StaticCastSharedRef<FMovieGraphRenderDataAccumulationArgs>(GetOrCreateAccumulator(GraphRenderer, InSampleState));
	TSharedRef<FMovieGraphSampleState> SampleStateCopy = InSampleState.Copy();

	// The legacy surface reader takes the payload just so it can shuffle it into our callback, but we can just include the data
	// directly in the callback, so this is just a dummy payload.
	TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> FramePayload = MakeShared<FImagePixelDataPayload, ESPMode::ThreadSafe>();
	auto Callback = [this, SampleStateCopy, FramePayload, AccumulationArgs](TUniquePtr<FImagePixelData>&& InPixelData)
	{
		// Transfer the framePayload to the returned data
		TUniquePtr<FImagePixelData> PixelDataWithPayload = nullptr;
		switch (InPixelData->GetType())
		{
		case EImagePixelType::Color:
		{
			TImagePixelData<FColor>* SourceData = static_cast<TImagePixelData<FColor>*>(InPixelData.Get());
			PixelDataWithPayload = MakeUnique<TImagePixelData<FColor>>(InPixelData->GetSize(), MoveTemp(SourceData->Pixels), FramePayload);
			break;
		}
		case EImagePixelType::Float16:
		{
			TImagePixelData<FFloat16Color>* SourceData = static_cast<TImagePixelData<FFloat16Color>*>(InPixelData.Get());
			PixelDataWithPayload = MakeUnique<TImagePixelData<FFloat16Color>>(InPixelData->GetSize(), MoveTemp(SourceData->Pixels), FramePayload);
			break;
		}
		case EImagePixelType::Float32:
		{
			TImagePixelData<FLinearColor>* SourceData = static_cast<TImagePixelData<FLinearColor>*>(InPixelData.Get());
			PixelDataWithPayload = MakeUnique<TImagePixelData<FLinearColor>>(InPixelData->GetSize(), MoveTemp(SourceData->Pixels), FramePayload);
			break;
		}
		default:
			checkNoEntry();
		}

		FAccumulatorSampleFunc AccumulateFunction = GetAccumulateSampleFunction();

		UE::Tasks::TTask<void> Task = UE::Tasks::Launch(UE_SOURCE_LOCATION, [PixelData = MoveTemp(PixelDataWithPayload), SampleStateCopy, AccumulationArgs, AccumulateFunction]() mutable
		{
			// Enqueue a encode for this frame onto our worker thread.
			AccumulateFunction(MoveTemp(PixelData), SampleStateCopy, AccumulationArgs);

			// We have to defer clearing the accumulator until after sample accumulation has finished
			if (SampleStateCopy->bFetchFromAccumulator)
			{
				// Final sample has now been executed, free the accumulator for reuse.
				AccumulationArgs->AccumulatorInstance->SetIsActive(false);
			}
		}, AccumulationArgs->AccumulatorInstance->TaskPrereq);

		// Make the next accumulation task that uses this accumulator use the task we just created as a pre-req.
		AccumulationArgs->AccumulatorInstance->TaskPrereq = Task;

		// Because we're run on a separate thread, we need to check validity differently. The standard
		// TWeakObjectPtr will report non-valid during GC (even if the object it's pointing to isn't being
		// GC'd).
		constexpr bool bEvenIfPendingKill = false;
		constexpr bool bThreadsafeTest = true;
		const bool bValid = this->WeakGraphRenderer.IsValid(bEvenIfPendingKill, bThreadsafeTest);
		if (ensureMsgf(bValid, TEXT("Renderer was garbage collected while outstanding PPM tasks existed, outstanding tasks were not flushed properly during shutdown!")))
		{
			// The regular Get() will fail during GC so we use the above check to see if it's valid
			// before ignoring it and directly getting the object.
			this->WeakGraphRenderer.GetEvenIfUnreachable()->AddOutstandingRenderTask_AnyThread(Task);
		}
	};

	return Callback;
}

TSharedRef<::MoviePipeline::IMoviePipelineAccumulationArgs> FMovieGraphImagePassBase::GetOrCreateAccumulator(const TObjectPtr<UMovieGraphDefaultRenderer> InGraphRenderer, const FMovieGraphSampleState& InSampleState) const
{
	const FMoviePipelineAccumulatorPoolPtr SampleAccumulatorPool = InGraphRenderer->GetOrCreateAccumulatorPool<FImageOverlappedAccumulator>();
	const DefaultRenderer::FSurfaceAccumulatorPool::FInstancePtr AccumulatorInstance = SampleAccumulatorPool->GetAccumulatorInstance_GameThread<FImageOverlappedAccumulator>(InSampleState.TraversalContext.Time.OutputFrameNumber, InSampleState.TraversalContext.RenderDataIdentifier);

	TSharedRef<FMovieGraphRenderDataAccumulationArgs> AccumulationArgs = MakeShared<FMovieGraphRenderDataAccumulationArgs>();
	AccumulationArgs->OutputMerger = InGraphRenderer->GetOwningGraph()->GetOutputMerger();
	AccumulationArgs->ImageAccumulator = StaticCastSharedPtr<FImageOverlappedAccumulator>(AccumulatorInstance->Accumulator);
	AccumulationArgs->AccumulatorInstance = SampleAccumulatorPool->GetAccumulatorInstance_GameThread<FImageOverlappedAccumulator>(InSampleState.TraversalContext.Time.OutputFrameNumber, InSampleState.TraversalContext.RenderDataIdentifier);

	return AccumulationArgs;
}

FMovieGraphImagePassBase::FAccumulatorSampleFunc FMovieGraphImagePassBase::GetAccumulateSampleFunction() const
{
	return AccumulateSample_TaskThread;
}

void AccumulateSample_TaskThread(TUniquePtr<FImagePixelData>&& InPixelData, const TSharedRef<::UE::MovieGraph::FMovieGraphSampleState> InSampleState, const TSharedRef<::MoviePipeline::IMoviePipelineAccumulationArgs> InAccumulatorArgs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline_AccumulateSample);
		
	TUniquePtr<FImagePixelData> SamplePixelData = MoveTemp(InPixelData);

	// Associate the sample state with the image as payload data, this allows downstream systems to fetch the values without us having to store the data
	// separately and ensure they stay paired the whole way down.
	TSharedPtr<FMovieGraphSampleState> SampleStatePayload = InSampleState->Copy();
	SamplePixelData->SetPayload(StaticCastSharedPtr<IImagePixelDataPayload>(SampleStatePayload));

	const TSharedRef<FMovieGraphRenderDataAccumulationArgs, ESPMode::ThreadSafe> AccumulatorArgs = StaticCastSharedRef<FMovieGraphRenderDataAccumulationArgs>(InAccumulatorArgs);

	const TSharedPtr<IMovieGraphOutputMerger, ESPMode::ThreadSafe> OutputMergerPin = AccumulatorArgs->OutputMerger.Pin();
	if (!OutputMergerPin.IsValid())
	{
		return;
	}

	const bool bIsWellFormed = SamplePixelData->IsDataWellFormed();
	check(bIsWellFormed);

	if (SampleStatePayload->bWriteSampleToDisk)
	{
		// Debug Feature: Write the raw sample to disk for debugging purposes. We copy the data here,
		// as we don't want to disturb the memory flow below.
		TUniquePtr<FImagePixelData> SampleData = SamplePixelData->CopyImageData();
		OutputMergerPin->OnSingleSampleDataAvailable_AnyThread(MoveTemp(SampleData));
	}

	// Optimization! If we don't need the accumulator (no tiling, no sub-sampling) then we'll skip it and just send it straight to the output stage.
	// This reduces memory requirements and improves performance in the baseline case.
	if (!SampleStatePayload->bRequiresAccumulator)
	{
		// PPM passes that don't output alpha arrive with alpha=0 from the render target clear color.
		// Fix up alpha here since we can't intercept the GPU readback (it's handled by the engine's
		// buffer visualization system, not the movie pipeline's surface reader).
		if (SampleStatePayload->bForceOpaqueAlpha)
		{
			SamplePixelData->SetAlphaOpaque();
		}

		OutputMergerPin->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(SamplePixelData));
		return;
	}

	const TSharedPtr<FImageOverlappedAccumulator> AccumulatorPin = StaticCastWeakPtr<FImageOverlappedAccumulator>(AccumulatorArgs->ImageAccumulator).Pin();
	if (AccumulatorPin->NumChannels == 0)
	{
		LLM_SCOPE_BYNAME(TEXT("MoviePipeline/ImageAccumulatorInitMemory"));

		// Use 3 channels (RGB) when forcing opaque alpha. FetchFullImageValue() initializes
		// Rgba[3]=1.0 and only overwrites channels 0..NumChannels-1, so a 3-channel accumulator
		// produces alpha=1.0 for free during the existing fetch pass -- no extra pixel iteration.
		const int32 ChannelCount = SampleStatePayload->bForceOpaqueAlpha ? 3 : 4;

		AccumulatorPin->InitMemory(SampleStatePayload->AccumulatorResolution, ChannelCount);
		AccumulatorPin->ZeroPlanes();
		AccumulatorPin->AccumulationGamma = 1.f;
	}

	// Accumulate the new sample to our target
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline_AccumulatePixelData);

		// TODO: This needs to be re-updated to support high resolution tiling,
		// and it's mostly copy/pasted from MRQ and could be refactored.

		// Some samples can come back at a different size than expected (post process materials) which
		// creates numerous issues with the accumulators. To work around this issue for now, we will resize
		// the image to the expected resolution. 
		FIntPoint RawSize = SamplePixelData->GetSize();
		const bool bCorrectSize = RawSize == SampleStatePayload->BackbufferResolution;

		if (!bCorrectSize)
		{
			const double ResizeConvertBeginTime = FPlatformTime::Seconds();

			// Convert the incoming data to full floats (the accumulator would do this later normally anyways)
			TArray64<FLinearColor> FullSizeData;
			FullSizeData.AddUninitialized(RawSize.X * RawSize.Y);

			if (SamplePixelData->GetType() == EImagePixelType::Float32)
			{
				const void* RawDataPtr = nullptr;
				int64 RawDataSize;

				if (SamplePixelData->GetRawData(RawDataPtr, RawDataSize) == true)
				{
					FMemory::Memcpy(FullSizeData.GetData(), RawDataPtr, RawDataSize);
				}
				else
				{
					UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to retrieve raw data from image data for writing. Bailing.");
					return;
				}
			}
			else if (SamplePixelData->GetType() == EImagePixelType::Float16)
			{
				const void* RawDataPtr = nullptr;
				int64 RawDataSize;

				if (SamplePixelData->GetRawData(RawDataPtr, RawDataSize) == true)
				{
					const FFloat16Color* DataAsColor = reinterpret_cast<const FFloat16Color*>(RawDataPtr);
					for (int64 Index = 0; Index < RawSize.X * RawSize.Y; Index++)
					{
						FullSizeData[Index] = FLinearColor(DataAsColor[Index]);
					}
				}
				else
				{
					UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to retrieve raw data from image data for writing. Bailing.");
					return;
				}
				// TODO: Produce a warning when this happens as it causes stretched Additional PPMs and the user won't know why.
			}
			else
			{
				check(0);
			}
			const double ResizeConvertEndTime = FPlatformTime::Seconds();

			// Now we can resize to our target size.
			FIntPoint TargetSize = SampleStatePayload->BackbufferResolution; // TODO: High Resolution TIling

			TArray64<FLinearColor> NewPixelData;
			NewPixelData.SetNumUninitialized(TargetSize.X * TargetSize.Y);

			FImageUtils::ImageResize(RawSize.X, RawSize.Y, MakeArrayView<FLinearColor>(FullSizeData.GetData(), FullSizeData.Num()), TargetSize.X, TargetSize.Y, MakeArrayView<FLinearColor>(NewPixelData.GetData(), NewPixelData.Num()));

			const float ElapsedConvertMs = float((ResizeConvertEndTime - ResizeConvertBeginTime) * 1000.0f);
			const float ElapsedResizeMs = float((FPlatformTime::Seconds() - ResizeConvertEndTime) * 1000.0f);

			UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "Resize Convert Time: %8.2fms Resize Time: %8.2fms", ElapsedConvertMs, ElapsedResizeMs);

			SamplePixelData = MakeUnique<TImagePixelData<FLinearColor>>(FIntPoint(TargetSize.X, TargetSize.Y), MoveTemp(NewPixelData), SampleStatePayload);

			// Update the raw size to match our new size.
			RawSize = SamplePixelData->GetSize();

			// TODO: We'd like to push this into a more central warning system so that we don't have to spam it every frame.
			UE_LOGF(LogMovieRenderPipeline, Warning, "'Additional Post Process Materials' do not support 'Aspect Ratio Constraint' or 'Screen Percentage'. The resulting data has been resized to match which may result in a stretched/squashed image.");
		}
				
		const FIntPoint TileSize = SampleStatePayload->UnpaddedTileSize;
		const FIntPoint OverlappedPad = SampleStatePayload->OverlappedPad;
		const FIntPoint OverlappedOffset = SampleStatePayload->OverlappedOffset;
		const FVector2D OverlappedSubpixelShift = SampleStatePayload->OverlappedSubpixelShift;
		::MoviePipeline::FTileWeight1D WeightFunctionX;
		::MoviePipeline::FTileWeight1D WeightFunctionY;
		WeightFunctionX.InitHelper(OverlappedPad.X, TileSize.X, OverlappedPad.X);
		WeightFunctionY.InitHelper(OverlappedPad.Y, TileSize.Y, OverlappedPad.Y);
				
		AccumulatorPin->AccumulatePixelData(*SamplePixelData, OverlappedOffset, OverlappedSubpixelShift, WeightFunctionX, WeightFunctionY);
	}

	// Finally on our last sample, we fetch the data out of the accumulator
	// and move it to the Output Merger.
	if (SampleStatePayload->bFetchFromAccumulator)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline_FetchAccumulatedPixelData);
		int32 FullSizeX = AccumulatorPin->PlaneSize.X;
		int32 FullSizeY = AccumulatorPin->PlaneSize.Y;

		// Now that a tile is fully built and accumulated we can notify the output builder that the
		// data is ready so it can pass that onto the output containers (if needed).
		if (SamplePixelData->GetType() == EImagePixelType::Float32)
		{
			// 32 bit FLinearColor
			TUniquePtr<TImagePixelData<FLinearColor> > FinalPixelData = MakeUnique<TImagePixelData<FLinearColor>>(FIntPoint(FullSizeX, FullSizeY), SampleStatePayload);
			AccumulatorPin->FetchFinalPixelDataLinearColor(FinalPixelData->Pixels);

			// Send the data to the Output Builder
			OutputMergerPin->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
		}
		else if (SamplePixelData->GetType() == EImagePixelType::Float16)
		{
			// 16 bit FLinearColor
			TUniquePtr<TImagePixelData<FFloat16Color> > FinalPixelData = MakeUnique<TImagePixelData<FFloat16Color>>(FIntPoint(FullSizeX, FullSizeY), SampleStatePayload);
			AccumulatorPin->FetchFinalPixelDataHalfFloat(FinalPixelData->Pixels);

			// Send the data to the Output Builder
			OutputMergerPin->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
		}
		else if (SamplePixelData->GetType() == EImagePixelType::Color)
		{
			// 8bit FColors
			TUniquePtr<TImagePixelData<FColor>> FinalPixelData = MakeUnique<TImagePixelData<FColor>>(FIntPoint(FullSizeX, FullSizeY), SampleStatePayload);
			AccumulatorPin->FetchFinalPixelDataByte(FinalPixelData->Pixels);

			// Send the data to the Output Builder
			OutputMergerPin->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
		}
		else
		{
			check(0);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline_FreeAccumulatedPixelData);
			// Free the memory in the accumulator.
			AccumulatorPin->Reset();
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ReleaseSampleData);
		SamplePixelData.Reset();
	}
}

} // namespace UE::MovieGraph::Rendering

