// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Renderers/MovieGraphPathTracerPass.h"
#include "Graph/Nodes/MovieGraphPathTracerPassNode.h"

namespace UE::MovieGraph::Rendering
{

UMovieGraphImagePassBaseNode* FMovieGraphPathTracerPass::GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const
{
	// This is a bit of a workaround for the fact that the pass doesn't have a strong pointer to the node it's supposed to be associated with,
	// since that instance changes every frame. So instead we have a virtual function here so the node can look it up by type, and then we can
	// call a bunch of virtual functions on the right instance to fetch values.
	const bool bIncludeCDOs = true;
	UMovieGraphPathTracerRenderPassNode* ParentNode = InConfig->GetSettingForBranch<UMovieGraphPathTracerRenderPassNode>(GetBranchName(), bIncludeCDOs);
	if (!ensureMsgf(ParentNode, TEXT("PathTracerPass should not exist without parent node in graph.")))
	{
		return nullptr;
	}

	return ParentNode;
}
namespace Private
{
	void GetSampleData(TSharedRef<FSceneViewFamilyContext> InOutFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo, int32& OutSampleCount, int32& OutSampleIndex)
	{
		// If motion blur is enabled:
		//    blend all spatial samples together while leaving the handling of temporal samples up to MRQ
		//    each temporal sample will include denoising and post-process effects
		// If motion blur is NOT enabled:
		//    blend all temporal+spatial samples within the path tracer and only apply denoising on the last temporal sample
		//    this way we minimize denoising cost and also allow a much higher number of temporal samples to be used which
		//    can help reduce strobing

		// NOTE: Tiling is not compatible with the reference motion blur mode because it changes the order of the loops over the image.
		const bool bHasTiles = InCameraInfo.TilingParams.TileCount.X * InCameraInfo.TilingParams.TileCount.Y > 1;
		const bool bAccumulateSpatialSamplesOnly = InOutFamily->EngineShowFlags.MotionBlur || bHasTiles;

		OutSampleCount = bAccumulateSpatialSamplesOnly ? InCameraInfo.SamplingParams.SpatialSampleCount : InCameraInfo.SamplingParams.TemporalSampleCount * InCameraInfo.SamplingParams.SpatialSampleCount;
		OutSampleIndex = bAccumulateSpatialSamplesOnly ? InCameraInfo.SamplingParams.SpatialSampleIndex : InCameraInfo.SamplingParams.TemporalSampleIndex * InCameraInfo.SamplingParams.SpatialSampleCount + InCameraInfo.SamplingParams.SpatialSampleIndex;
	}
}

bool FMovieGraphPathTracerPass::ShouldDiscardOutput(const TSharedRef<FSceneViewFamilyContext>& InFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	// We intentionally skip calling the super because it discards any samples not done during Rendering state,
	// but we need to not skip the results due to Temporal Denoising cooldown frames.

	int32 SampleCount, SampleIndex;
	Private::GetSampleData(InFamily, InCameraInfo, SampleCount, SampleIndex);

	// Discard the result, unless its the last sample
	return !(SampleIndex == SampleCount - 1);
}

void FMovieGraphPathTracerPass::ApplyMovieGraphOverridesToSceneView(TSharedRef<FSceneViewFamilyContext> InOutFamily, const FViewFamilyInitData& InInitData, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	FMovieGraphImagePassBase::ApplyMovieGraphOverridesToSceneView(InOutFamily, InInitData, InCameraInfo);

	// Override whatever settings came from PostProcessVolume or Camera
	int32 SampleCount, SampleIndex;
	Private::GetSampleData(InOutFamily, InCameraInfo, SampleCount, SampleIndex);

	// TODO: pass along FrameIndex (which includes SampleIndex) to make sure sampling is fully deterministic
		
	// Overwrite whatever sampling count came from the PostProcessVolume
	FSceneView* View = const_cast<FSceneView*>(InOutFamily->Views[0]);
	View->FinalPostProcessSettings.bOverride_PathTracingSamplesPerPixel = true;
	View->FinalPostProcessSettings.PathTracingSamplesPerPixel = SampleCount;

	// If we are using reference motion blur, also force the use of reference DOF (as the post-processed DOF cannot behave well with motion blurred input)
	// TODO: Is there a way to directly access bEnableReferenceMotionBlur from here?
	const bool bHasTiles = InCameraInfo.TilingParams.TileCount.X * InCameraInfo.TilingParams.TileCount.Y > 1;
	const bool bAccumulateSpatialSamplesOnly = InOutFamily->EngineShowFlags.MotionBlur || bHasTiles;
	if (!bAccumulateSpatialSamplesOnly)
	{
		View->FinalPostProcessSettings.bOverride_PathTracingEnableReferenceDOF = true;
		View->FinalPostProcessSettings.PathTracingEnableReferenceDOF = true;
	}

	// Update the post processing settings if the node has overridden any of them
	if (const UMovieGraphPathTracerRenderPassNode* PathTracerNode = Cast<UMovieGraphPathTracerRenderPassNode>(GetParentNode(InInitData.TimeData.EvaluatedConfig)))
	{
#define OVERRIDE_COMPONENT(ComponentName) \
		if (PathTracerNode->bOverride_bLightingComponents_Include##ComponentName) \
		{ \
			View->FinalPostProcessSettings.bOverride_PathTracingInclude##ComponentName = true; \
			View->FinalPostProcessSettings.PathTracingInclude##ComponentName = PathTracerNode->bLightingComponents_Include##ComponentName; \
		} \
		
		OVERRIDE_COMPONENT(Emissive);
		OVERRIDE_COMPONENT(Diffuse);
		OVERRIDE_COMPONENT(IndirectDiffuse);
		OVERRIDE_COMPONENT(Specular);
		OVERRIDE_COMPONENT(IndirectSpecular);
		OVERRIDE_COMPONENT(Volume);
		OVERRIDE_COMPONENT(IndirectVolume);
		
#undef OVERRIDE_COMPONENT
	}
	
	// reset path tracer's accumulation at the start of each sample
	View->bForcePathTracerReset = SampleIndex == 0;
	// set path tracer seed from sampling parameters
	View->PathTracerSeedOffset = InCameraInfo.SamplingParams.SeedOffset;
}

void FMovieGraphPathTracerPass::ApplyMovieGraphOverridesToSampleState(FMovieGraphSampleState& SampleState) const
{
	// Cancel out the subpixel shift because the path tracer does its own anti-aliasing
	SampleState.OverlappedSubpixelShift = FVector2D(0.5, 0.5);
}

} // UE::MovieGraph::Rendering