// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphImageSequenceOutputNode.h"

#include "Graph/Nodes/MovieGraphBurnInNode.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "Graph/Nodes/MovieGraphRenderLayerNode.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphOCIOHelper.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphFilenameResolveParams.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieRenderGraphEditorSettings.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "MoviePipelineImageSequenceOutput.h" // for FAsyncImageQuantization
#include "MoviePipelineEXROutput.h"
#include "MovieRenderPipelineCoreModule.h"

#include "Modules/ModuleManager.h"
#include "ImageWriteQueue.h"
#include "Misc/Paths.h"
#include "Async/TaskGraphInterfaces.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphImageSequenceOutputNode)

#define INJECT_BURN_IN_NODE(OUTPUT_NAME)																		\
	if (bEnableBurnIn)																							\
	{																											\
		UMovieGraphOutputBurnInNode* BurnInNode = NewObject<UMovieGraphOutputBurnInNode>(InEvaluatedConfig);	\
		BurnInNode->bOverride_OutputName = true;																\
		BurnInNode->bOverride_FileNameFormat = true;															\
		BurnInNode->bOverride_OutputRestriction = true;															\
		BurnInNode->bOverride_bCompositeOntoFinalImage = true;													\
		BurnInNode->bOverride_BurnInClass = true;																\
		BurnInNode->OutputName = OUTPUT_NAME;																	\
		BurnInNode->FileNameFormat = BurnInFileNameFormat;														\
		BurnInNode->OutputRestriction = FSoftClassPath(GetClass());												\
		BurnInNode->bCompositeOntoFinalImage = bCompositeOntoFinalImage;										\
		BurnInNode->BurnInClass = BurnInClass;																	\
																												\
		OutInjectedNodes.Add(BurnInNode);																		\
	}																											\

#define INJECT_BURN_IN_MULTILAYER_NODE(OUTPUT_NAME)																\
	if (bEnableBurnIn)																							\
	{																											\
		UMovieGraphOutputBurnInNode* BurnInNode = NewObject<UMovieGraphOutputBurnInNode>(InEvaluatedConfig);	\
		BurnInNode->bOverride_OutputName = true;																\
		BurnInNode->bOverride_FileNameFormat = true;															\
		BurnInNode->bOverride_OutputRestriction = true;															\
		BurnInNode->bOverride_bCompositeOntoFinalImage = true;													\
		BurnInNode->bOverride_BurnInClass = true;																\
		BurnInNode->bOverride_LayerNameFormat = true;															\
		BurnInNode->OutputName = OUTPUT_NAME;																	\
		BurnInNode->FileNameFormat = BurnInFileNameFormat;														\
		BurnInNode->OutputRestriction = FSoftClassPath(GetClass());												\
		BurnInNode->bCompositeOntoFinalImage = false;															\
		BurnInNode->BurnInClass = BurnInClass;																	\
		BurnInNode->LayerNameFormat = BurnInLayerNameFormat;													\
																												\
		OutInjectedNodes.Add(BurnInNode);																		\
	}																											\

namespace UE::MovieGraph
{
	void UpdateMetadataLayerNamesToMatchExrLayerNames(const FMovieGraphRenderDataIdentifier& InIdentifier, const FString& LayerName, TMap<FString, FStringFormatArg>& InOutMetadata)
	{
		// We rename the metadata in the file when we write it to disk, because there's no other way to know what metadata goes with the actual layer name in the exr,
		// and we don't want to require complex string manipulation in pipeline tools, so we rename `unreal/<layerName>/<rendererName>/<cameraName>/foo` to just
		// `unreal/layerData/<exrLayerName>/foo`, where exrLayerName comes from this function that decided if the current layer is either "rgba" or something more complex.
		TMap<FString, FStringFormatArg> NewMetadataNames;
		TArray<FString> OldMetadataKeys;

		const FString MetadataPrefixForLayer = UE::MoviePipeline::GetMetadataPrefixPath(InIdentifier);

		for (const TPair<FString, FStringFormatArg>& KVP : InOutMetadata)
		{
			if (KVP.Key.StartsWith(MetadataPrefixForLayer))
			{
				const FString NewMetadataPrefix = FString::Printf(TEXT("unreal/layerData/%s"), LayerName.IsEmpty() ? TEXT("rgba") : *LayerName);
				const FString OldMetadataPostfix = KVP.Key.RightChop(MetadataPrefixForLayer.Len());
				const FString NewKey = NewMetadataPrefix + OldMetadataPostfix;

				NewMetadataNames.Add(NewKey, KVP.Value);

				// We want to remove the old keys to prevent confusion
				OldMetadataKeys.Add(KVP.Key);
			}
		}

		// Remove the old keys from the map
		for (const FString& OldKey : OldMetadataKeys)
		{
			InOutMetadata.Remove(OldKey);
		}

		// Add the renamed ones
		InOutMetadata.Append(NewMetadataNames);
	}
}

UMovieGraphImageSequenceOutputNode::UMovieGraphImageSequenceOutputNode()
{
	ImageWriteQueue = &FModuleManager::Get().LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue").GetWriteQueue();
}

void UMovieGraphImageSequenceOutputNode::OnAllFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph)
{
	FinalizeFence = ImageWriteQueue->CreateFence();
}

bool UMovieGraphImageSequenceOutputNode::IsFinishedWritingToDiskImpl() const
{
	// Wait until the finalization fence is reached meaning we've written everything to disk.
	return Super::IsFinishedWritingToDiskImpl() && (!FinalizeFence.IsValid() || FinalizeFence.WaitFor(0));
}

FString UMovieGraphImageSequenceOutputNode::CreateFileName(
	UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData,
	const UMovieGraphImageSequenceOutputNode* InParentNode,
	const UMovieGraphPipeline* InPipeline,
	const TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& InRenderData,
	const EImageFormat InImageFormat,
	FMovieGraphResolveArgs& OutMergedFormatArgs,
	FString& OutFrameTemplatedFileName) const
{
	const TCHAR* Extension = TEXT("");
	switch (InImageFormat)
	{
	case EImageFormat::PNG: Extension = TEXT("png"); break;
	case EImageFormat::JPEG: Extension = TEXT("jpeg"); break;
	case EImageFormat::BMP: Extension = TEXT("bmp"); break;
	case EImageFormat::EXR: Extension = TEXT("exr"); break;
	}

	UMovieGraphGlobalOutputSettingNode* OutputSettingNode = InRawFrameData->EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName);
	if (!OutputSettingNode)
	{
		return FString();
	}

	const UE::MovieGraph::FMovieGraphSampleState* Payload = InRenderData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
	if (!ensureMsgf(Payload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
	{
		return FString();
	}

	// The file name format usually comes from the output node directly, but the payload has a chance to override it.
	const FString FinalFileNameFormat = !Payload->FilenameFormatOverride.IsEmpty() ? Payload->FilenameFormatOverride : InParentNode->FileNameFormat;

	// Generate one string that puts the directory combined with the filename format.
	FString FileNameFormatString = OutputSettingNode->OutputDirectory.Path / FinalFileNameFormat;

	// Insert tokens like {layer_name} as appropriate to make sure outputs don't clash with each other.
	DisambiguateFilename(FileNameFormatString, InRawFrameData, InParentNode, InRenderData);

	// Previous method is preserved for output frame number validation.
	constexpr bool bIncludeRenderPass = false;
	constexpr bool bTestFrameNumber = true;
	constexpr bool bIncludeCameraName = false;
	UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber, bIncludeCameraName);
	
	// Map the .ext to be specific to our output data.
	TMap<FString, FString> AdditionalFormatArgs;
	AdditionalFormatArgs.Add(TEXT("ext"), Extension);

	FMovieGraphFilenameResolveParams Params = FMovieGraphFilenameResolveParams::MakeResolveParams(
		InRenderData.Key,
		InPipeline,
		InRawFrameData->EvaluatedConfig.Get(),
		Payload->TraversalContext,
		AdditionalFormatArgs,
		Payload);

	// Take our string path from the Output Setting and resolve it.
	const FString ResolvedFileName = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FileNameFormatString, Params, OutMergedFormatArgs);

	OutFrameTemplatedFileName = GetFrameTemplatedFileName(Params, FileNameFormatString, OutMergedFormatArgs);

	return ResolvedFileName;
}

FString UMovieGraphImageSequenceOutputNode::GetFrameTemplatedFileName(
	const FMovieGraphFilenameResolveParams& InParams, const FString& InFileNameFormatString, FMovieGraphResolveArgs& OutMergedFormatArgs) const
{
	const FString FrameTemplatedFormatString = MoviePipeline::GetFrameTemplatedFilePath(InFileNameFormatString);

	// If time dilation is being used, the parameters will ask ResolveFilenameFormatArguments to warn the user if there's no _rel frame number
	// found, but we're intentionally overriding them above to be able to replace them with a completely unrelated token, so we don't actually
	// want that warning to be produced.
	FMovieGraphFilenameResolveParams ParamsCopy = InParams;
	ParamsCopy.bForceRelativeFrameNumbers = false;
	
	return UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FrameTemplatedFormatString, ParamsCopy, OutMergedFormatArgs);
}

void UMovieGraphImageSequenceOutputNode::OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask)
{
	// ------------------------
	// This method is called on the CDO!
	// ------------------------

	check(InRawFrameData);

	TArray<TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>> CompositedPasses = GetCompositedPasses(InRawFrameData);

	// ToDo:
	// The ImageWriteQueue is set up in a fire-and-forget manner. This means that the data needs to be placed in the WriteQueue
	// as a TUniquePtr (so it can free the data when its done). Unfortunately if we have multiple output formats at once,
	// we can't MoveTemp the data so we need to make a copy.
	// 
	// Copying can be expensive (3ms @ 1080p, 12ms at 4k for a single layer image) so ideally we'd like to do it on the task graph
	// but this isn't really compatible with the ImageWriteQueue API as we need the future returned by the ImageWriteQueue to happen
	// in order, so that we push our futures to the main Movie Pipeline in order, otherwise when we encode files to videos they'll
	// end up with frames out of order. A workaround for this would be to chain all of the send-to-imagewritequeue tasks to each
	// other with dependencies, but I'm not sure that's going to scale to the potentialy high data volume going wide MRQ will eventually
	// need.

	// Do a first pass to determine which layers/payloads should actually be written by this output node. This needs to be done upfront so we can
	// provide a list of payloads output to this node to the filename disambiguation logic.
	TMap<const UMovieGraphFileOutputNode*, TArray<UE::MovieGraph::FMovieGraphSampleState*>> PayloadsToWrite;
	for (TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& RenderData : InRawFrameData->ImageOutputData)
	{
		checkf(RenderData.Value.IsValid(), TEXT("Unexpected empty image data: incorrectly moved or its production failed?"));
		UE::MovieGraph::FMovieGraphSampleState* Payload = RenderData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
		if (!ensureMsgf(Payload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
		{
			continue;
		}

		// This payload may have opted out of being written by this output type.
		if (!Payload->OutputTypeRestrictions.IsEmpty() && !Payload->OutputTypeRestrictions.Contains(FSoftClassPath(GetClass())))
		{
			continue;
		}

		// If this pass is composited, skip it for now
		if (CompositedPasses.ContainsByPredicate([&RenderData](const TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& CompositedPass)
			{
				return CompositedPass.Key == RenderData.Key;
			}))
		{
			continue;
		}
		
		// A layer within this output data may have chosen to not be written to disk by this CDO node
		if (!InMask.Contains(RenderData.Key))
		{
			continue;
		}
		
		constexpr bool bIncludeCDOs = false;
        constexpr bool bExactMatch = true;
        const UMovieGraphImageSequenceOutputNode* ParentNode = Cast<UMovieGraphImageSequenceOutputNode>(
        	InRawFrameData->EvaluatedConfig->GetSettingForBranch(GetClass(), RenderData.Key.RootBranchName, bIncludeCDOs, bExactMatch));
        checkf(ParentNode, TEXT("Image sequence output should not exist without a parent node in the graph."));

		PayloadsToWrite.FindOrAdd(ParentNode).Add(Payload);
	}
	
	// NodeInstanceToPayloads needs to be populated before the loop below; it's used in filename disambiguation
	InRawFrameData->NodeInstanceToPayloads.Append(PayloadsToWrite);

	// The base ImageSequenceOutputNode doesn't support any multilayer formats, so we write out each render pass separately.
	for (TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& RenderData : InRawFrameData->ImageOutputData)
	{
		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = true;
		const UMovieGraphImageSequenceOutputNode* ParentNode = Cast<UMovieGraphImageSequenceOutputNode>(
			InRawFrameData->EvaluatedConfig->GetSettingForBranch(GetClass(), RenderData.Key.RootBranchName, bIncludeCDOs, bExactMatch));
		if (!IsValid(ParentNode))
		{
			continue;
		}

		TArray<UE::MovieGraph::FMovieGraphSampleState*>* EligiblePayloads = PayloadsToWrite.Find(ParentNode);
		if (!EligiblePayloads)
		{
			continue;
		}

		// Only write payloads that have been vetted by the prior loop and placed in NodeInstanceToPayloads.
		UE::MovieGraph::FMovieGraphSampleState* Payload = RenderData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
		if (!ensureMsgf(Payload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
		{
			continue;
		}

		if (!EligiblePayloads->Contains(Payload))
		{
			continue;
		}

		FMovieGraphResolveArgs FinalResolvedKVPs;
		FString FrameTemplatedFileName;
		FString FileName = CreateFileName(InRawFrameData, ParentNode, InPipeline, RenderData, OutputFormat, FinalResolvedKVPs, FrameTemplatedFileName);
		if (!ensureMsgf(!FileName.IsEmpty(), TEXT("Unexpected empty file name, skipping frame.")))
		{
			continue;
		}

		TUniquePtr<FImageWriteTask> TileImageTask = MakeUnique<FImageWriteTask>();
		TileImageTask->Format = OutputFormat;
		TileImageTask->CompressionQuality = 100;
		TileImageTask->Filename = FileName;

		// Pixel data can only be moved if there are no other active output image sequence nodes on the branch
		if (GetNumFileOutputNodes(*InRawFrameData->EvaluatedConfig, RenderData.Key.RootBranchName) > 1)
		{
			TileImageTask->PixelData = RenderData.Value->CopyImageData();
		}
		else
		{
			TileImageTask->PixelData = RenderData.Value->MoveImageDataToNew();
		}

		// If the overscan isn't cropped from the final image, offset any composites to the top-left of the original frustum
		FIntPoint CompositeOffset = FIntPoint::ZeroValue;
		const bool bIsCropRectValid = !Payload->CropRectangle.IsEmpty();
		const bool bCanCropResolution = RenderData.Value->GetSize() == Payload->OverscannedResolution;
		if (ShouldCropOverscanImpl() && bIsCropRectValid && bCanCropResolution)
		{
			switch (RenderData.Value->GetType())
			{
			case EImagePixelType::Color:
				TileImageTask->PixelPreProcessors.Add(TAsyncCropImage<FColor>(TileImageTask.Get(), Payload->CropRectangle));
				break;

			case EImagePixelType::Float16:
				TileImageTask->PixelPreProcessors.Add(TAsyncCropImage<FFloat16Color>(TileImageTask.Get(), Payload->CropRectangle));
				break;

			case EImagePixelType::Float32:
				TileImageTask->PixelPreProcessors.Add(TAsyncCropImage<FLinearColor>(TileImageTask.Get(), Payload->CropRectangle));
				break;
			}
		}
		else
		{
			CompositeOffset = Payload->CropRectangle.Min;
		}
		
		bool bQuantizationEncodeSRGB = true;
#if WITH_OCIO
		if (FMovieGraphOCIOHelper::GenerateOcioPixelPreProcessor(Payload, InPipeline, InRawFrameData->EvaluatedConfig.Get(), ParentNode->OCIOConfiguration, ParentNode->OCIOContext, TileImageTask->PixelPreProcessors))
		{
			// We assume that any encoding on the output transform should be done by OCIO
			bQuantizationEncodeSRGB = false;
		}
#endif // WITH_OCIO

		EImagePixelType PixelType = TileImageTask->PixelData->GetType();

		if (bQuantizeTo8Bit && TileImageTask->PixelData->GetBitDepth() > 8u)
		{
			TileImageTask->PixelPreProcessors.Emplace(UE::MoviePipeline::FAsyncImageQuantization(TileImageTask.Get(), bQuantizationEncodeSRGB));
			PixelType = EImagePixelType::Color;
		}

		// Perform compositing if any composited passes were found earlier
		for (TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& CompositedPass : CompositedPasses)
		{
			// This pass may not allow other passes to be composited on it
			if (!Payload->bAllowsCompositing)
			{
				continue;
			}
			
			// This composited pass will only composite on top of renders w/ the same branch and camera
			if (!CompositedPass.Key.IsBranchAndCameraEqual(RenderData.Key))
			{
				continue;
			}
			
			UE::MovieGraph::FMovieGraphSampleState* CompositePayload = CompositedPass.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
			if (!ensureMsgf(CompositePayload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
			{
				continue;
			}

			// This composite may have opted out of being written by this output type.
			if (!CompositePayload->OutputTypeRestrictions.IsEmpty() && !CompositePayload->OutputTypeRestrictions.Contains(FSoftClassPath(GetClass())))
			{
				continue;
			}

			// There could be multiple renders within this branch using the composited pass, so we have to copy the image data
			switch (PixelType)
			{
			case EImagePixelType::Color:
				TileImageTask->PixelPreProcessors.Add(TAsyncCompositeImage<FColor>(CompositedPass.Value->CopyImageData(), CompositeOffset));
				break;
			case EImagePixelType::Float16:
				TileImageTask->PixelPreProcessors.Add(TAsyncCompositeImage<FFloat16Color>(CompositedPass.Value->CopyImageData(), CompositeOffset));
				break;
			case EImagePixelType::Float32:
				TileImageTask->PixelPreProcessors.Add(TAsyncCompositeImage<FLinearColor>(CompositedPass.Value->CopyImageData(), CompositeOffset));
				break;
			}
		}

		UE::MovieGraph::FMovieGraphOutputFutureData OutputData;
		OutputData.Shot = InPipeline->GetActiveShotList()[Payload->TraversalContext.ShotIndex];
		OutputData.FilePath = FileName;
		OutputData.FrameTemplatedFilePath = FrameTemplatedFileName;
		OutputData.DataIdentifier = RenderData.Key;
		OutputData.OriginNodeClass = GetClass();
		OutputData.RenderLayerIndex = Payload->RenderLayerIndex;

		TFuture<bool> Future = ImageWriteQueue->Enqueue(MoveTemp(TileImageTask));

		InPipeline->AddOutputFuture(MoveTemp(Future), OutputData);
	}
}

#if WITH_UNREALEXR
TUniquePtr<FEXRImageWriteTask> UMovieGraphImageSequenceOutputNode_EXR::CreateImageWriteTask(FString InFileName, EEXRCompressionFormat InCompression, bool bMultiPart, int32 InCompressionLevel) const
{
	// Ensure our OpenExrRTTI module gets loaded.
	UE_CALL_ONCE([]
		{
			check(IsInGameThread());
			FModuleManager::Get().LoadModule(TEXT("UEOpenExrRTTI"));
		});

	// If not using multi-part, we have to pad all layers up to the maximum resolution. If multi-part is on, different header
	// data window sizes are suppported, so check the cvar to see if we should pad
	const bool bPadToDataWindowSize = !bMultiPart || UE::MoviePipeline::CVarMoviePipelinePadLayersForMultiPartEXR.GetValueOnGameThread();
	
	TUniquePtr<FEXRImageWriteTask> ImageWriteTask = MakeUnique<FEXRImageWriteTask>();
	ImageWriteTask->Filename = MoveTemp(InFileName);
	ImageWriteTask->bMultipart = bMultiPart;
	ImageWriteTask->bPadToDataWindowSize = bPadToDataWindowSize;
	ImageWriteTask->Compression = InCompression;
	ImageWriteTask->CompressionLevel = InCompressionLevel;

	return MoveTemp(ImageWriteTask);
}

void UMovieGraphImageSequenceOutputNode_EXR::PrepareTaskGlobalMetadata(FEXRImageWriteTask& InOutImageTask, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, TMap<FString, FString>& InMetadata) const
{
	// Add in hardware usage & diagnostic metadata
	constexpr bool bIsGraph = true;
	UE::MoviePipeline::GetHardwareUsageMetadata(InMetadata, FPaths::GetPath(InOutImageTask.Filename));
	UE::MoviePipeline::GetDiagnosticMetadata(InMetadata, bIsGraph);

	// Add passed in resolved metadata
	for (const TPair<FString, FString>& Metadata : InMetadata)
	{
		InOutImageTask.FileMetadata.Emplace(Metadata.Key, Metadata.Value);
	}

	// Add in any metadata from the output merger frame
	for (const TPair<FString, FString>& Metadata : InRawFrameData->FileMetadata)
	{
		InOutImageTask.FileMetadata.Add(Metadata.Key, Metadata.Value);
	}
}

void UMovieGraphImageSequenceOutputNode_EXR::UpdateTaskPerLayer(
	FEXRImageWriteTask& InOutImageTask,
	const UMovieGraphImageSequenceOutputNode* InParentNode,
	TUniquePtr<FImagePixelData> InImageData,
	int32 InLayerIndex,
	const FString& InLayerName,
	const TMap<FString, FString>& InResolvedOCIOContext) const
{
	const UE::MovieGraph::FMovieGraphSampleState* Payload = InImageData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
	if (!ensureMsgf(Payload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
	{
		return;
	}

	bool bEnabledOCIO = false;
#if WITH_OCIO
	if (FMovieGraphOCIOHelper::GenerateOcioPixelPreProcessorWithContext(Payload, InParentNode->OCIOConfiguration, InResolvedOCIOContext, InOutImageTask.PixelPreprocessors.FindOrAdd(InLayerIndex)))
	{
		bEnabledOCIO = true;
	}
#endif // WITH_OCIO

	if (InLayerIndex == 0)
	{
		// Add task information that is common to all layers. This metadata may be redundant with unreal/* metadata,
		// but these are "standard" fields in EXR metadata.
		InOutImageTask.FileMetadata.Add("owner", UE::MoviePipeline::GetJobAuthor(Payload->TraversalContext.Job));
		InOutImageTask.FileMetadata.Add("comments", Payload->TraversalContext.Job->Comment);

		const FIntPoint& Resolution = InImageData->GetSize();
		InOutImageTask.Width = Resolution.X;
		InOutImageTask.Height = Resolution.Y;

		InOutImageTask.OverscanPercentage = Payload->OverscanFraction;
		InOutImageTask.CropRectangle = Payload->CropRectangle;
		
#if WITH_OCIO
		if (bEnabledOCIO)
		{
			UE::MoviePipeline::UpdateColorSpaceMetadata(InParentNode->OCIOConfiguration.ColorConfiguration, InOutImageTask);
		}
		else
#endif // WITH_OCIO
		{
			UE::MoviePipeline::UpdateColorSpaceMetadata(Payload->SceneCaptureSource, InOutImageTask);
		}
	}

	if (!InLayerName.IsEmpty())
	{
		InOutImageTask.LayerNames.FindOrAdd(InImageData.Get(), InLayerName);
	}

	InOutImageTask.Layers.Add(MoveTemp(InImageData));
}
#endif // WITH_UNREALEXR

#if WITH_UNREALEXR
void UMovieGraphImageSequenceOutputNode_EXR::OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask)
{
	check(InRawFrameData);

	TArray<TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>> CompositedPasses = GetCompositedPasses(InRawFrameData);

	// Do a first pass to determine which layers/payloads should actually be written by this output node. This needs to be done upfront so we can
	// provide a list of payloads output by this node to the filename disambiguation logic.
	TMap<const UMovieGraphFileOutputNode*, TArray<UE::MovieGraph::FMovieGraphSampleState*>> PayloadsToWrite;
	for (TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& RenderData : InRawFrameData->ImageOutputData)
	{
		checkf(RenderData.Value.IsValid(), TEXT("Unexpected empty image data: incorrectly moved or its production failed?"));
		UE::MovieGraph::FMovieGraphSampleState* Payload = RenderData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
		if (!ensureMsgf(Payload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
		{
			continue;
		}

		// This payload may have opted out of being written by this output type.
		if (!Payload->OutputTypeRestrictions.IsEmpty() && !Payload->OutputTypeRestrictions.Contains(FSoftClassPath(GetClass())))
		{
			continue;
		}

		// If this pass is composited, skip it for now
		if (CompositedPasses.ContainsByPredicate([&RenderData](const TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& CompositedPass)
			{
				return RenderData.Key == CompositedPass.Key;
			}))
		{
			continue;
		}

		// A layer within this output data may have chosen to not be written to disk by this CDO node
		if (!InMask.Contains(RenderData.Key))
		{
			continue;
		}
		
		constexpr bool bIncludeCDOs = false;
        constexpr bool bExactMatch = true;
        const UMovieGraphImageSequenceOutputNode* ParentNode = Cast<UMovieGraphImageSequenceOutputNode>(
        	InRawFrameData->EvaluatedConfig->GetSettingForBranch(GetClass(), RenderData.Key.RootBranchName, bIncludeCDOs, bExactMatch));
        checkf(ParentNode, TEXT("Image sequence output should not exist without a parent node in the graph."));

		PayloadsToWrite.FindOrAdd(ParentNode).Add(Payload);
	}

	// NodeInstanceToPayloads needs to be populated before the loop below; it's used in filename disambiguation
	InRawFrameData->NodeInstanceToPayloads.Append(PayloadsToWrite);

	for (TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& RenderData : InRawFrameData->ImageOutputData)
	{
		constexpr bool bIncludeCDOs = false;
		constexpr bool bExactMatch = true;
		const UMovieGraphImageSequenceOutputNode_EXR* ParentNode = InRawFrameData->EvaluatedConfig->GetSettingForBranch<UMovieGraphImageSequenceOutputNode_EXR>(
			RenderData.Key.RootBranchName, bIncludeCDOs, bExactMatch);
		if (!IsValid(ParentNode))
		{
			continue;
		}

		TArray<UE::MovieGraph::FMovieGraphSampleState*>* EligiblePayloads = PayloadsToWrite.Find(ParentNode);
		if (!EligiblePayloads)
		{
			continue;
		}

		// Only write payloads that have been vetted by the prior loop and placed in NodeInstanceToPayloads.
		UE::MovieGraph::FMovieGraphSampleState* Payload = RenderData.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
		if (!ensureMsgf(Payload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
		{
			continue;
		}

		if (!EligiblePayloads->Contains(Payload))
		{
			continue;
		}

		FMovieGraphResolveArgs ResolvedFormatArgs;
		FString FrameTemplatedFileName;
		FString FileName = CreateFileName(InRawFrameData, ParentNode, InPipeline, RenderData, OutputFormat, ResolvedFormatArgs, FrameTemplatedFileName);
		if (!ensureMsgf(!FileName.IsEmpty(), TEXT("Unexpected empty file name, skipping frame.")))
		{
			continue;
		}

		// The pass may be need to be written out with lossless compression (eg, Object ID forces this on).
		const EEXRCompressionFormat CompressionType = Payload->bForceLosslessCompression ? ParentNode->LosslessCompression : ParentNode->Compression;
		
		constexpr bool bMultiPart = false;
		TUniquePtr<FEXRImageWriteTask> ImageWriteTask = CreateImageWriteTask(FileName, CompressionType, bMultiPart, ParentNode->CompressionLevel);
		
		PrepareTaskGlobalMetadata(*ImageWriteTask, InRawFrameData, ResolvedFormatArgs.FileMetadata);

		// No layer is equivalent to a zero-index layer
		constexpr int32 LayerIndex = 0;
		TUniquePtr<FImagePixelData> PixelData;
		if (GetNumFileOutputNodes(*InRawFrameData->EvaluatedConfig, RenderData.Key.RootBranchName) > 1)
		{
			PixelData = RenderData.Value->CopyImageData();
		}
		else
		{
			PixelData = RenderData.Value->MoveImageDataToNew();
		}

		TMap<FString, FString> ResolvedOCIOContext = {};
#if WITH_OCIO
		ResolvedOCIOContext = FMovieGraphOCIOHelper::ResolveOpenColorIOContext(
			ParentNode->OCIOContext,
			RenderData.Key,
			InPipeline,
			InRawFrameData->EvaluatedConfig.Get(),
			Payload->TraversalContext
		);
#endif // WITH_OCIO

		UpdateTaskPerLayer(*ImageWriteTask, ParentNode, MoveTemp(PixelData), LayerIndex, FString(), ResolvedOCIOContext);

		// Perform compositing if any composited passes were found earlier
		for (TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& CompositedPass : CompositedPasses)
		{
			// This pass may not allow other passes to be composited on it
			if (!Payload->bAllowsCompositing)
			{
				continue;
			}

			UE::MovieGraph::FMovieGraphSampleState* CompositePayload = CompositedPass.Value->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
			if (!ensureMsgf(CompositePayload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
			{
				continue;
			}

			// This composite may have opted out of being written by this output type.
			if (!CompositePayload->OutputTypeRestrictions.IsEmpty() && !CompositePayload->OutputTypeRestrictions.Contains(FSoftClassPath(GetClass())))
			{
				continue;
			}
			
			// This composited pass will only composite on top of renders w/ the same branch and camera
			if (CompositedPass.Key.IsBranchAndCameraEqual(RenderData.Key))
			{
				EImagePixelType PixelType = RenderData.Value->GetType();

				// There could be multiple renders within this branch using the composited pass, so we have to copy the image data
				switch (PixelType)
				{
				case EImagePixelType::Color:
					ImageWriteTask->PixelPreprocessors.FindOrAdd(LayerIndex).Add(TAsyncCompositeImage<FColor>(CompositedPass.Value->CopyImageData(), Payload->CropRectangle.Min));
					break;
				case EImagePixelType::Float16:
					ImageWriteTask->PixelPreprocessors.FindOrAdd(LayerIndex).Add(TAsyncCompositeImage<FFloat16Color>(CompositedPass.Value->CopyImageData(), Payload->CropRectangle.Min));
					break;
				case EImagePixelType::Float32:
					ImageWriteTask->PixelPreprocessors.FindOrAdd(LayerIndex).Add(TAsyncCompositeImage<FLinearColor>(CompositedPass.Value->CopyImageData(), Payload->CropRectangle.Min));
					break;
				}
			}
		}


		// Rename the per-layer metadata to match the EXR layer names to make fetching metadata simple string lookups.
		UE::MovieGraph::UpdateMetadataLayerNamesToMatchExrLayerNames(RenderData.Key, TEXT(""), ImageWriteTask->FileMetadata);

		UE::MovieGraph::FMovieGraphOutputFutureData OutputFutureData;
		OutputFutureData.Shot = InPipeline->GetActiveShotList()[Payload->TraversalContext.ShotIndex];
		OutputFutureData.FilePath = FileName;
		OutputFutureData.FrameTemplatedFilePath = FrameTemplatedFileName;
		OutputFutureData.DataIdentifier = RenderData.Key;
		OutputFutureData.OriginNodeClass = GetClass();
		OutputFutureData.RenderLayerIndex = Payload->RenderLayerIndex;

		TFuture<bool> Future = ImageWriteQueue->Enqueue(MoveTemp(ImageWriteTask));

		InPipeline->AddOutputFuture(MoveTemp(Future), OutputFutureData);
	}
}
#endif // WITH_UNREALEXR

void UMovieGraphImageSequenceOutputNode_EXR::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	INJECT_BURN_IN_NODE(TEXT("EXR"))
}

void UMovieGraphImageSequenceOutputNode_EXR::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesEXR = true;
}

#if WITH_UNREALEXR
void UMovieGraphImageSequenceOutputNode_MultiLayerEXR::OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask)
{
	auto GetPayloadCompressionType = [](
		const UE::MovieGraph::FMovieGraphSampleState* InPayload, const UMovieGraphImageSequenceOutputNode_MultiLayerEXR* InParentNode) -> EEXRCompressionFormat
	{
		return InPayload->bForceLosslessCompression ? InParentNode->LosslessCompression : InParentNode->Compression;
	};
	
	check(InRawFrameData);

	constexpr bool bIncludeCDOs = false;
	constexpr bool bExactMatch = true;
	const UMovieGraphImageSequenceOutputNode_MultiLayerEXR* ParentNode = InRawFrameData->EvaluatedConfig->GetSettingForBranch<UMovieGraphImageSequenceOutputNode_MultiLayerEXR>(
		UMovieGraphNode::GlobalsPinName, bIncludeCDOs, bExactMatch);
	checkf(ParentNode, TEXT("Multi-Layer EXR should not exist without a parent node in the graph."));

	// Generate the output config for each filename, which contains the Render IDs, the resolve args, the frame-templated filename, and the maximum resolution
	// to use when outputting to the corresponding EXR file
	TMap<FString, FEXROutputConfigForFilename> FilenameToRenderConfig;
	GetFilenameToEXROutputConfigMappings(ParentNode, InPipeline, InRawFrameData, FilenameToRenderConfig);

	// Write an EXR for each filename, which potentially contains multiple passes (render IDs).
	for (TPair<FString, FEXROutputConfigForFilename>& RenderConfigForFilename : FilenameToRenderConfig)
	{
		const FString& Filename = RenderConfigForFilename.Key;
		FEXROutputConfigForFilename& RenderConfig = RenderConfigForFilename.Value;
		
		// If all layers are not using the same compression type, force multipart on.
		bool bUseMultipart = ParentNode->bMultipart;
		if (!bUseMultipart)
		{
			EEXRCompressionFormat LastCompressionType = EEXRCompressionFormat::Max;
			
			for (const FMovieGraphRenderDataIdentifier& RenderID : RenderConfig.RenderIDs)
			{
				const TUniquePtr<FImagePixelData>& ImageData = InRawFrameData->ImageOutputData[RenderID];
				checkf(ImageData.IsValid(), TEXT("Unexpected empty image data: incorrectly moved or its production failed?"));

				const UE::MovieGraph::FMovieGraphSampleState* Payload = ImageData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
				if (!ensureMsgf(Payload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
				{
					continue;
				}

				const EEXRCompressionFormat CompressionType = GetPayloadCompressionType(Payload, ParentNode);
				if (LastCompressionType == EEXRCompressionFormat::Max)
				{
					LastCompressionType = CompressionType;
				}
				else if (CompressionType != LastCompressionType)
				{
					MRG_LOG_ONCE_PER_RENDER(ForceMultiPart, LogMovieRenderPipeline, Warning, TEXT("An EXR is being generated that containins layers with differing compression types (can happen especially when generating Object IDs or using PPMs). Forcing EXR multipart ON."));
					bUseMultipart = true;
					break;
				}
			}
		}
		
		TUniquePtr<FEXRImageWriteTask> MultiLayerImageTask = CreateImageWriteTask(Filename, ParentNode->Compression, bUseMultipart, ParentNode->CompressionLevel);
		PrepareTaskGlobalMetadata(*MultiLayerImageTask, InRawFrameData, RenderConfig.ResolveArgs.FileMetadata);

		// Keep track of the lowest render layer index found among the layers that are included. This will be used as the index provided to the output
		// future. This index is used to determine what the first render layer is when "First Render Layer Only" is turned on for displaying media
		// post-render, so for multi-layer EXRs, the layer with the lowest index should be used as the index for the file.
		int32 LowestRenderLayerIndex = 100000;

		// Add each render pass as a layer to the EXR
		bool bHasGeneratedPrimaryRGBALayer = false;
		int32 LayerIndex = 0;
		int32 ShotIndex = 0;
		for (const FMovieGraphRenderDataIdentifier& RenderID : RenderConfig.RenderIDs)
		{
			const TUniquePtr<FImagePixelData>& ImageData = InRawFrameData->ImageOutputData[RenderID];
			checkf(ImageData.IsValid(), TEXT("Unexpected empty image data: incorrectly moved or its production failed?"));

			const UE::MovieGraph::FMovieGraphSampleState* Payload = ImageData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
			if (!ensureMsgf(Payload, TEXT("Image data is missing the expected FMovieGraphSampleState payload.")))
			{
				continue;
			}

			// This payload may have opted out of being written by this output type.
			if (!Payload->OutputTypeRestrictions.IsEmpty() && !Payload->OutputTypeRestrictions.Contains(FSoftClassPath(GetClass())))
			{
				continue;
			}
			
			ShotIndex = Payload->TraversalContext.ShotIndex;

			LowestRenderLayerIndex = FMath::Min(LowestRenderLayerIndex, Payload->RenderLayerIndex);

			// If using multipart, specify the compression type for each layer.
			if (bUseMultipart)
			{
				const EEXRCompressionFormat LayerCompressionType = GetPayloadCompressionType(Payload, ParentNode);
				MultiLayerImageTask->CompressionByLayer.Add(LayerCompressionType);
			}

			// The first layer that doesn't have an explicitly-specified name will have an empty layer name in the EXR -- this is the "primary"/RGBA
			// layer. Otherwise, the layer name will be procedurally generated if the payload didn't specify a layer name override. Having a "primary"
			// layer in the EXR expands compatibility with a number of applications that read EXRs.
			FString LayerName = Payload->LayerNameOverride;
			if (bHasGeneratedPrimaryRGBALayer && LayerName.IsEmpty())
			{
				// If there is more than one layer, then we will prefix the layer. The first layer is not prefixed (and gets inserted as RGBA)
				// as most programs that handle EXRs expect the main image data to be in an unnamed layer. We only postfix with cameraname
				// if there's multiple cameras, as pipelines may be already be built around the generic "one camera" support.
				// TODO: The number of cameras may be inaccurate -- no camera setting in the graph yet
				UMoviePipelineExecutorShot* CurrentShot = InPipeline->GetActiveShotList()[ShotIndex];
				int32 NumCameras = CurrentShot->SidecarCameras.Num();

				UE::MovieGraph::FMovieGraphRenderDataValidationInfo ValidationInfo = InRawFrameData->GetValidationInfo(RenderID, /*bInDiscardCompositedRenders*/ false);
				TArray<FString> Tokens;

				if (ValidationInfo.BranchCount > 1)
				{
					if (ValidationInfo.LayerCount < ValidationInfo.BranchCount)
					{
						Tokens.Add(RenderID.RootBranchName.ToString());
					}
					else
					{
						Tokens.Add(RenderID.LayerName);
					}
				}

				if (ValidationInfo.ActiveBranchRendererCount > 1)
				{
					Tokens.Add(RenderID.RendererName);
				}

				if (ValidationInfo.ActiveRendererSubresourceCount > 1)
				{
					Tokens.Add(RenderID.SubResourceName);
				}

				const bool bTreatAsMultiCamera = !Payload->CameraNameTokenOverrides.IsEmpty();
				if (NumCameras > 1 || bTreatAsMultiCamera)
				{
					Tokens.Add(RenderID.CameraName);
				}

				if (ensureMsgf(!Tokens.IsEmpty(), TEXT("Missing expected EXR layer token.")))
				{
					LayerName = Tokens[0];
					
					for (int32 Index = 1; Index < Tokens.Num(); ++Index)
					{
						LayerName = FString::Printf(TEXT("%s_%s"), *LayerName, *Tokens[Index]);
					}
				}
			}
			else
			{
				// Don't generate a layer name. This layer will be the "primary" RGBA layer without a name.
				bHasGeneratedPrimaryRGBALayer = true;
			}

			TUniquePtr<FImagePixelData> PixelData;
			if (GetNumFileOutputNodes(*InRawFrameData->EvaluatedConfig, RenderID.RootBranchName) > 1)
			{
				PixelData = ImageData->CopyImageData();
			}
			else
			{
				PixelData = ImageData->MoveImageDataToNew();
			}

			TMap<FString, FString> ResolvedOCIOContext = {};
#if WITH_OCIO
			ResolvedOCIOContext = FMovieGraphOCIOHelper::ResolveOpenColorIOContext(
				ParentNode->OCIOContext,
				RenderID,
				InPipeline,
				InRawFrameData->EvaluatedConfig.Get(),
				Payload->TraversalContext
			);
#endif // WITH_OCIO
			UpdateTaskPerLayer(*MultiLayerImageTask, ParentNode, MoveTemp(PixelData), LayerIndex, LayerName, ResolvedOCIOContext);

			// Ensure the write task uses the maximum resolution of all the layers being written
			MultiLayerImageTask->Width = RenderConfig.MaximumResolution.X;
			MultiLayerImageTask->Height = RenderConfig.MaximumResolution.Y;

			// Rename the per-layer metadata to match the EXR layer names to make fetching metadata simple string lookups.
			UE::MovieGraph::UpdateMetadataLayerNamesToMatchExrLayerNames(RenderID, LayerName, MultiLayerImageTask->FileMetadata);

			LayerIndex++;
		}

		if (MultiLayerImageTask->Layers.IsEmpty())
		{
			continue;
		}

		UE::MovieGraph::FMovieGraphOutputFutureData OutputFutureData;
		OutputFutureData.Shot = InPipeline->GetActiveShotList()[ShotIndex];
		OutputFutureData.FilePath = Filename;
		OutputFutureData.FrameTemplatedFilePath = RenderConfig.FrameTemplatedFilename;
		OutputFutureData.DataIdentifier = FMovieGraphRenderDataIdentifier(); // EXRs put all the render passes internally so this resolves to a ""
		OutputFutureData.OriginNodeClass = GetClass();
		OutputFutureData.RenderLayerIndex = LowestRenderLayerIndex;

		InPipeline->AddOutputFuture(ImageWriteQueue->Enqueue(MoveTemp(MultiLayerImageTask)), OutputFutureData);
	}
}
#endif // WITH_UNREALEXR

void UMovieGraphImageSequenceOutputNode_MultiLayerEXR::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	INJECT_BURN_IN_MULTILAYER_NODE(TEXT("MULTI_EXR"))
}

void UMovieGraphImageSequenceOutputNode_MultiLayerEXR::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesMultiEXR = true;
}

void UMovieGraphImageSequenceOutputNode_MultiLayerEXR::GetFilenameToEXROutputConfigMappings(
	const UMovieGraphImageSequenceOutputNode_MultiLayerEXR* InParentNode, UMovieGraphPipeline* InPipeline,
	UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, TMap<FString, FEXROutputConfigForFilename>& OutFilenameToOutputConfigs) const
{
	// Merge one layer's resolve args (InNewResolveArgs) into an existing set of resolve args (InExistingResolveArgs).
	auto MergeResolveArgs = [](FMovieGraphResolveArgs& InNewResolveArgs, FMovieGraphResolveArgs& InExistingResolveArgs)
	{
		// Covert the filename arguments to FormatNamedArguments once; this is needed by FString::Format() in the loop
		FStringFormatNamedArguments NamedArguments;
		for (const TPair<FString, FString>& FilenameArgument : InNewResolveArgs.FilenameArguments)
		{
			NamedArguments.Add(FilenameArgument.Key, FilenameArgument.Value);
		}

		for (TPair<FString, FString>& MetadataPair : InNewResolveArgs.FileMetadata)
		{
			// The metadata key and/or value may contain filename format {tokens}; resolve any of them BEFORE merging in with existing metadata. This
			// is important because the metadata may contain a {token} that, once resolved, prevents a collision with an existing key.
			MetadataPair.Key = FString::Format(*MetadataPair.Key, NamedArguments);
			MetadataPair.Value = FString::Format(*MetadataPair.Value, NamedArguments);

			// Merge in the resolved metadata into the existing metadata
			InExistingResolveArgs.FileMetadata.Add(MetadataPair.Key, MetadataPair.Value);
		}

		// The filename arguments are not needed after merging + resolving; however, the last set of arguments is passed along anyway if they are needed.
		// They aren't merged though, because they differ too much between layers to make merging of any practical usefulness (eg, {layer_name}).
		InExistingResolveArgs.FilenameArguments = InNewResolveArgs.FilenameArguments;
	};
	
	// First, generate filename -> renderID mapping, and filename -> resolution mapping.
	// This assumes that all render passes will have the same resolution, so we use 0 as the resolution index.
	// Once we know the resolutions of all the render passes, they can be binned together into groups with the same
	// resolution, and the filenames can be regenerated to ensure that passes of differing resolutions go to
	// different files.
	//
	// This two-step process is necessary due to the flexibility in file naming, and the multi-layer nature of EXRs.
	// For example, if the file name format is "{sequence_name}.{frame_number}", and the second of two branches in the
	// graph has a differing resolution, only after resolving the output filenames for all outputs is a problem found;
	// layers of differing resolutions will be written to the same file. Using "{layer_name}.{sequence_name}.{frame_number}"
	// as the file name format would prevent the issue, but the two-step process is a generic way of approaching the
	// problem.
	for (const TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& RenderPassData : InRawFrameData->ImageOutputData)
	{
		FString FrameTemplatedFilename;
		constexpr int32 ResolutionIndex = 0;
		FMovieGraphResolveArgs ResolveArgs;
		const FString PreliminaryFileName = ResolveOutputFilename(InParentNode, InPipeline, InRawFrameData, RenderPassData.Key, ResolveArgs, FrameTemplatedFilename);

		FEXROutputConfigForFilename& OutputConfig = OutFilenameToOutputConfigs.FindOrAdd(PreliminaryFileName);
		
		OutputConfig.RenderIDs.Add(RenderPassData.Key);
		OutputConfig.FrameTemplatedFilename = FrameTemplatedFilename;
		
		OutputConfig.MaximumResolution.X = FMath::Max(OutputConfig.MaximumResolution.X, RenderPassData.Value->GetSize().X);
		OutputConfig.MaximumResolution.Y = FMath::Max(OutputConfig.MaximumResolution.Y, RenderPassData.Value->GetSize().Y);

		MergeResolveArgs(ResolveArgs, OutputConfig.ResolveArgs);
	}
}

FString UMovieGraphImageSequenceOutputNode_MultiLayerEXR::ResolveOutputFilename(
	const UMovieGraphImageSequenceOutputNode_MultiLayerEXR* InParentNode,
	const UMovieGraphPipeline* InPipeline,
	const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData,
	const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier,
	FMovieGraphResolveArgs& OutResolveArgs,
	FString& OutFrameTemplatedFilename) const
{
	if (!ensure(InRawFrameData))
	{
		return FString();
	}
	CA_ASSUME(InRawFrameData != nullptr);

	const TCHAR* Extension = TEXT("exr");

	constexpr bool bIncludeCDOs = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSettings = InRawFrameData->EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(InRenderDataIdentifier.RootBranchName, bIncludeCDOs);
	if (!ensure(OutputSettings))
	{
		return FString();
	}
	
	FString FileNameFormatString = InParentNode->FileNameFormat;
	
	// If we're writing more than one render pass out, we need to ensure the file name has the format string in it so we don't
	// overwrite the same file multiple times. Burn In overlays don't count because they get composited on top of an existing file.
	constexpr bool bIncludeRenderPass = false;
	constexpr bool bTestFrameNumber = true;
	constexpr bool bIncludeCameraName = false;
	UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber, bIncludeCameraName);

	const TUniquePtr<FImagePixelData>* ImageData = InRawFrameData->ImageOutputData.Find(InRenderDataIdentifier);
	const UE::MovieGraph::FMovieGraphSampleState* Payload = ImageData ? (*ImageData)->GetPayload<UE::MovieGraph::FMovieGraphSampleState>() : nullptr;
	
	// Create specific data that needs to override 
	TMap<FString, FString> FormatOverrides;
	FormatOverrides.Add(TEXT("render_pass"), TEXT("")); // Render Passes are included inside the exr file by named layers.
	FormatOverrides.Add(TEXT("ext"), Extension);

	// The layer's render data identifier is used here in the resolve. Usually this is not a problem. However, the user may include some tokens, like
	// {layer_name}, that come from the identifier, which will prevent all layers from being placed in the same multi-layer EXR (because now the path
	// isn't resolving to the path that other layers are resolving to). We have to assume that the user is doing this intentionally, even though it's
	// a bit strange. Including the full identifier here is important so all custom metadata is resolved correctly (see
	// UMovieGraphSetMetadataAttributesNode) when ResolveFilenameFormatArguments() is called.
	const FMovieGraphFilenameResolveParams Params = FMovieGraphFilenameResolveParams::MakeResolveParams(
		InRenderDataIdentifier, InPipeline, InRawFrameData->EvaluatedConfig.Get(), InRawFrameData->TraversalContext, FormatOverrides, Payload);
	
	const FString FilePathFormatString = OutputSettings->OutputDirectory.Path / FileNameFormatString;

	FString FinalFilePath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FilePathFormatString, Params, OutResolveArgs);

	OutFrameTemplatedFilename = GetFrameTemplatedFileName(Params, FilePathFormatString, OutResolveArgs);

	if (FPaths::IsRelative(FinalFilePath))
	{
		FinalFilePath = FPaths::ConvertRelativePathToFull(FinalFilePath);
	}

	return FinalFilePath;
}

void UMovieGraphImageSequenceOutputNode_BMP::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	INJECT_BURN_IN_NODE(TEXT("BMP"))
}

void UMovieGraphImageSequenceOutputNode_BMP::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesBMP = true;
}

void UMovieGraphImageSequenceOutputNode_JPG::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	INJECT_BURN_IN_NODE(TEXT("JPG"))
}

void UMovieGraphImageSequenceOutputNode_JPG::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesJPG = true;
}

void UMovieGraphImageSequenceOutputNode_PNG::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	INJECT_BURN_IN_NODE(TEXT("PNG"))
}

void UMovieGraphImageSequenceOutputNode_PNG::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesPNG = true;
}
