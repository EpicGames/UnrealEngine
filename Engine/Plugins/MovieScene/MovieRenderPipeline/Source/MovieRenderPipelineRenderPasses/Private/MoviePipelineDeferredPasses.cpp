// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineDeferredPasses.h"
#include "MoviePipelineOutputBase.h"
#include "Engine/TextureRenderTarget2D.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "MovieRenderPipelineDataTypes.h"
#include "GameFramework/PlayerController.h"
#include "MoviePipelineRenderPass.h"
#include "EngineModule.h"
#include "Engine/World.h"
#include "Engine/TextureRenderTarget.h"
#include "MoviePipeline.h"
#include "Misc/FrameRate.h"
#include "MoviePipelineShotConfig.h"
#include "MovieRenderOverlappedImage.h"
#include "MovieRenderPipelineCoreModule.h"
#include "ImagePixelData.h"
#include "MoviePipelineOutputBuilder.h"
#include "BufferVisualizationData.h"
#include "Containers/Array.h"
#include "FinalPostProcessSettings.h"
#include "Materials/Material.h"
#include "MoviePipelineCameraSetting.h"
#include "MoviePipelineHighResSetting.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineAntiAliasingSetting.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Components/PrimitiveComponent.h"
#include "EngineUtils.h"
#include "Engine/RendererSettings.h"
#include "Camera/CameraComponent.h"
#include "CineCameraComponent.h"
#include "Interfaces/Interface_PostProcessVolume.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineDeferredPasses)

FString UMoviePipelineDeferredPassBase::StencilLayerMaterialAsset = TEXT("/MovieRenderPipeline/Materials/MoviePipeline_StencilCutout.MoviePipeline_StencilCutout");
FString UMoviePipelineDeferredPassBase::DefaultDepthAsset = TEXT("/MovieRenderPipeline/Materials/MovieRenderQueue_WorldDepth.MovieRenderQueue_WorldDepth");
FString UMoviePipelineDeferredPassBase::DefaultMotionVectorsAsset = TEXT("/MovieRenderPipeline/Materials/MovieRenderQueue_MotionVectors.MovieRenderQueue_MotionVectors");



UMoviePipelineDeferredPassBase::UMoviePipelineDeferredPassBase() 
	: UMoviePipelineImagePassBase()
{
	PassIdentifier = FMoviePipelinePassIdentifier("FinalImage");

	// To help user knowledge we pre-seed the additional post processing materials with an array of potentially common passes.
	TArray<FString> DefaultPostProcessMaterials;
	DefaultPostProcessMaterials.Add(DefaultDepthAsset);
	DefaultPostProcessMaterials.Add(DefaultMotionVectorsAsset);

	for (FString& MaterialPath : DefaultPostProcessMaterials)
	{
		FMoviePipelinePostProcessPass& NewPass = AdditionalPostProcessMaterials.AddDefaulted_GetRef();
		NewPass.Material = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(MaterialPath));
		NewPass.bEnabled = false;
		NewPass.bHighPrecisionOutput = MaterialPath.Equals(DefaultDepthAsset);
	}
	bRenderMainPass = true;
}

void UMoviePipelineDeferredPassBase::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bUse32BitPostProcessMaterials_DEPRECATED)
	{
		for (FMoviePipelinePostProcessPass& Pass : AdditionalPostProcessMaterials)
		{
			Pass.bHighPrecisionOutput = true;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}

FIntPoint UMoviePipelineDeferredPassBase::GetEffectiveOutputResolutionForCamera(const int32 InCameraIndex) const
{
	return GetOverscannedResolutionForCamera(InCameraIndex);
}

FIntPoint UMoviePipelineDeferredPassBase::GetOverscannedResolutionForCamera(const int32 InCameraIndex) const
{
	UMoviePipelinePrimaryConfig* PrimaryConfig = GetPipeline()->GetPipelinePrimaryConfig();
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];

	// Get any cached overscan value for this camera. If there is none, query the live overscan value and cache it
	float CameraOverscan = 0.0f;
	if (GetPipeline()->HasCachedCameraOverscan(InCameraIndex))
	{
		CameraOverscan = GetPipeline()->GetCachedCameraOverscan(InCameraIndex);
	}
	else
	{
		// Get the camera view info to retrieve the camera's overscan, which is used when the settings do not override the overscan
		FMinimalViewInfo CameraViewInfo;

		if (GetNumCamerasToRender() == 1)
		{
			CameraViewInfo = GetPipeline()->GetWorld()->GetFirstPlayerController()->PlayerCameraManager->GetCameraCacheView();
		}
		else
		{
			UCameraComponent* CameraComponent;
			GetPipeline()->GetSidecarCameraData(CurrentShot, InCameraIndex, CameraViewInfo, &CameraComponent);
		}

		CameraOverscan = CameraViewInfo.GetOverscan();
		GetPipeline()->CacheCameraOverscan(InCameraIndex, CameraOverscan);
	}
	
	return UMoviePipelineBlueprintLibrary::GetOverscannedResolution(PrimaryConfig, CurrentShot, CameraOverscan);
}

bool UMoviePipelineDeferredPassBase::GetManualExposureEnabledForCamera(const int32 InCameraIndex) const
{
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];

	if (GetNumCamerasToRender() == 1)
	{
		APlayerCameraManager* PlayerCameraManager = GetPipeline()->GetWorld()->GetFirstPlayerController()->PlayerCameraManager;

		// By design, we don't use PlayerCameraManager->GetCameraCacheView() or PlayerCameraManager->ViewTarget.POV, as these cached
		// values haven't been initialized at the beginning of the first tick where the movie passes are initialized.  Instead, we
		// need to pull values from the component on the target actor.
		if (PlayerCameraManager->ViewTarget.Target)
		{
			UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromActor(PlayerCameraManager->ViewTarget.Target);
			if (CameraComponent && CameraComponent->PostProcessBlendWeight > 0.0f && CameraComponent->PostProcessSettings.bOverride_AutoExposureMethod)
			{
				return CameraComponent->PostProcessSettings.AutoExposureMethod == AEM_Manual;
			}
		}
	}
	else
	{
		FMinimalViewInfo CameraViewInfo;
		UCameraComponent* CameraComponent;
		GetPipeline()->GetSidecarCameraData(CurrentShot, InCameraIndex, CameraViewInfo, &CameraComponent);

		if (CameraViewInfo.PostProcessBlendWeight > 0.0f && CameraViewInfo.PostProcessSettings.bOverride_AutoExposureMethod)
		{
			return CameraViewInfo.PostProcessSettings.AutoExposureMethod == AEM_Manual;
		}
	}

	return false;
}

FMoviePipelineRenderPassMetrics UMoviePipelineDeferredPassBase::GetRenderPassMetricsForCamera(const int32 InCameraIndex, const FMoviePipelineRenderPassMetrics& InSampleState) const
{
	// Add per-camera custom backbuffer size support here.
	UMoviePipelinePrimaryConfig* PrimaryConfig = GetPipeline()->GetPipelinePrimaryConfig();
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	check(PrimaryConfig);
	check(CurrentShot);

	return UE::MoviePipeline::GetRenderPassMetrics(PrimaryConfig, CurrentShot, InSampleState, GetOverscannedResolutionForCamera(InCameraIndex));
}

int32 UMoviePipelineDeferredPassBase::GetNumCamerasToRender() const
{
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	UMoviePipelineCameraSetting* CameraSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineCameraSetting>(CurrentShot);

	return CameraSettings->bRenderAllCameras ? CurrentShot->SidecarCameras.Num() : 1;
}

int32 UMoviePipelineDeferredPassBase::GetCameraIndexForRenderPass(const int32 InCameraIndex) const
{
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	UMoviePipelineCameraSetting* CameraSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineCameraSetting>(CurrentShot);

	// If we're not rendering all cameras, we need to pass -1 so we pick up the real camera name.
	return CameraSettings->bRenderAllCameras ? InCameraIndex : -1;
}

FString UMoviePipelineDeferredPassBase::GetCameraName(const int32 InCameraIndex) const
{
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	UMoviePipelineCameraSetting* CameraSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineCameraSetting>(CurrentShot);

	// If we're not rendering all cameras, we need to pass -1 so we pick up the real camera name.
	const int32 LocalCameraIndex = CameraSettings->bRenderAllCameras ? InCameraIndex : -1;

	return CurrentShot->GetCameraName(LocalCameraIndex);
}

FString UMoviePipelineDeferredPassBase::GetCameraNameOverride(const int32 InCameraIndex) const
{
	// Custom camera name used to override ouput file name param
	return TEXT("");
}

void UMoviePipelineDeferredPassBase::MoviePipelineRenderShowFlagOverride(FEngineShowFlags& OutShowFlag)
{
	if (bDisableMultisampleEffects)
	{
		OutShowFlag.SetAntiAliasing(false);
		OutShowFlag.SetDepthOfField(false);
		OutShowFlag.SetMotionBlur(false);
		OutShowFlag.SetBloom(false);
		OutShowFlag.SetSceneColorFringe(false);
	}
}

void UMoviePipelineDeferredPassBase::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	Super::SetupImpl(InPassInitSettings);
	LLM_SCOPE_BYNAME(TEXT("MoviePipeline/DeferredPassSetup"));

	if (bAddDefaultLayer && (GetNumStencilLayers() == 0))
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "The 'Add Default Layer' deferred rendering option requires at least one Actor or Data Layer to be specified.");
		GetPipeline()->Shutdown(true);
		return;
	}

	{
		TSoftObjectPtr<UMaterialInterface> StencilMatRef = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(StencilLayerMaterialAsset));
		StencilLayerMaterial = StencilMatRef.LoadSynchronous();
		if (!StencilLayerMaterial)
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Failed to load Stencil Mask material, stencil layers will be incorrect. Path: %ls", *StencilMatRef.ToString());
		}
	}
	
	for (FMoviePipelinePostProcessPass& AdditionalPass : AdditionalPostProcessMaterials)
	{
		if (AdditionalPass.bEnabled)
		{
			UMaterialInterface* Material = AdditionalPass.Material.LoadSynchronous();
			if (Material)
			{
				ActivePostProcessMaterials.Add(Material);
				
				if (AdditionalPass.bHighPrecisionOutput)
				{
					ActiveHighPrecisionPostProcessMaterials.Add(Material);
				}
			}
		}
	}

	// Create a view state. Each individual camera, tile, and stencil layer need their own unique state as this includes visual history for anti-aliasing, etc. 
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	UMoviePipelineHighResSetting* HighResSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineHighResSetting>(CurrentShot);
	const int32 NumCameras = GetNumCamerasToRender();

	int32 TotalNumberOfAccumulators = 0;
	for (int32 CamIndex = 0; CamIndex < NumCameras; CamIndex++)
	{
		const FIntPoint CameraResolution = GetOverscannedResolutionForCamera(CamIndex);

		// Figure out how big each sub-region (tile) is.
		FIntPoint BackbufferResolution = FIntPoint(
			FMath::CeilToInt((float)CameraResolution.X / (float)HighResSettings->TileCount),
			FMath::CeilToInt((float)CameraResolution.Y / (float)HighResSettings->TileCount));

		// Then increase each sub-region by the overlap amount.
		BackbufferResolution = HighResSettings->CalculatePaddedBackbufferSize(BackbufferResolution);

		// Re-initialize the render target and surface queue for the current camera
		GetOrCreateViewRenderTarget(BackbufferResolution);
		CreateSurfaceQueueImpl(BackbufferResolution);

		FMultiCameraViewStateData& CameraData = CameraViewStateData.AddDefaulted_GetRef();

		// We don't always want to allocate a unique history per tile as very large resolutions can OOM the GPU in backbuffer images alone.
		// But we do need the history for some features (like Lumen) to work, so it's optional.
		int32 HighResTileCount = HighResSettings->bAllocateHistoryPerTile ? HighResSettings->TileCount : 1;

		if (HighResTileCount > 1 && HighResSettings->bPageToSystemMemory)
		{
			CameraData.SystemMemoryMirror = FSceneViewStateInterface::SystemMemoryMirrorAllocate();
		}

		if (HighResTileCount > 1 && bRenderMainPass && !GetManualExposureEnabledForCamera(CamIndex))
		{
			// Add view state for auto-exposure render pass
			FMultiCameraViewStateData::FPerTile& PerTile = CameraData.TileData.FindOrAdd(FIntPoint(-1, -1));
			PerTile.SceneViewStates.AddDefaulted();
		}

		for (int32 TileIndexX = 0; TileIndexX < HighResTileCount; TileIndexX++)
		{
			for (int32 TileIndexY = 0; TileIndexY < HighResTileCount; TileIndexY++)
			{
				FMultiCameraViewStateData::FPerTile& PerTile = CameraData.TileData.FindOrAdd(FIntPoint(TileIndexX, TileIndexY));
				// If they want to render the main pass (most likely) add a view state for it
				if (bRenderMainPass)
				{
					PerTile.SceneViewStates.AddDefaulted();
				}

				// If they want to render a "default" stencil layer (that has everything not in another layer) add that...
				if (GetNumStencilLayers() > 0 && bAddDefaultLayer)
				{
					PerTile.SceneViewStates.AddDefaulted();
				}

				// Finally all of the other stencil layers
				for (int32 Index = 0; Index < GetNumStencilLayers(); Index++)
				{
					PerTile.SceneViewStates.AddDefaulted();
				}
			}
		}

		// We have to add up the number of accumulators needed separately, because we don't make
		// one accumulator per high-res tile.
		if (bRenderMainPass)
		{
			TotalNumberOfAccumulators++;
		}
		if (GetNumStencilLayers() > 0 && bAddDefaultLayer)
		{
			TotalNumberOfAccumulators++;
		}
		for (int32 Index = 0; Index < GetNumStencilLayers(); Index++)
		{
			TotalNumberOfAccumulators++;
		}

		// Now that we have an array of view states, allocate each one.
		for (TPair<FIntPoint, FMultiCameraViewStateData::FPerTile>& Pair : CameraData.TileData)
		{
			for (int32 Index = 0; Index < Pair.Value.SceneViewStates.Num(); Index++)
			{
				Pair.Value.SceneViewStates[Index].Allocate(InPassInitSettings.FeatureLevel);
			}
		}
	}

	// We must allocate one accumulator per output, because when we submit a sample we tie up an accumulator, but because of temporal sampling
	// the accumulators can be tied up for multiple game frames, thus we must have at least one per output and we can only reuse them between
	// actual output frames (not engine frames). This doesn't allocate memory until they're actually used so it's ok to over-allocate.
	int32 PoolSize = (TotalNumberOfAccumulators + (ActivePostProcessMaterials.Num()*NumCameras) + 1) * 3;
	AccumulatorPool = MakeShared<TAccumulatorPool<FImageOverlappedAccumulator>, ESPMode::ThreadSafe>(PoolSize);
	
	PreviousCustomDepthValue.Reset();

	// This scene view extension will be released automatically as soon as Render Sequence is torn down.
	// One Extension per sequence, since each sequence has its own OCIO settings.
	OCIOSceneViewExtension = FSceneViewExtensions::NewExtension<FOpenColorIODisplayExtension>();

	const bool bEnableStencilPass = bAddDefaultLayer || GetNumStencilLayers() > 0;
	if (bEnableStencilPass)
	{
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.CustomDepth"));
		if (CVar)
		{
			PreviousCustomDepthValue = CVar->GetInt();
			const int32 CustomDepthWithStencil = 3;
			if (PreviousCustomDepthValue != CustomDepthWithStencil)
			{
				UE_LOGF(LogMovieRenderPipeline, Log, "Overriding project custom depth/stencil value to support a stencil pass.");
				// We use ECVF_SetByProjectSetting otherwise once this is set once by rendering, the UI silently fails
				// if you try to change it afterwards. This SetByProjectSetting will fail if they have manipulated the cvar via the console
				// during their current session but it's less likely than changing the project settings.
				CVar->Set(CustomDepthWithStencil, EConsoleVariableFlags::ECVF_SetByProjectSetting);
			}
		}
	}

	// Cache out the stencil layer names (from data layers or actor layers) and de-duplicate. If layers with the same name
	// are provided, renders may fail, which is why the the names need to be de-duplicated.
	if (IsUsingDataLayers())
	{
		for (FSoftObjectPath DataLayerAssetPath : DataLayers)
		{
			UDataLayerAsset* DataLayerAsset = Cast<UDataLayerAsset>(DataLayerAssetPath.TryLoad());
			if (DataLayerAsset)
			{
				UniqueStencilLayerNames.Add(DataLayerAsset->GetName());
			}
		}
	}
	else
	{
		for (const FActorLayer& Layer : ActorLayers)
		{
			UniqueStencilLayerNames.Add(Layer.Name.ToString());
		}
	}
	
	UE::MoviePipeline::DeduplicateNameArray(UniqueStencilLayerNames);
}

void UMoviePipelineDeferredPassBase::TeardownImpl()
{
	ActivePostProcessMaterials.Reset();
	ActiveHighPrecisionPostProcessMaterials.Reset();
	UniqueStencilLayerNames.Reset();

	for (FMultiCameraViewStateData& CameraData : CameraViewStateData)
	{
		for (TPair<FIntPoint, FMultiCameraViewStateData::FPerTile>& Pair : CameraData.TileData)
		{
			for (int32 Index = 0; Index < Pair.Value.SceneViewStates.Num(); Index++)
			{
				FSceneViewStateInterface* Ref = Pair.Value.SceneViewStates[Index].GetReference();
				if (Ref)
				{
					Ref->ClearMIDPool();
				}
				Pair.Value.SceneViewStates[Index].Destroy();
			}
		}
	}
	CameraViewStateData.Reset();
	
	OCIOSceneViewExtension.Reset();
	OCIOSceneViewExtension = nullptr;

	if (PreviousCustomDepthValue.IsSet())
	{
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.CustomDepth"));
		if (CVar)
		{
			if (CVar->GetInt() != PreviousCustomDepthValue.GetValue())
			{
				UE_LOGF(LogMovieRenderPipeline, Log, "Restoring custom depth/stencil value to: %d", PreviousCustomDepthValue.GetValue());
				CVar->Set(PreviousCustomDepthValue.GetValue(), EConsoleVariableFlags::ECVF_SetByProjectSetting);
			}
		}
	}

	// Preserve our view state until the rendering thread has been flushed.
	Super::TeardownImpl();
}

void UMoviePipelineDeferredPassBase::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UMoviePipelineDeferredPassBase& This = *CastChecked<UMoviePipelineDeferredPassBase>(InThis);

	for (FMultiCameraViewStateData& CameraData : This.CameraViewStateData)
	{
		for (TPair<FIntPoint, FMultiCameraViewStateData::FPerTile>& Pair : CameraData.TileData)
		{
			for (int32 Index = 0; Index < Pair.Value.SceneViewStates.Num(); Index++)
			{
				FSceneViewStateInterface* Ref = Pair.Value.SceneViewStates[Index].GetReference();
				if (Ref)
				{
					Ref->AddReferencedObjects(Collector);
				}
			}
		}
	}
}

namespace UE
{
namespace MoviePipeline
{
struct FDeferredPassRenderStatePayload : public UMoviePipelineImagePassBase::IViewCalcPayload
{
	int32 CameraIndex;
	FIntPoint TileIndex; // Will always be 1,1 if no history-per-tile is enabled
	int32 SceneViewIndex;
};
}
}


FSceneViewStateInterface* UMoviePipelineDeferredPassBase::GetSceneViewStateInterface(IViewCalcPayload* OptPayload)
{
	UE::MoviePipeline::FDeferredPassRenderStatePayload* Payload = (UE::MoviePipeline::FDeferredPassRenderStatePayload*)OptPayload;
	check(Payload);

	// When not using multi-camera support, CameraIndex is -1, but means "first entry" in the array.
	int32 LocalCameraIndex = FMath::Clamp(Payload->CameraIndex, 0, Payload->CameraIndex);

	FMultiCameraViewStateData& CameraData = CameraViewStateData[LocalCameraIndex];
	FIntPoint TileIndex = CameraData.TileData.Num() == 1 ? FIntPoint(0, 0) : Payload->TileIndex;
	if (FMultiCameraViewStateData::FPerTile* TileData = CameraData.TileData.Find(TileIndex))
	{
		return TileData->SceneViewStates[Payload->SceneViewIndex].GetReference();
	}

	return nullptr;
}

FSceneViewStateInterface* UMoviePipelineDeferredPassBase::GetExposureSceneViewStateInterface(IViewCalcPayload* OptPayload, int32 CubeFaceIndex)
{
	UE::MoviePipeline::FDeferredPassRenderStatePayload* Payload = (UE::MoviePipeline::FDeferredPassRenderStatePayload*)OptPayload;
	check(Payload);

	// When not using multi-camera support, CameraIndex is -1, but means "first entry" in the array.
	int32 LocalCameraIndex = FMath::Clamp(Payload->CameraIndex, 0, Payload->CameraIndex);

	// Special tile {-1,-1} holds the auto-exposure view state, if present
	FMultiCameraViewStateData& CameraData = CameraViewStateData[LocalCameraIndex];
	if (FMultiCameraViewStateData::FPerTile* TileData = CameraData.TileData.Find(FIntPoint(-1,-1)))
	{
		return TileData->SceneViewStates[0].GetReference();
	}
	else
	{
		return nullptr;
	}
}

void UMoviePipelineDeferredPassBase::GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses)
{
	// No super call here because multiple cameras makes this all complicated
	// Super::GatherOutputPassesImpl(ExpectedRenderPasses);

	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];

	const int32 NumCameras = GetNumCamerasToRender();
	for (int32 CameraIndex = 0; CameraIndex < NumCameras; CameraIndex++)
	{
		FMoviePipelinePassIdentifier PassIdentifierForCurrentCamera;
		PassIdentifierForCurrentCamera.Name = PassIdentifier.Name;
		PassIdentifierForCurrentCamera.CameraName = GetCameraName(CameraIndex);

		// Add the default backbuffer
		if (bRenderMainPass)
		{
			ExpectedRenderPasses.Add(PassIdentifierForCurrentCamera);
		}

		// Each camera will render everything in the Post Process Material stack.
		TArray<FString> RenderPasses;
		for (UMaterialInterface* Material : ActivePostProcessMaterials)
		{
			if (Material)
			{
				RenderPasses.Add(GetNameForPostProcessMaterial(Material));
			}
		}

		for (const FString& Pass : RenderPasses)
		{
			ExpectedRenderPasses.Add(FMoviePipelinePassIdentifier(PassIdentifierForCurrentCamera.Name + Pass, PassIdentifierForCurrentCamera.CameraName));
		}

		// Stencil Layer Time!
		if (GetNumStencilLayers() > 0 && bAddDefaultLayer)
		{
			ExpectedRenderPasses.Add(FMoviePipelinePassIdentifier(PassIdentifierForCurrentCamera.Name + TEXT("DefaultLayer"), PassIdentifierForCurrentCamera.CameraName));
		}

		for (const FString& StencilLayerName : GetStencilLayerNames())
		{
			ExpectedRenderPasses.Add(FMoviePipelinePassIdentifier(PassIdentifierForCurrentCamera.Name + StencilLayerName, PassIdentifierForCurrentCamera.CameraName));
		}
	}
}

void UMoviePipelineDeferredPassBase::AddViewExtensions(FSceneViewFamilyContext& InContext, FMoviePipelineRenderPassMetrics& InOutSampleState)
{
	// OCIO Scene View Extension is a special case and won't be registered like other view extensions.
	if (InOutSampleState.OCIOConfiguration && InOutSampleState.OCIOConfiguration->bIsEnabled)
	{
		FOpenColorIODisplayConfiguration* OCIOConfigNew = const_cast<FMoviePipelineRenderPassMetrics&>(InOutSampleState).OCIOConfiguration;
		FOpenColorIODisplayConfiguration& OCIOConfigCurrent = OCIOSceneViewExtension->GetDisplayConfiguration();

		// We only need to set this once per render sequence.
		if (OCIOConfigNew->ColorConfiguration.ConfigurationSource && OCIOConfigNew->ColorConfiguration.ConfigurationSource != OCIOConfigCurrent.ColorConfiguration.ConfigurationSource)
		{
			OCIOSceneViewExtension->SetDisplayConfiguration(*OCIOConfigNew);
		}

		InContext.ViewExtensions.Add(OCIOSceneViewExtension.ToSharedRef());
	}
}

void UMoviePipelineDeferredPassBase::RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState)
{
	// Wait for a surface to be available to write to. This will stall the game thread while the RHI/Render Thread catch up.
	Super::RenderSample_GameThreadImpl(InSampleState);

	const int32 NumCameras = GetNumCamerasToRender();
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];

	for (int32 CameraIndex = 0; CameraIndex < NumCameras; CameraIndex++)
	{
		FMoviePipelinePassIdentifier PassIdentifierForCurrentCamera;
		PassIdentifierForCurrentCamera.Name = PassIdentifier.Name;

		// If we're not rendering all cameras, we need to pass -1 so we pick up the real camera name.
		const int32 CameraIndexForRenderPass = GetCameraIndexForRenderPass(CameraIndex);
		PassIdentifierForCurrentCamera.CameraName = GetCameraName(CameraIndex);

		// Main Render Pass
		if (bRenderMainPass)
		{
			// We have an auto-exposure pass if we allocated a view state for it
			bool bHasAutoExposurePass = CameraViewStateData.Num() && CameraViewStateData[0].TileData.Contains(FIntPoint(-1, -1));

			// Auto-exposure pass is rendered only for the first tile and sample, actual pass is always rendered
			bool bAutoExposurePass = bHasAutoExposurePass && InSampleState.TileIndexes.X == 0 && InSampleState.TileIndexes.Y == 0 && InSampleState.TemporalSampleIndex == 0 && InSampleState.SpatialSampleIndex == 0;
			bool bActualPass = true;

			while (bAutoExposurePass || bActualPass)
			{
				FMoviePipelineRenderPassMetrics InOutSampleState = GetRenderPassMetricsForCamera(CameraIndexForRenderPass, InSampleState);
				// InOutSampleState.OutputState.CameraCount = NumCameras;
				InOutSampleState.OutputState.CameraIndex = CameraIndexForRenderPass;
				InOutSampleState.OutputState.CameraNameOverride = GetCameraNameOverride(CameraIndex);

				if (bAutoExposurePass)
				{
					// Auto exposure reference pass uses tile indices -1,-1
					InOutSampleState.TileIndexes = FIntPoint(-1, -1);
				}

				UE::MoviePipeline::FDeferredPassRenderStatePayload Payload;
				Payload.CameraIndex = CameraIndexForRenderPass;
				Payload.TileIndex = InOutSampleState.TileIndexes;

				// Main renders use index 0.
				Payload.SceneViewIndex = 0;

				TSharedPtr<FSceneViewFamilyContext> ViewFamily = CalculateViewFamily(InOutSampleState, &Payload);

				// Add post-processing materials if needed
				FSceneView* View = const_cast<FSceneView*>(ViewFamily->Views[0]);
				View->FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Empty();
				View->FinalPostProcessSettings.BufferVisualizationPipes.Empty();

				for (UMaterialInterface* Material : ActivePostProcessMaterials)
				{
					if (Material)
					{
						View->FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Add(Material);
					}
				}

				for (UMaterialInterface* VisMaterial : View->FinalPostProcessSettings.BufferVisualizationOverviewMaterials)
				{
					// If this was just to contribute to the history buffer, no need to go any further.
					if (InOutSampleState.bDiscardResult)
					{
						continue;
					}

					FMoviePipelinePassIdentifier LayerPassIdentifier = FMoviePipelinePassIdentifier(PassIdentifier.Name + GetNameForPostProcessMaterial(VisMaterial), PassIdentifierForCurrentCamera.CameraName);

					auto BufferPipe = MakeShared<FImagePixelPipe, ESPMode::ThreadSafe>();
					BufferPipe->bIsExpecting32BitPixelData = ActiveHighPrecisionPostProcessMaterials.Contains(VisMaterial);
					BufferPipe->AddEndpoint(MakeForwardingEndpoint(LayerPassIdentifier, InOutSampleState));

					View->FinalPostProcessSettings.BufferVisualizationPipes.Add(VisMaterial->GetFName(), BufferPipe);
				}


				int32 NumValidMaterials = View->FinalPostProcessSettings.BufferVisualizationPipes.Num();
				View->FinalPostProcessSettings.bBufferVisualizationDumpRequired = NumValidMaterials > 0;

				// Submit to be rendered. Main render pass always uses target 0.
				TWeakObjectPtr<UTextureRenderTarget2D> ViewRenderTarget = GetOrCreateViewRenderTarget(InOutSampleState.BackbufferSize, (IViewCalcPayload*)(&Payload));
				check(ViewRenderTarget.IsValid());

				FRenderTarget* RenderTarget = ViewRenderTarget->GameThread_GetRenderTargetResource();
				check(RenderTarget);

				FCanvas Canvas = FCanvas(RenderTarget, nullptr, GetPipeline()->GetWorld(), View->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, 1.0f);
				GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.Get());
			
				ENQUEUE_RENDER_COMMAND(TransitionTextureSRVState)(
				[RenderTarget](FRHICommandListImmediate& RHICmdList) mutable
				{
					// Transition our render target from a render target view to a shader resource view to allow the UMG preview material to read from this Render Target.
					RHICmdList.Transition(FRHITransitionInfo(RenderTarget->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVGraphicsPixel));
				});

				if (bAutoExposurePass)
				{
					// Finished rendering auto-exposure pass
					bAutoExposurePass = false;
				}
				else
				{
					// Readback + Accumulate.
					PostRendererSubmission(InOutSampleState, PassIdentifierForCurrentCamera, GetOutputFileSortingOrder(), Canvas);

					// Finished rendering actual pass
					bActualPass = false;
				}
			}

		}


		// Now do the stencil layer submission (which doesn't support additional post processing materials)
		{
			FMoviePipelineRenderPassMetrics InOutSampleState = GetRenderPassMetricsForCamera(CameraIndexForRenderPass, InSampleState);
			InOutSampleState.OutputState.CameraIndex = CameraIndexForRenderPass;
			InOutSampleState.OutputState.CameraNameOverride = GetCameraNameOverride(CameraIndex);

			struct FStencilValues
			{
				FStencilValues()
					: bRenderCustomDepth(false)
					, StencilMask(ERendererStencilMask::ERSM_Default)
					, CustomStencil(0)
				{
				}

				bool bRenderCustomDepth;
				ERendererStencilMask StencilMask;
				int32 CustomStencil;
			};

			// Now for each stencil layer we reconfigure all the actors custom depth/stencil 
			TArray<FString> AllStencilLayerNames = GetStencilLayerNames();
			if (bAddDefaultLayer)
			{
				AllStencilLayerNames.Add(TEXT("DefaultLayer"));
			}

			// If we're going to be using stencil layers, we need to cache all of the users
			// custom stencil/depth settings since we're changing them to do the mask.
			TMap<UPrimitiveComponent*, FStencilValues> PreviousValues;
			if (AllStencilLayerNames.Num() > 0)
			{
				for (TActorIterator<AActor> ActorItr(GetWorld()); ActorItr; ++ActorItr)
				{
					AActor* Actor = *ActorItr;
					if (Actor)
					{
						for (UActorComponent* Component : Actor->GetComponents())
						{
							if (Component && Component->IsA<UPrimitiveComponent>())
							{
								UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(Component);
								FStencilValues& Values = PreviousValues.Add(PrimitiveComponent);
								Values.StencilMask = PrimitiveComponent->CustomDepthStencilWriteMask;
								Values.CustomStencil = PrimitiveComponent->CustomDepthStencilValue;
								Values.bRenderCustomDepth = PrimitiveComponent->bRenderCustomDepth;
							}
						}
					}
				}
			}


			for (int32 StencilLayerIndex = 0; StencilLayerIndex < AllStencilLayerNames.Num(); StencilLayerIndex++)
			{
				const FString& LayerName = AllStencilLayerNames[StencilLayerIndex];
				FMoviePipelinePassIdentifier LayerPassIdentifier = FMoviePipelinePassIdentifier(PassIdentifierForCurrentCamera.Name + LayerName);
				LayerPassIdentifier.CameraName = PassIdentifierForCurrentCamera.CameraName;

				// Modify all of the actors in this world so they have the right stencil settings (so we can use the stencil buffer as a mask later)
				for (TActorIterator<AActor> ActorItr(GetWorld()); ActorItr; ++ActorItr)
				{
					AActor* Actor = *ActorItr;
					if (Actor)
					{
						// The way stencil masking works is that we draw the actors on the given layer to the stencil buffer.
						// Then we apply a post-processing material which colors pixels outside those actors black, before
						// post processing. Then, TAA, Motion Blur, etc. is applied to all pixels. An alpha channel can preserve
						// which pixels were the geometry and which are dead space which lets you apply that as a mask later.
						bool bInLayer = true;
						if (bAddDefaultLayer && LayerName == TEXT("DefaultLayer"))
						{
							// If we're trying to render the default layer, the logic is different - we only add objects who
							// aren't in any of the stencil layers.
							bInLayer = IsActorInAnyStencilLayer(Actor);
						
						}
						else
						{
							// If this a normal layer, we only add the actor if it exists on this layer.
							bInLayer = IsActorInLayer(Actor, StencilLayerIndex);
						}

						for (UActorComponent* Component : Actor->GetComponents())
						{
							if (Component && Component->IsA<UPrimitiveComponent>())
							{
								UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(Component);
								// We want to render all objects not on the layer to stencil too so that foreground objects mask.
								PrimitiveComponent->SetCustomDepthStencilValue(bInLayer ? 1 : 0);
								PrimitiveComponent->SetCustomDepthStencilWriteMask(ERendererStencilMask::ERSM_Default);
								PrimitiveComponent->SetRenderCustomDepth(true);
							}
						}
					}
				}

				// Submit the actual render now
				if (StencilLayerMaterial)
				{
					UE::MoviePipeline::FDeferredPassRenderStatePayload Payload;
					Payload.CameraIndex = CameraIndexForRenderPass;
					Payload.TileIndex = InOutSampleState.TileIndexes;
					Payload.SceneViewIndex = StencilLayerIndex + (bRenderMainPass ? 1 : 0);
					TSharedPtr<FSceneViewFamilyContext> ViewFamily = CalculateViewFamily(InOutSampleState, &Payload);
					FSceneView* View = const_cast<FSceneView*>(ViewFamily->Views[0]);

					// Now that we've modified all of the stencil values, we can submit them to be rendered.
					View->FinalPostProcessSettings.AddBlendable(StencilLayerMaterial, 1.0f);
					IBlendableInterface* BlendableInterface = Cast<IBlendableInterface>(StencilLayerMaterial);
					BlendableInterface->OverrideBlendableSettings(*View, 1.f);

					{
						TWeakObjectPtr<UTextureRenderTarget2D> ViewRenderTarget = GetOrCreateViewRenderTarget(InOutSampleState.BackbufferSize, (IViewCalcPayload*)(&Payload));
						check(ViewRenderTarget.IsValid());

						FRenderTarget* RenderTarget = ViewRenderTarget->GameThread_GetRenderTargetResource();
						check(RenderTarget);

						FCanvas Canvas = FCanvas(RenderTarget, nullptr, GetPipeline()->GetWorld(), View->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, 1.0f);
						GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.Get());
						
						ENQUEUE_RENDER_COMMAND(TransitionTextureSRVState)(
						[RenderTarget](FRHICommandListImmediate& RHICmdList) mutable
						{
							// Transition our render target from a render target view to a shader resource view to allow the UMG preview material to read from this Render Target.
							RHICmdList.Transition(FRHITransitionInfo(RenderTarget->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVGraphicsPixel));
						});

						// Readback + Accumulate.
						PostRendererSubmission(InOutSampleState, LayerPassIdentifier, GetOutputFileSortingOrder() + 1, Canvas);
					}
				}
			}

			// Now that all stencil layers have been rendered, we can restore the custom depth/stencil/etc. values so that the main render pass acts as the user expects next time.
			for (TPair<UPrimitiveComponent*, FStencilValues>& KVP : PreviousValues)
			{
				KVP.Key->SetCustomDepthStencilValue(KVP.Value.CustomStencil);
				KVP.Key->SetCustomDepthStencilWriteMask(KVP.Value.StencilMask);
				KVP.Key->SetRenderCustomDepth(KVP.Value.bRenderCustomDepth);
			}
		}
	}
}

void UMoviePipelineDeferredPassBase::OnTileStartImpl(FIntPoint TileIndexes)
{
	for (FMultiCameraViewStateData& CameraData : CameraViewStateData)
	{
		if (CameraData.SystemMemoryMirror && bRenderMainPass)
		{
			FMultiCameraViewStateData::FPerTile* TileData = CameraData.TileData.Find(TileIndexes);
			if (TileData && TileData->SceneViewStates.Num())
			{
				TileData->SceneViewStates[0].GetReference()->SystemMemoryMirrorRestore(CameraData.SystemMemoryMirror.Get());
			}
		}
	}
}

void UMoviePipelineDeferredPassBase::OnTileEndImpl(FIntPoint TileIndexes)
{
	for (FMultiCameraViewStateData& CameraData : CameraViewStateData)
	{
		if (CameraData.SystemMemoryMirror && bRenderMainPass)
		{
			FMultiCameraViewStateData::FPerTile* TileData = CameraData.TileData.Find(TileIndexes);
			if (TileData && TileData->SceneViewStates.Num())
			{
				TileData->SceneViewStates[0].GetReference()->SystemMemoryMirrorBackup(CameraData.SystemMemoryMirror.Get());
			}
		}
	}
}

TFunction<void(TUniquePtr<FImagePixelData>&&)> UMoviePipelineDeferredPassBase::MakeForwardingEndpoint(const FMoviePipelinePassIdentifier InPassIdentifier, const FMoviePipelineRenderPassMetrics& InSampleState)
{
	// We have a pool of accumulators - we multi-thread the accumulation on the task graph, and for each frame,
	// the task has the previous samples as pre-reqs to keep the accumulation in order. However, each accumulator
	// can only work on one frame at a time, so we create a pool of them to work concurrently. This needs a limit
	// as large accumulations (16k) can take a lot of system RAM.
	TSharedPtr<FAccumulatorPool::FAccumulatorInstance, ESPMode::ThreadSafe> SampleAccumulator = nullptr;
	{
		SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_WaitForAvailableAccumulator);
		SampleAccumulator = AccumulatorPool->BlockAndGetAccumulator_GameThread(InSampleState.OutputState.OutputFrameNumber, InPassIdentifier);
	}

	TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> FramePayload = MakeShared<FImagePixelDataPayload, ESPMode::ThreadSafe>();
	FramePayload->PassIdentifier = InPassIdentifier;
	FramePayload->SampleState = InSampleState;
	FramePayload->SortingOrder = GetOutputFileSortingOrder() + 1;

	MoviePipeline::FImageSampleAccumulationArgs AccumulationArgs;
	{
		AccumulationArgs.OutputMerger = GetPipeline()->OutputBuilder;
		AccumulationArgs.ImageAccumulator = StaticCastSharedPtr<FImageOverlappedAccumulator>(SampleAccumulator->Accumulator);
		AccumulationArgs.bAccumulateAlpha = bAccumulatorIncludesAlpha;
	}

	auto Callback = [this, FramePayload, AccumulationArgs, SampleAccumulator](TUniquePtr<FImagePixelData>&& InPixelData)
	{
		// Transfer the framePayload to the returned data
		TUniquePtr<FImagePixelData> PixelDataWithPayload = nullptr;
		switch (InPixelData->GetType())
		{
		case EImagePixelType::Color:
		{
			TImagePixelData<FColor>* SourceData = static_cast<TImagePixelData<FColor>*>(InPixelData.Get());
			PixelDataWithPayload = MakeUnique<TImagePixelData<FColor>>(InPixelData->GetSize(), MoveTemp(SourceData->Pixels), FramePayload);
			break;
		}
		case EImagePixelType::Float16:
		{
			TImagePixelData<FFloat16Color>* SourceData = static_cast<TImagePixelData<FFloat16Color>*>(InPixelData.Get());
			PixelDataWithPayload = MakeUnique<TImagePixelData<FFloat16Color>>(InPixelData->GetSize(), MoveTemp(SourceData->Pixels), FramePayload);
			break;
		}
		case EImagePixelType::Float32:
		{
			TImagePixelData<FLinearColor>* SourceData = static_cast<TImagePixelData<FLinearColor>*>(InPixelData.Get());
			PixelDataWithPayload = MakeUnique<TImagePixelData<FLinearColor>>(InPixelData->GetSize(), MoveTemp(SourceData->Pixels), FramePayload);
			break;
		}
		default:
			checkNoEntry();
		}

		bool bFinalSample = FramePayload->IsLastTile() && FramePayload->IsLastTemporalSample();
		bool bFirstSample = FramePayload->IsFirstTile() && FramePayload->IsFirstTemporalSample();

		FMoviePipelineBackgroundAccumulateTask Task;
		// There may be other accumulations for this accumulator which need to be processed first
		Task.LastCompletionEvent = SampleAccumulator->TaskPrereq;

		FGraphEventRef Event = Task.Execute([PixelData = MoveTemp(PixelDataWithPayload), AccumulationArgs, bFinalSample, SampleAccumulator]() mutable
		{
			// Enqueue a encode for this frame onto our worker thread.
			MoviePipeline::AccumulateSample_TaskThread(MoveTemp(PixelData), AccumulationArgs);
			if (bFinalSample)
			{
				SampleAccumulator->TaskPrereq = nullptr;
				SampleAccumulator->bIsActive = false;
			}
		});
		SampleAccumulator->TaskPrereq = Event;

		this->OutstandingTasks.Add(Event);
	};

	return Callback;
}

UE::MoviePipeline::FImagePassCameraViewData UMoviePipelineDeferredPassBase::GetCameraInfo(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload) const
{
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	const int32 NumCameras = GetNumCamerasToRender();
	
	if (NumCameras == 1)
	{
		// If there's only one camera being used we can use the parent class which assumes the camera comes from the PlayerCameraManager
		return Super::GetCameraInfo(InOutSampleState, OptPayload);
	}
	else
	{
		UE::MoviePipeline::FImagePassCameraViewData OutCameraData;

		// Here's where it gets a lot more complicated. There's a number of properties we need to fetch from a camera manually to fill out the minimal view info.
		UCameraComponent* OutCamera = nullptr;

		GetPipeline()->GetSidecarCameraData(CurrentShot, InOutSampleState.OutputState.CameraIndex, OutCameraData.ViewInfo, &OutCamera);
		if (OutCamera)
		{
			// This has to come from the main camera for consistency's sake, and it's not a per-camera setting in the editor.
			OutCameraData.ViewActor = GetPipeline()->GetWorld()->GetFirstPlayerController()->GetViewTarget();

			// Try adding cine-camera specific metadata (not all animated cameras are cine cameras though)
			UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(OutCamera);
			if (CineCameraComponent)
			{
				// ToDo: This is still wrong, PassIdentifier.CameraName needs to come in from the InOutSampleState somewhere.
				UE::MoviePipeline::GetMetadataFromCineCamera(CineCameraComponent, PassIdentifier.CameraName, PassIdentifier.Name, OutCameraData.FileMetadata);

				// We only do this in the multi-camera case because the single camera case is covered by the main Rendering loop.
				FVector PreviousLocation = OutCameraData.ViewInfo.Location;
				FRotator PreviousRotation = OutCameraData.ViewInfo.Rotation;
				if (OutCameraData.ViewInfo.PreviousViewTransform.IsSet())
				{
					PreviousLocation = OutCameraData.ViewInfo.PreviousViewTransform->GetLocation();
					PreviousRotation = FRotator(OutCameraData.ViewInfo.PreviousViewTransform->GetRotation());
				}

				UE::MoviePipeline::GetMetadataFromCameraLocRot(PassIdentifier.CameraName, PassIdentifier.Name, OutCameraData.ViewInfo.Location, OutCameraData.ViewInfo.Rotation, PreviousLocation, PreviousRotation, OutCameraData.FileMetadata);
			}
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "Failed to find Camera Component for Shot: %d CameraIndex: %d", GetPipeline()->GetCurrentShotIndex(), InOutSampleState.OutputState.CameraIndex);
		}

		return OutCameraData;
	}
}

void UMoviePipelineDeferredPassBase::BlendPostProcessSettings(FSceneView* InView, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload)
{
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	const int32 NumCameras = GetNumCamerasToRender();

	// The primary camera should still respect the world post processing volumes and should already be the viewtarget.
	if (NumCameras == 1)
	{
		// If there's only one camera being used we can use the parent class which assumes the camera comes from the PlayerCameraManager
		Super::BlendPostProcessSettings(InView, InOutSampleState, OptPayload);
	}
	else
	{
		UCameraComponent* OutCamera = nullptr;
		FMinimalViewInfo OutViewInfo;

		GetPipeline()->GetSidecarCameraData(CurrentShot, InOutSampleState.OutputState.CameraIndex, OutViewInfo, &OutCamera);
		if (!OutCamera)
		{
			// GetCameraInfo will have already printed a warning
			return;
		}

		// For sidecar cameras we need to do the blending of PP volumes and camera PP manually.
		FVector ViewLocation = OutCamera->GetComponentLocation();
		UE::MoviePipeline::DoPostProcessBlend(ViewLocation, GetWorld(), OutViewInfo, InView);
	}
}


void UMoviePipelineDeferredPassBase::PostRendererSubmission(const FMoviePipelineRenderPassMetrics& InSampleState, const FMoviePipelinePassIdentifier InPassIdentifier, const int32 InSortingOrder, FCanvas& InCanvas)
{
	// If this was just to contribute to the history buffer, no need to go any further.
	if (InSampleState.bDiscardResult)
	{
		return;
	}

	// Draw letterboxing, except for 1 pixel outline for anti-aliasing purposes.
	// This extra pixel thickness will be done later after accumulating the last tile.

	MoviePipeline::FLetterboxData LetterboxData;

	APlayerCameraManager* PlayerCameraManager = GetPipeline()->GetWorld()->GetFirstPlayerController()->PlayerCameraManager;
	if(PlayerCameraManager && PlayerCameraManager->GetCameraCacheView().bConstrainAspectRatio)
	{
		const FMinimalViewInfo CameraCache = PlayerCameraManager->GetCameraCacheView();
		UMoviePipelineOutputSetting* OutputSettings = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
		check(OutputSettings);
		
		// Taking overscan into account.
		const FIntPoint FullOutputSize = InSampleState.OverscannedResolution;

		const float OutputSizeAspectRatio = FullOutputSize.X / (float)FullOutputSize.Y;
		const float CameraAspectRatio = bAllowCameraAspectRatio ? CameraCache.AspectRatio : OutputSizeAspectRatio;

		// Note: We use RoundToInt to match the unscaled/constrained view rectangle calculated in FViewport::CalculateViewExtents
		const FIntPoint ConstrainedFullSize = CameraAspectRatio > OutputSizeAspectRatio ?
			FIntPoint(FullOutputSize.X, FMath::RoundToInt((double)FullOutputSize.X / (double)CameraAspectRatio)) :
			FIntPoint(FMath::RoundToInt(CameraAspectRatio * FullOutputSize.Y), FullOutputSize.Y);

		const FIntPoint TileViewMin = InSampleState.OverlappedOffset;
		const FIntPoint TileViewMax = TileViewMin + InSampleState.BackbufferSize;

		// Calculate camera ratio constrained rect.

		FIntPoint ConstrainedViewMin = (FullOutputSize - ConstrainedFullSize) / 2;
		FIntPoint ConstrainedViewMax = ConstrainedViewMin + ConstrainedFullSize;

		// When there is overlap, adjust the offsets to leave a 1-pixel margin around the active pixel area, so that
		// anti-aliasing is able to sample the original pixel during accumulation, instead of 
		// the letterbox pixel which would otherwise darken the edges of the active area.
		LetterboxData.bDrawLetterboxBorder = InSampleState.OverlappedPad != FIntPoint::ZeroValue;
		LetterboxData.FrameActiveArea = FIntRect(ConstrainedViewMin, ConstrainedViewMax);

		// Clip it by the tile rect.

		ConstrainedViewMin = FIntPoint(FMath::Clamp(ConstrainedViewMin.X, TileViewMin.X, TileViewMax.X),
			FMath::Clamp(ConstrainedViewMin.Y, TileViewMin.Y, TileViewMax.Y));

		ConstrainedViewMax = FIntPoint(FMath::Clamp(ConstrainedViewMax.X, TileViewMin.X, TileViewMax.X),
			FMath::Clamp(ConstrainedViewMax.Y, TileViewMin.Y, TileViewMax.Y));

		// Difference between the clipped constrained rect and the tile rect

		const FIntPoint OffsetMin = ConstrainedViewMin - TileViewMin;
		const FIntPoint OffsetMax = TileViewMax - ConstrainedViewMax;

		const int32 AntiAliasMargin = LetterboxData.bDrawLetterboxBorder ? 1 : 0;

		const int32 LeftThickness   = FMath::Max(OffsetMin.X - AntiAliasMargin, 0);
		const int32 RightThickness  = FMath::Max(OffsetMax.X - AntiAliasMargin, 0);
		const int32 TopThickness    = FMath::Max(OffsetMin.Y - AntiAliasMargin, 0);
		const int32 BottomThickness = FMath::Max(OffsetMax.Y - AntiAliasMargin, 0);

		// These thicknesses will inform a later stage which pixels have been cleared out and that 
		// should not be sampled.
		LetterboxData.LeftSamplePixelsClearedBeforeAccumulation = LeftThickness;
		LetterboxData.RightSamplePixelsClearedBeforeAccumulation = RightThickness;
		LetterboxData.TopSamplePixelsClearedBeforeAccumulation = TopThickness;
		LetterboxData.BottomSamplePixelsClearedBeforeAccumulation = BottomThickness;

		// Clear left
		if (LeftThickness > 0)
		{
			InCanvas.DrawTile(0, 0, LeftThickness, InSampleState.BackbufferSize.Y,
				0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::Black, nullptr, false);
		}

		// Clear right
		if (RightThickness > 0)
		{
			InCanvas.DrawTile(InSampleState.BackbufferSize.X - RightThickness, 0, InSampleState.BackbufferSize.X, InSampleState.BackbufferSize.Y,
				0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::Black, nullptr, false);
		}

		// Clear top
		if (TopThickness > 0)
		{
			InCanvas.DrawTile(0, 0, InSampleState.BackbufferSize.X, TopThickness,
				0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::Black, nullptr, false);
		}

		// Clear bottom
		if (BottomThickness > 0)
		{
			InCanvas.DrawTile(0, InSampleState.BackbufferSize.Y - BottomThickness, InSampleState.BackbufferSize.X, InSampleState.BackbufferSize.Y,
				0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::Black, nullptr, false);
		}

		InCanvas.Flush_GameThread(true);
	}

	// We have a pool of accumulators - we multi-thread the accumulation on the task graph, and for each frame,
	// the task has the previous samples as pre-reqs to keep the accumulation in order. However, each accumulator
	// can only work on one frame at a time, so we create a pool of them to work concurrently. This needs a limit
	// as large accumulations (16k) can take a lot of system RAM.
	TSharedPtr<FAccumulatorPool::FAccumulatorInstance, ESPMode::ThreadSafe> SampleAccumulator = nullptr;
	{
		SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_WaitForAvailableAccumulator);
		SampleAccumulator = AccumulatorPool->BlockAndGetAccumulator_GameThread(InSampleState.OutputState.OutputFrameNumber, InPassIdentifier);
	}

	TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> FramePayload = MakeShared<FImagePixelDataPayload, ESPMode::ThreadSafe>();
	FramePayload->PassIdentifier = InPassIdentifier;
	FramePayload->SampleState = InSampleState;
	FramePayload->SortingOrder = InSortingOrder;

	TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> LocalSurfaceQueue = GetOrCreateSurfaceQueue(InSampleState.BackbufferSize, (IViewCalcPayload*)(&FramePayload.Get()));

	MoviePipeline::FImageSampleAccumulationArgs AccumulationArgs;
	{
		AccumulationArgs.OutputMerger = GetPipeline()->OutputBuilder;
		AccumulationArgs.ImageAccumulator = StaticCastSharedPtr<FImageOverlappedAccumulator>(SampleAccumulator->Accumulator);
		AccumulationArgs.bAccumulateAlpha = bAccumulatorIncludesAlpha;
		AccumulationArgs.LetterboxData = LetterboxData;
	}

	auto Callback = [this, FramePayload, AccumulationArgs, SampleAccumulator](TUniquePtr<FImagePixelData>&& InPixelData)
	{
		bool bFinalSample = FramePayload->IsLastTile() && FramePayload->IsLastTemporalSample();
		bool bFirstSample = FramePayload->IsFirstTile() && FramePayload->IsFirstTemporalSample();

		FMoviePipelineBackgroundAccumulateTask Task;
		// There may be other accumulations for this accumulator which need to be processed first
		Task.LastCompletionEvent = SampleAccumulator->TaskPrereq;

		FGraphEventRef Event = Task.Execute([PixelData = MoveTemp(InPixelData), AccumulationArgs, bFinalSample, SampleAccumulator]() mutable
		{
			// Enqueue a encode for this frame onto our worker thread.
			MoviePipeline::AccumulateSample_TaskThread(MoveTemp(PixelData), AccumulationArgs);

			if (bFinalSample)
			{
				// Final sample has now been executed, break the pre-req chain and free the accumulator for reuse.
				SampleAccumulator->bIsActive = false;
				SampleAccumulator->TaskPrereq = nullptr;
			}
		});
		SampleAccumulator->TaskPrereq = Event;

		this->OutstandingTasks.Add(Event);
	};

	FRenderTarget* RenderTarget = InCanvas.GetRenderTarget();

	ENQUEUE_RENDER_COMMAND(CanvasRenderTargetResolveCommand)(
		[LocalSurfaceQueue, FramePayload, Callback, RenderTarget](FRHICommandListImmediate& RHICmdList) mutable
		{
			// Enqueue a encode for this frame onto our worker thread.
			LocalSurfaceQueue->OnRenderTargetReady_RenderThread(RenderTarget->GetRenderTargetTexture(), FramePayload, MoveTemp(Callback));
		});

}

bool UMoviePipelineDeferredPassBase::IsAutoExposureAllowed(const FMoviePipelineRenderPassMetrics& InSampleState) const
{
	// For high-res tiling, check if a view state exists for auto-exposure, with key {-1,-1}
	return (InSampleState.GetTileCount() == 1) || (CameraViewStateData.Num() && CameraViewStateData[0].TileData.Contains(FIntPoint(-1, -1)));
}

#if WITH_EDITOR
FText UMoviePipelineDeferredPass_PathTracer::GetFooterText(UMoviePipelineExecutorJob* InJob) const {
	return NSLOCTEXT(
		"MovieRenderPipeline",
		"DeferredBasePassSetting_FooterText_PathTracer",
		"Sampling for the Path Tracer is controlled by the Anti-Aliasing settings and the Reference Motion Blur setting.\n"
		"All other Path Tracer settings are taken from the Post Process settings.");
}
#endif

bool UMoviePipelineDeferredPassBase::CheckIfPathTracerIsSupported() const
{
	bool bSupportsPathTracing = false;
	if (IsRayTracingEnabled())
	{
		IConsoleVariable* PathTracingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PathTracing"));
		if (PathTracingCVar)
		{
			bSupportsPathTracing = PathTracingCVar->GetInt() != 0;
		}
	}
	return bSupportsPathTracing;
}

void UMoviePipelineDeferredPassBase::PathTracerValidationImpl()
{
	const bool bSupportsPathTracing = CheckIfPathTracerIsSupported();

	if (!bSupportsPathTracing)
	{
		const FText ValidationWarning = NSLOCTEXT("MovieRenderPipeline", "PathTracerValidation_Unsupported", "Path Tracing is currently not enabled for this project and this render pass will not work.");
		ValidationResults.Add(ValidationWarning);
		ValidationState = EMoviePipelineValidationState::Warnings;
	}
}

void UMoviePipelineDeferredPass_PathTracer::ValidateStateImpl()
{
	Super::ValidateStateImpl();
	PathTracerValidationImpl();
}

void UMoviePipelineDeferredPass_PathTracer::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	if (!CheckIfPathTracerIsSupported())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Cannot render a Path Tracer pass, Path Tracer is not enabled by this project.");
		GetPipeline()->Shutdown(true);
		return;
	}

	Super::SetupImpl(InPassInitSettings);
}

TSharedPtr<FSceneViewFamilyContext> UMoviePipelineDeferredPass_PathTracer::CalculateViewFamily(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload)
{
	// remove sub-pixel shift, since the path tracer does its own anti-aliasing
	InOutSampleState.SpatialShiftX = 0;
	InOutSampleState.SpatialShiftY = 0;
	InOutSampleState.OverlappedSubpixelShift = FVector2d(0.5, 0.5);
	return Super::CalculateViewFamily(InOutSampleState, OptPayload);
}

void UMoviePipelineDeferredPass_PathTracer::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesPathTracer = true;
	InTelemetry->bUsesPPMs |= Algo::AnyOf(AdditionalPostProcessMaterials, [](const FMoviePipelinePostProcessPass& Pass) { return Pass.bEnabled; });
}

bool UMoviePipelineDeferredPassBase::IsUsingDataLayers() const
{
	int32 NumDataLayers = 0;
	for (FSoftObjectPath DataLayerAssetPath : DataLayers)
	{
		UDataLayerAsset* DataLayerAsset = Cast<UDataLayerAsset>(DataLayerAssetPath.TryLoad());
		if (DataLayerAsset)
		{
			NumDataLayers++;
		}
	}
	return NumDataLayers > 0;
}

int32 UMoviePipelineDeferredPassBase::GetNumStencilLayers() const
{
	if (IsUsingDataLayers())
	{
		// Because DataLayers are an asset, they can actually be null despite being in this list.
		int32 NumDataLayers = 0;
		for (FSoftObjectPath DataLayerAssetPath : DataLayers)
		{
			UDataLayerAsset* DataLayerAsset = Cast<UDataLayerAsset>(DataLayerAssetPath.TryLoad());
			if (DataLayerAsset)
			{
				NumDataLayers++;
			}
		}
		return NumDataLayers;
	}
	return ActorLayers.Num();
}

TArray<FString> UMoviePipelineDeferredPassBase::GetStencilLayerNames() const
{
	return UniqueStencilLayerNames;
}

FSoftObjectPath UMoviePipelineDeferredPassBase::GetValidDataLayerByIndex(const int32 InIndex) const
{
	int32 NumValidDataLayers = 0;
	for (FSoftObjectPath DataLayerAssetPath : DataLayers)
	{
		UDataLayerAsset* DataLayerAsset = Cast<UDataLayerAsset>(DataLayerAssetPath.TryLoad());
		if (DataLayerAsset)
		{
			if (InIndex == NumValidDataLayers)
			{
				return DataLayerAssetPath;
			}

			NumValidDataLayers++;
		}
	}

	return FSoftObjectPath();
}

bool UMoviePipelineDeferredPassBase::IsActorInLayer(AActor* InActor, int32 InLayerIndex) const
{
	if (IsUsingDataLayers())
	{
		FSoftObjectPath DataLayerAssetPath = GetValidDataLayerByIndex(InLayerIndex);
		UDataLayerAsset* DataLayerAsset = Cast<UDataLayerAsset>(DataLayerAssetPath.TryLoad());
		if (DataLayerAsset)
		{
			return InActor->ContainsDataLayer(DataLayerAsset);
		}
	}
	else
	{
		const FName& LayerName = ActorLayers[InLayerIndex].Name;
		return InActor->Layers.Contains(LayerName);
	}

	return false;
}

bool UMoviePipelineDeferredPassBase::IsActorInAnyStencilLayer(AActor* InActor) const
{
	bool bInLayer = false;
	if (IsUsingDataLayers())
	{
		for (FSoftObjectPath DataLayerAssetPath : DataLayers)
		{
			UDataLayerAsset* DataLayerAsset = Cast<UDataLayerAsset>(DataLayerAssetPath.TryLoad());
			if (DataLayerAsset)
			{
				bInLayer = !InActor->ContainsDataLayer(DataLayerAsset);

				if (!bInLayer)
				{
					break;
				}
			}
		}
	}
	else
	{
		for (const FActorLayer& AllLayer : ActorLayers)
		{
			bInLayer = !InActor->Layers.Contains(AllLayer.Name);
			if (!bInLayer)
			{
				break;
			}
		}
	}

	return bInLayer;
}

void UMoviePipelineDeferredPassBase::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesDeferred = true;
	InTelemetry->bUsesPPMs |= Algo::AnyOf(AdditionalPostProcessMaterials, [](const FMoviePipelinePostProcessPass& Pass) { return Pass.bEnabled; });
}

FString UMoviePipelineDeferredPassBase::GetNameForPostProcessMaterial(const UMaterialInterface* InMaterial)
{
	FString MaterialName = InMaterial->GetName();

	// Use the name specified in the post process pass if it's not empty. Otherwise fall back to the material's name.
	const FMoviePipelinePostProcessPass* MatchingPass = Algo::FindByPredicate(AdditionalPostProcessMaterials, [InMaterial](const FMoviePipelinePostProcessPass& InPass) { return InPass.Material == InMaterial; });
	if (MatchingPass && !MatchingPass->Name.IsEmpty())
	{
		MaterialName = MatchingPass->Name;
	}

	return MaterialName;
}
