// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineEXROutput.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Async/Async.h"
#include "Misc/Paths.h"
#include "HAL/PlatformTime.h"
#include "Math/Float16.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineCameraSetting.h"
#include "ImageWriteQueue.h"
#include "MoviePipeline.h"
#include "MoviePipelineImageQuantization.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineTelemetry.h"
#include "IOpenExrRTTIModule.h"
#include "Modules/ModuleManager.h"
#include "MoviePipelineUtils.h"
#include "ColorManagement/ColorSpace.h"
#include "HDRHelper.h"
#include "Algo/Accumulate.h"

THIRD_PARTY_INCLUDES_START
#include "OpenEXR/ImfChannelList.h"
#include "OpenEXR/ImfStandardAttributes.h"
#include "OpenEXR/ImfMultiPartOutputFile.h"
#include "OpenEXR/ImfOutputPart.h"

THIRD_PARTY_INCLUDES_END

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineEXROutput)

#if WITH_UNREALEXR

// NOTE: see also ExrImageWrapper

namespace UE
{
	namespace MoviePipeline
	{
		TAutoConsoleVariable<bool> CVarMoviePipelinePadLayersForMultiPartEXR(
			TEXT("MoviePipeline.PadLayersForMultiPartEXR"),
			true,
			TEXT("Indicates that layers in a multi-part EXR file should be padded to match the resolution of the largest layer.\n")
			TEXT("When enabled, padding will be applied to all layers to ensure their data windows match when written to a multi-part EXR file. When disabled,\n")
			TEXT("each layer will have its own resolution and data window, which may reduce filesize, but not all software supports this when using multi-part EXR files"),
			ECVF_Default);
	}
}

class FExrMemStreamOut : public Imf::OStream
{
public:

	FExrMemStreamOut()
		: Imf::OStream("")
		, Pos(0)
	{
	}

	// InN must be 32bit to match the abstract interface.
	virtual void write(const char c[/*n*/], int32 InN)
	{
		int64 SrcN = (int64)InN;
		int64 DestPost = Pos + SrcN;
		if (DestPost > Data.Num())
		{
			Data.AddUninitialized(DestPost - Data.Num());
		}

		for (int64 i = 0; i < SrcN; ++i)
		{
			Data[Pos + i] = c[i];
		}
		Pos += SrcN;
	}


	//---------------------------------------------------------
	// Get the current writing position, in bytes from the
	// beginning of the file.  If the next call to write() will
	// start writing at the beginning of the file, tellp()
	// returns 0.
	//---------------------------------------------------------

	uint64_t tellp() override
	{
		return Pos;
	}


	//-------------------------------------------
	// Set the current writing position.
	// After calling seekp(i), tellp() returns i.
	//-------------------------------------------

	void seekp(uint64_t pos) override
	{
		Pos = pos;
	}


	int64 Pos;
	TArray64<uint8> Data;
};

bool FEXRImageWriteTask::RunTask()
{
	bool bSuccess = WriteToDisk();

	if (OnCompleted)
	{
		AsyncTask(ENamedThreads::GameThread, [bSuccess, LocalOnCompleted = MoveTemp(OnCompleted)] { LocalOnCompleted(bSuccess); });
	}

	return bSuccess;
}

void FEXRImageWriteTask::OnAbandoned()
{
	if (OnCompleted)
	{
		AsyncTask(ENamedThreads::GameThread, [LocalOnCompleted = MoveTemp(OnCompleted)] { LocalOnCompleted(false); });
	}
}

bool FEXRImageWriteTask::GenerateFilePartsForResolution(const FIntPoint& Resolution, const TArray<FImagePixelData*>& InLayers, const EEXRCompressionFormat CompressionType, const FString& InPartName, FEXRPart& OutPart, TArray<TUniquePtr<FImagePixelData>>& OutQuantizedLayers)
{
	// Generate the header that's either used for the entire file, or per-layer in multi-part EXRs.
	auto GenerateHeader = [this, &InPartName]
		(const IMATH_NAMESPACE::Box2i& InDisplayWindow, const IMATH_NAMESPACE::Box2i& InDataWindow, const Imf::Compression CompressionType) -> Imf::Header
	{
		Imf::Header Header(InDisplayWindow, InDataWindow, 1, Imath::V2f(0, 0), 1, Imf::LineOrder::INCREASING_Y, CompressionType);

		if (bMultipart)
		{
			Header.setName(TCHAR_TO_ANSI(*InPartName));
			Header.setType("scanlineimage");
		}

		// Add all of the `FileMetadata` metadata into the header.
        AddFileMetadata(Header);

		return Header;
	};
	
	bool bSuccess = false;

	OutPart.Scanlines = bPadToDataWindowSize ? Height : Resolution.Y;
	
	const bool bIsCropRectValid = !CropRectangle.IsEmpty() &&
		CropRectangle.Min.X >= 0 && CropRectangle.Min.Y >= 0 &&
		CropRectangle.Max.X <= Width && CropRectangle.Max.Y <= Height;

	// Display window is always the size of the crop rectangle if one is provided, otherwise the resolution of the EXR file
	const FIntPoint DisplayWindowRes = bIsCropRectValid ? CropRectangle.Size() : FIntPoint(Width, Height);

	// When overscan/crop is provided, offset the data window into the negative region
	const FIntPoint DataWindowOffset = bIsCropRectValid ? CropRectangle.Min : FIntPoint::ZeroValue;

	// If the part's resolution does not match the file's overall resolution, we center the layers' pixels in the
	// overall file, so offset the data window accordingly
	const FIntPoint ResolutionDiffOffset = bPadToDataWindowSize ? FIntPoint::ZeroValue : (FIntPoint(Width, Height) - Resolution) / 2;
	const FIntPoint DataWindowResolution = bPadToDataWindowSize ? FIntPoint(Width, Height) : Resolution;
	
	IMATH_NAMESPACE::V2i DataWindowTopLeft = IMATH_NAMESPACE::V2i(-DataWindowOffset.X + ResolutionDiffOffset.X, -DataWindowOffset.Y + ResolutionDiffOffset.Y);
	IMATH_NAMESPACE::V2i DataWindowBottomRight = IMATH_NAMESPACE::V2i(DataWindowTopLeft.x + DataWindowResolution.X - 1, DataWindowTopLeft.y + DataWindowResolution.Y - 1);
	
	// Data Window specifies how much data is in the actual file, ie: 1920x1080
	IMATH_NAMESPACE::Box2i DataWindow = IMATH_NAMESPACE::Box2i(DataWindowTopLeft, DataWindowBottomRight);

	// Display Window specifies the total 'visible' area of the output file. 
	// The Display Window always starts at 0,0, but Data Window can go negative to
	// support having pixels out of bounds (such as camera overscan).
	IMATH_NAMESPACE::Box2i DisplayWindow = IMATH_NAMESPACE::Box2i(IMATH_NAMESPACE::V2i(0, 0),IMATH_NAMESPACE::V2i(DisplayWindowRes.X - 1, DisplayWindowRes.Y - 1));

	static_assert(static_cast<uint8>(EEXRCompressionFormat::Max) == Imf::Compression::NUM_COMPRESSION_METHODS);
	Imf::Compression FileCompression = static_cast<Imf::Compression>(CompressionType);
	
	OutPart.Header = GenerateHeader(DisplayWindow, DataWindow, FileCompression);

	// If using DWA compression, specify the compression level directly on the header.
	if (FileCompression == Imf::Compression::DWAA_COMPRESSION ||
		FileCompression == Imf::Compression::DWAB_COMPRESSION)
	{
		OutPart.Header.dwaCompressionLevel() = static_cast<float>(CompressionLevel);
	}

	if (ColorSpaceChromaticities.Num() > 0)
	{
		if (ensureMsgf(ColorSpaceChromaticities.Num() == 4, TEXT("Four chromaticity coordinates expected.")))
		{
			Imf::Chromaticities Chromaticities = {
				IMATH_NAMESPACE::V2f((float)ColorSpaceChromaticities[0].X, (float)ColorSpaceChromaticities[0].Y),
				IMATH_NAMESPACE::V2f((float)ColorSpaceChromaticities[1].X, (float)ColorSpaceChromaticities[1].Y),
				IMATH_NAMESPACE::V2f((float)ColorSpaceChromaticities[2].X, (float)ColorSpaceChromaticities[2].Y),
				IMATH_NAMESPACE::V2f((float)ColorSpaceChromaticities[3].X, (float)ColorSpaceChromaticities[3].Y),
			};
			Imf::addChromaticities(OutPart.Header, Chromaticities);
		}
	}

	// If we have to quantize the data (ie: Upscale 8 bit to 16 bit) we need to store them long enough for the file to get written.
	for (FImagePixelData* Layer : InLayers)
	{
		uint8 RawBitDepth = Layer->GetBitDepth();
		check(RawBitDepth == 8 || RawBitDepth == 16 || RawBitDepth == 32);

		if (!bPadToDataWindowSize && !ensureAlwaysMsgf(Layer->GetSize().X == Resolution.X && Layer->GetSize().Y == Resolution.Y, TEXT("Skipping layer due to mismatched width/height from rest of EXR file!")))
		{
			continue;
		}

		void const* RawDataPtr = nullptr;
		int64 RawDataSize;
		bSuccess = Layer->GetRawData(RawDataPtr, RawDataSize);
		if (!bSuccess)
		{
			UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to retrieve raw data from image data for writing. Bailing.");
			break;
		}

		const bool bNeedsInflation = Layer->GetSize().X < Width || Layer->GetSize().Y < Height;
		if (bPadToDataWindowSize && bNeedsInflation)
		{
			const FIntPoint InflateOffset = (Layer->GetSize() - FIntPoint(Width, Height)) / 2;
			const FIntRect InflateRect = FIntRect(InflateOffset.X, InflateOffset.Y, InflateOffset.X + Width, InflateOffset.Y + Height);

			switch (Layer->GetType())
			{
			case EImagePixelType::Color:
				{
					TAsyncCropImage<FColor> CropImagePreprocessor(InflateRect);
					CropImagePreprocessor(Layer);
					OutQuantizedLayers.Add(MoveTemp(CropImagePreprocessor.OutCroppedImage));
				}
				break;

			case EImagePixelType::Float16:
				{
					TAsyncCropImage<FFloat16Color> CropImagePreprocessor(InflateRect);
					CropImagePreprocessor(Layer);
					OutQuantizedLayers.Add(MoveTemp(CropImagePreprocessor.OutCroppedImage));
				}
				break;

			case EImagePixelType::Float32:
				{
					TAsyncCropImage<FLinearColor> CropImagePreprocessor(InflateRect);
					CropImagePreprocessor(Layer);
					OutQuantizedLayers.Add(MoveTemp(CropImagePreprocessor.OutCroppedImage));
				}
				break;
			}
			
			FString LayerName = LayerNames.FindOrAdd(Layer);
			if (LayerName.Len() > 0)
			{
				LayerNames.Add(OutQuantizedLayers.Last().Get(), LayerName);
			}
			
			Layer = OutQuantizedLayers.Last().Get();
		}
		
		switch (RawBitDepth)
		{
		case 8:
		{
			TUniquePtr<FImagePixelData> QuantizedPixelData = UE::MoviePipeline::QuantizeImagePixelDataToBitDepth(Layer, 16);
			OutQuantizedLayers.Add(MoveTemp(QuantizedPixelData));

			// Add an entry in the LayerNames table if needed since it matches by Layer pointer but that has changed.
			FString LayerName = LayerNames.FindOrAdd(Layer);
			if (LayerName.Len() > 0)
			{
				LayerNames.Add(OutQuantizedLayers.Last().Get(), LayerName);
			}

			OutPart.BytesWritten = CompressRaw<Imf::HALF>(OutPart.Header, OutPart.FrameBuffer, OutQuantizedLayers.Last().Get());
		}
			break;
		case 16:
			OutPart.BytesWritten = CompressRaw<Imf::HALF>(OutPart.Header, OutPart.FrameBuffer, Layer);
			break;
		case 32:
			OutPart.BytesWritten = CompressRaw<Imf::FLOAT>(OutPart.Header, OutPart.FrameBuffer, Layer);
			break;
		default:
			checkNoEntry();
		}
	}

	return bSuccess;
}

bool FEXRImageWriteTask::WriteToDisk()
{
	// Ensure that the payload filename has the correct extension for the format
	const TCHAR* FormatExtension = TEXT(".exr");
	if (FormatExtension && !Filename.EndsWith(FormatExtension))
	{
		Filename = FPaths::GetBaseFilename(Filename, false) + FormatExtension;
	}

	bool bSuccess = EnsureWritableFile();

	if (bSuccess)
	{
		PreProcess();

		FExrMemStreamOut OutputFile;
		TArray<FEXRPart> Parts;
		
		// If we have to quantize the data (ie: Upscale 8 bit to 16 bit) we need to store them long enough for the file to get written.
		TArray<TUniquePtr<FImagePixelData>> QuantizedLayers;
		
		if (bMultipart)
		{
			for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); ++LayerIndex)
			{
				FEXRPart Part;
				TUniquePtr<FImagePixelData>& Layer = Layers[LayerIndex];
				FString PartName = LayerNames.Contains(Layer.Get()) ? LayerNames[Layer.Get()] : TEXT("FinalImage");

				// Override the compression level for the layer if requested.
				const bool bUsingPerLayerCompressionType = CompressionByLayer.Num() == Layers.Num();
				const EEXRCompressionFormat LayerCompressionType = bUsingPerLayerCompressionType ? CompressionByLayer[LayerIndex] : Compression;
				
				if (!GenerateFilePartsForResolution(Layer->GetSize(), { Layer.Get() }, LayerCompressionType, PartName, Part, QuantizedLayers))
				{
					UE_LOGF(LogMovieRenderPipelineIO, Error, "There was an error writing layer '%ls' to the EXR file. The pixel format may not be compatible with this image type, or there was a resolution mismatch.", *LayerNames[Layer.Get()]);
					bSuccess = false;
					break;
				}
				
				Parts.Add(Part);
			}
		}
		else
		{
			FEXRPart Part;
			TArray<FImagePixelData*> LayerRawPtrs;
			Algo::Transform(Layers, LayerRawPtrs, [](const TUniquePtr<FImagePixelData>& Layer) { return Layer.Get(); });
			
			if (!GenerateFilePartsForResolution(FIntPoint(Width, Height), LayerRawPtrs, Compression, TEXT("FinalImage"), Part, QuantizedLayers))
			{
				bSuccess = false;
			}
			else
			{
				Parts.Add(Part);
			}
		}
		
		if (Parts.Num() == 0)
		{
			UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to write any layers to EXR file. The pixel format may not be compatible with this image type, or there was a resolution mismatch.");
			return false;
		}
		
		const int64 TotalBytesWritten = Algo::Accumulate(Parts, int64(0), [](int64 Sum, const FEXRPart& Part) { return Sum + Part.BytesWritten; });
		if (!ensureMsgf(TotalBytesWritten >= 0, TEXT("Invalid estimated EXR byte count: %lld"), TotalBytesWritten))
		{
			return false;
		}

		OutputFile.Data.Reserve(TotalBytesWritten);

		// This scope ensures that IMF::Outputfile creates a complete file by closing the file when it goes out of scope.
		// To complete the file, EXR seeks back into the file and writes the scanline offsets when the file is closed, 
		// which moves the tellp location. So file length is stored in advance for later use. The output file needs to be
		// created after the header information is filled.
		//
		// Note: OutputFile has an option to control the number of threads used to write the file. The default is fine;
		// providing too many threads here will massively decrease performance and bloat memory usage.
#if WITH_EDITOR
		try
#endif
		{
			if (bMultipart)
			{
				TArray<Imf::Header> Headers;
				Algo::Transform(Parts, Headers, [](const FEXRPart& Part) { return Part.Header; });
		
				Imf::Header* HeaderData = Headers.GetData();
				const int32 HeaderCount = Headers.Num();
		
				Imf::MultiPartOutputFile ImfMultiPartFile(OutputFile, HeaderData, HeaderCount);
				for (int32 PartIdx = 0; PartIdx < Parts.Num(); ++PartIdx)
				{
					FEXRPart& Part = Parts[PartIdx];
					
					Imf::OutputPart OutputPart(ImfMultiPartFile, PartIdx);
					OutputPart.setFrameBuffer(Part.FrameBuffer);
					OutputPart.writePixels(Part.Scanlines);
				}
			}
			else
			{
				if (Parts.Num() > 1)
				{
					UE_LOGF(LogMovieRenderPipelineIO, Warning, "Multiple headers were created which is only supported for multi-part EXR files, only the first header will be used");
				}
				
				Imf::OutputFile ImfFile(OutputFile, Parts[0].Header);
				ImfFile.setFrameBuffer(Parts[0].FrameBuffer);
				ImfFile.writePixels(Parts[0].Scanlines);
			}
		}
#if WITH_EDITOR
		catch (const IEX_NAMESPACE::BaseExc& Exception)
		{
			UE_LOGF(LogMovieRenderPipelineIO, Error, "Caught exception: %s", Exception.message().c_str());
		}
#endif
		
		// Now that the scope has closed for the Imf::OutputFile, now we can write the data to disk.
		if (bSuccess)
		{
			bSuccess = FFileHelper::SaveArrayToFile(OutputFile.Data, *Filename);
		} 
	}

	if (!bSuccess)
	{
		UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to write image to '%ls'. The pixel format may not be compatible with this image type, or there was an error writing to that filename.", *Filename);
	}

	return bSuccess;
}

static FString GetChannelName(const FString& InLayerName, const int32 InChannelIndex, const ERGBFormat InFormat)
{
	const int32 MaxChannels = 4;
	static const char* RGBAChannelNames[] = { "R", "G", "B", "A" };
	static const char* BGRAChannelNames[] = { "B", "G", "R", "A" };
	static const char* GrayChannelNames[] = { "G" };
	check(InChannelIndex < MaxChannels);

	const char** ChannelNames = BGRAChannelNames;

	switch (InFormat)
	{
		case ERGBFormat::RGBA:
		case ERGBFormat::RGBAF:
		{
			ChannelNames = RGBAChannelNames;
		}
		break;
		case ERGBFormat::BGRA:
		{
			ChannelNames = BGRAChannelNames;
		}
		break;
		case ERGBFormat::Gray:
		case ERGBFormat::GrayF:
		{
			check(InChannelIndex < UE_ARRAY_COUNT(GrayChannelNames));
			ChannelNames = GrayChannelNames;
		}
		break;
		default:
			checkNoEntry();
	}

	if (InLayerName.Len() > 0)
	{
		return FString::Printf(TEXT("%s.%hs"), *InLayerName, ChannelNames[InChannelIndex]);
	}
	else
	{
		return FString(ChannelNames[InChannelIndex]);
	}
}

static int32 GetComponentWidth(const EImagePixelType InPixelType)
{
	switch (InPixelType)
	{
	case EImagePixelType::Color: return 1;
	case EImagePixelType::Float16: return 2;
	case EImagePixelType::Float32: return 4;
	default:
		checkNoEntry();
	}
	return 1;
}

template <Imf::PixelType OutputFormat>
int64 FEXRImageWriteTask::CompressRaw(Imf::Header& InHeader, Imf::FrameBuffer& InFrameBuffer, FImagePixelData* InLayer)
{
	void const* RawDataPtr = nullptr;
	int64 RawDataSize;

	if (InLayer->GetRawData(RawDataPtr, RawDataSize) == false)
	{
		UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to retrieve raw data from image data for writing. Bailing.");
		return 0;
	}
		

	// Look up our layer name (if any).
	FString& LayerName = LayerNames.FindOrAdd(InLayer);
	int32 NumChannels = InLayer->GetNumChannels();
	int32 ComponentWidth = GetComponentWidth(InLayer->GetType());
	int32 LayerWidth = InLayer->GetSize().X;
	int32 LayerHeight = InLayer->GetSize().Y;

	for (int32 Channel = 0; Channel < NumChannels; Channel++)
	{
		FString ChannelName = GetChannelName(LayerName, Channel, InLayer->GetPixelLayout());

		// Insert the channel into the header with the right datatype.
		InHeader.channels().insert(TCHAR_TO_ANSI(*ChannelName), Imf::Channel(OutputFormat));
		IMATH_NAMESPACE::Box2i& DataWindow = InHeader.dataWindow();

		// Now insert the data for this channel. Unreal stores them interleaved.
		InFrameBuffer.insert(TCHAR_TO_ANSI(*ChannelName),	// Name
			Imf::Slice::Make(OutputFormat,						// Type
			(char*)RawDataPtr + (ComponentWidth * Channel), // Data Start (offset by component to match interleave)
				DataWindow,
				ComponentWidth * NumChannels,				// xStride
				LayerWidth * ComponentWidth * NumChannels));		// yStride
	}
	
	return int64(LayerWidth) * int64(LayerHeight) * NumChannels * int64(OutputFormat == 2 ? 4 : 2);
}

bool FEXRImageWriteTask::EnsureWritableFile()
{
	FString Directory = FPaths::GetPath(Filename);

	if (!IFileManager::Get().DirectoryExists(*Directory))
	{
		IFileManager::Get().MakeDirectory(*Directory);
	}

	// If the file doesn't exist, we're ok to continue
	if (IFileManager::Get().FileSize(*Filename) == -1)
	{
		return true;
	}
	// If we're allowed to overwrite the file, and we deleted it ok, we can continue
	else if (bOverwriteFile && FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*Filename))
	{
		return true;
	}
	// We can't write to the file
	else
	{
		UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to write image to '%ls'. Should Overwrite: %d - If we should have overwritten the file, we failed to delete the file. If we shouldn't have overwritten the file the file already exists so we can't replace it.", *Filename, bOverwriteFile);
		return false;
	}
}

void FEXRImageWriteTask::AddFileMetadata(Imf::Header& InHeader)
{
	static const FName RTTIExtensionModuleName("UEOpenExrRTTI");
	IOpenExrRTTIModule* OpenExrModule = FModuleManager::LoadModulePtr<IOpenExrRTTIModule>(RTTIExtensionModuleName);
	if (OpenExrModule)
	{
		OpenExrModule->AddFileMetadata(FileMetadata, InHeader);
	}
}

void FEXRImageWriteTask::PreProcess()
{
	if (PixelPreprocessors.IsEmpty())
	{
		return;
	}

	for (int32 LayerIdx = 0; LayerIdx < Layers.Num(); ++LayerIdx)
	{
		if (const TArray<FPixelPreProcessor>* LayerPixelPreprocessorsPtr = PixelPreprocessors.Find(LayerIdx))
		{
			for (const FPixelPreProcessor& PreProcessor : *LayerPixelPreprocessorsPtr)
			{
				// PreProcessors are assumed to be valid. Fetch the Data pointer each time
				// in case a pre-processor changes our pixel data.
				FImagePixelData* Data = Layers[LayerIdx].Get();
				PreProcessor(Data);
			}
		}
	}
}

namespace UE
{
	namespace MoviePipeline
	{
		static TPair<UE::Color::EColorSpace, FString> GetDisplayGamutType(EDisplayColorGamut InDisplayGamut)
		{
			switch (InDisplayGamut)
			{
			case EDisplayColorGamut::sRGB_D65: return { UE::Color::EColorSpace::sRGB, TEXT("sRGB") };
			case EDisplayColorGamut::DCIP3_D65: return { UE::Color::EColorSpace::P3DCI, TEXT("P3DCI") };
			case EDisplayColorGamut::Rec2020_D65: return { UE::Color::EColorSpace::Rec2020, TEXT("Rec2020") };
			case EDisplayColorGamut::ACES_D60: return { UE::Color::EColorSpace::ACESAP0, TEXT("ACESAP0") };
			case EDisplayColorGamut::ACEScg_D60: return { UE::Color::EColorSpace::ACESAP1, TEXT("ACESAP1") };

			default:
				checkNoEntry();
				return {};
			}
		};

		void UpdateColorSpaceMetadataImpl(const FEXRColorSpaceMetadata& InColorSpaceMetadata, FEXRImageWriteTask& InOutImageTask)
		{
			if (!InColorSpaceMetadata.SourceName.IsEmpty())
			{
				InOutImageTask.FileMetadata.Add("unreal/colorSpace/source", InColorSpaceMetadata.SourceName);
			}
			if (!InColorSpaceMetadata.DestinationName.IsEmpty())
			{
				InOutImageTask.FileMetadata.Add("unreal/colorSpace/destination", InColorSpaceMetadata.DestinationName);
			}

			InOutImageTask.ColorSpaceChromaticities = InColorSpaceMetadata.Chromaticities;
		}

		void UpdateColorSpaceMetadata(const FOpenColorIOColorConversionSettings& InConversionSettings, FEXRImageWriteTask& InOutImageTask)
		{
			FEXRColorSpaceMetadata ColorSpaceMetadata;

			if (InConversionSettings.IsValid())
			{
				// Note: OpenColorIO does not expose chromaticity information so we only provide transform names.
				if (InConversionSettings.IsDisplayView())
				{
					switch (InConversionSettings.DisplayViewDirection)
					{
					case EOpenColorIOViewTransformDirection::Forward:
						ColorSpaceMetadata.SourceName = InConversionSettings.SourceColorSpace.ToString();
						ColorSpaceMetadata.DestinationName = InConversionSettings.DestinationDisplayView.ToString();
						break;
					case EOpenColorIOViewTransformDirection::Inverse:
						ColorSpaceMetadata.SourceName = InConversionSettings.DestinationDisplayView.ToString();
						ColorSpaceMetadata.DestinationName = InConversionSettings.SourceColorSpace.ToString();
						break;

					default:
						checkNoEntry();
					}
				}
				else
				{
					ColorSpaceMetadata.SourceName = InConversionSettings.SourceColorSpace.ToString();
					ColorSpaceMetadata.DestinationName = InConversionSettings.DestinationColorSpace.ToString();
				}
			}

			UpdateColorSpaceMetadataImpl(ColorSpaceMetadata, InOutImageTask);
		}

		void UpdateColorSpaceMetadata(ESceneCaptureSource InSceneCaptureSource, FEXRImageWriteTask& InOutImageTask)
		{
			FEXRColorSpaceMetadata ColorSpaceMetadata;

			switch (InSceneCaptureSource)
			{
			case SCS_FinalColorLDR:
			case SCS_FinalToneCurveHDR:
			{
				// We are in output display space
				TPair<UE::Color::EColorSpace, FString> ColorSpaceType = GetDisplayGamutType(HDRGetDefaultDisplayColorGamut());
				UE::Color::FColorSpace OutputCS = UE::Color::FColorSpace(ColorSpaceType.Key);

				ColorSpaceMetadata.DestinationName = ColorSpaceType.Value;
				ColorSpaceMetadata.Chromaticities = { OutputCS.GetRedChromaticity(), OutputCS.GetGreenChromaticity(), OutputCS.GetBlueChromaticity(), OutputCS.GetWhiteChromaticity() };
				break;
			}
			case SCS_SceneColorHDR:
			case SCS_SceneColorHDRNoAlpha:
			case SCS_FinalColorHDR:
			case SCS_BaseColor:
			{
				// We are in working color space
				const UE::Color::FColorSpace& WCS = UE::Color::FColorSpace::GetWorking();
				ColorSpaceMetadata.Chromaticities = { WCS.GetRedChromaticity(), WCS.GetGreenChromaticity(), WCS.GetBlueChromaticity(), WCS.GetWhiteChromaticity() };
				break;
			}
			default:
				break;
			}

			UpdateColorSpaceMetadataImpl(ColorSpaceMetadata, InOutImageTask);
		}
	}
}

void UMoviePipelineImageSequenceOutput_EXR::OnReceiveImageDataImpl(FMoviePipelineMergerOutputFrame* InMergedOutputFrame)
{
	if (!bMultilayer)
	{
		// Some software doesn't support multi-layer, so in that case we fall back to the single-layer-multiple-file
		// codepath of our parent.
		Super::OnReceiveImageDataImpl(InMergedOutputFrame);
		return;
	}
	check(InMergedOutputFrame);

	// Ensure our OpenExrRTTI module gets loaded. This needs to happen from the main thread, if it's not loaded then metadata silently fails when writing.
	static const FName RTTIExtensionModuleName("UEOpenExrRTTI");
	FModuleManager::Get().LoadModule(RTTIExtensionModuleName);


	UMoviePipelineOutputSetting* OutputSettings = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	check(OutputSettings);

	
	// Find the maximum resolution over all layers, which will be used to pad lower resolution layers to a matching size
	FIntPoint MaximumResolution = FIntPoint::ZeroValue;
	for (TPair<FMoviePipelinePassIdentifier, TUniquePtr<FImagePixelData>>& RenderPassData : InMergedOutputFrame->ImageOutputData)
	{
		MaximumResolution.X = FMath::Max(MaximumResolution.X, RenderPassData.Value->GetSize().X);
		MaximumResolution.Y = FMath::Max(MaximumResolution.Y, RenderPassData.Value->GetSize().Y);
	}

	FString OutputDirectory = OutputSettings->OutputDirectory.Path;

	// We need to resolve the filename format string. We combine the folder and file name into one long string first
	FString FinalFilePath;
	FString FrameTemplatedFilePath;
	FMoviePipelineFormatArgs FinalFormatArgs;
	FString FinalImageSequenceFileName;
	FString ClipName;
	const TCHAR* Extension = TEXT("exr");
	{
		FString FileNameFormatString = OutputSettings->FileNameFormat;

		// If we're writing more than one render pass out, we need to ensure the file name has the format string in it so we don't
		// overwrite the same file multiple times. Burn In overlays don't count because they get composited on top of an existing file.
		const bool bIncludeRenderPass = false;
		const bool bTestFrameNumber = true;

		UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber);

		// Create specific data that needs to override 
		TMap<FString, FString> FormatOverrides;
		FormatOverrides.Add(TEXT("render_pass"), TEXT("")); // Render Passes are included inside the exr file by named layers.
		FormatOverrides.Add(TEXT("ext"), Extension);

		// This resolves the filename format and gathers metadata from the settings at the same time.
		GetPipeline()->ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, FinalImageSequenceFileName, FinalFormatArgs, &InMergedOutputFrame->FrameOutputState, -InMergedOutputFrame->FrameOutputState.ShotOutputFrameNumber);

		FString FilePathFormatString = OutputDirectory / FileNameFormatString;
		GetPipeline()->ResolveFilenameFormatArguments(FilePathFormatString, FormatOverrides, FinalFilePath, FinalFormatArgs, &InMergedOutputFrame->FrameOutputState);

		if (FPaths::IsRelative(FinalFilePath))
		{
			FinalFilePath = FPaths::ConvertRelativePathToFull(FinalFilePath);
		}

		// Re-resolve using the frame-templated format string
		const FString FrameTemplatedFormatString = MoviePipeline::GetFrameTemplatedFilePath(FilePathFormatString);	
		GetPipeline()->ResolveFilenameFormatArguments(FrameTemplatedFormatString, FormatOverrides, FrameTemplatedFilePath, FinalFormatArgs, &InMergedOutputFrame->FrameOutputState);

		if (FPaths::IsRelative(FrameTemplatedFilePath))
		{
			FrameTemplatedFilePath = FPaths::ConvertRelativePathToFull(FrameTemplatedFilePath);
		}

		// Create a deterministic clipname by removing frame numbers, file extension, and any trailing .'s
		UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);
		GetPipeline()->ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, ClipName, FinalFormatArgs, &InMergedOutputFrame->FrameOutputState);
		ClipName.RemoveFromEnd(Extension);
		ClipName.RemoveFromEnd(".");
	}

	// If not using multi-part, we have to pad all layers up to the maximum resolution. If multi-part is on, different header
	// data window sizes are suppported, so check the cvar to see if we should pad
	const bool bPadToDataWindowSize = !bMultipart || UE::MoviePipeline::CVarMoviePipelinePadLayersForMultiPartEXR.GetValueOnGameThread();
	TUniquePtr<FEXRImageWriteTask> MultiLayerImageTask = MakeUnique<FEXRImageWriteTask>();
	MultiLayerImageTask->Filename = FinalFilePath;
	MultiLayerImageTask->bMultipart = bMultipart;
	MultiLayerImageTask->bPadToDataWindowSize = bPadToDataWindowSize;
	MultiLayerImageTask->Compression = Compression;
	// MRQ keeps the default DWA compression level. MRG exposes this on its EXR output nodes.

	// FinalFormatArgs.FileMetadata has been merged by ResolveFilenameFormatArgs with the FrameOutputState,
	// but we need to convert from FString, FString (needed for BP/Python purposes) to a FStringFormatArg as
	// we need to preserve numeric metadata types later in the image writing process (for compression level)
	TMap<FString, FStringFormatArg> NewFileMetdataMap;
	for (const TPair<FString, FString>& Metadata : FinalFormatArgs.FileMetadata)
	{
		NewFileMetdataMap.Add(Metadata.Key, Metadata.Value);
	}
	MultiLayerImageTask->FileMetadata = NewFileMetdataMap;

	// Add color space metadata to the output: xy chromaticity coordinates and/or the color space source/dest names.
	// TODO: Support is also needed for regular exrs via the image wrapper module.
	UMoviePipelineColorSetting* ColorSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineColorSetting>();
	if (ColorSetting && ColorSetting->OCIOConfiguration.bIsEnabled)
	{
		UE::MoviePipeline::UpdateColorSpaceMetadata(ColorSetting->OCIOConfiguration.ColorConfiguration, *MultiLayerImageTask);
	}
	else
	{
		ESceneCaptureSource SceneCaptureSource = (ColorSetting && ColorSetting->bDisableToneCurve) ? ESceneCaptureSource::SCS_FinalColorHDR : ESceneCaptureSource::SCS_FinalToneCurveHDR;
		UE::MoviePipeline::UpdateColorSpaceMetadata(SceneCaptureSource, *MultiLayerImageTask);
	}

	int32 LayerIndex = 0;
	bool bRequiresTransparentOutput = false;
	int32 ShotIndex = 0;
	for (TPair<FMoviePipelinePassIdentifier, TUniquePtr<FImagePixelData>>& RenderPassData : InMergedOutputFrame->ImageOutputData)
	{
		// No quantization required, just copy the data as we will move it into the image write task.
		TUniquePtr<FImagePixelData> PixelData = RenderPassData.Value->CopyImageData();
		FImagePixelDataPayload* Payload = RenderPassData.Value->GetPayload<FImagePixelDataPayload>();
		ShotIndex = Payload->SampleState.OutputState.ShotIndex;

		// If there is more than one layer, then we will prefix the layer. The first layer is not prefixed (and gets inserted as RGBA)
		// as most programs that handle EXRs expect the main image data to be in an unnamed layer.
		if (LayerIndex == 0)
		{
			// Only check the main image pass for transparent output since that's generally considered the 'preview'.
			bRequiresTransparentOutput = Payload->bRequireTransparentOutput;
			MultiLayerImageTask->OverscanPercentage = Payload->SampleState.OverscanPercentage;
			MultiLayerImageTask->CropRectangle = Payload->SampleState.CropRectangle;
		}
		else
		{
			// If there is more than one layer, then we will prefix the layer. The first layer is not prefixed (and gets inserted as RGBA)
			// as most programs that handle EXRs expect the main image data to be in an unnamed layer. We only postfix with cameraname
			// if there's multiple cameras, as pipelines may be already be built around the generic "one camera" support.
			UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[ShotIndex];
			UMoviePipelineCameraSetting* CameraSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineCameraSetting>(CurrentShot);
			int32 NumCameras = CameraSettings->bRenderAllCameras ? CurrentShot->SidecarCameras.Num() : 1;
			
			FString CombinedName;
			if (NumCameras == 1)
			{
				CombinedName = RenderPassData.Key.Name;
			}
			else
			{
				CombinedName = FString::Printf(TEXT("%s_%s"), *RenderPassData.Key.Name, *RenderPassData.Key.CameraName);
			}
			MultiLayerImageTask->LayerNames.FindOrAdd(PixelData.Get(), CombinedName);
		}

		MultiLayerImageTask->Width = MaximumResolution.X;
		MultiLayerImageTask->Height = MaximumResolution.Y;
		MultiLayerImageTask->Layers.Add(MoveTemp(PixelData));
		LayerIndex++;
	}

	MoviePipeline::FMoviePipelineOutputFutureData OutputData;
	OutputData.Shot = GetPipeline()->GetActiveShotList()[ShotIndex];
	OutputData.PassIdentifier = FMoviePipelinePassIdentifier(TEXT("")); // exrs put all the render passes internally so this resolves to a ""
	OutputData.OriginSettingClass = GetClass();
	OutputData.FilePath = FinalFilePath;
	OutputData.FrameTemplatedFilePath = FrameTemplatedFilePath;
	GetPipeline()->AddOutputFuture(ImageWriteQueue->Enqueue(MoveTemp(MultiLayerImageTask)), OutputData);

#if WITH_EDITOR
	GetPipeline()->AddFrameToOutputMetadata(ClipName, FinalImageSequenceFileName, InMergedOutputFrame->FrameOutputState, Extension, bRequiresTransparentOutput);
#endif
}

#endif // WITH_UNREALEXR

void UMoviePipelineImageSequenceOutput_EXR::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	if (bMultilayer)
	{
		InTelemetry->bUsesMultiEXR = true;
	}
	else
	{
		InTelemetry->bUsesEXR = true;
	}
}