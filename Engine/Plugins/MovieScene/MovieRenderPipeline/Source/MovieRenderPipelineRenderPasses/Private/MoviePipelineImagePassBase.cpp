// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineImagePassBase.h"

// For Cine Camera Variables in Metadata
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MoviePipeline.h"
#include "GameFramework/PlayerController.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineViewFamilySetting.h"
#include "MoviePipelineQueue.h"
#include "LegacyScreenPercentageDriver.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineGameOverrideSetting.h"
#include "EngineModule.h"
#include "Engine/LocalPlayer.h"
#include "Engine/RendererSettings.h"
#include "Engine/TextureRenderTarget.h"
#include "MovieRenderOverlappedImage.h"
#include "ImageUtils.h"
#include "SceneManagement.h"
#include "TextureResource.h"

// For Cine Camera Variables in Metadata
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "MoviePipelineUtils.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineImagePassBase)


DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_AccumulateSample_TT"), STAT_AccumulateSample_TaskThread, STATGROUP_MoviePipeline);

void UMoviePipelineImagePassBase::GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const
{
	OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
	OutViewModeIndex = EViewModeIndex::VMI_Lit;
}

void UMoviePipelineImagePassBase::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	Super::SetupImpl(InPassInitSettings);

	// Allocate 
	ViewState.Allocate(InPassInitSettings.FeatureLevel);
}

void UMoviePipelineImagePassBase::WaitUntilTasksComplete()
{
	GetPipeline()->SetPreviewTexture(nullptr);

	// This may call FlushRenderingCommands if there are outstanding readbacks that need to happen.
	for (TPair<FIntPoint, TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe>> SurfaceQueueIt : SurfaceQueues)
	{
		if (SurfaceQueueIt.Value.IsValid())
		{
			SurfaceQueueIt.Value->Shutdown();
		}
	}

	// Stall until the task graph has completed any pending accumulations.
	FTaskGraphInterface::Get().WaitUntilTasksComplete(OutstandingTasks, ENamedThreads::GameThread);
	OutstandingTasks.Reset();
};

void UMoviePipelineImagePassBase::TeardownImpl()
{
	for (TPair<FIntPoint, TWeakObjectPtr<UTextureRenderTarget2D>>& TileRenderTargetIt : TileRenderTargets)
	{
		if (TileRenderTargetIt.Value.IsValid())
		{
			TileRenderTargetIt.Value->RemoveFromRoot();
		}
	}

	SurfaceQueues.Empty();
	TileRenderTargets.Empty();

	FSceneViewStateInterface* Ref = ViewState.GetReference();
	if (Ref)
	{
		Ref->ClearMIDPool();
	}
	ViewState.Destroy();

	Super::TeardownImpl();
}

void UMoviePipelineImagePassBase::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UMoviePipelineImagePassBase& This = *CastChecked<UMoviePipelineImagePassBase>(InThis);
	FSceneViewStateInterface* Ref = This.ViewState.GetReference();
	if (Ref)
	{
		Ref->AddReferencedObjects(Collector);
	}
}

void UMoviePipelineImagePassBase::RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState)
{
	Super::RenderSample_GameThreadImpl(InSampleState);

	// Wait for a all surfaces to be available to write to. This will stall the game thread while the RHI/Render Thread catch up.
	SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_WaitForAvailableSurface);
	for(TPair<FIntPoint, TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe>> SurfaceQueueIt : SurfaceQueues)
	{
		if (SurfaceQueueIt.Value.IsValid())
		{
			SurfaceQueueIt.Value->BlockUntilAnyAvailable();
		}
	}
}

TWeakObjectPtr<UTextureRenderTarget2D> UMoviePipelineImagePassBase::GetOrCreateViewRenderTarget(const FIntPoint& InSize, IViewCalcPayload* OptPayload)
{
	if (const TWeakObjectPtr<UTextureRenderTarget2D>* ExistViewRenderTarget = TileRenderTargets.Find(InSize))
	{
		return *ExistViewRenderTarget;
	}

	const TWeakObjectPtr<UTextureRenderTarget2D> NewViewRenderTarget = CreateViewRenderTargetImpl(InSize, OptPayload);
	TileRenderTargets.Emplace(InSize, NewViewRenderTarget);

	return NewViewRenderTarget;
}

TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> UMoviePipelineImagePassBase::GetOrCreateSurfaceQueue(const FIntPoint& InSize, IViewCalcPayload* OptPayload)
{
	if (const TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe>* ExistSurfaceQueue = SurfaceQueues.Find(InSize))
	{
		return *ExistSurfaceQueue;
	}

	const TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> NewSurfaceQueue = CreateSurfaceQueueImpl(InSize, OptPayload);
	SurfaceQueues.Emplace(InSize, NewSurfaceQueue);

	return NewSurfaceQueue;
}

TWeakObjectPtr<UTextureRenderTarget2D> UMoviePipelineImagePassBase::CreateViewRenderTargetImpl(const FIntPoint& InSize, IViewCalcPayload* OptPayload) const
{
	TWeakObjectPtr<UTextureRenderTarget2D> NewTarget = NewObject<UTextureRenderTarget2D>(GetTransientPackage());
	NewTarget->ClearColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);

	// OCIO: Since this is a manually created Render target we don't need Gamma to be applied.
	// We use this render target to render to via a display extension that utilizes Display Gamma
	// which has a default value of 2.2 (DefaultDisplayGamma), therefore we need to set Gamma on this render target to 2.2 to cancel out any unwanted effects.
	NewTarget->TargetGamma = UTextureRenderTarget::GetDefaultDisplayGamma();

	// Initialize to the tile size (not final size) and use a 16 bit back buffer to avoid precision issues when accumulating later
	NewTarget->InitCustomFormat(InSize.X, InSize.Y, EPixelFormat::PF_FloatRGBA, false);
	NewTarget->AddToRoot();

	// Always update the preview texture to the new texture, so that in cases where resolution is changing between frames (e.g. animated overscan)
	// the preview texture continues to be for the most recent frame.
	// TODO: Multi-camera support - As there is only one preview texture, and there is no way to distinguish which camera we are creating the texture for,
	// we can't be sure that the newest preview texture is for the same camera as previous frames.
	GetPipeline()->SetPreviewTexture(NewTarget.Get());

	return NewTarget;
}

TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> UMoviePipelineImagePassBase::CreateSurfaceQueueImpl(const FIntPoint& InSize, IViewCalcPayload* OptPayload) const
{
	TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> SurfaceQueue = MakeShared<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe>(InSize, EPixelFormat::PF_FloatRGBA, 3, true);

	return SurfaceQueue;
}

TSharedPtr<FSceneViewFamilyContext> UMoviePipelineImagePassBase::CalculateViewFamily(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload)
{
	const FMoviePipelineFrameOutputState::FTimeData& TimeData = InOutSampleState.OutputState.TimeData;

	FEngineShowFlags ShowFlags = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
	EViewModeIndex  ViewModeIndex;
	GetViewShowFlags(ShowFlags, ViewModeIndex);
	MoviePipelineRenderShowFlagOverride(ShowFlags);
	TWeakObjectPtr<UTextureRenderTarget2D> ViewRenderTarget = GetOrCreateViewRenderTarget(InOutSampleState.BackbufferSize, OptPayload);
	check(ViewRenderTarget.IsValid());

	FRenderTarget* RenderTarget = ViewRenderTarget->GameThread_GetRenderTargetResource();

	TSharedPtr<FSceneViewFamilyContext> OutViewFamily = MakeShared<FSceneViewFamilyContext>(FSceneViewFamily::ConstructionValues(
		RenderTarget,
		GetPipeline()->GetWorld()->Scene,
		ShowFlags)
		.SetTime(FGameTime::CreateUndilated(TimeData.WorldSeconds, TimeData.FrameDeltaTime))
		.SetRealtimeUpdate(true));

	OutViewFamily->SceneCaptureSource = InOutSampleState.SceneCaptureSource;
	OutViewFamily->bWorldIsPaused = InOutSampleState.bWorldIsPaused;
	OutViewFamily->ViewMode = ViewModeIndex;
	OutViewFamily->bOverrideVirtualTextureThrottle = true;
	
	// Kept as an if/else statement to avoid the confusion with setting all of these values to some permutation of !/!!bHasRenderedFirstViewThisFrame.
	if (!GetPipeline()->bHasRenderedFirstViewThisFrame)
	{
		GetPipeline()->bHasRenderedFirstViewThisFrame = true;
		
		OutViewFamily->bIsFirstViewInMultipleViewFamily = true;
		OutViewFamily->bAdditionalViewFamily = false;
	}
	else
	{
		OutViewFamily->bIsFirstViewInMultipleViewFamily = false;
		OutViewFamily->bAdditionalViewFamily = true;
	}

	const bool bIsPerspective = true;
	ApplyViewMode(OutViewFamily->ViewMode, bIsPerspective, OutViewFamily->EngineShowFlags);

	EngineShowFlagOverride(ESFIM_Game, OutViewFamily->ViewMode, OutViewFamily->EngineShowFlags, false);
	
	const UMoviePipelineExecutorShot* Shot = GetPipeline()->GetActiveShotList()[InOutSampleState.OutputState.ShotIndex];

	for (UMoviePipelineGameOverrideSetting* OverrideSetting : GetPipeline()->FindSettingsForShot<UMoviePipelineGameOverrideSetting>(Shot))
	{
		if (OverrideSetting->bOverrideVirtualTextureFeedbackFactor)
		{
			OutViewFamily->VirtualTextureFeedbackFactor = OverrideSetting->VirtualTextureFeedbackFactor;
		}
	}

	// Auto exposure pass is specified with a tile index of {-1,-1}
	const bool bAutoExposurePass = (InOutSampleState.TileIndexes.X == -1) && (InOutSampleState.TileIndexes.Y == -1);
	const bool bScreenPercentageSupported = IsScreenPercentageSupported() && !bAutoExposurePass;

	// Force disable screen percentage and motion blur for auto-exposure passes.  These are already at lowered resolution relative to the overall high res
	// tiled view, and only the eye adaptation is used from these, making blur irrelevant.  Saves history memory and performance.
	if (bAutoExposurePass)
	{
		OutViewFamily->EngineShowFlags.ScreenPercentage = false;
		OutViewFamily->EngineShowFlags.MotionBlur = false;
	}

	// No need to do anything if screen percentage is not supported. 
	if (bScreenPercentageSupported)
	{
		// Allows all Output Settings to have an access to View Family. This allows to modify rendering output settings.
		for (UMoviePipelineViewFamilySetting* Setting : GetPipeline()->FindSettingsForShot<UMoviePipelineViewFamilySetting>(Shot))
		{
			Setting->SetupViewFamily(*OutViewFamily);
		}
	}

	// If UMoviePipelineViewFamilySetting never set a Screen percentage interface we fallback to default.
	if (OutViewFamily->GetScreenPercentageInterface() == nullptr)
	{
		OutViewFamily->SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(*OutViewFamily, bScreenPercentageSupported ? InOutSampleState.GlobalScreenPercentageFraction : 1.f));
	}

	int32 ViewCount = InOutSampleState.bAutoExposureCubePass ? 6 : 1;
	for (int32 ViewIndex = 0; ViewIndex < ViewCount; ViewIndex++)
	{
		// Ignored in downstream code if this isn't an auto exposure cube pass
		InOutSampleState.AutoExposureCubeFace = ViewIndex;

		// View is added as a child of the OutViewFamily-> 
		FSceneView* View = GetSceneViewForSampleState(OutViewFamily.Get(), /*InOut*/ InOutSampleState, OptPayload);

		SetupViewForViewModeOverride(View);

		// Override the view's FrameIndex to be based on our progress through the sequence. This greatly increases
		// determinism with things like TAA.
		View->OverrideFrameIndexValue = InOutSampleState.FrameIndex;
		View->OverrideOutputFrameIndexValue = InOutSampleState.OutputState.OutputFrameNumber;
		View->bCameraCut = InOutSampleState.bCameraCut;
		View->bIsOfflineRender = true;
		View->AntiAliasingMethod = IsAntiAliasingSupported() ? InOutSampleState.AntiAliasingMethod : EAntiAliasingMethod::AAM_None;

		// Override the Motion Blur settings since these are controlled by the movie pipeline.
		{
			FFrameRate OutputFrameRate = GetPipeline()->GetPipelinePrimaryConfig()->GetEffectiveFrameRate(GetPipeline()->GetTargetSequence());

			// We need to inversly scale the target FPS by time dilation to counteract slowmo. If scaling isn't applied then motion blur length
			// stays the same length despite the smaller delta time and the blur ends up too long.
			View->FinalPostProcessSettings.MotionBlurTargetFPS = FMath::RoundToInt(OutputFrameRate.AsDecimal() / FMath::Max(SMALL_NUMBER, InOutSampleState.OutputState.TimeData.TimeDilation));
			View->FinalPostProcessSettings.MotionBlurAmount = InOutSampleState.OutputState.TimeData.MotionBlurFraction;
			View->FinalPostProcessSettings.MotionBlurMax = 100.f;
			View->FinalPostProcessSettings.bOverride_MotionBlurAmount = true;
			View->FinalPostProcessSettings.bOverride_MotionBlurTargetFPS = true;
			View->FinalPostProcessSettings.bOverride_MotionBlurMax = true;

			// Skip the whole pass if they don't want motion blur.
			if (FMath::IsNearlyZero(InOutSampleState.OutputState.TimeData.MotionBlurFraction))
			{
				OutViewFamily->EngineShowFlags.SetMotionBlur(false);
			}
		}

		// Locked Exposure
		const bool bAutoExposureAllowed = IsAutoExposureAllowed(InOutSampleState);
		{
			// If the rendering pass doesn't allow autoexposure and they dont' have manual exposure set up, warn.
			if (!bAutoExposureAllowed && (View->FinalPostProcessSettings.AutoExposureMethod != EAutoExposureMethod::AEM_Manual))
			{
				// Skip warning if the project setting is disabled though, as exposure will be forced off in the renderer anyways.
				const URendererSettings* RenderSettings = GetDefault<URendererSettings>();
				if (RenderSettings->bDefaultFeatureAutoExposure != false)
				{
					UE_LOGF(LogMovieRenderPipeline, Warning, "Camera Auto Exposure Method not supported by one or more render passes. Change the Auto Exposure Method to Manual!");
					View->FinalPostProcessSettings.AutoExposureMethod = EAutoExposureMethod::AEM_Manual;
				}
			}
		}
	}

	OutViewFamily->ViewExtensions.Append(GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(GetWorld()->Scene)));

	AddViewExtensions(*OutViewFamily, InOutSampleState);

	for (auto ViewExt : OutViewFamily->ViewExtensions)
	{
		ViewExt->SetupViewFamily(*OutViewFamily.Get());
	}

	// Support scene captures with the "bMainViewFamily" flag set
	OutViewFamily->bIsMainViewFamily = true;

	// Post view family extension setup, do some more work on each view
	for (int32 ViewIndex = 0; ViewIndex < ViewCount; ViewIndex++)
	{
		FSceneView* View = const_cast<FSceneView*>(OutViewFamily->Views[ViewIndex]);

		for (int ViewExt = 0; ViewExt < OutViewFamily->ViewExtensions.Num(); ViewExt++)
		{
			OutViewFamily->ViewExtensions[ViewExt]->SetupView(*OutViewFamily.Get(), *View);
		}

		// The requested configuration may not be supported, warn user and fall back. We can't call
		// FSceneView::SetupAntiAliasingMethod because it reads the value from the cvar which would
		// cause the value set by the MoviePipeline UI to be ignored.
		{
			bool bMethodWasUnsupported = false;
			if (View->AntiAliasingMethod == AAM_TemporalAA && !SupportsGen4TAA(View->GetShaderPlatform()))
			{
				UE_LOGF(LogMovieRenderPipeline, Error, "TAA was requested but this hardware does not support it.");
				bMethodWasUnsupported = true;
			}
			else if (View->AntiAliasingMethod == AAM_TSR && !SupportsTSR(View->GetShaderPlatform()))
			{
				UE_LOGF(LogMovieRenderPipeline, Error, "TSR was requested but this hardware does not support it.");
				bMethodWasUnsupported = true;
			}

			if (bMethodWasUnsupported)
			{
				View->AntiAliasingMethod = AAM_None;
			}
		}

		// Anti Aliasing
		{
			// If we're not using Temporal Anti-Aliasing or Path Tracing we will apply the View Matrix projection jitter. Normally TAA sets this
			// inside FSceneRenderer::PreVisibilityFrameSetup. Path Tracing does its own anti-aliasing internally.
			bool bApplyProjectionJitter =
				!OutViewFamily->EngineShowFlags.PathTracing
				&& !IsTemporalAccumulationBasedMethod(View->AntiAliasingMethod);
			if (bApplyProjectionJitter)
			{
				View->ViewMatrices.HackAddTemporalAAProjectionJitter(InOutSampleState.ProjectionMatrixJitterAmount);
			}
		}

		// Path Tracer Sampling
		if (OutViewFamily->EngineShowFlags.PathTracing)
		{
			// override whatever settings came from PostProcessVolume or Camera

			// If motion blur is enabled:
			//    blend all spatial samples together while leaving the handling of temporal samples up to MRQ
			//    each temporal sample will include denoising and post-process effects
			// If motion blur is NOT enabled:
			//    blend all temporal+spatial samples within the path tracer and only apply denoising on the last temporal sample
			//    this way we minimize denoising cost and also allow a much higher number of temporal samples to be used which
			//    can help reduce strobing

			// NOTE: Tiling is not compatible with the reference motion blur mode because it changes the order of the loops over the image.
			const bool bAccumulateSpatialSamplesOnly = OutViewFamily->EngineShowFlags.MotionBlur || InOutSampleState.GetTileCount() > 1;

			const int32 SampleCount = bAccumulateSpatialSamplesOnly ? InOutSampleState.SpatialSampleCount : InOutSampleState.TemporalSampleCount * InOutSampleState.SpatialSampleCount;
			const int32 SampleIndex = bAccumulateSpatialSamplesOnly ? InOutSampleState.SpatialSampleIndex : InOutSampleState.TemporalSampleIndex * InOutSampleState.SpatialSampleCount + InOutSampleState.SpatialSampleIndex;

			// TODO: pass along FrameIndex (which includes SampleIndex) to make sure sampling is fully deterministic

			// Overwrite whatever sampling count came from the PostProcessVolume
			View->FinalPostProcessSettings.bOverride_PathTracingSamplesPerPixel = true;
			View->FinalPostProcessSettings.PathTracingSamplesPerPixel = SampleCount;

			// reset path tracer's accumulation at the start of each sample
			View->bForcePathTracerReset = SampleIndex == 0;

			// discard the result, unless its the last sample
			InOutSampleState.bDiscardResult |= !(SampleIndex == SampleCount - 1);
		}

		// Object Occlusion/Histories
		{
			// If we're using tiling, we force the reset of histories each frame so that we don't use the previous tile's
			// object occlusion queries, as that causes things to disappear from some views.
			if (InOutSampleState.GetTileCount() > 1)
			{
				View->bForceCameraVisibilityReset = true;
			}
		}

		// Bias all mip-mapping to pretend to be working at our target resolution and not our tile resolution
		// so that the images don't end up soft.
		{
			float EffectivePrimaryResolutionFraction = 1.f / InOutSampleState.TileCounts.X;
			View->MaterialTextureMipBias = FMath::Log2(EffectivePrimaryResolutionFraction);

			// Add an additional bias per user settings. This allows them to choose to make the textures sharper if it
			// looks better with their particular settings.
			View->MaterialTextureMipBias += InOutSampleState.TextureSharpnessBias;
		}
	}

	return OutViewFamily;
}

void UMoviePipelineImagePassBase::SetupViewForViewModeOverride(FSceneView* View)
{
	UE::MovieRenderPipeline::UpdateSceneViewForShowFlags(View);
}

void UMoviePipelineImagePassBase::OnFrameStartImpl()
{
	Super::OnFrameStartImpl();

	// Clean up and shutdown any stale surface queues. This is necessary for anything that changes resolution between frames, such as animated overscan.
	// The surface queue pool is keyed off of resolution, so if every frame has a new resolution, a new surface queue is created, and subsequently,
	// only one surface is ever added to the queue (that for the frame that needed that resolution of surface queue). However, when a surface queue isn't full
	// it can't properly mark surfaces as complete and ready for readback because surface queues natively track "staleness" by how far from the current surface in the queue
	// a previously queued surface is. So, in order to prevent the surface queue from growing too large, and to force surfaces to complete their readback,
	// we track the last frame the queue was used on, and if it has been enough frames, we clean it up, forcing any surfaces to read back. This staleness amount
	// should give any queued surfaces enough frames to complete rendering so that they can be read back by the time Shutdown is called
	for (auto Iter = SurfaceQueues.CreateIterator(); Iter; ++Iter)
	{
		if (Iter->Value->IsStale())
		{
			Iter->Value->Shutdown();
			Iter.RemoveCurrent();
		}
	}
}

void UMoviePipelineImagePassBase::GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses)
{
	Super::GatherOutputPassesImpl(ExpectedRenderPasses);
	ExpectedRenderPasses.Add(PassIdentifier);
}

// Cube capture is arranged in 3x2 square tiles, rounded down to a multiple of 8 pixels.
static int32 ComputeAutoExposureCubeCaptureSize(FIntPoint Resolution)
{
	return AlignDown(FMath::Min(Resolution.X / 3, Resolution.Y / 2), 8);
}

FSceneView* UMoviePipelineImagePassBase::GetSceneViewForSampleState(FSceneViewFamily* ViewFamily, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload)
{
	APlayerController* LocalPlayerController = GetPipeline()->GetWorld()->GetFirstPlayerController();

	int32 TileSizeX;
	int32 TileSizeY;

	// Auto exposure pass is specified with a tile index of {-1,-1}
	const bool bAutoExposurePass = (InOutSampleState.TileIndexes.X == -1) && (InOutSampleState.TileIndexes.Y == -1);

	if (bAutoExposurePass)
	{
		if (InOutSampleState.bAutoExposureCubePass)
		{
			int32 CubeCaptureSize = ComputeAutoExposureCubeCaptureSize(InOutSampleState.BackbufferSize);
			check(CubeCaptureSize > 0);

			TileSizeX = CubeCaptureSize;
			TileSizeY = CubeCaptureSize;
		}
		else
		{
			// Auto exposure pass renders full screen, but at single tile resolution.  Uses the same back buffer size, so it doesn't require separate render targets.
			// EffectiveOutputResolution is deprecated in favor of OverscannedResolution in all other code paths, but for this specific code path, we want no overscan.
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			TileSizeX = InOutSampleState.EffectiveOutputResolution.X / InOutSampleState.TileCounts.X;
			TileSizeY = InOutSampleState.EffectiveOutputResolution.Y / InOutSampleState.TileCounts.Y;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			check(TileSizeX <= InOutSampleState.BackbufferSize.X);
			check(TileSizeY <= InOutSampleState.BackbufferSize.Y);
		}

		InOutSampleState.OverscanPercentage = 0.0f;
	}
	else
	{
		TileSizeX = InOutSampleState.BackbufferSize.X;
		TileSizeY = InOutSampleState.BackbufferSize.Y;
	}

	UE::MoviePipeline::FImagePassCameraViewData CameraInfo = GetCameraInfo(InOutSampleState, OptPayload);

	const float DestAspectRatio = TileSizeX / (float)TileSizeY;
	const float CameraAspectRatio = bAllowCameraAspectRatio ? CameraInfo.ViewInfo.AspectRatio : DestAspectRatio;

	// Auto exposure cube map faces are rendered as 3x2 split screen tiles.
	static const FIntPoint GCubeFaceViewRectOffsets[6] =
	{
		{ 0,0 },
		{ 1,0 },
		{ 2,0 },
		{ 0,1 },
		{ 1,1 },
		{ 2,1 },
	};
	FIntPoint ViewRectOffset = InOutSampleState.bAutoExposureCubePass ? GCubeFaceViewRectOffsets[InOutSampleState.AutoExposureCubeFace] * FIntPoint(TileSizeX, TileSizeY) : FIntPoint(0, 0);

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = ViewFamily;
	ViewInitOptions.ViewOrigin = CameraInfo.ViewInfo.Location;
	FIntRect ViewRect = FIntRect(ViewRectOffset, ViewRectOffset + FIntPoint(TileSizeX, TileSizeY));
	ViewInitOptions.SetViewRectangle(ViewRect);

	ViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(CameraInfo.ViewInfo.Rotation);
	ViewInitOptions.ViewActor = CameraInfo.ViewActor;

	// Rotate the view 90 degrees (reason: unknown)
	ViewInitOptions.ViewRotationMatrix = ViewInitOptions.ViewRotationMatrix * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	if (InOutSampleState.bAutoExposureCubePass)
	{
		ViewInitOptions.ViewRotationMatrix = ViewInitOptions.ViewRotationMatrix * CalcCubeFaceTransform((ECubeFace)InOutSampleState.AutoExposureCubeFace);
	}

	if (bAutoExposurePass)
	{
		// Overscan is irrelevant for the auto exposure pass
		CameraInfo.ViewInfo.ClearOverscan();
	}
	else if (InOutSampleState.bOverrideCameraOverscan)
	{
		// If we are overriding the camera's overscan, clear out any overscan the camera added to the view info, and apply the overriding overscan
		CameraInfo.ViewInfo.ClearOverscan();
		CameraInfo.ViewInfo.ApplyOverscan(InOutSampleState.OverscanPercentage);
	}
	else
	{
		const float CachedOverscan = GetPipeline()->GetCachedCameraOverscan(InOutSampleState.OutputState.CameraIndex);
		
		// Current overscan is different from originally cached value, indicating overscan changed since start of frame, so output a warning message
		if (CameraInfo.ViewInfo.GetOverscan() != CachedOverscan && InOutSampleState.OutputState.IsFirstTemporalSample())
		{
			UE_LOGF(
				LogMovieRenderPipeline,
				Warning,
				"Overscan on camera %ls changed since start of frame %d in shot %ls, scaling resolution by cached overscan value of %f instead to keep frame resolution consistent",
				*InOutSampleState.OutputState.CameraName,
				InOutSampleState.OutputState.ShotOutputFrameNumber,
				*InOutSampleState.OutputState.ShotName,
				CachedOverscan);
		}

		// Update the sample state with this camera's overscan instead of the config overscan it is filled with initially
		InOutSampleState.OverscanPercentage = CachedOverscan;
	}
	
	ViewInitOptions.FOV = CameraInfo.ViewInfo.FOV;
	ViewInitOptions.DesiredFOV = CameraInfo.ViewInfo.FOV;

	float DofSensorScale = 1.0f;

	if (InOutSampleState.bAutoExposureCubePass)
	{
		// Auto exposure cube faces just use fixed 90 degree FOV
		ViewInitOptions.FOV = 90.0f;
		ViewInitOptions.DesiredFOV = 90.0f;

		const float MatrixFOV = 90.0f * (float)UE_PI / 360.0f;
		const float ClippingPlane = GNearClippingPlane;

		ViewInitOptions.ProjectionMatrix = FReversedZPerspectiveMatrix(MatrixFOV, MatrixFOV, 1.0f, 1.0f, ClippingPlane, ClippingPlane);
	}
	else if (CameraInfo.bUseCustomProjectionMatrix)
	{
		ViewInitOptions.ProjectionMatrix = CameraInfo.CustomProjectionMatrix;

		// Auto exposure pass is full screen, and doesn't use tiling
		if (!bAutoExposurePass)
		{
			// Modify the custom matrix to do an off center projection, with overlap for high-res tiling
			const bool bOrthographic = false;
			ModifyProjectionMatrixForTiling(InOutSampleState, bOrthographic, /*InOut*/ ViewInitOptions.ProjectionMatrix, /*Out*/ DofSensorScale);
		}
	}
	else
	{
		// If they're using high-resolution tiling we can't support letterboxing (as the blended areas we would render with
		// would have been cropped via letterboxing), so to handle this scenario we disable aspect ratio constraints and then
		// manually rescale the view (if needed) to mimick the effect of letterboxing.
		TEnumAsByte<EAspectRatioAxisConstraint> AspectRatioAxisConstraint = CameraInfo.ViewInfo.AspectRatioAxisConstraint.Get(EAspectRatioAxisConstraint::AspectRatio_MaintainXFOV);
		if (InOutSampleState.GetTileCount() > 1 && CameraInfo.ViewInfo.bConstrainAspectRatio)
		{
			if (CameraAspectRatio < DestAspectRatio)
			{
				AspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainYFOV;
				CameraInfo.ViewInfo.OrthoWidth *= (DestAspectRatio / CameraAspectRatio);

				// Off-center camera projections are calculated based on constrained aspect ratios, but those are disabled
				// when using high-resolution tiling. This means that we need to scale the offset projection as well.
				// 
				// To calculate the required size change, we can look at an Aspect Ratio of 0.5 inside a square output, 
				// ie: the rendered area is 1000 x 2000 for an output that is 2000x2000 (this is 0.5 of 1.0). With an
				// off-center projection, an offset of 1.0 on X originally only moved by 500 pixels (1000x0.5), but with the aspect
				// ratio constraint disabled, it now applies to the full output image (2000x0.5) resulting in a move that is twice as big.
				// 
				// To resolve this, we scale the offset by the CameraAspectRatio / DestAspectRatio, which is 0.5 / 1.0 for this example,
				// meaning we multiply the user-intended offset (1.0) by 0.5, resulting in the originally desired 500px offset.
				const double Ratio = CameraAspectRatio / DestAspectRatio; // ex: Ratio = 0.5 / 1
				CameraInfo.ViewInfo.OffCenterProjectionOffset.X *= Ratio;
			}
			else if (CameraAspectRatio > DestAspectRatio)
			{
				// Don't rescale the width and keep it X-constrained.
				AspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainXFOV;

				// Like above, off-center projections need to be rescaled too.
				const double Ratio = DestAspectRatio / CameraAspectRatio;
				CameraInfo.ViewInfo.OffCenterProjectionOffset.Y *= Ratio;
			}
			CameraInfo.ViewInfo.bConstrainAspectRatio = false;
		}


		FIntRect ViewExtents = FViewport::CalculateViewExtents(CameraInfo.ViewInfo.AspectRatio, DestAspectRatio, ViewRect, InOutSampleState.BackbufferSize);
		FMinimalViewInfo::CalculateProjectionMatrixGivenViewRectangle(CameraInfo.ViewInfo, AspectRatioAxisConstraint, ViewExtents, ViewInitOptions);

		// Auto exposure pass is full screen, and doesn't use tiling
		if (!bAutoExposurePass)
		{
			ModifyProjectionMatrixForTiling(InOutSampleState, CameraInfo.ViewInfo.ProjectionMode == ECameraProjectionMode::Orthographic,  /*InOut*/ ViewInitOptions.ProjectionMatrix, /*Out*/ DofSensorScale);
		}
	}


	// Scale the DoF sensor scale to counteract overscan, otherwise the size of Bokeh changes when you have Overscan enabled.
	DofSensorScale *= 1.0 + InOutSampleState.OverscanPercentage;

	ViewInitOptions.SceneViewStateInterface = GetSceneViewStateInterface(OptPayload);

	// If not the auto exposure pass, attempt to get the view state interface from the auto exposure pass
	if (!bAutoExposurePass)
	{
		ViewInitOptions.ExposureSceneViewStateInterface = GetExposureSceneViewStateInterface(OptPayload);
	}

	FSceneView* View = new FSceneView(ViewInitOptions);
	ViewFamily->Views.Add(View);

	
	View->ViewLocation = CameraInfo.ViewInfo.Location;
	View->ViewRotation = CameraInfo.ViewInfo.Rotation;
	// Override previous/current view transforms so that tiled renders don't use the wrong occlusion/motion blur information.
	View->PreviousViewTransform = CameraInfo.ViewInfo.PreviousViewTransform;

	View->StartFinalPostprocessSettings(View->ViewLocation);
	BlendPostProcessSettings(View, InOutSampleState, OptPayload);

	// Scaling sensor size inversely with the the projection matrix [0][0] should physically
	// cause the circle of confusion to be unchanged.
	View->FinalPostProcessSettings.DepthOfFieldSensorWidth *= DofSensorScale;

	// Disable anti-aliasing and temporal upscale for auto-exposure passes.  Auto-exposure is calculated before those passes, so this is wasted work (and memory for history).
	if (bAutoExposurePass)
	{
		View->AntiAliasingMethod = AAM_None;
		View->PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::SpatialUpscale;
	}

	// Auto exposure pass is full screen, and doesn't use tiling
	if (!bAutoExposurePass)
	{
		// Modify the 'center' of the lens to be offset for high-res tiling, helps some effects (vignette) etc. still work.
		View->LensPrincipalPointOffsetScale = (FVector4f)CalculatePrinciplePointOffsetForTiling(InOutSampleState); // LWC_TODO: precision loss. CalculatePrinciplePointOffsetForTiling() could return float, it's normalized?
	}
	View->EndFinalPostprocessSettings(ViewInitOptions);

	// This metadata is per-file and not per-view, but we need the blended result from the view to actually match what we rendered.
	// To solve this, we'll insert metadata per renderpass, separated by render pass name.
	InOutSampleState.OutputState.FileMetadata.Add(FString::Printf(TEXT("unreal/%s/%s/fstop"), *PassIdentifier.CameraName, *PassIdentifier.Name), FString::SanitizeFloat(View->FinalPostProcessSettings.DepthOfFieldFstop));
	InOutSampleState.OutputState.FileMetadata.Add(FString::Printf(TEXT("unreal/%s/%s/fov"), *PassIdentifier.CameraName, *PassIdentifier.Name), FString::SanitizeFloat(ViewInitOptions.FOV));
	InOutSampleState.OutputState.FileMetadata.Add(FString::Printf(TEXT("unreal/%s/%s/focalDistance"), *PassIdentifier.CameraName, *PassIdentifier.Name), FString::SanitizeFloat(View->FinalPostProcessSettings.DepthOfFieldFocalDistance));
	InOutSampleState.OutputState.FileMetadata.Add(FString::Printf(TEXT("unreal/%s/%s/sensorWidth"), *PassIdentifier.CameraName, *PassIdentifier.Name), FString::SanitizeFloat(View->FinalPostProcessSettings.DepthOfFieldSensorWidth));
	InOutSampleState.OutputState.FileMetadata.Add(FString::Printf(TEXT("unreal/%s/%s/overscanPercent"), *PassIdentifier.CameraName, *PassIdentifier.Name), FString::SanitizeFloat(InOutSampleState.OverscanPercentage));

	InOutSampleState.OutputState.FileMetadata.Append(CameraInfo.FileMetadata);
	return View;
}

void UMoviePipelineImagePassBase::BlendPostProcessSettings(FSceneView* InView, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload)
{
	check(InView);

	APlayerController* LocalPlayerController = GetPipeline()->GetWorld()->GetFirstPlayerController();
	// CameraAnim override
	if (LocalPlayerController->PlayerCameraManager)
	{
		TArray<FPostProcessSettings> const* CameraAnimPPSettings;
		TArray<float> const* CameraAnimPPBlendWeights;
		LocalPlayerController->PlayerCameraManager->GetCachedPostProcessBlends(CameraAnimPPSettings, CameraAnimPPBlendWeights);

		if (LocalPlayerController->PlayerCameraManager->bEnableFading)
		{
			InView->OverlayColor = LocalPlayerController->PlayerCameraManager->FadeColor;
			InView->OverlayColor.A = FMath::Clamp(LocalPlayerController->PlayerCameraManager->FadeAmount, 0.f, 1.f);
		}

		if (LocalPlayerController->PlayerCameraManager->bEnableColorScaling)
		{
			FVector ColorScale = LocalPlayerController->PlayerCameraManager->ColorScale;
			InView->ColorScale = FLinearColor(ColorScale.X, ColorScale.Y, ColorScale.Z);
		}

		FMinimalViewInfo ViewInfo = LocalPlayerController->PlayerCameraManager->GetCameraCacheView();
		for (int32 PPIdx = 0; PPIdx < CameraAnimPPBlendWeights->Num(); ++PPIdx)
		{
			InView->OverridePostProcessSettings((*CameraAnimPPSettings)[PPIdx], (*CameraAnimPPBlendWeights)[PPIdx]);
		}

		InView->OverridePostProcessSettings(ViewInfo.PostProcessSettings, ViewInfo.PostProcessBlendWeight);
	}
}

FVector4 UMoviePipelineImagePassBase::CalculatePrinciplePointOffsetForTiling(const FMoviePipelineRenderPassMetrics& InSampleState) const
{
	// We need our final view parameters to be in the space of [-1,1], including all the tiles.
		// Starting with a single tile, the middle of the tile in offset screen space is:
	FVector2D TilePrincipalPointOffset;

	TilePrincipalPointOffset.X = (float(InSampleState.TileIndexes.X) + 0.5f - (0.5f * float(InSampleState.TileCounts.X))) * 2.0f;
	TilePrincipalPointOffset.Y = (float(InSampleState.TileIndexes.Y) + 0.5f - (0.5f * float(InSampleState.TileCounts.Y))) * 2.0f;

	// For the tile size ratio, we have to multiply by (1.0 + overlap) and then divide by tile num
	FVector2D OverlapScale;
	OverlapScale.X = (1.0f + float(2 * InSampleState.OverlappedPad.X) / float(InSampleState.TileSize.X));
	OverlapScale.Y = (1.0f + float(2 * InSampleState.OverlappedPad.Y) / float(InSampleState.TileSize.Y));

	TilePrincipalPointOffset.X /= OverlapScale.X;
	TilePrincipalPointOffset.Y /= OverlapScale.Y;

	FVector2D TilePrincipalPointScale;
	TilePrincipalPointScale.X = OverlapScale.X / float(InSampleState.TileCounts.X);
	TilePrincipalPointScale.Y = OverlapScale.Y / float(InSampleState.TileCounts.Y);

	TilePrincipalPointOffset.X *= TilePrincipalPointScale.X;
	TilePrincipalPointOffset.Y *= TilePrincipalPointScale.Y;

	return FVector4(TilePrincipalPointOffset.X, -TilePrincipalPointOffset.Y, TilePrincipalPointScale.X, TilePrincipalPointScale.Y);
}

void UMoviePipelineImagePassBase::ModifyProjectionMatrixForTiling(const FMoviePipelineRenderPassMetrics& InSampleState, const bool bInOrthographic, FMatrix& InOutProjectionMatrix, float& OutDoFSensorScale) const
{
	float PadRatioX = 1.0f;
	float PadRatioY = 1.0f;

	if (InSampleState.OverlappedPad.X > 0 && InSampleState.OverlappedPad.Y > 0)
	{
		PadRatioX = float(InSampleState.OverlappedPad.X * 2 + InSampleState.TileSize.X) / float(InSampleState.TileSize.X);
		PadRatioY = float(InSampleState.OverlappedPad.Y * 2 + InSampleState.TileSize.Y) / float(InSampleState.TileSize.Y);
	}

	float ScaleX = PadRatioX / float(InSampleState.TileCounts.X);
	float ScaleY = PadRatioY / float(InSampleState.TileCounts.Y);

	InOutProjectionMatrix.M[0][0] /= ScaleX;
	InOutProjectionMatrix.M[1][1] /= ScaleY;
	OutDoFSensorScale = ScaleX;

	// this offset would be correct with no pad
	float OffsetX = -((float(InSampleState.TileIndexes.X) + 0.5f - float(InSampleState.TileCounts.X) / 2.0f) * 2.0f);
	float OffsetY = ((float(InSampleState.TileIndexes.Y) + 0.5f - float(InSampleState.TileCounts.Y) / 2.0f) * 2.0f);

	if (bInOrthographic)
	{
		// Scale the off-center projection matrix too so that it's appropriately sized down for each tile.
		InOutProjectionMatrix.M[3][0] /= ScaleX;
		InOutProjectionMatrix.M[3][1] /= ScaleY;
		InOutProjectionMatrix.M[3][0] += OffsetX / PadRatioX;
		InOutProjectionMatrix.M[3][1] += OffsetY / PadRatioY;
	}
	else
	{
		// Scale the off-center projection matrix too so that it's appropriately sized down for each tile.
		InOutProjectionMatrix.M[2][0] /= ScaleX;
		InOutProjectionMatrix.M[2][1] /= ScaleY;
		// Then offset it for this particular tile.
		InOutProjectionMatrix.M[2][0] += OffsetX / PadRatioX;
		InOutProjectionMatrix.M[2][1] += OffsetY / PadRatioY;
	}
}

/** Creates a transformation for a cubemap face, following the D3D cubemap layout. */
FMatrix UMoviePipelineImagePassBase::CalcCubeFaceTransform(ECubeFace Face) const
{
	static const FVector XAxis(1.f, 0.f, 0.f);
	static const FVector YAxis(0.f, 1.f, 0.f);
	static const FVector ZAxis(0.f, 0.f, 1.f);

	// vectors we will need for our basis
	FVector vUp(YAxis);
	FVector vDir;
	switch (Face)
	{
	case CubeFace_PosX:
		vDir = XAxis;
		break;
	case CubeFace_NegX:
		vDir = -XAxis;
		break;
	case CubeFace_PosY:
		vUp = -ZAxis;
		vDir = YAxis;
		break;
	case CubeFace_NegY:
		vUp = ZAxis;
		vDir = -YAxis;
		break;
	case CubeFace_PosZ:
		vDir = ZAxis;
		break;
	case CubeFace_NegZ:
		vDir = -ZAxis;
		break;
	}
	// derive right vector
	FVector vRight(vUp^ vDir);
	// create matrix from the 3 axes
	return FBasisVectorMatrix(vRight, vUp, vDir, FVector::ZeroVector);
}

UE::MoviePipeline::FImagePassCameraViewData UMoviePipelineImagePassBase::GetCameraInfo(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload) const
{
	UE::MoviePipeline::FImagePassCameraViewData OutCameraData;

	// Default implementation doesn't support multi-camera and always provides the information from the current PlayerCameraManager
	if (GetPipeline()->GetWorld()->GetFirstPlayerController()->PlayerCameraManager)
	{
		OutCameraData.ViewInfo = GetPipeline()->GetWorld()->GetFirstPlayerController()->PlayerCameraManager->GetCameraCacheView();

		// Now override some of the properties with things that come from MRQ
		OutCameraData.ViewInfo.Location = InOutSampleState.FrameInfo.CurrViewLocation;
		OutCameraData.ViewInfo.Rotation = InOutSampleState.FrameInfo.CurrViewRotation;
		OutCameraData.ViewInfo.PreviousViewTransform = FTransform(InOutSampleState.FrameInfo.PrevViewRotation, InOutSampleState.FrameInfo.PrevViewLocation);

		// And some fields that aren't in FMinimalViewInfo
		OutCameraData.ViewActor = GetPipeline()->GetWorld()->GetFirstPlayerController()->GetViewTarget();

		// This only works if you use a Cine Camera (which is almost guranteed with Sequencer) and it's easier (and less human error prone) than re-deriving the information
		ACineCameraActor* CineCameraActor = Cast<ACineCameraActor>(GetWorld()->GetFirstPlayerController()->PlayerCameraManager->GetViewTarget());
		if (CineCameraActor)
		{
			UCineCameraComponent* CineCameraComponent = CineCameraActor->GetCineCameraComponent();
			if (CineCameraComponent)
			{
				// Add camera-specific metadata
				UE::MoviePipeline::GetMetadataFromCineCamera(CineCameraComponent, PassIdentifier.CameraName, PassIdentifier.Name, OutCameraData.FileMetadata);
			}
		}
	}

	return OutCameraData;
}


TSharedPtr<FAccumulatorPool::FAccumulatorInstance, ESPMode::ThreadSafe> FAccumulatorPool::BlockAndGetAccumulator_GameThread(int32 InFrameNumber, const FMoviePipelinePassIdentifier& InPassIdentifier)
{
	FScopeLock ScopeLock(&CriticalSection);

	int32 AvailableIndex = INDEX_NONE;
	while (AvailableIndex == INDEX_NONE)
	{
		for (int32 Index = 0; Index < Accumulators.Num(); Index++)
		{
			if (InFrameNumber == Accumulators[Index]->ActiveFrameNumber && InPassIdentifier == Accumulators[Index]->ActivePassIdentifier)
			{
				AvailableIndex = Index;
				break;
			}
		}

		if (AvailableIndex == INDEX_NONE)
		{
			// If we don't have an accumulator already working on it let's look for a free one.
			for (int32 Index = 0; Index < Accumulators.Num(); Index++)
			{
				if (!Accumulators[Index]->IsActive())
				{
					// Found a free one, tie it to this output frame.
					Accumulators[Index]->ActiveFrameNumber = InFrameNumber;
					Accumulators[Index]->ActivePassIdentifier = InPassIdentifier;
					Accumulators[Index]->bIsActive = true;
					Accumulators[Index]->TaskPrereq = nullptr;
					AvailableIndex = Index;
					break;
				}
			}
		}

		// If a free accumulator wasn't found, try creating a new one
		if (AvailableIndex == INDEX_NONE)
		{
			if (TSharedPtr<FAccumulatorInstance, ESPMode::ThreadSafe> NewAccumulatorInstance = CreateNewAccumulatorInstance())
			{
				NewAccumulatorInstance->ActiveFrameNumber = InFrameNumber;
				NewAccumulatorInstance->ActivePassIdentifier = InPassIdentifier;
				NewAccumulatorInstance->bIsActive = true;
				NewAccumulatorInstance->TaskPrereq = nullptr;
			
				AvailableIndex = Accumulators.Num();
				Accumulators.Add(NewAccumulatorInstance);
				UE_LOGF(LogMovieRenderPipeline, Log, "Allocated a Accumulator for Pool %ls, New Pool Count: %d", *GetPoolName().ToString(), Accumulators.Num());
			}
		}
	}

	return Accumulators[AvailableIndex];
}

bool FAccumulatorPool::FAccumulatorInstance::IsActive() const
{
	return bIsActive;
}

void FAccumulatorPool::FAccumulatorInstance::SetIsActive(const bool bInIsActive)
{
	bIsActive = bInIsActive;
}

namespace MoviePipeline
{
	/**
	 * Clears the letterbox border that was not already cleared in GPU.
	 * Note: It was left this way for proper anti-aliasing at the edges of the frame.
	 * 
	 * @param LetterboxData - Data about the border, including whether it is enabled or not.
	 * @param ImageData     - Pixel data to draw on.
	 */
	void DrawLetterboxBorder(const FLetterboxData& LetterboxData, FImagePixelData* ImageData)
	{
		if (!ImageData || !LetterboxData.bDrawLetterboxBorder)
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline::DrawLetterboxBorder);

		constexpr int32 BorderThickness = 2;
		const FIntRect& FrameActiveArea = LetterboxData.FrameActiveArea;

		// Get the overall image dimensions.
		const FIntPoint ImageSize = ImageData->GetSize();
		const int32 FullWidth = ImageSize.X;
		const int32 FullHeight = ImageSize.Y;

		// Generic lambda to clear a rectangular region within a pixel array.
		auto ClearRegion = [FullWidth](auto& Pixels, int32 X0, int32 X1, int32 Y0, int32 Y1)
			{
				if (X0 >= X1 || Y0 >= Y1)
				{
					return;
				}

				for (int32 Y = Y0; Y < Y1; ++Y)
				{
					for (int32 X = X0; X < X1; ++X)
					{
						Pixels[Y * FullWidth + X] = {}; // Transparent black
					}
				}
			};

		// Lambda to draw all four borders using the ClearRegion helper.
		auto DrawBorders = [&](auto& Pixels)
			{
				// Top border (includes top-left and top-right corners)
				{
					const int32 X0 = FMath::Max(FrameActiveArea.Min.X - BorderThickness, 0);
					const int32 X1 = FMath::Min(FrameActiveArea.Max.X + BorderThickness, FullWidth);
					const int32 Y0 = FMath::Max(FrameActiveArea.Min.Y - BorderThickness, 0);
					const int32 Y1 = FrameActiveArea.Min.Y;

					ClearRegion(Pixels, X0, X1, Y0, Y1);
				}

				// Bottom border (includes bottom-left and bottom-right corners)
				{
					const int32 X0 = FMath::Max(FrameActiveArea.Min.X - BorderThickness, 0);
					const int32 X1 = FMath::Min(FrameActiveArea.Max.X + BorderThickness, FullWidth);
					const int32 Y0 = FrameActiveArea.Max.Y;
					const int32 Y1 = FMath::Min(FrameActiveArea.Max.Y + BorderThickness, FullHeight);

					ClearRegion(Pixels, X0, X1, Y0, Y1);
				}

				// Left border
				{
					const int32 X0 = FMath::Max(FrameActiveArea.Min.X - BorderThickness, 0);
					const int32 X1 = FrameActiveArea.Min.X;
					const int32 Y0 = FrameActiveArea.Min.Y;
					const int32 Y1 = FrameActiveArea.Max.Y;

					ClearRegion(Pixels, X0, X1, Y0, Y1);
				}

				// Right border
				{
					const int32 X0 = FrameActiveArea.Max.X;
					const int32 X1 = FMath::Min(FrameActiveArea.Max.X + BorderThickness, FullWidth);
					const int32 Y0 = FrameActiveArea.Min.Y;
					const int32 Y1 = FrameActiveArea.Max.Y;

					ClearRegion(Pixels, X0, X1, Y0, Y1);
				}
			};

		// Dispatch based on the pixel type.
		switch (ImageData->GetType())
		{
		case EImagePixelType::Color:
		{
			DrawBorders(static_cast<TImagePixelData<FColor>*>(ImageData)->Pixels);
			break;
		}
		case EImagePixelType::Float16:
		{
			DrawBorders(static_cast<TImagePixelData<FFloat16Color>*>(ImageData)->Pixels);
			break;
		}
		case EImagePixelType::Float32:
		{
			DrawBorders(static_cast<TImagePixelData<FLinearColor>*>(ImageData)->Pixels);
			break;
		}
		default:
			checkNoEntry();
			break;
		}
	}

	void AccumulateSample_TaskThread(TUniquePtr<FImagePixelData>&& InPixelData, const MoviePipeline::FImageSampleAccumulationArgs& InParams)
	{
		SCOPE_CYCLE_COUNTER(STAT_AccumulateSample_TaskThread);

		TUniquePtr<FImagePixelData> SamplePixelData = MoveTemp(InPixelData);
		const bool bIsWellFormed = SamplePixelData->IsDataWellFormed();

		if (!bIsWellFormed)
		{
			// figure out why it is not well formed, and print a warning.
			int64 RawSize = SamplePixelData->GetRawDataSizeInBytes();

			int64 SizeX = SamplePixelData->GetSize().X;
			int64 SizeY = SamplePixelData->GetSize().Y;
			int64 ByteDepth = int64(SamplePixelData->GetBitDepth() / 8);
			int64 NumChannels = int64(SamplePixelData->GetNumChannels());
			int64 ExpectedTotalSize = SizeX * SizeY * ByteDepth * NumChannels;
			int64 ActualTotalSize = SamplePixelData->GetRawDataSizeInBytes();

			UE_LOGF(LogMovieRenderPipeline, Log, "AccumulateSample_RenderThread: Data is not well formed.");
			UE_LOGF(LogMovieRenderPipeline, Log, "Image dimension: %lldx%lld, %lld, %lld", SizeX, SizeY, ByteDepth, NumChannels);
			UE_LOGF(LogMovieRenderPipeline, Log, "Expected size: %lld", ExpectedTotalSize);
			UE_LOGF(LogMovieRenderPipeline, Log, "Actual size:   %lld", ActualTotalSize);
		}

		check(bIsWellFormed);

		FImagePixelDataPayload* OriginalFramePayload = SamplePixelData->GetPayload<FImagePixelDataPayload>();
		check(OriginalFramePayload);

		// We duplicate the payload for now because there are multiple cases where we need to create a new 
		// image payload and we can't transfer the existing payload over.
		TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> NewPayload = OriginalFramePayload->Copy();

		// Writing tiles can be useful for debug reasons. These get passed onto the output every frame.
		if (NewPayload->SampleState.bWriteSampleToDisk)
		{
			// Send the data to the Output Builder. This has to be a copy of the pixel data from the GPU, since
			// it enqueues it onto the game thread and won't be read/sent to write to disk for another frame. 
			// The extra copy is unfortunate, but is only the size of a single sample (ie: 1920x1080 -> 17mb)
			TUniquePtr<FImagePixelData> SampleData = SamplePixelData->CopyImageData();
			ensure(InParams.OutputMerger.IsValid());
			InParams.OutputMerger.Pin()->OnSingleSampleDataAvailable_AnyThread(MoveTemp(SampleData));
		}

		const bool bHasOverlap = NewPayload->SampleState.OverlappedPad != FIntPoint::ZeroValue;

		// Optimization! If we don't need the accumulator (no tiling, no supersampling, no overlap) then we'll skip it 
		// and just send it straight to the output stage, significantly improving performance in the baseline case.
		{
			const bool bOneTile = NewPayload->IsFirstTile() && NewPayload->IsLastTile();
			const bool bOneTS = NewPayload->IsFirstTemporalSample() && NewPayload->IsLastTemporalSample();
			const bool bOneSS = NewPayload->SampleState.SpatialSampleCount == 1;

			if (bOneTile && bOneTS && bOneSS && !bHasOverlap)
			{
				// We do not expect deferred letterbox drawing without tile overlap present.
				check(!InParams.LetterboxData.bDrawLetterboxBorder);

				// Send the data directly to the Output Builder and skip the accumulator.
				ensure(InParams.OutputMerger.IsValid());
				InParams.OutputMerger.Pin()->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(SamplePixelData));
				return;
			}
		}

		// Allocate memory if the ImageAccumulator has not been initialized yet for this output
		// This usually happens on the first sample (regular case), or on the last spatial sample of the first temporal sample (path tracer)
		MoviePipeline::FTileWeight1D WeightFunctionX;
		MoviePipeline::FTileWeight1D WeightFunctionY;
		NewPayload->GetWeightFunctionParams(/*Out*/ WeightFunctionX, /*Out*/ WeightFunctionY);

		// Adjust the weights to account for the pixels that were cleared before accumulation,
		// and should therefore not be sampled.
		// 
		// Note: We exclude overlap cases which should have the anti-aliasing margin with real pixels
		// already and do not really need this sampling protection. Doing so is slightly more complicated
		// because they will have the finite slopes in the _/-\_ weights 1D curve and would probably need
		// to add MinX and MaxX limit notions to FTileWeight1D and use that instead to keep the slopes intact.
		if (!bHasOverlap)
		{
			WeightFunctionX.X0 = FMath::Max(WeightFunctionX.X0, InParams.LetterboxData.LeftSamplePixelsClearedBeforeAccumulation);
			WeightFunctionX.X1 = FMath::Max(WeightFunctionX.X1, InParams.LetterboxData.LeftSamplePixelsClearedBeforeAccumulation);
			WeightFunctionX.X2 = FMath::Min(WeightFunctionX.X2, SamplePixelData->GetSize().X - InParams.LetterboxData.RightSamplePixelsClearedBeforeAccumulation);
			WeightFunctionX.X3 = FMath::Min(WeightFunctionX.X3, SamplePixelData->GetSize().X - InParams.LetterboxData.RightSamplePixelsClearedBeforeAccumulation);

			WeightFunctionY.X0 = FMath::Max(WeightFunctionY.X0, InParams.LetterboxData.TopSamplePixelsClearedBeforeAccumulation);
			WeightFunctionY.X1 = FMath::Max(WeightFunctionY.X1, InParams.LetterboxData.TopSamplePixelsClearedBeforeAccumulation);
			WeightFunctionY.X2 = FMath::Min(WeightFunctionY.X2, SamplePixelData->GetSize().Y - InParams.LetterboxData.BottomSamplePixelsClearedBeforeAccumulation);
			WeightFunctionY.X3 = FMath::Min(WeightFunctionY.X3, SamplePixelData->GetSize().Y - InParams.LetterboxData.BottomSamplePixelsClearedBeforeAccumulation);
		}

		TSharedPtr<FImageOverlappedAccumulator> PinnedImageAccumulator = InParams.ImageAccumulator.Pin();
		TSharedPtr<IMoviePipelineOutputMerger> PinnedOutputMerger = InParams.OutputMerger.Pin();

		ensure(PinnedImageAccumulator.IsValid());
		ensure(PinnedOutputMerger.IsValid());
		if (PinnedImageAccumulator->NumChannels == 0)
		{
			LLM_SCOPE_BYNAME(TEXT("MoviePipeline/ImageAccumulatorInitMemory"));
			int32 ChannelCount = InParams.bAccumulateAlpha ? 4 : 3;
			PinnedImageAccumulator->InitMemory(NewPayload->GetAccumulatorSize(), ChannelCount); 
			PinnedImageAccumulator->ZeroPlanes();
			PinnedImageAccumulator->AccumulationGamma = NewPayload->SampleState.AccumulationGamma;
		}

		// Accumulate the new sample to our target
		{
			// Some samples can come back at a different size than expected (post process materials) which
			// creates numerous issues with the accumulators. To work around this issue for now, we will resize
			// the image to the expected resolution. 
			FIntPoint RawSize = SamplePixelData->GetSize();
			const bool bCorrectSize = NewPayload->GetOverlapPaddedSizeIsValid(RawSize);
			
			if (!bCorrectSize)
			{
				const double ResizeConvertBeginTime = FPlatformTime::Seconds();

				// Convert the incoming data to full floats (the accumulator would do this later normally anyways)
				TArray64<FLinearColor> FullSizeData;
				FullSizeData.AddUninitialized(RawSize.X * RawSize.Y);

				if (SamplePixelData->GetType() == EImagePixelType::Float32)
				{
					const void* RawDataPtr = nullptr;
					int64 RawDataSize;
					
					if(SamplePixelData->GetRawData(RawDataPtr, RawDataSize) == true)
					{
						FMemory::Memcpy(FullSizeData.GetData(), RawDataPtr, RawDataSize);
					}
					else
					{
						UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to retrieve raw data from image data for writing. Bailing.");
						return;
					}
				}
				else if (SamplePixelData->GetType() == EImagePixelType::Float16)
				{
					const void* RawDataPtr = nullptr;
					int64 RawDataSize;

					if(SamplePixelData->GetRawData(RawDataPtr, RawDataSize) == true)
					{
						const FFloat16Color* DataAsColor = reinterpret_cast<const FFloat16Color*>(RawDataPtr);
						for (int64 Index = 0; Index < RawSize.X * RawSize.Y; Index++)
						{
							FullSizeData[Index] = FLinearColor(DataAsColor[Index]);
						}
					}
					else
					{
						UE_LOGF(LogMovieRenderPipelineIO, Error, "Failed to retrieve raw data from image data for writing. Bailing.");
						return;
					}

				}
				else
				{
					check(0);
				}
				const double ResizeConvertEndTime = FPlatformTime::Seconds();

				// Now we can resize to our target size.
				FIntPoint TargetSize = NewPayload->GetOverlapPaddedSize();

				TArray64<FLinearColor> NewPixelData;
				NewPixelData.SetNumUninitialized(TargetSize.X* TargetSize.Y);

				FImageUtils::ImageResize(RawSize.X, RawSize.Y, MakeArrayView<FLinearColor>(FullSizeData.GetData(), FullSizeData.Num()), TargetSize.X, TargetSize.Y, MakeArrayView<FLinearColor>(NewPixelData.GetData(), NewPixelData.Num()));

				const float ElapsedConvertMs = float((ResizeConvertEndTime - ResizeConvertBeginTime) * 1000.0f);
				const float ElapsedResizeMs = float((FPlatformTime::Seconds() - ResizeConvertEndTime) * 1000.0f);

				UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "Resize Convert Time: %8.2fms Resize Time: %8.2fms", ElapsedConvertMs, ElapsedResizeMs);

				SamplePixelData = MakeUnique<TImagePixelData<FLinearColor>>(FIntPoint(TargetSize.X, TargetSize.Y), MoveTemp(NewPixelData), NewPayload);

				// Update the raw size to match our new size.
				RawSize = SamplePixelData->GetSize();
			}

			const double AccumulateBeginTime = FPlatformTime::Seconds();

			// This should have been rescaled now if needed, so we can just check again to validate.
			check(NewPayload->GetOverlapPaddedSizeIsValid(RawSize));

			// bool bSkip = NewPayload->SampleState.TileIndexes.X != 0 || NewPayload->SampleState.TileIndexes.Y != 1;
			// if (!bSkip)
			{
				PinnedImageAccumulator->AccumulatePixelData(*SamplePixelData, NewPayload->GetOverlappedOffset(), NewPayload->GetOverlappedSubpixelShift(), WeightFunctionX, WeightFunctionY);
			}

			const double AccumulateEndTime = FPlatformTime::Seconds();
			const float ElapsedMs = float((AccumulateEndTime - AccumulateBeginTime) * 1000.0f);

			UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "Accumulation time: %8.2fms", ElapsedMs);
			
		}

		if (NewPayload->IsLastTile() && NewPayload->IsLastTemporalSample())
		{
			int32 FullSizeX = PinnedImageAccumulator->PlaneSize.X;
			int32 FullSizeY = PinnedImageAccumulator->PlaneSize.Y;

			// Now that a tile is fully built and accumulated we can notify the output builder that the
			// data is ready so it can pass that onto the output containers (if needed).
			if (SamplePixelData->GetType() == EImagePixelType::Float32)
			{
				// 32 bit FLinearColor
				TUniquePtr<TImagePixelData<FLinearColor> > FinalPixelData = MakeUnique<TImagePixelData<FLinearColor>>(FIntPoint(FullSizeX, FullSizeY), NewPayload);
				PinnedImageAccumulator->FetchFinalPixelDataLinearColor(FinalPixelData->Pixels);

				// Apply letterbox outline. Will only do any work if enabled.
				DrawLetterboxBorder(InParams.LetterboxData, FinalPixelData.Get());

				// Send the data to the Output Builder
				PinnedOutputMerger->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
			}
			else if (SamplePixelData->GetType() == EImagePixelType::Float16)
			{
				// 16 bit FLinearColor
				TUniquePtr<TImagePixelData<FFloat16Color> > FinalPixelData = MakeUnique<TImagePixelData<FFloat16Color>>(FIntPoint(FullSizeX, FullSizeY), NewPayload);
				PinnedImageAccumulator->FetchFinalPixelDataHalfFloat(FinalPixelData->Pixels);

				// Apply letterbox outline. Will only do any work if enabled.
				DrawLetterboxBorder(InParams.LetterboxData, FinalPixelData.Get());

				// Send the data to the Output Builder
				PinnedOutputMerger->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
			}
			else if (SamplePixelData->GetType() == EImagePixelType::Color)
			{
				// 8bit FColors
				TUniquePtr<TImagePixelData<FColor>> FinalPixelData = MakeUnique<TImagePixelData<FColor>>(FIntPoint(FullSizeX, FullSizeY), NewPayload);
				PinnedImageAccumulator->FetchFinalPixelDataByte(FinalPixelData->Pixels);

				// Apply letterbox outline. Will only do any work if enabled.
				DrawLetterboxBorder(InParams.LetterboxData, FinalPixelData.Get());

				// Send the data to the Output Builder
				PinnedOutputMerger->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
			}
			else
			{
				check(0);
			}

			// Free the memory in the accumulator.
			PinnedImageAccumulator->Reset();
		}

		{
			// Explicitly free the SamplePixelData (which by now has been copied into the accumulator)
			// so that we can profile how long freeing the allocation takes.
			TRACE_CPUPROFILER_EVENT_SCOPE(ReleasePixelDataSample);
			SamplePixelData.Reset();
		}
	}
}


