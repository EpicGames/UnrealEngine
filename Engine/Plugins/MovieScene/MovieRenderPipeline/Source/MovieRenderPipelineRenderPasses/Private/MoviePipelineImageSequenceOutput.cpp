// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineImageSequenceOutput.h"
#include "ImageWriteTask.h"
#include "ImagePixelData.h"
#include "Modules/ModuleManager.h"
#include "ImageWriteQueue.h"
#include "MoviePipeline.h"
#include "ImageWriteStream.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MovieRenderTileImage.h"
#include "MovieRenderOverlappedImage.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Misc/FrameRate.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineBurnInSetting.h"
#include "Containers/UnrealString.h"
#include "Misc/StringFormatArg.h"
#include "MoviePipelineOutputBase.h"
#include "MoviePipelineImageQuantization.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineWidgetRenderSetting.h"
#include "MoviePipelineUtils.h"
#include "HAL/PlatformTime.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineImageSequenceOutput)

DECLARE_CYCLE_STAT(TEXT("ImgSeqOutput_RecieveImageData"), STAT_ImgSeqRecieveImageData, STATGROUP_MoviePipeline);

namespace UE
{
	namespace MoviePipeline
	{
		FAsyncImageQuantization::FAsyncImageQuantization(FImageWriteTask* InWriteTask, const bool bInConvertToSRGB)
			: ParentWriteTask(InWriteTask)
			, bConvertToSRGB(bInConvertToSRGB)
		{
		}

		void FAsyncImageQuantization::operator()(FImagePixelData* PixelData)
		{
			// Note: Ideally we would use FImageCore routines here, but there is no easy way to construct pixel data from an FImage currently.

			// Convert the incoming data to 8-bit, potentially with sRGB applied.
			TUniquePtr<FImagePixelData> QuantizedPixelData = QuantizeImagePixelDataToBitDepth(PixelData, 8, nullptr, bConvertToSRGB);
			ParentWriteTask->PixelData = MoveTemp(QuantizedPixelData);
		}
	}
}

UMoviePipelineImageSequenceOutputBase::UMoviePipelineImageSequenceOutputBase()
{
	if (!HasAnyFlags(RF_ArchetypeObject))
	{
		ImageWriteQueue = &FModuleManager::Get().LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue").GetWriteQueue();
	}
}

void UMoviePipelineImageSequenceOutputBase::BeginFinalizeImpl()
{
	FinalizeFence = ImageWriteQueue->CreateFence();
}

bool UMoviePipelineImageSequenceOutputBase::HasFinishedProcessingImpl()
{ 
	// Wait until the finalization fence is reached meaning we've written everything to disk.
	return Super::HasFinishedProcessingImpl() && (!FinalizeFence.IsValid() || FinalizeFence.WaitFor(0));
}

void UMoviePipelineImageSequenceOutputBase::OnShotFinishedImpl(const UMoviePipelineExecutorShot* InShot, const bool bFlushToDisk)
{
	if (bFlushToDisk)
	{
		UE_LOGF(LogMovieRenderPipelineIO, Log, "ImageSequenceOutputBase flushing %d tasks to disk, inserting a fence in the queue and then waiting...", ImageWriteQueue->GetNumPendingTasks());
		const double FlushBeginTime = FPlatformTime::Seconds();

		TFuture<void> Fence = ImageWriteQueue->CreateFence();
		Fence.Wait();
		const float ElapsedS = float((FPlatformTime::Seconds() - FlushBeginTime));
		UE_LOGF(LogMovieRenderPipelineIO, Log, "Finished flushing tasks to disk after %2.2fs!", ElapsedS);
	}
}

void UMoviePipelineImageSequenceOutputBase::OnReceiveImageDataImpl(FMoviePipelineMergerOutputFrame* InMergedOutputFrame)
{
	SCOPE_CYCLE_COUNTER(STAT_ImgSeqRecieveImageData);

	check(InMergedOutputFrame);

	// Special case for extracting Burn Ins and Widget Renderer 
	TArray<MoviePipeline::FCompositePassInfo> CompositedPasses;
	MoviePipeline::GetPassCompositeData(InMergedOutputFrame, CompositedPasses);


	UMoviePipelineOutputSetting* OutputSettings = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	check(OutputSettings);

	UMoviePipelineColorSetting* ColorSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineColorSetting>();

	FString OutputDirectory = OutputSettings->OutputDirectory.Path;

	// The InMergedOutputFrame->ImageOutputData map contains both RenderPasses and CompositePasses.
	// We determine how we gather pixel data based on the number of RenderPasses we have done, not counting the CompositePasses.
	// This is the reason for using a separate RenderPassIteration counter with a foreach loop, only incrementing it for RenderPasses.
	int32 RenderPassIteration = 0;
	const int32 RenderPassCount = InMergedOutputFrame->ImageOutputData.Num() - CompositedPasses.Num();
	for (TPair<FMoviePipelinePassIdentifier, TUniquePtr<FImagePixelData>>& RenderPassData : InMergedOutputFrame->ImageOutputData)
	{
		// Don't write out a composited pass in this loop, as it will be merged with the Final Image and not written separately. 
		bool bSkip = false;
		for (const MoviePipeline::FCompositePassInfo& CompositePass : CompositedPasses)
		{
			if (CompositePass.PassIdentifier == RenderPassData.Key)
			{
				bSkip = true;
				break;
			}
		}

		if (bSkip)
		{
			continue;
		}

		EImageFormat PreferredOutputFormat = OutputFormat;

		FImagePixelDataPayload* Payload = RenderPassData.Value->GetPayload<FImagePixelDataPayload>();

		// If the output requires a transparent output (to be useful) then we'll on a per-case basis override their intended
		// filetype to something that makes that file useful.
		if (Payload->bRequireTransparentOutput)
		{
			if (PreferredOutputFormat == EImageFormat::BMP ||
				PreferredOutputFormat == EImageFormat::JPEG)
			{
				PreferredOutputFormat = EImageFormat::PNG;
			}
		}

		const TCHAR* Extension = TEXT("");
		switch (PreferredOutputFormat)
		{
		case EImageFormat::PNG: Extension = TEXT("png"); break;
		case EImageFormat::JPEG: Extension = TEXT("jpeg"); break;
		case EImageFormat::BMP: Extension = TEXT("bmp"); break;
		case EImageFormat::EXR: Extension = TEXT("exr"); break;
		}

		MoviePipeline::FMoviePipelineOutputFutureData OutputData;
		OutputData.Shot = GetPipeline()->GetActiveShotList()[Payload->SampleState.OutputState.ShotIndex];
		OutputData.PassIdentifier = RenderPassData.Key;
		OutputData.OriginSettingClass = GetClass();

		struct FXMLData
		{
			FString ClipName;
			FString ImageSequenceFileName;
		};
		
		FXMLData XMLData;
		{
			// We need to resolve the filename format string. We combine the folder and file name into one long string first.
			FString FileNameFormatString = OutputDirectory / OutputSettings->FileNameFormat;

			// If we're writing more than one render pass out, we need to ensure the file name has the format string in it so we don't
			// overwrite the same file multiple times. Burn In overlays don't count if they are getting composited on top of an existing file.
			const bool bIncludeRenderPass = InMergedOutputFrame->HasDataFromMultipleRenderPasses(CompositedPasses);
			const bool bIncludeCameraName = InMergedOutputFrame->HasDataFromMultipleCameras();
			const bool bTestFrameNumber = true;

			UE::MoviePipeline::ValidateOutputFormatString(/*InOut*/ FileNameFormatString, bIncludeRenderPass, bTestFrameNumber, bIncludeCameraName);

			// Create specific data that needs to override 
			TMap<FString, FString> FormatOverrides;
			FormatOverrides.Add(TEXT("render_pass"), RenderPassData.Key.Name);
			FormatOverrides.Add(TEXT("ext"), Extension);
			FMoviePipelineFormatArgs FinalFormatArgs;

			// Resolve for XMLs
			{
				GetPipeline()->ResolveFilenameFormatArguments(/*In*/ FileNameFormatString, FormatOverrides, /*Out*/ XMLData.ImageSequenceFileName, FinalFormatArgs, &Payload->SampleState.OutputState, -Payload->SampleState.OutputState.ShotOutputFrameNumber);
			}
			
			// Resolve the final absolute file path to write this to
			{
				GetPipeline()->ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, OutputData.FilePath, FinalFormatArgs, &Payload->SampleState.OutputState);

				if (FPaths::IsRelative(OutputData.FilePath))
				{
					OutputData.FilePath = FPaths::ConvertRelativePathToFull(OutputData.FilePath);
				}

				// Re-resolve using the frame-templated format string
				const FString FrameTemplatedFormatString = MoviePipeline::GetFrameTemplatedFilePath(FileNameFormatString);	
				GetPipeline()->ResolveFilenameFormatArguments(FrameTemplatedFormatString, FormatOverrides, OutputData.FrameTemplatedFilePath, FinalFormatArgs, &Payload->SampleState.OutputState);

				if (FPaths::IsRelative(OutputData.FrameTemplatedFilePath))
				{
					OutputData.FrameTemplatedFilePath = FPaths::ConvertRelativePathToFull(OutputData.FrameTemplatedFilePath);
				}
			}

			// More XML resolving. Create a deterministic clipname by removing frame numbers, file extension, and any trailing .'s
			{
				UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);
				GetPipeline()->ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, XMLData.ClipName, FinalFormatArgs, &Payload->SampleState.OutputState);
				XMLData.ClipName.RemoveFromEnd(Extension);
				XMLData.ClipName.RemoveFromEnd(".");
			}
		}

		TUniquePtr<FImageWriteTask> TileImageTask = MakeUnique<FImageWriteTask>();
		TileImageTask->Format = PreferredOutputFormat;
		TileImageTask->CompressionQuality = 100;
		TileImageTask->Filename = OutputData.FilePath;

		// If the overscan isn't cropped from the final image, offset any composites to the top-left of the original frustum
		FIntPoint CompositeOffset = FIntPoint::ZeroValue;

		// For now, only passes that were rendered at the overscanned resolution can be cropped using the crop rectangle
		const bool bIsCropRectValid = !Payload->SampleState.CropRectangle.IsEmpty();
		const bool bCanCropResolution = RenderPassData.Value->GetSize() == Payload->SampleState.OverscannedResolution;
		if (ShouldCropOverscanImpl() && bIsCropRectValid && bCanCropResolution)
		{
			switch (RenderPassData.Value->GetType())
			{
			case EImagePixelType::Color:
				TileImageTask->PixelPreProcessors.Add(TAsyncCropImage<FColor>(TileImageTask.Get(), Payload->SampleState.CropRectangle));
				break;

			case EImagePixelType::Float16:
				TileImageTask->PixelPreProcessors.Add(TAsyncCropImage<FFloat16Color>(TileImageTask.Get(), Payload->SampleState.CropRectangle));
				break;

			case EImagePixelType::Float32:
				TileImageTask->PixelPreProcessors.Add(TAsyncCropImage<FLinearColor>(TileImageTask.Get(), Payload->SampleState.CropRectangle));
				break;
			}
		}
		else
		{
			CompositeOffset = Payload->SampleState.CropRectangle.Min;
		}
		
		TUniquePtr<FImagePixelData> QuantizedPixelData = RenderPassData.Value->CopyImageData();
		EImagePixelType QuantizedPixelType = QuantizedPixelData->GetType();

		switch (PreferredOutputFormat)
		{
		case EImageFormat::PNG:
		case EImageFormat::JPEG:
		case EImageFormat::BMP:
		{
			// All three of these formats only support 8 bit data, so we need to take the incoming buffer type,
			// copy it into a new 8-bit array and apply a little noise to the data to help hide gradient banding.
			const bool bApplysRGB = !(ColorSetting && ColorSetting->OCIOConfiguration.bIsEnabled);
			TileImageTask->PixelPreProcessors.Add(UE::MoviePipeline::FAsyncImageQuantization(TileImageTask.Get(), bApplysRGB));

			// The pixel type will get changed by this pre-processor so future calculations below need to know the correct type they'll be editing.
			QuantizedPixelType = EImagePixelType::Color; 
			break;
		}
		case EImageFormat::EXR:
			// No quantization required, just copy the data as we will move it into the image write task.
			break;
		default:
			check(false);
		}


		// We composite before flipping the alpha so that it is consistent for all formats.
		if (RenderPassData.Key.Name == TEXT("FinalImage") || RenderPassData.Key.Name == TEXT("PathTracer")) 
		{
			for (const MoviePipeline::FCompositePassInfo& CompositePass : CompositedPasses)
			{
				// Match them up by camera name so multiple passes intended for different camera names work.
				if (RenderPassData.Key.CameraName != CompositePass.PassIdentifier.CameraName)
				{
					continue;
				}

				// Check that the composite resolution matches the original frustum resolution to ensure the composite pass doesn't fail.
				// This can happen if multiple cameras with different amounts of overscan are rendered, since composite passes
				// don't support rendering at multiple resolutions
				const FIntPoint CompositeResolution = CompositePass.PixelData->GetSize();
				const FIntPoint CameraOutputResolution = Payload->SampleState.CropRectangle.Size();
				if (CompositeResolution != CameraOutputResolution)
				{
					UE_LOGF(LogMovieRenderPipeline, Warning, "Composite resolution %dx%d does not match output resolution %dx%d, skipping composite for %ls on camera %ls",
						CompositeResolution.X, CompositeResolution.Y,
						CameraOutputResolution.X, CameraOutputResolution.Y,
						*CompositePass.PassIdentifier.Name,
						*RenderPassData.Key.CameraName);

					continue;
				}

				// If there's more than one render pass, we need to copy the composite passes for the first render pass then move for the remaining ones
				const bool bShouldCopyImageData = RenderPassCount > 1 && RenderPassIteration == 0;
				TUniquePtr<FImagePixelData> PixelData = bShouldCopyImageData ? CompositePass.PixelData->CopyImageData() : CompositePass.PixelData->MoveImageDataToNew();
				
				// We don't need to copy the data here (even though it's being passed to a async system) because we already made a unique copy of the
				// burn in/widget data when we decided to composite it.
				switch (QuantizedPixelType)
				{
					case EImagePixelType::Color:
						TileImageTask->PixelPreProcessors.Add(TAsyncCompositeImage<FColor>(MoveTemp(PixelData), CompositeOffset));
						break;
					case EImagePixelType::Float16:
						TileImageTask->PixelPreProcessors.Add(TAsyncCompositeImage<FFloat16Color>(MoveTemp(PixelData), CompositeOffset));
						break;
					case EImagePixelType::Float32:
						TileImageTask->PixelPreProcessors.Add(TAsyncCompositeImage<FLinearColor>(MoveTemp(PixelData), CompositeOffset));
						break;
				}
			}
		}

		// A payload _requiring_ alpha output will override the Write Alpha option, because that flag is used to indicate that the output is
		// no good without alpha, and we already did logic above to ensure it got turned into a filetype that could write alpha.
		if (!IsAlphaAllowed() && !Payload->bRequireTransparentOutput)
		{
			TileImageTask->AddPreProcessorToSetAlphaOpaque();
		}


		TileImageTask->PixelData = MoveTemp(QuantizedPixelData);
		
#if WITH_EDITOR
		GetPipeline()->AddFrameToOutputMetadata(XMLData.ClipName, XMLData.ImageSequenceFileName, Payload->SampleState.OutputState, Extension, Payload->bRequireTransparentOutput);
#endif

		GetPipeline()->AddOutputFuture(ImageWriteQueue->Enqueue(MoveTemp(TileImageTask)), OutputData);

		RenderPassIteration++;
	}
}

void UMoviePipelineImageSequenceOutputBase::GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const
{
	// Stub in a dummy extension (so people know it exists)
	// InOutFormatArgs.Arguments.Add(TEXT("ext"), TEXT("jpg/png/exr")); Hidden since we just always post-pend with an extension.
	InOutFormatArgs.FilenameArguments.Add(TEXT("render_pass"), TEXT("RenderPassName"));
}

void UMoviePipelineImageSequenceOutput_BMP::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesBMP = true;
}

void UMoviePipelineImageSequenceOutput_PNG::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesPNG = true;
}

void UMoviePipelineImageSequenceOutput_JPG::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesJPG = true;
}
