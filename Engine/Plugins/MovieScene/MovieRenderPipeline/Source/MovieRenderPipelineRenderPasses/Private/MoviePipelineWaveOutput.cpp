// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineWaveOutput.h"
#include "MoviePipeline.h"
#include "Sound/SampleBufferIO.h"
#include "SampleBuffer.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "AudioDevice.h"
#include "AudioMixerDevice.h"
#include "AudioThread.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineWaveOutput)

static FAudioDevice* GetAudioDeviceFromWorldContext(const UObject* WorldContextObject)
{
	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->GetNetMode() == NM_DedicatedServer)
	{
		return nullptr;
	}

	return ThisWorld->GetAudioDeviceRaw();
}

static Audio::FMixerDevice* GetAudioMixerDeviceFromWorldContext(const UObject* WorldContextObject)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFromWorldContext(WorldContextObject))
	{
		return static_cast<Audio::FMixerDevice*>(AudioDevice);
	}
	return nullptr;
}

static bool IsMoviePipelineAudioOutputSupported(const UObject* WorldContextObject)
{
	Audio::FMixerDevice* MixerDevice = GetAudioMixerDeviceFromWorldContext(WorldContextObject);
	Audio::IAudioMixerPlatformInterface* AudioMixerPlatform = MixerDevice ? MixerDevice->GetAudioMixerPlatform() : nullptr;

	// If the current audio mixer is non-realtime, audio output is supported
	if (AudioMixerPlatform && AudioMixerPlatform->IsNonRealtime())
	{
		return true;
	}

	// If there is no async audio processing (e.g. we're in the editor), it's possible to create a new non-realtime audio mixer
	if (!FAudioThread::IsUsingThreadedAudio())
	{
		return true;
	}

	// Otherwise, we can't support audio output
	return false;
}

void UMoviePipelineWaveOutput::BeginFinalizeImpl()
{
	if (!IsMoviePipelineAudioOutputSupported(this))
	{
		return;
	}

	ActiveWriters.Reset();

	// There should be no active submixes by the time we finalize - they should all have been converted to recorded samples.
	check(GetPipeline()->GetAudioState().ActiveSubmixes.Num() == 0);

	// If we didn't end up recording audio, don't try outputting any files.
	if (GetPipeline()->GetAudioState().FinishedSegments.Num() == 0)
	{
		return;
	}

	UMoviePipelineOutputSetting* OutputSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	FString OutputFilename = FileNameFormatOverride.Len() > 0 ? FileNameFormatOverride : OutputSetting->FileNameFormat;
	FString FileNameFormatString = OutputSetting->OutputDirectory.Path / OutputFilename;

	const bool bIncludeRenderPass = false;
	const bool bTestFrameNumber = false;
	UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber);
	// Strip any frame number tags so we don't get one audio file per frame.
	UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);

	struct FOutputSegment
	{
		FString FilePath;
		Audio::TSampleBuffer<int16> SampleBuffer;
		int32 ShotIndex;
	};

	TArray<FOutputSegment> FinalSegments;

	// Won't have finished segments for shots that didn't get rendered
	for (int32 SegmentIndex = 0; SegmentIndex < GetPipeline()->GetAudioState().FinishedSegments.Num(); SegmentIndex++)
	{
		const MoviePipeline::FAudioState::FAudioSegment& Segment = GetPipeline()->GetAudioState().FinishedSegments[SegmentIndex];
		if (AlreadyWrittenSegments.Contains(Segment.Id))
		{
			continue;
		}

		// This function may get called many times over the course of a rendering (if flushing to disk). But because we don't own the audio
		// data we can't remove it after we've written it to disk. So we keep track of which segments we've already written to disk for and
		// skip them. This avoids us generating extra output futures for previous shots when we render a new shot.
		AlreadyWrittenSegments.Add(Segment.Id);

		// Generate a filename for this output file.
		TMap<FString, FString> FormatOverrides;
		FormatOverrides.Add(TEXT("render_pass"), TEXT("Audio"));
		FormatOverrides.Add(TEXT("ext"), TEXT("wav"));

		// Append the submix name to have a different .wav output per submix
		FString FileNameWithSubmix = FileNameFormatString + Segment.SubmixName;
		FMoviePipelineFormatArgs FinalFormatArgs;
		FString FinalFilePath;
		GetPipeline()->ResolveFilenameFormatArguments(FileNameWithSubmix, FormatOverrides, FinalFilePath, FinalFormatArgs, &Segment.OutputState);

		if (FPaths::IsRelative(FinalFilePath))
		{
			FinalFilePath = FPaths::ConvertRelativePathToFull(FinalFilePath);
		}

		// Look to see if we already have a output file to append this to.
		FOutputSegment* OutputSegment = nullptr;
		for(int32 Index = 0; Index < FinalSegments.Num(); Index++)
		{
			if (FinalSegments[Index].FilePath == FinalFilePath)
			{
				OutputSegment = &FinalSegments[Index];
				break;
			}
		}

		if (!OutputSegment)
		{
			FinalSegments.AddDefaulted();
			OutputSegment = &FinalSegments[FinalSegments.Num() -1];
			OutputSegment->FilePath = FinalFilePath;
			OutputSegment->ShotIndex = Segment.OutputState.ShotIndex;
		}

		// Convert our samples and append them to the existing array
		double StartTime = FPlatformTime::Seconds();
		Audio::TSampleBuffer<int16> SampleBuffer = Audio::TSampleBuffer<int16>(Segment.SegmentData.GetData(), Segment.SegmentData.Num(), Segment.NumChannels, Segment.SampleRate);
		OutputSegment->SampleBuffer.Append(SampleBuffer.GetData(), Segment.SegmentData.Num(), Segment.NumChannels, Segment.SampleRate);
		UE_LOGF(LogMovieRenderPipeline, Log, "Audio Segment took %f seconds to convert to a sample buffer.", (FPlatformTime::Seconds() - StartTime));
	}

	for(FOutputSegment& Segment : FinalSegments)
	{
		MoviePipeline::FMoviePipelineOutputFutureData OutputData;
		OutputData.Shot = GetPipeline()->GetActiveShotList()[Segment.ShotIndex];
		OutputData.PassIdentifier = FMoviePipelinePassIdentifier(TEXT("Audio"));
		OutputData.OriginSettingClass = GetClass();

		// Do this before we start manipulating the filepath for the audio API
		OutputData.FilePath = Segment.FilePath;

		FString FileName = FPaths::GetBaseFilename(Segment.FilePath);
		FString FileFolder = FPaths::GetPath(Segment.FilePath);

		TPromise<bool> Completed;
		GetPipeline()->AddOutputFuture(Completed.GetFuture(), OutputData);

		TUniquePtr<Audio::FSoundWavePCMWriter> Writer = MakeUnique<Audio::FSoundWavePCMWriter>();
		bool bSuccess = Writer->BeginWriteToWavFile(Segment.SampleBuffer, FileName, FileFolder, [this](){});

		Completed.SetValue(bSuccess);
		ActiveWriters.Add(MoveTemp(Writer));
	}

	// The FSoundWavePCMWriter is unfortunately async, and the completion callbacks don't work unless the main thread
	// can be spun (as it enqueues a callback onto the main thread). We're going to just cheat here and stall the main thread
	// for 0.5s to give it a chance to write to disk. It'll only potentially be an issue with command line encoding if it takes
	// longer than 0.5s to write to disk.
	if (const TConsoleVariableData<float>* CVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("MovieRenderPipeline.WaveOutput.WriteDelay")))
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "Delaying main thread for %f seconds while audio writes to disk.", CVar->GetValueOnGameThread());
		FPlatformProcess::Sleep(CVar->GetValueOnGameThread());
	}
}

void UMoviePipelineWaveOutput::OnShotFinishedImpl(const UMoviePipelineExecutorShot* InShot, const bool bFlushToDisk)
{
	if (bFlushToDisk)
	{
		UE_LOGF(LogMovieRenderPipeline, Log, "UMoviePipelineWaveOutput flushing disk...");
		const double FlushBeginTime = FPlatformTime::Seconds();

		BeginFinalizeImpl();
		FinalizeImpl();

		const float ElapsedS = float((FPlatformTime::Seconds() - FlushBeginTime));
		UE_LOGF(LogMovieRenderPipeline, Log, "Finished flushing tasks to disk after %2.2fs!", ElapsedS);
	}
}

bool UMoviePipelineWaveOutput::HasFinishedProcessingImpl()
{
	return true;
}

void UMoviePipelineWaveOutput::ValidateStateImpl()
{
	Super::ValidateStateImpl();

	if (!IsMoviePipelineAudioOutputSupported(this))
	{
		ValidationState = EMoviePipelineValidationState::Warnings;
		ValidationResults.Add(NSLOCTEXT("MovieRenderPipeline", "WaveOutput_NotUsingDeterministicAudio", "Process must be launched with \"-deterministicaudio\" for this to work. Using a remote render will automatically add this argument."));
	}
}

void UMoviePipelineWaveOutput::BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const 
{
	// Always add this even if we're not disabled so that audio is muted, it'll never line up during preview anyways.
	InOutCommandLineArgs.Add("-deterministicaudio");
}

void UMoviePipelineWaveOutput::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesAudio = true;
}
