// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineDeferredPasses.h"
#include "UObject/ICookInfo.h"

class FMovieRenderPipelineRenderPassesModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
#if WITH_EDITOR
		if (IsRunningCookCommandlet())
		{
			UE::Cook::FDelegates::ModifyCook.AddLambda(
				[](UE::Cook::ICookInfo& CookInfo, TArray<UE::Cook::FPackageCookRule>& InOutPackageCookRules)
				{
					// Ensure these assets (which are referenced only by code) get packaged
					const FString* Assets[] =
					{
						&UMoviePipelineDeferredPassBase::StencilLayerMaterialAsset,
						&UMoviePipelineDeferredPassBase::DefaultDepthAsset,
						&UMoviePipelineDeferredPassBase::DefaultMotionVectorsAsset
					};

					for (const FString* Asset : Assets)
					{
						InOutPackageCookRules.Add(
							UE::Cook::FPackageCookRule{
								.PackageName = FName(FSoftObjectPath(*Asset).GetLongPackageName()),
								.InstigatorName = FName("FMovieRenderPipelineRenderPassesModule"),
								.CookRule = UE::Cook::EPackageCookRule::AddToCook
							}
						);
					}
				}
			);
		}
#endif
	}
};

IMPLEMENT_MODULE(FMovieRenderPipelineRenderPassesModule, MovieRenderPipelineRenderPasses);
