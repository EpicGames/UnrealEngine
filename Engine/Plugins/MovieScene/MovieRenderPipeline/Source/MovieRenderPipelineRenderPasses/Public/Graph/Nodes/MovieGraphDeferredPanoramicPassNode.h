// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphTimeStepData.h"
#include "Graph/Nodes/MovieGraphImagePassBaseNode.h"
#include "MoviePipelinePanoramicBlenderBase.h"
#include "MovieGraphDeferredPanoramicPassNode.generated.h"

namespace UE::MovieGraph::Private
{
	struct FLumenCvarCache;
}

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphDeferredPanoramicNode final : public UMovieGraphImagePassBaseNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphDeferredPanoramicNode();
	UE_API virtual ~UMovieGraphDeferredPanoramicNode() override;

	// UMovieGraphRenderPassNode Interface
	UE_API virtual void RenderImpl(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) override;
	// ~UMovieGraphRenderPassNode Interface

	// UMovieGraphImagePassBaseNode Interface
	UE_API virtual FEngineShowFlags GetShowFlags() const override;
	UE_API virtual EViewModeIndex GetViewModeIndex() const override;
	UE_API virtual TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> CreateInstance() const override;
	UE_API virtual bool GetWriteAllSamples() const override;
	UE_API virtual int32 GetNumSpatialSamples() const override;
	UE_API virtual bool GetDisableToneCurve() const override;
	UE_API virtual bool GetAllowOCIO() const override;
	virtual bool GetOverrideAntiAliasing() const override { return bOverride_AntiAliasingMethod; }
	UE_API virtual EAntiAliasingMethod GetAntiAliasingMethod() const override;
	virtual bool GetEnableHistoryPerTile() const { return bAllocateHistoryPerPane; }
	UE_API virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const;
	UE_API virtual void PreLayerRender(
		const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		const FMovieGraphTimeStepData& InTimeData) override;
	UE_API virtual void PostLayerRender(
		const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		const FMovieGraphTimeStepData& InTimeData) override;
	// ~UMovieGraphImagePassBaseNode Interface

	// UMovieGraphNode Interface
#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif
	// ~UMovieGraphNode Interface

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_NumHorizontalSteps : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_NumVerticalSteps : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bFollowCameraOrientation : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bAllocateHistoryPerPane : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bPageToSystemMemory : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SpatialSampleCount : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AntiAliasingMethod : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Filter : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bDisableToneCurve : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bAllowOCIO : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ViewModeIndex : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bWriteAllSamples : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 8, ClampMin = 8), Category = "Sampling", meta = (EditCondition = "bOverride_NumHorizontalSteps"))
	int32 NumHorizontalSteps;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 3, ClampMin = 3), Category = "Sampling", meta = (EditCondition = "bOverride_NumVerticalSteps"))
	int32 NumVerticalSteps;
	
	/**
	* Should the Pitch, Yaw and Roll of the camera be respected? If false, only the location will be taken from the camera.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sampling", meta = (EditCondition = "bOverride_bFollowCameraOrientation"))
	bool bFollowCameraOrientation;
	
	/**
	* Should we store the render scene history per individual render? This can consume a great deal of memory with many renders,
	* but enables TAA/TSR and other history-based effects (denoisers, auto-exposure, Lumen, etc.) to work.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sampling", meta = (EditCondition = "bOverride_bAllocateHistoryPerPane"))
	bool bAllocateHistoryPerPane;
	
	/*
	* If true, persistented GPU data per panoramic pane is paged to system memory, allowing higher resolutions, but significantly 
	* increasing render times. The GPU data is downloaded after each tile is rendered, and then re-uploaded for the next tile.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sampling", meta = (EditCondition = "bOverride_bPageToSystemMemory"))
	bool bPageToSystemMemory;

	/**
	* How many sub-pixel jitter renders should we do per temporal sample? This can be used to achieve high
	* sample counts without Temporal Sub-Sampling (allowing high sample counts without motion blur being enabled),
	* but we generally recommend using Temporal Sub-Samples when possible. It can also be combined with
	* temporal samples and you will get SpatialSampleCount many renders per temporal sample.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 1, ClampMin = 1), Category = "Sampling", meta = (EditCondition = "bOverride_SpatialSampleCount"))
	int32 SpatialSampleCount;

	/**
	* Which anti-aliasing method should this render use. If this is set to None, then Movie Render Graph
	* will handle anti-aliasing by doing a sub-pixel jitter (one for each temporal/spatial sample). Some
	* rendering effects rely on TSR or TAA to reduce noise so we recommend leaving them enabled
	* where possible. All options work with Spatial and Temporal samples, but TSR/TAA may introduce minor
	* visual artifacts (such as ghosting). MSAA is not supported in the deferred renderer.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sampling", meta = (EditCondition = "bOverride_AntiAliasingMethod"))
	TEnumAsByte<EAntiAliasingMethod> AntiAliasingMethod;

	/**
	* Filter used when blending panoramic. Bilinear is fastest (Samples a 2x2 pixel grid) and produces nearly as good results as the others which require sampling 4x4 pixels.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sampling", meta = (EditCondition = "bOverride_Filter"))
	EMoviePipelinePanoramicFilterType Filter;

	/**
	* Debug Feature. Can use this to write out each individual Temporal and Spatial sample rendered by this render pass,
	* which allows you to see which images are being accumulated together. Can be useful for debugging incorrect looking
	* frames to see which sub-frame evaluations were incorrect.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sampling", AdvancedDisplay, DisplayName = "Write All Samples (Debug)", meta = (EditCondition = "bOverride_bWriteAllsamples"))
	bool bWriteAllSamples;

	/**
	* If true, the tone curve will be disabled for this render pass. This will result in values greater than 1.0 in final renders
	* and can optionally be combined with OCIO profiles on the file output nodes to convert from Linear Values in Working Color Space
	* (which is sRGB  (Rec. 709) by default, unless changed in the project settings).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Processing", meta = (EditCondition = "bOverride_bDisableToneCurve"))
	bool bDisableToneCurve;

	/**
	* Allow the output file OpenColorIO transform to be used on this render.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Processing", meta = (EditCondition = "bOverride_bAllowOCIO"))
	bool bAllowOCIO;

	/**
	* The view mode index that will be applied to renders. These mirror the View Modes you find in the Viewport,
	* but most view modes other than Lit are used for debugging so they may not do what you expect, or may
	* have to be used in combination with certain Show Flags to produce a result similar to what you see in
	* the viewport.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View Mode", meta = (EditCondition = "bOverride_ViewModeIndex", InvalidEnumValues = "VMI_PathTracing,VMI_VisualizeBuffer,VMI_Unknown"))
	TEnumAsByte<EViewModeIndex> ViewModeIndex;

private:
	/**
	 * Lumen cvar caches, one per pass instance, populated by PreLayerRender and consumed by
	 * PostLayerRender. Keyed by the instance pointer so that layers are isolated from one another.
	 * TUniquePtr guarantees the cache's destructor runs on removal (as a safety net).
	 */
	TMap<const UE::MovieGraph::Rendering::FMovieGraphImagePassBase*, TUniquePtr<UE::MovieGraph::Private::FLumenCvarCache>> ActiveLayerCaches;

	/**
	 * Shared across every PreLayerRender call within a single RenderImpl invocation so its
	 * RenderedFrameNumber accumulates monotonically across all layers' warm-up renders. That
	 * monotonic progression is required for stable Lumen lighting.
	 */
	FMovieGraphTimeStepData CurrentWarmUpTimeStepData;
};

#undef UE_API
