// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphTimeStepData.h"
#include "Graph/Nodes/MovieGraphImagePassBaseNode.h"
#include "MoviePipelineDeferredPasses.h"
#include "MovieGraphDeferredPassNode.generated.h"

class UMovieGraphEvaluatedConfig;

namespace UE::MovieGraph::Private
{
	struct FLumenCvarCache;
}

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

/**
* A render node which uses the Deferred Renderer.
*/
UCLASS(MinimalAPI)
class UMovieGraphDeferredRenderPassNode : public UMovieGraphImagePassBaseNode 
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphDeferredRenderPassNode();
	UE_API virtual ~UMovieGraphDeferredRenderPassNode() override;

	UE_API virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	UE_API virtual void ResolveTokenContainingProperties(TFunction<void(FString&)>& ResolveFunc, const FMovieGraphTokenResolveContext& InContext) override;

	/** Context provided to pass instance factories during setup. */
	struct FPassInstanceFactoryContext
	{
		const FMovieGraphRenderPassLayerData& LayerData;
		UMovieGraphDefaultRenderer* Renderer = nullptr;
		UMovieGraphEvaluatedConfig* EvaluatedConfig = nullptr;
	};

	/**
	 * Factory function type for plugins that want to override deferred pass instance creation.
	 * Called once per layer during setup. Return a valid instance to override the default
	 * FMovieGraphDeferredPass, or nullptr to decline (fall back to next factory or default).
	 *
	 * Note: If multiple plugins register factories that claim the same layer, a warning
	 * is logged. The first registered factory that returns a non-null pass wins.
	 */
	using FPassInstanceFactory = TFunction<TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>(
		const FPassInstanceFactoryContext& /*InContext*/)>;

	/**
	 * Register a named factory that can override pass instance creation. This is used to tweak the
	 * behavior of the Deferred Renderer node. Use sparingly; generally, creating a new node is the
	 * best approach.
	 * @param InName     Unique name identifying the pass instance factory.
	 *                   If a factory with this name is already registered, it is replaced.
	 * @param InFactory  The factory function.
	 */
	static UE_API void RegisterPassInstanceFactory(FName InName, FPassInstanceFactory InFactory);

	/** Unregister a previously registered factory by name. */
	static UE_API void UnregisterPassInstanceFactory(FName InName);

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

protected:
	// UMovieGraphRenderPassNode Interface
	UE_API virtual void SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData) override;
	UE_API virtual void RenderImpl(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) override;
	UE_API virtual TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> CreateInstance() const override;
	UE_API virtual bool GetWriteAllSamples() const override;
	UE_API virtual TArray<FMoviePipelinePostProcessPass> GetAdditionalPostProcessMaterials() const override;
	UE_API virtual FString GetPPMFileNameFormatString() const override;
	UE_API virtual int32 GetNumSpatialSamples() const override;
	UE_API virtual bool GetDisableToneCurve() const override;
	UE_API virtual bool GetAllowOCIO() const override;
	virtual bool GetOverrideAntiAliasing() const override { return bOverride_AntiAliasingMethod; }
	UE_API virtual EAntiAliasingMethod GetAntiAliasingMethod() const override;
	virtual bool GetEnableHighResolutionTiling() const override { return bEnableHighResolutionTiling; }
	virtual FIntPoint GetTileCount() const override { return FIntPoint(TileCount, TileCount); }
	virtual float GetTileOverlapPercentage() const override { return OverlapPercentage; }
	virtual bool GetEnablePageToSystemMemory() const override { return bPageToSystemMemory; }
	virtual bool GetEnableHistoryPerTile() const override { return bAllocateHistoryPerTile; }
	// ~UMovieGraphRenderPassNode Interface

	// UMovieGraphImagePassBaseNode Interface
	UE_API virtual EViewModeIndex GetViewModeIndex() const override;
	UE_API virtual bool GetWriteBeautyPassToDisk() const override;
	UE_API virtual void PreLayerRender(
		const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		const FMovieGraphTimeStepData& InTimeData) override;
	UE_API virtual void PostLayerRender(
		const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		const FMovieGraphTimeStepData& InTimeData) override;
	// ~UMovieGraphImagePassBaseNode Interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SpatialSampleCount: 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AntiAliasingMethod : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bDisableToneCurve : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bAllowOCIO : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ViewModeIndex : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bWriteAllSamples : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bIncludeBeautyRenderInOutput : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_PPMFileNameFormat : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AdditionalPostProcessMaterials : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableHighResolutionTiling : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_TileCount : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OverlapPercentage : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bAllocateHistoryPerTile : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bPageToSystemMemory : 1;

	/**
	* How many sub-pixel jitter renders should we do per temporal sample? This can be used to achieve high
	* sample counts without Temporal Sub-Sampling (allowing high sample counts without motion blur being enabled),
	* but we generally recommend using Temporal Sub-Samples when possible. It can also be combined with
	* temporal samples and you will get SpatialSampleCount many renders per temporal sample.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 1, ClampMin = 1), Category = "Sampling", meta=(EditCondition="bOverride_SpatialSampleCount"))
	int32 SpatialSampleCount;

	/**
	* Which anti-aliasing method should this render use. If this is set to None, then Movie Render Graph
	* will handle anti-aliasing by doing a sub-pixel jitter (one for each temporal/spatial sample). Some
	* rendering effects rely on TSR or TAA to reduce noise so we recommend leaving them enabled
	* where possible. All options work with Spatial and Temporal samples, but TSR/TAA may introduce minor
	* visual artifacts (such as ghosting). MSAA is not supported in the deferred renderer.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sampling", meta = (EditCondition = "bOverride_AntiAliasingMethod"))
	TEnumAsByte<EAntiAliasingMethod> AntiAliasingMethod;
	
	/**
	* Debug Feature. Can use this to write out each individual Temporal and Spatial sample rendered by this render pass,
	* which allows you to see which images are being accumulated together. Can be useful for debugging incorrect looking
	* frames to see which sub-frame evaluations were incorrect.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sampling", AdvancedDisplay, DisplayName="Write All Samples (Debug)", meta = (EditCondition = "bOverride_bWriteAllsamples"))
	bool bWriteAllSamples;

	/**
	* If true, the tone curve will be disabled for this render pass. This will result in values greater than 1.0 in final renders
	* and can optionally be combined with OCIO profiles on the file output nodes to convert from Linear Values in Working Color Space
	* (which is sRGB  (Rec. 709) by default, unless changed in the project settings).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Processing", meta = (EditCondition = "bOverride_bDisableToneCurve"))
	bool bDisableToneCurve;

	/**
	* Allow the output file OpenColorIO transform to be used on this render.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Processing", meta = (EditCondition = "bOverride_bAllowOCIO"))
	bool bAllowOCIO;
	
	/** 
	* The view mode index that will be applied to renders. These mirror the View Modes you find in the Viewport,
	* but most view modes other than Lit are used for debugging so they may not do what you expect, or may
	* have to be used in combination with certain Show Flags to produce a result similar to what you see in
	* the viewport.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View Mode", meta=(EditCondition="bOverride_ViewModeIndex", InvalidEnumValues = "VMI_PathTracing,VMI_VisualizeBuffer,VMI_Unknown"))
	TEnumAsByte<EViewModeIndex> ViewModeIndex;

	/**
	 * Whether the main beauty pass should be written to disk.
	 *
	 * If just Post Process Material passes should be written to disk, turn this off. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Process Materials", meta=(EditCondition="bOverride_bIncludeBeautyRenderInOutput"))
	bool bIncludeBeautyRenderInOutput;

	/**
	 * If specified, overrides the output node's file name format, and uses this file name format for Post Process Material passes instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Process Materials", DisplayName = "PPM File Name Format", meta=(EditCondition="bOverride_PPMFileNameFormat", MrgTokenAutocomplete))
	FString PPMFileNameFormat;

	/**
	* An array of additional post-processing materials to run after the frame is rendered. Using this feature may add a notable amount of render time.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Process Materials", meta=(EditCondition="bOverride_AdditionalPostProcessMaterials"))
	TArray<FMoviePipelinePostProcessPass> AdditionalPostProcessMaterials;

	/**
	* If true, the render will be done using a "tiled" render, which can overcome size limitations of GPUs but comes with a significant number
	* of limitations. The internal GBuffer used for rendering is quite memory intensive, so a very large (ie: 8-16k) render may be impractical
	* from a memory standpoint. You can enable High Resolution Tiling to render this in multiple smaller passes, but there is overhead to each
	* tile as well, in terms of raytracing and lumen acceleration structures.
	* 
	* - If you have VRAM available, it's better to render with 1 tile instead of 2, and instead increase the TDR (Timeout Device Recovery) in
	* your OS to allow frames to take longer than the default 2s limit.
	* - If you need to use tiling, and you need to use Lumen, TAA/TSR, or other rendering features that require the previous frame's buffer,
	* then you'll need to enable bAllocateHistoryPerTile. This can come at a large VRAM cost but may require less vram than having a larger
	* GBuffer resolution.
	* - If you have spare system memory (RAM), you can use the experimental bPageToSystemMemory feature to download all of the per-tile rendering history
	* into CPU memory after each tile, and then upload it again before the next time that tile is used. This comes at a _significant_ performance and system
	* memory cost but can allow utilizing significantly larger overall resolutions while still supporting Lumen, TAA/TSR and other features.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_bEnableHighResolutionTiling"))
	bool bEnableHighResolutionTiling;

	/**
	* If bEnableHighResolutionTiling is enabled, what is the tile count that the screen should be broken into. This is not in pixels, but in number of tiles per
	* side, ie: an output resolution of 4k, and a tile count of 2, produces 4 tiles (2 horizontal, 2 vertical) with each tile being 1080p. Larger tile counts
	* shrink the individual render resolution, but increase the total number of renders needed.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_TileCount", UIMin = "1", ClampMin = "1", UIMax = "16"))
	int32 TileCount;

	/**
	* Rendering effects such as Depth of Field may produce different results near the edge of a tile (as it cannot sample outside of the tile itself), so this
	* setting allows you to create an overlapped region between tiles. 10% is a good starting point, but may need to be increased if you have extremely large
	* depth of field.
	* 
	* Note: This uses 0-50 and not 0-.5 like the previous system did to bring it in-line with other usages of overscan in the engine (nDisplay).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_OverlapPercentage", UIMin = "0", UIMax = "50", ClampMin = "0", ClampMax = "50"))
	float OverlapPercentage;

	/**
	* If enabled, a FSceneViewStateInterface is allocated for each tile in the high resolution image. This is required for TAA/TSR/Lumen and other modern rendering features
	* to work correctly, but can consume significant amounts of VRAM to store the state for each tile. This can be mitigated (at significant render time impact) with 
	* the new experimental bPageToSystemMemory cost.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_bAllocateHistoryPerTile"))
	bool bAllocateHistoryPerTile;

	/**
	* Experimental Feature: When enabled, after each tile is rendered, MRQ will download the per-tile image history back to system RAM, and then the next time
	* the tile is rendered on the subsequent frame the data is transfered from sytem memory back to GPU memory for use. This significantly impacts rendering time,
	* but can allow using Lumen and other features that rely on bAllocateHistoryPerTile to be used on GPUs that do not have enough VRAM to store all of the history
	* data for every tile at once.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_bPageToSystemMemory"))
	bool bPageToSystemMemory;

private:
	/** Information identifying a registered pass instance factory. */
	struct FNamedPassFactory
	{
		/** The name of the pass instance factory. */
		FName Name;
		
		/** The function that will create the pass instance factory. */
		FPassInstanceFactory Factory;
	};

	/** Protects mutations to PassInstanceFactories. */
	static FCriticalSection PassInstanceFactoriesCS;

	/** The pass instance factories that have been registered. */
	static TArray<FNamedPassFactory> PassInstanceFactories;

	/**
	 * Lumen cvar caches, one per pass instance, populated by PreLayerRender and consumed by
	 * PostLayerRender. Keyed by the instance pointer so that layers are isolated from one another.
	 * TUniquePtr guarantees the cache's destructor runs on removal (as a safety net).
	 */
	TMap<const UE::MovieGraph::Rendering::FMovieGraphImagePassBase*, TUniquePtr<UE::MovieGraph::Private::FLumenCvarCache>> ActiveLayerCaches;

	/**
	 * Shared across every PreLayerRender call within a single RenderImpl invocation so its
	 * RenderedFrameNumber accumulates monotonically across all layers' warm-up renders. That
	 * monotonic progression is required for stable Lumen lighting.
	 */
	FMovieGraphTimeStepData CurrentWarmUpTimeStepData;
};

#undef UE_API
