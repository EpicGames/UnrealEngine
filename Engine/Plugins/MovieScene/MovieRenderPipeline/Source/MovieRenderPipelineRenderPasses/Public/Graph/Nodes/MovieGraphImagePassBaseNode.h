// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/Renderers/MovieGraphImagePassBase.h"
#include "Graph/Renderers/MovieGraphShowFlags.h"
#include "Graph/Nodes/MovieGraphRenderPassNode.h"
#include "MoviePipelineDeferredPasses.h"
#include "MovieGraphImagePassBaseNode.generated.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

// Forward Declare
struct FMovieGraphRenderPassSetupData;
struct FMovieGraphTimeStepData;
struct FMoviePipelineEndShotRenderTelemetryContext;

/**
* The UMovieGraphImagePassBaseNode is an abstract base-class for render nodes that wish to create
* renders of the 3d scene. You are not required to inherit from this node (can inherit from
* UMovieGraphRenderPassNode), but this node provides a helpful set of functions and default values
* for constructing the required matrices and settings for viewport-like renders.
*/
UCLASS(MinimalAPI, Abstract)
class UMovieGraphImagePassBaseNode : public UMovieGraphRenderPassNode
{
	GENERATED_BODY()
public:
	UE_API UMovieGraphImagePassBaseNode();

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
		
	/** Gets the view mode index that should be active for this renderer. */
	UE_API virtual EViewModeIndex GetViewModeIndex() const;

	/** Gets the show flags that should be active for this renderer. */
	UE_API virtual FEngineShowFlags GetShowFlags() const;

	/** Gets any show flags that should be applied as defaults, before user changes are applied. */
	UE_API virtual TArray<uint32> GetDefaultShowFlags() const;

	/** Should each individual sample rendered be written out for debugging? */
	virtual bool GetWriteAllSamples() const { return false; }

	/** Get an array of user-added post-process materials for the render */
	virtual TArray<FMoviePipelinePostProcessPass> GetAdditionalPostProcessMaterials() const { return {}; }

	/**
	 * Get the post-process material file name format string, if any. If empty, no file name format override will be used for PPMs, and the
	 * value on the output node will be used.
	 */
	virtual FString GetPPMFileNameFormatString() const { return FString(); }

	/** How many spatial samples should be rendered each frame? */
	virtual int32 GetNumSpatialSamples() const { return 1; }

	/** DEPRECATED - How many spatial samples should be used during warm-up frames? */
	UE_DEPRECATED(5.8, "Override the version that takes arguments instead, this version is no longer called.")
	virtual int32 GetNumSpatialSamplesDuringWarmUp() const { return GetNumSpatialSamples(); }

	/**
	 * The number of spatial samples that are performed during warm-up frames. This is called per-frame, so the value returned here can vary
	 * during the warm-up process.
	 */
	UE_API virtual int32 GetNumSpatialSamplesDuringWarmUp(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) const;

	/** Offset to apply to random number generator seed. */
	virtual int32 GetSeedOffset() const { return 0; }

	/** Should the tone curve be disabled while rendering? Allows for linear values in exrs but changes the look of the final image. */
	virtual bool GetDisableToneCurve() const { return false; }

	/** Should the output file be allowed to apply an OCIO transform on this render? */
	virtual bool GetAllowOCIO() const { return true; }

	/** Should the denoiser be run on the resulting image (only has any effect with the Path Tracer) */
	virtual bool GetAllowDenoiser() const { return true; }

	/** Should we override the anti-aliasing setting specified by the Project Settings? */
	virtual bool GetOverrideAntiAliasing() const { return false; }

	/** Which AA Method should be used? */
	virtual EAntiAliasingMethod GetAntiAliasingMethod() const { return EAntiAliasingMethod::AAM_None; }
	
	/** Whether this node allows changing the Show Flags in the details panel. */
	virtual bool GetAllowsShowFlagsCustomization() const { return true; }

	/** Whether this pass allows other passes to be composited on it. */
	virtual bool GetAllowsCompositing() const { return true; }

	/** Whether this node's beauty pass should be written to disk. For some nodes, this will have no effect because there is no "beauty" pass. */
	virtual bool GetWriteBeautyPassToDisk() const { return true; }

	/** Whether lossless compression must be forced on for this node. */
	virtual bool GetForceLosslessCompression() const { return false; }
	
	/** Are we using high resolution tiling? */
	virtual bool GetEnableHighResolutionTiling() const { return false; }

	/** If using high resolution tiling, how many tiles are being rendered? If not using tiling should be (1, 1). */
	virtual FIntPoint GetTileCount() const { return FIntPoint(1, 1); }
	
	/** If using high resolution tiling, what percentage of overlap should be used between tiles? 0-100 scale. */
	virtual float GetTileOverlapPercentage() const { return 0.f; }
	
	/** If using high resolution tiling, should each tile be paged to system memory after rendering? */
	virtual bool GetEnablePageToSystemMemory() const { return false; }
	
	/** If using high resolution tiling, do we keep a unique scene view history for each tile? */
	virtual bool GetEnableHistoryPerTile() const { return false; }

protected:
	// Note: Since *individual* show flags are overridden instead of the entire ShowFlags property, manually set to
	// overridden so the traversal picks the changes up (otherwise they will be ignored).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ShowFlags : 1 = 1;

	/** The show flags that should be active during a render for this node. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Show Flags")
	TObjectPtr<UMovieGraphShowFlags> ShowFlags;

	TArray<TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>> CurrentInstances;

	static UE_API FString DefaultDepthAsset;
	static UE_API FString DefaultMotionVectorsAsset;

protected:
	// UMovieGraphRenderPassNode Interface
	UE_API virtual void SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData) override;
	UE_API virtual void TeardownImpl() override;
	UE_API virtual void RenderImpl(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) override;
	UE_API virtual void GatherOutputPassesImpl(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const override;
	UE_API virtual void UpdateRenderPassTelemetryImpl(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const override;
	UE_API virtual void GatherRenderCameraSourcesImpl(TArray<TSharedRef<FMovieGraphRenderCameraSource>>& OutSources) const override;
	virtual int32 GetNumSceneViewsRenderedImpl() const { return CurrentInstances.Num(); }
	UE_API virtual void GetPreviewBannerMessagesImpl(const FMovieGraphRenderDataIdentifier& InIdentifier, TArray<FMovieGraphPreviewBannerMessage>& OutMessages) const override;
    // ~UMovieGraphRenderPassNode Interface

	virtual TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> CreateInstance() const { return nullptr; }

	/**
	 * Called once per pass instance, immediately after the render layer modifiers are activated and
	 * immediately before the instance's Render() is invoked. Override to insert per-layer setup.
	 * Does nothing by default.
	 */
	virtual void PreLayerRender(
		const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		const FMovieGraphTimeStepData& InTimeData) {}

	/**
	 * Called once per pass instance, immediately after the instance's Render() returns and before
	 * the render-layer modifiers are cleared. Override to insert per-layer teardown. Does nothing
	 * by default.
	 */
	virtual void PostLayerRender(
		const TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>& InInstance,
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		const FMovieGraphTimeStepData& InTimeData) {}
};

#undef UE_API
