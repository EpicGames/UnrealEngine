// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Nodes/MovieGraphImagePassBaseNode.h"

#include "MovieGraphPathTracerPassNode.generated.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API


UENUM(BlueprintType)
enum class EMovieGraphPathTracerDenoiserType : uint8
{
	/** 
	* The active spatial denoiser plugin will be used for denoising. If the denoiser is not loaded, a warning will show in the log.
	* If multiple spatial denoiser plugins are enabled, the last one to get loaded will be the one used.
	*/
	Spatial = 0,

	/** 
	* The active spatial-temporal denoiser plugin will be used for denoising. It provides more temporal stability than spatial denoiser 
	* if the Frame Count of past/future frames are used (Frame Count > 0) in the plugin. The user needs to config `Frame Count` to
	* match the requirements of the chosen denoiser plugin. If the denoiser is not loaded, a warning will show in the log. If multiple
	* spatial-temporal denoiser plugins are enabled, the last one to get loaded will be the one used.
	*/
	Temporal = 1
};


/** A render node which uses the path tracer. */
UCLASS(MinimalAPI)
class UMovieGraphPathTracerRenderPassNode : public UMovieGraphImagePassBaseNode
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphPathTracerRenderPassNode();

#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
#endif

	// UMovieGraphRenderPassNode Interface
	UE_API virtual void SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData) override;
	UE_API virtual void TeardownImpl() override;
	// ~UMovieGraphRenderPassNode Interface

	// UMovieGraphImagePassBaseNode Interface
	UE_API virtual bool GetWriteAllSamples() const override;
	UE_API virtual TArray<FMoviePipelinePostProcessPass> GetAdditionalPostProcessMaterials() const override;
	UE_API virtual FString GetPPMFileNameFormatString() const override;
	UE_API virtual int32 GetNumSpatialSamples() const override;
	UE_API virtual int32 GetNumSpatialSamplesDuringWarmUp(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) const override;
	UE_API virtual int32 GetSeedOffset() const override;
	UE_API virtual bool GetDisableToneCurve() const override;
	UE_API virtual bool GetAllowOCIO() const override;
	UE_API virtual bool GetAllowDenoiser() const override;
	UE_API virtual bool GetWriteBeautyPassToDisk() const override;
	UE_API virtual TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> CreateInstance() const override;
	UE_API virtual FEngineShowFlags GetShowFlags() const override;
	UE_API virtual bool GetEnableHighResolutionTiling() const override;
	UE_API virtual FIntPoint GetTileCount() const override;
	UE_API virtual float GetTileOverlapPercentage() const override;
	UE_API virtual bool GetEnablePageToSystemMemory() const override;
	UE_API virtual bool GetEnableHistoryPerTile() const override;
	// ~UMovieGraphImagePassBaseNode Interface

	// UMovieGraphSettingNode Interface
	UE_API virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const override;
	// ~UMovieGraphSettingNode Interface

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	UE_API virtual void ResolveTokenContainingProperties(TFunction<void(FString&)>& ResolveFunc, const FMovieGraphTokenResolveContext& InContext) override;

protected:
	// UMovieGraphRenderPassNode Interface
	UE_API virtual int32 GetCoolingDownFrameCount() const override;
	// ~UMovieGraphRenderPassNode Interface

	// UMovieGraphCoreRenderPassNode Interface
	UE_API virtual EViewModeIndex GetViewModeIndex() const override;
	// ~UMovieGraphCoreRenderPassNode Interface

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SpatialSampleCount : 1;

	UPROPERTY(BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SeedOffset : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableReferenceMotionBlur : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableDenoiser : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DenoiserType : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_FrameCount : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bDisableToneCurve : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bAllowOCIO : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bLightingComponents_IncludeEmissive : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bLightingComponents_IncludeDiffuse : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bLightingComponents_IncludeIndirectDiffuse : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bLightingComponents_IncludeSpecular : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bLightingComponents_IncludeIndirectSpecular : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bLightingComponents_IncludeVolume : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bLightingComponents_IncludeIndirectVolume : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bWriteAllSamples : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bIncludeBeautyRenderInOutput : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_PPMFileNameFormat : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AdditionalPostProcessMaterials : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableHighResolutionTiling : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_TileCount : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OverlapPercentage : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bAllocateHistoryPerTile : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bPageToSystemMemory : 1;

	/**
	* How many sub-pixel jitter renders should we do per temporal sample? This can be used to achieve high
	* sample counts without Temporal Sub-Sampling (allowing high sample counts without motion blur being enabled),
	* but we generally recommend using Temporal Sub-Samples when possible. It can also be combined with
	* temporal samples and you will get SpatialSampleCount many renders per temporal sample.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 1, ClampMin = 1), Category = "Sampling", meta = (EditCondition = "bOverride_SpatialSampleCount"))
	int32 SpatialSampleCount;

	/**
	* Offset to apply to random number generator seed.
	* Intentionally not exposed to the UI as meant for automated pipelines. Should be edited via scripting if needs to be changed.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "Sampling", AdvancedDisplay, meta = (EditCondition = "bOverride_SeedOffset"))
	int32 SeedOffset;

	/** 
	 *  When enabled, the path tracer will blend all spatial and temporal samples prior to the denoising and will disable post-processed motion blur.
	 *  In this mode it is possible to use higher temporal sample counts to improve the motion blur quality. This mode also automatically enabled reference DOF.
	 *  When this option is disabled, the path tracer will accumulate spatial samples, but denoise them prior to accumulation of temporal samples.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 1, ClampMin = 1), Category = "Reference Motion Blur", meta = (EditCondition = "bOverride_bEnableReferenceMotionBlur"))
	bool bEnableReferenceMotionBlur;

	/** If true the resulting image will be denoised at the end of each set of Spatial Samples. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Denoiser", meta = (EditCondition = "bOverride_bEnableDenoiser"))
	bool bEnableDenoiser;

	/**
	* Select which type of denoiser to use when the denoiser is enabled. Temporal denoisers will provide better results when
	* denoising animated sequences (the denoising results will look more stable), especially when combined with an appropriate 
	* Frame Count (non-zero). Denoisers are implemented as plugins so you may need to enable a plugin as well for this to work.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Denoiser", meta = (EditCondition = "bOverride_DenoiserType"))
	EMovieGraphPathTracerDenoiserType DenoiserType;

	/** 
	* The number of frames to consider when using temporal-based denoisers. Generally higher numbers will result in longer
	* denoising times and higher memory requirements. For NFOR this number refers to how many frames to consider on both sides
	* of the current frame (ie: 2 means consider 2 before, and 2 after the currently denoised frame), but other denoiser 
	* implementations may interpret this value differently.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, UIMax = 3), Category = "Denoiser")
	int32 FrameCount;

	/**
	* Debug Feature. This can be used to write out each individual spatial sample rendered by this render pass,
	* which allows you to see which images are being accumulated together. Can be useful for debugging incorrect looking
	* frames to see which sub-frame evaluations were incorrect.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sampling", AdvancedDisplay, DisplayName = "Write All Samples (Debug)", meta = (EditCondition = "bOverride_bWriteAllsamples"))
	bool bWriteAllSamples;

	/**
	* If true, the tone curve will be disabled for this render pass. This will result in values greater than 1.0 in final renders
	* and can optionally be combined with OCIO profiles on the file output nodes to convert from Linear Values in Working Color Space
	* (which is sRGB  (Rec. 709) by default, unless changed in the project settings).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Processing", meta = (EditCondition = "bOverride_bDisableToneCurve"))
	bool bDisableToneCurve;

	/**
	* Allow the output file OpenColorIO transform to be used on this render.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Processing", meta = (EditCondition = "bOverride_bAllowOCIO"))
	bool bAllowOCIO;

	/** Whether the render should include directly visible emissive components. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lighting Components", DisplayName = "Emissive", meta = (EditCondition = "bOverride_bLightingComponents_IncludeEmissive"))
	bool bLightingComponents_IncludeEmissive = true;

	/** Whether the render should include diffuse lighting contributions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lighting Components", DisplayName = "Diffuse", meta = (EditCondition = "bOverride_bLightingComponents_IncludeDiffuse"))
	bool bLightingComponents_IncludeDiffuse = true;

	/** Whether the render should include indirect diffuse lighting contributions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lighting Components", DisplayName = "Indirect Diffuse", meta = (EditCondition = "bOverride_bLightingComponents_IncludeIndirectDiffuse"))
	bool bLightingComponents_IncludeIndirectDiffuse = true;

	/** Whether the render should include specular lighting contributions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lighting Components", DisplayName = "Specular", meta = (EditCondition = "bOverride_bLightingComponents_IncludeSpecular"))
	bool bLightingComponents_IncludeSpecular = true;

	/** Whether the render should include indirect specular lighting contributions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lighting Components", DisplayName = "Indirect Specular", meta = (EditCondition = "bOverride_bLightingComponents_IncludeIndirectSpecular"))
	bool bLightingComponents_IncludeIndirectSpecular = true;

	/** Whether the render should include volume lighting contributions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lighting Components", DisplayName = "Volume", meta = (EditCondition = "bOverride_bLightingComponents_IncludeVolume"))
	bool bLightingComponents_IncludeVolume = true;

	/** Whether the render should include indirect volume lighting contributions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lighting Components", DisplayName = "Indirect Volume", meta = (EditCondition = "bOverride_bLightingComponents_IncludeIndirectVolume"))
	bool bLightingComponents_IncludeIndirectVolume = true;

	/**
	 * Whether the main beauty pass should be written to disk.
	 *
	 * If just Post Process Material passes should be written to disk, turn this off. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Process Materials", meta=(EditCondition="bOverride_bIncludeBeautyRenderInOutput"))
	bool bIncludeBeautyRenderInOutput;

	/**
	 * If specified, overrides the output node's file name format, and uses this file name format for Post Process Material passes instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Process Materials", DisplayName = "PPM File Name Format", meta=(EditCondition="bOverride_PPMFileNameFormat", MrgTokenAutocomplete))
	FString PPMFileNameFormat;

	/**
	* An array of additional post-processing materials to run after the frame is rendered. Using this feature may add a notable amount of render time.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Process Materials", meta=(EditCondition="bOverride_AdditionalPostProcessMaterials"))
	TArray<FMoviePipelinePostProcessPass> AdditionalPostProcessMaterials;

	/**
	* If true, the render will be done using a "tiled" render, which can overcome size limitations of GPUs but comes with a significant number
	* of limitations. The internal GBuffer used for rendering is quite memory intensive, so a very large (ie: 8-16k) render may be impractical
	* from a memory standpoint. You can enable High Resolution Tiling to render this in multiple smaller passes, but there is overhead to each
	* tile as well, in terms of raytracing and lumen acceleration structures.
	*
	* - If you have VRAM available, it's better to render with 1 tile instead of 2, and instead increase the TDR (Timeout Device Recovery) in
	* your OS to allow frames to take longer than the default 2s limit.
	* - If you need to use tiling, and you need to use Lumen, TAA/TSR, or other rendering features that require the previous frame's buffer,
	* then you'll need to enable bAllocateHistoryPerTile. This can come at a large VRAM cost but may require less vram than having a larger
	* GBuffer resolution.
	* - If you have spare system memory (RAM), you can use the experimental bPageToSystemMemory feature to download all of the per-tile rendering history
	* into CPU memory after each tile, and then upload it again before the next time that tile is used. This comes at a _significant_ performance and system
	* memory cost but can allow utilizing significantly larger overall resolutions while still supporting Lumen, TAA/TSR and other features.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_bEnableHighResolutionTiling"))
	bool bEnableHighResolutionTiling;

	/**
	* If bEnableHighResolutionTiling is enabled, what is the tile count that the screen should be broken into. This is not in pixels, but in number of tiles per
	* side, ie: an output resolution of 4k, and a tile count of 2, produces 4 tiles (2 horizontal, 2 vertical) with each tile being 1080p. Larger tile counts
	* shrink the individual render resolution, but increase the total number of renders needed.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_TileCount", UIMin = "1", ClampMin = "1", UIMax = "16"))
	int32 TileCount;

	/**
	* Rendering effects such as Depth of Field may produce different results near the edge of a tile (as it cannot sample outside of the tile itself), so this
	* setting allows you to create an overlapped region between tiles. 10% is a good starting point, but may need to be increased if you have extremely large
	* depth of field. Only has an effect if bEnableHighResolutionTiling is enabled.
	*
	* Note: This uses 0-50 and not 0-.5 like the previous system did to bring it in-line with other usages of overscan in the engine (nDisplay).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_OverlapPercentage", UIMin = "0", UIMax = "50", ClampMin = "0", ClampMax = "50"))
	float OverlapPercentage;

	/**
	* If enabled, a FSceneViewStateInterface is allocated for each tile in the high resolution image. This is required for TAA/TSR/Lumen and other modern rendering features
	* to work correctly, but can consume significant amounts of VRAM to store the state for each tile. This can be mitigated (at significant render time impact) with
	* the new experimental bPageToSystemMemory cost.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_bAllocateHistoryPerTile"))
	bool bAllocateHistoryPerTile;

	/**
	* Experimental Feature: When enabled, after each tile is rendered, MRQ will download the per-tile image history back to system RAM, and then the next time
	* the tile is rendered on the subsequent frame the data is transfered from sytem memory back to GPU memory for use. This significantly impacts rendering time,
	* but can allow using Lumen and other features that rely on bAllocateHistoryPerTile to be used on GPUs that do not have enough VRAM to store all of the history
	* data for every tile at once.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "High Resolution Tiling", meta = (EditCondition = "bOverride_bPageToSystemMemory"))
	bool bPageToSystemMemory;

private:
	/**
	 * The original value of the "r.PathTracing.ProgressDisplay" cvar before the render starts. The progress display
	 * will be hidden during the render.
	 */
	bool bOriginalProgressDisplayCvarValue = false;

	/**
	 * The original value of the "r.NFOR.FrameCount" cvar before the render starts. Will use the new value set in
	 * this node during the render.
	 */
	int32 OriginalFrameCountCvarValue = 2;

	/**
	 * The original value of the "r.PathTracing.SpatialDenoiser.Type" cvar before the render starts. Will use the new value set in
	 * this node during the render.
	 */
	int32 OriginalDenoiserType = 0;
};

#undef UE_API
