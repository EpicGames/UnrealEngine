// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Renderers/MovieGraphImagePassBase.h"
#include "Graph/Nodes/MovieGraphDeferredPanoramicPassNode.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

struct FSceneViewStateSystemMemoryMirror;

namespace UE::MovieGraph::Rendering
{
	struct FMovieGraphDeferredPanoramicPass : public FMovieGraphImagePassBase
	{
		UE_API FMovieGraphDeferredPanoramicPass();

		// FMovieGraphImagePassBase Interface
		UE_API virtual void Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer) override;
		UE_API virtual void Teardown() override;
		UE_API virtual UMovieGraphImagePassBaseNode* GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const override;
		UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		UE_API virtual TSharedRef<::MoviePipeline::IMoviePipelineAccumulationArgs> GetOrCreateAccumulator(TObjectPtr<UMovieGraphDefaultRenderer> InGraphRenderer, const UE::MovieGraph::FMovieGraphSampleState& InSampleState) const override;
		UE_API virtual FAccumulatorSampleFunc GetAccumulateSampleFunction() const override;
		UE_API virtual void GatherOutputPasses(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const override;
		UE_API virtual FName GetBranchName() const;
		UE_API virtual FString GetCameraName() const override;
		UE_API virtual int32 GetRenderCameraIndex() const override;
		// ~FMovieGraphImagePassBase Interface

		// FMovieGraphDeferredPass Interface
		UE_API virtual void Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) override;
		// ~FMovieGraphDeferredPass Interface

	protected:
		UE_API virtual bool ShouldDiscardOutput(const TSharedRef<FSceneViewFamilyContext>& InFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const;
		UE_API virtual FSceneViewStateInterface* GetSceneViewState(UMovieGraphDeferredPanoramicNode* ParentNodeThisFrame, int32_t PaneX, int32_t PaneY);

	protected:
		// A view state for each Pane (if History Per Pane is enabled)
		TArray<FSceneViewStateReference> PaneViewStates;

		// When using an auto exposure render pass, holds view states for 6 cube faces
		TArray<FSceneViewStateReference> AutoExposureViewStates;

		// Used when using Page to System Memory
		TPimplPtr<FSceneViewStateSystemMemoryMirror> SystemMemoryMirror;

		bool bHasPrintedRenderingInfo = false;
		bool bHasPrintedWarnings = false;
		FMovieGraphRenderDataIdentifier RenderDataIdentifier;
		FMovieGraphRenderPassLayerData LayerData;
		TSharedPtr<UE::MovieGraph::IMovieGraphOutputMerger> PanoramicOutputBlender;
		
	};
}

#undef UE_API
