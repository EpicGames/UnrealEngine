// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/Renderers/MovieGraphImagePassBase.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

struct FSceneViewStateSystemMemoryMirror;

namespace UE::MovieGraph::Rendering
{
	struct FMovieGraphPostRendererSubmissionParams
	{
		UE::MovieGraph::FMovieGraphSampleState SampleState;
		UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams RenderTargetInitParams;
		UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo;
	};

	struct FMovieGraphDeferredPass : public FMovieGraphImagePassBase
	{
		// FMovieGraphImagePassBase Interface
		UE_API virtual void Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer) override;
		UE_API virtual void Teardown() override;
		UE_API virtual void Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) override;
		UE_API virtual void GatherOutputPasses(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const override;
		UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		UE_API virtual FName GetBranchName() const override;
		UE_API virtual FString GetCameraName() const override;
		UE_API virtual UMovieGraphImagePassBaseNode* GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const override;
		UE_API virtual bool ShouldDiscardOutput(const TSharedRef<FSceneViewFamilyContext>& InFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const override;
		UE_API virtual int32 GetRenderCameraIndex() const override;
		// End FMovieGraphImagePassBase

	protected:
		/** Data about the resolution and camera that can be used in the next render. */
		struct FResolutionAndCameraInfo
		{
			float OverscanFraction = 0.f;
			float CameraAspectRatio = 0.f;
			bool bRenderAllCameras = false;
			DefaultRenderer::FCameraInfo CameraInfo;
			FIntPoint BackbufferResolution;
			FIntPoint AccumulatorResolution;
			FIntRect AccumulatorResolutionCropRect;
			FIntPoint TileSize;
		};

		/** Result of CreateConfiguredView() — contains the fully configured ViewFamily, View, FViewFamilyInitData, and FSceneViewInitOptions. */
		struct FCreateViewResult
		{
			explicit FCreateViewResult(const TSharedRef<FSceneViewFamilyContext> InViewFamily);

			TSharedRef<FSceneViewFamilyContext> ViewFamily;
			FSceneView* View = nullptr;
			FViewFamilyInitData ViewFamilyInitData;
			FSceneViewInitOptions SceneViewInitOptions;
		};

		UE_API bool HasRenderResourceParametersChanged(const FIntPoint& AccumulatorResolution, const FIntPoint& BackbufferResolution) const;
		UE_API virtual void PostRendererSubmission(const UE::MovieGraph::FMovieGraphSampleState& InSampleState, const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams, FCanvas& InCanvas, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const override;
		UE_API virtual FSceneViewStateInterface* GetSceneViewState(UMovieGraphImagePassBaseNode* ParentNodeThisFrame, int32_t TileX, int32_t TileY);

		/** Provides the resolution and camera information that should be used for the next render. */
		UE_API FResolutionAndCameraInfo GetResolutionAndCameraInfo(UMovieGraphEvaluatedConfig* InEvaluatedConfig) const;

		/**
		 * Encapsulates the shared view creation pipeline: GetSceneViewState -> CreateSceneViewFamily ->
		 * CreateViewInitOptions -> CalculateProjectionMatrix -> ModifyProjectionMatrixForTiling ->
		 * CreateSceneView -> ApplyMovieGraphOverrides. Used by both the base Render() and derived passes.
		 * Respects the bAutoRestoreViewStateMirror flag: when false, the per-call SystemMemoryMirrorRestore()
		 * is skipped so derived passes can manage restore/backup at their own granularity.
		 */
		UE_API FCreateViewResult CreateConfiguredView(
			UE::MovieGraph::DefaultRenderer::FCameraInfo& InOutCameraInfo,
			const FViewFamilyInitData& InViewFamilyInitData,
			const FIntPoint& InBackbufferResolution,
			const FIntPoint& InAccumulatorResolution,
			UMovieGraphImagePassBaseNode* InParentNode,
			const int32 TileX,
			const int32 TileY);

		/** Computes the projection matrix jitter amount for the current sample. Override to provide a custom jitter. */
		UE_API virtual FVector2D ComputeProjectionMatrixJitter(const FVector2D& InDefaultJitter, const FIntPoint& InBackbufferResolution) const;

		/**
		 * Populates the standard per-layer/per-view metadata (camera matrices, location/rotation,
		 * cine-camera attributes, blended PP settings) shared by all deferred-style passes. Derived passes
		 * that build their own SampleState should call this so their layers carry the same camera/view
		 * metadata as stock deferred passes.
		 */
		static UE_API void WritePerViewMetadata(
			TMap<FString, FString>& InOutMetadata,
			const FSceneView& InView,
			const FSceneViewInitOptions& InViewInitOptions,
			const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo,
			const FMovieGraphRenderDataIdentifier& InIdentifier);

	protected:
		FMovieGraphRenderPassLayerData LayerData;

		/** Unique identifier passed in GatherOutputPasses and with each render that identifies the data produced by this renderer. */
		FMovieGraphRenderDataIdentifier RenderDataIdentifier;

		UE_DEPRECATED(5.6, "SceneViewState is no longer used. Use SceneViewStates instead with reference at FIntPoint(0,0).")
		FSceneViewStateReference SceneViewState;

		// Scene View history used by the renderer. When using an auto-exposure pass it'll use (-1, -1), otherwise one-per tile (and one at 0,0 if not using tiling).
		TMap<FIntPoint, FSceneViewStateReference> SceneViewStates;

		// Used when using Page to System Memory
		TPimplPtr<FSceneViewStateSystemMemoryMirror> SystemMemoryMirror;

		/**
		 * When true (default), CreateConfiguredView() calls SystemMemoryMirrorRestore() on the view state
		 * returned by GetSceneViewState (gated on GetEnableHistoryPerTile() && GetEnablePageToSystemMemory()).
		 * Derived passes that reuse the same view state across multiple renders within a single logical
		 * iteration should clear this flag in Setup() and backup/restore view state as needed.
		 */
		bool bAutoRestoreViewStateMirror = true;

		// The number of frames to delay to send frames from SubmissionQueue to post-render submission.
		int32 FramesToDelayPostSubmission;
		
		// If using cooldown, the number of cool-down frames we still need to process.
		int32 RemainingCooldownReadbackFrames;

		// FIFO queue of rendered frames. It allows frames to be sent to post-render submission with a delay if needed (e.g., when temporal denoising is used with path tracers).  
		TQueue<FMovieGraphPostRendererSubmissionParams> SubmissionQueue;

		// Did we initialize a auto-exposure sceneview history during setup?
		bool bHasAutoExposurePass = false;

		/** Whether the main beauty pass should be written to disk. Turn this off if only the PPM passes should be written. */
		bool bWriteBeautyPassToDisk = true;

		// Track the last Accumulator resolution we used, so that we can detect when it is changed and log that information.
		FIntPoint PrevAccumulatorResolution;

		// Track the last backbuffer resolution we used, so that we can detect when it is changed and log that information.
		FIntPoint PrevBackbufferResolution;
	}; 
}

#undef UE_API
