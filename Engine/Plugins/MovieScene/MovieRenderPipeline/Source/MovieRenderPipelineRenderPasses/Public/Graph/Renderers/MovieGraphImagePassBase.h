// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphRenderCameraSource.h"
#include "SceneView.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

// Forward Declares
class UMovieGraphImagePassBaseNode;
struct FMoviePipelineEndShotRenderTelemetryContext;

namespace UE::MovieGraph::Rendering
{
	struct FMovieGraphRenderDataAccumulationArgs : ::MoviePipeline::IMoviePipelineAccumulationArgs
	{
	public:
		TWeakPtr<::MoviePipeline::IMoviePipelineOverlappedAccumulator, ESPMode::ThreadSafe> ImageAccumulator;
		DefaultRenderer::FSurfaceAccumulatorPool::FInstancePtr AccumulatorInstance;
		TWeakPtr<IMovieGraphOutputMerger, ESPMode::ThreadSafe> OutputMerger;

		UE_DEPRECATED(5.6, "No longer used, calling code now looks at sample state directly.")
		bool bIsFirstSample;
		UE_DEPRECATED(5.6, "No longer used, calling code now looks at sample state directly.")
		bool bIsLastSample;


		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FMovieGraphRenderDataAccumulationArgs() = default;
		~FMovieGraphRenderDataAccumulationArgs() = default;
		FMovieGraphRenderDataAccumulationArgs(const FMovieGraphRenderDataAccumulationArgs&) = default;
		FMovieGraphRenderDataAccumulationArgs(FMovieGraphRenderDataAccumulationArgs&&) = default;
		FMovieGraphRenderDataAccumulationArgs& operator=(const FMovieGraphRenderDataAccumulationArgs&) = default;
		FMovieGraphRenderDataAccumulationArgs& operator=(FMovieGraphRenderDataAccumulationArgs&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	// Forward Declare
	void AccumulateSample_TaskThread(TUniquePtr<FImagePixelData>&& InPixelData, const TSharedRef<::UE::MovieGraph::FMovieGraphSampleState> InSampleState, const TSharedRef<::MoviePipeline::IMoviePipelineAccumulationArgs> InAccumulatorArgs);

	struct FViewFamilyInitData
	{
		FViewFamilyInitData()
			: RenderTarget(nullptr)
			, World(nullptr)
			, SceneCaptureSource(ESceneCaptureSource::SCS_MAX)
			, bWorldIsPaused(false)
			, FrameIndex(-1)
			, AntiAliasingMethod(EAntiAliasingMethod::AAM_None)
			, ShowFlags(ESFIM_Game)
			, ViewModeIndex(VMI_Lit)
			, bIsCameraCut(false)
		{
		}

		class FRenderTarget* RenderTarget;
		class UWorld* World;
		FMovieGraphTimeStepData TimeData;
		ESceneCaptureSource SceneCaptureSource;
		bool bWorldIsPaused;
		int32 FrameIndex;
		EAntiAliasingMethod AntiAliasingMethod;
		FEngineShowFlags ShowFlags;
		EViewModeIndex ViewModeIndex;
		TEnumAsByte<ECameraProjectionMode::Type> ProjectionMode;
		bool bIsCameraCut;
	};
	


	struct FMovieGraphImagePassBase
	{
		FMovieGraphImagePassBase() = default;
		virtual ~FMovieGraphImagePassBase() = default;

		UE_API virtual void Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer);
		UE_API virtual void Teardown();

		virtual void Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) {}
		virtual void GatherOutputPasses(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const {}
		UE_API virtual void UpdateRenderPassTelemetry(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const;
		UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector);
		virtual FName GetBranchName() const { return NAME_None; }
		virtual FString GetCameraName() const { return FString(); }
		virtual TWeakObjectPtr<UMovieGraphDefaultRenderer> GetRenderer() const { return WeakGraphRenderer; }

		/**
		* Optional per-tile banner messages to overlay on this pass's preview tile in the Render Preview window. The default
		* implementation contributes nothing.
		*/
		virtual void GetPreviewBannerMessages(TArray<FMovieGraphPreviewBannerMessage>& OutMessages) const {}

		/** Returns the shared camera source for this pass. Use when a shared reference is needed to extend lifetime. */
		virtual TSharedPtr<const FMovieGraphRenderCameraSource> GetRenderCameraSource() const { return RenderCameraSource; }
		virtual TSharedPtr<FMovieGraphRenderCameraSource> GetRenderCameraSource() { return RenderCameraSource; }

	protected:
		typedef TFunction<void(TUniquePtr<FImagePixelData>&&, const TSharedRef<FMovieGraphSampleState>, const TSharedRef<::MoviePipeline::IMoviePipelineAccumulationArgs>)> FAccumulatorSampleFunc;
		
		/** Returns the camera index used for this pass.
		 * Returns [0..N) when using multi-camera rendering; otherwise INDEX_NONE.
		 */
		virtual int32 GetRenderCameraIndex() const { return INDEX_NONE; }

		/** Returns the camera info for the given camera index, forwarding this pass's camera source to the renderer. */
		UE_API virtual UE::MovieGraph::DefaultRenderer::FCameraInfo GetRenderCameraInfo() const;

		/** Returns the overscan fraction for the given camera index, forwarding this pass's camera source to the renderer. */
		UE_API virtual float GetRenderCameraOverscan() const;

		/** Utility function for calculating a Projection Matrix (Orthographic or Perspective), and modify it based on the overscan percentage, aspect ratio, etc. */
		UE_API virtual void CalculateProjectionMatrix(UE::MovieGraph::DefaultRenderer::FCameraInfo& InOutCameraInfo, FSceneViewProjectionData& InOutProjectionData, const FIntPoint InBackbufferResolution, const FIntPoint InAccumulatorResolution) const;
		
		/** Utility function for modifying the projection matrix to match the given tiling parameters. */
		UE_API virtual void ModifyProjectionMatrixForTiling(const UE::MovieGraph::DefaultRenderer::FMovieGraphTilingParams& InTilingParams, const bool bInOrthographic, FMatrix& InOutProjectionMatrix, float& OutDoFSensorScale) const;
		
		/** Utility function for calculating the PrinciplePointOffset with the given tiling parameters. Used to make some effects (like vignette) work with tiling. */
		UE_API virtual FVector4f CalculatePrinciplePointOffsetForTiling(const UE::MovieGraph::DefaultRenderer::FMovieGraphTilingParams& InTilingParams) const;
		
		/** Utility function for creating FSceneViewInitOptions based on the specified camera info. */
		UE_API virtual FSceneViewInitOptions CreateViewInitOptions(const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo, FSceneViewFamilyContext* InViewFamily, FSceneViewStateInterface* InViewStateInterface) const;

		/**
		 * Reapplies an overscan magnitude to ViewInfo and enables engine-side view-rect scaling so the renderer
		 * supersamples for quality preservation. 
		 * 
		 * Note: The camera's bScaleResolutionWithOverscan is intentionally ignored to preserve offline render quality.
		 *
		 * Render passes should call this instead of ClearOverscan + ApplyOverscan directly because the raw
		 * pattern silently drops OverscanResolutionFraction.
		 */
		static UE_API void ReapplyOverscanPreservingEngineScaling(
			UE::MovieGraph::DefaultRenderer::FCameraInfo& InOutCameraInfo,
			float InOverscanMagnitude,
			const FIntPoint& InAccumulatorResolution,
			const FIntPoint& InOutputResolution);

		UE_DEPRECATED(5.6, "Use the version which takes a FSceneViewStateInterface pointer instead.")
		UE_API virtual FSceneViewInitOptions CreateViewInitOptions(const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo, FSceneViewFamilyContext* InViewFamily, FSceneViewStateReference& InViewStateRef) const;
		
		/** Utility function for creating a FSceneView for the given InitOptions, Family, and Camera. */
		UE_API virtual FSceneView* CreateSceneView(const FSceneViewInitOptions& InInitOptions, TSharedRef<FSceneViewFamilyContext> InViewFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const;

		/** Gets the render target init params that will be used when creating the render target. */
		UE_API virtual DefaultRenderer::FRenderTargetInitParams GetRenderTargetInitParams(const FMovieGraphTimeStepData& InTimeData, const FIntPoint& InResolution);

		UE_API virtual void ApplyCameraManagerPostProcessBlends(FSceneView* InView, const FMinimalViewInfo& InViewInfo, bool bApplyCameraManagerPostProcessBlends) const;
		UE_API virtual TSharedRef<FSceneViewFamilyContext> CreateSceneViewFamily(const FViewFamilyInitData& InInitData) const;
		UE_API virtual void ApplyMovieGraphOverridesToSceneView(TSharedRef<FSceneViewFamilyContext> InOutFamily, const FViewFamilyInitData& InInitData, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const;
		UE_API virtual void ApplyMovieGraphOverridesToViewFamily(TSharedRef<FSceneViewFamilyContext> InOutFamily, const FViewFamilyInitData& InInitData) const;
		UE_API virtual void PostRendererSubmission(const UE::MovieGraph::FMovieGraphSampleState& InSampleState, const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams, FCanvas& InCanvas, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const;
		UE_API virtual TFunction<void(TUniquePtr<FImagePixelData>&&)> MakeForwardingEndpoint(const FMovieGraphSampleState& InSampleState, const FMovieGraphTimeStepData& InTimeData);
		virtual bool ShouldDiscardOutput(const TSharedRef<FSceneViewFamilyContext>& InFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const { return false; }
		virtual void ApplyMovieGraphOverridesToSampleState(FMovieGraphSampleState& SampleState) const {}
		/** For this image pass, look up the associated node by type for the given config. */
		virtual UMovieGraphImagePassBaseNode* GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const { return nullptr; }

		/** Gets an accumulator that's in use for the provided sample state (or creates a new one), and returns the accumulator args that are in use. */
		UE_API virtual TSharedRef<::MoviePipeline::IMoviePipelineAccumulationArgs> GetOrCreateAccumulator(TObjectPtr<UMovieGraphDefaultRenderer> InGraphRenderer, const FMovieGraphSampleState& InSampleState) const;

		/** Gets the function that the accumulator will use to perform accumulation. */
		UE_API virtual FAccumulatorSampleFunc GetAccumulateSampleFunction() const;

	protected:
		TWeakObjectPtr<UMovieGraphDefaultRenderer> WeakGraphRenderer;

		/**
		 * Optional camera source shared among all FMovieGraphImagePassBase instances spawned by the same
		 * UMovieGraphRenderPassNode. The node creates at most one FMovieGraphRenderCameraSource and hands
		 * the same TSharedPtr to every pass instance it owns. Null when using default sequence cameras.
		 *
		 * FMovieGraphRenderCameraSource is a non-UObject, so the GC cannot trace into it. Each pass
		 * instance forwards AddReferencedObjects() to the shared source, so the source's UObject
		 * references are reported once per pass instance. Duplicate reports are safe; Unreal's GC
		 * handles them correctly.
		 */
		TSharedPtr<FMovieGraphRenderCameraSource> RenderCameraSource;
	};
}

#undef UE_API
