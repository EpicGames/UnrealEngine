// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MovieGraphDeferredPass.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

namespace UE::MovieGraph::Rendering
{
	struct FMovieGraphPathTracerPass : public FMovieGraphDeferredPass
	{
		UE_API virtual UMovieGraphImagePassBaseNode* GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const override;
		UE_API virtual void ApplyMovieGraphOverridesToSceneView(TSharedRef<FSceneViewFamilyContext> InOutFamily, const FViewFamilyInitData& InInitData, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const override;
		UE_API virtual void ApplyMovieGraphOverridesToSampleState(FMovieGraphSampleState& SampleState) const override;
		UE_API virtual bool ShouldDiscardOutput(const TSharedRef<FSceneViewFamilyContext>& InFamily, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const override;
	}; 
}

#undef UE_API
