// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphConfig.h"
#include "Graph/Nodes/MovieGraphFileOutputNode.h"

#include "Async/Future.h"
#include "IImageWrapper.h"
#include "MoviePipelineEXROutput.h"
#include "Styling/AppStyle.h"

#include "MovieGraphImageSequenceOutputNode.generated.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

// Forward Declare
class IImageWriteQueue;

/**
* The UMovieGraphImageSequenceOutputNode node is the base class for all image sequence outputs, such as 
* a series of jpeg, png, bmp, or .exr images. Create an instance of the appropriate class (such as 
* UMovieGraphImageSequenceOutputNode_JPG) instead of this abstract base class.
*/
UCLASS(MinimalAPI, Abstract)
class UMovieGraphImageSequenceOutputNode : public UMovieGraphFileOutputNode
{
	GENERATED_BODY()
public:
	UE_API UMovieGraphImageSequenceOutputNode();
	
	// UMovieGraphFileOutputNode Interface
	UE_API virtual void OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask) override;
	UE_API virtual void OnAllFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) override;
	UE_API virtual bool IsFinishedWritingToDiskImpl() const override;
	// ~UMovieGraphFileOutputNode Interface

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OCIOConfiguration : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OCIOContext : 1;

	/**
	* OCIO configuration/transform settings.
	*
	* Note: There are differences from the previous implementation in MRQ given that we are now doing CPU-side processing.
	* 1) This feature only works on desktop platforms when the OpenColorIO library is available.
	* 2) Users are now responsible for setting the renderer output space to Final Color (HDR) in Linear Working Color Space (SCS_FinalColorHDR).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", DisplayName="OCIO Configuration", meta = (EditCondition = "bOverride_OCIOConfiguration"))
	FOpenColorIODisplayConfiguration OCIOConfiguration;

	/**
	* OCIO context of key-value string pairs, typically used to apply shot-specific looks (such as a CDL color correction, or a 1D grade LUT).
	* 
	* Notes:
	* 1) If a configuration asset base context was set, it remains active but can be overridden here with new key-values.
	* 2) Format tokens such as {shot_name} are supported and will get resolved before submission.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", DisplayName = "OCIO Context", meta = (EditCondition = "bOverride_OCIOContext"))
	TMap<FString, FString> OCIOContext;

protected:
	/** Convenience function to create the output file name. */
	UE_API FString CreateFileName(
		UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData,
		const UMovieGraphImageSequenceOutputNode* InParentNode,
		const UMovieGraphPipeline* InPipeline,
		const TPair<FMovieGraphRenderDataIdentifier, TUniquePtr<FImagePixelData>>& InRenderData,
		const EImageFormat InImageFormat,
		FMovieGraphResolveArgs& OutMergedFormatArgs,
		FString& OutFrameTemplatedFileName) const;

	/** Gets a "frame-number templated" filename (eg, Seq.Shot.{frame_placeholder}.exr) where '{frame_placeholder}' is used in place of the frame number. */
	UE_API FString GetFrameTemplatedFileName(
		const FMovieGraphFilenameResolveParams& InParams,
		const FString& InFileNameFormatString,
		FMovieGraphResolveArgs& OutMergedFormatArgs) const;

protected:
	/** The output format (as known used by the ImageWriteQueue) to output into. */
	EImageFormat OutputFormat;

	/** Whether we enforce 8-bit depth on the output. */
	bool bQuantizeTo8Bit;

	/** A pointer to the image write queue used for asynchronously writing images */
	IImageWriteQueue* ImageWriteQueue;

	/** A fence to keep track of when the Image Write queue has fully flushed. */
	TFuture<void> FinalizeFence;
};

/**
 * Image sequence output node that can write single-layer EXR files.
 */
UCLASS(MinimalAPI, meta=(PrioritizeCategories="EXR OCIO"))
class UMovieGraphImageSequenceOutputNode_EXR : public UMovieGraphImageSequenceOutputNode, public IMovieGraphEvaluationNodeInjector
{
	GENERATED_BODY()

public:
	UMovieGraphImageSequenceOutputNode_EXR()
	{
		OutputFormat = EImageFormat::EXR;
		bQuantizeTo8Bit = false;
		Compression = EEXRCompressionFormat::PIZ;
		LosslessCompression = EEXRCompressionFormat::PIZ;
	}

#if WITH_UNREALEXR
	UE_API virtual void OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask) override;
#endif // WITH_UNREALEXR

	virtual bool ShouldCropOverscanImpl() const override { return false; }
	
#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override 
	{ 
		static const FText EXRSequenceNodeName = NSLOCTEXT("MovieGraph", "NodeName_EXRSequence", ".exr Sequence (Single-Layer)");
		return EXRSequenceNodeName;
	}

	virtual FText GetKeywords() const override
	{
		static const FText Keywords = NSLOCTEXT("MovieGraph", "ImageSequenceOutputNode_EXR_Keywords", "exr image single layer");
		return Keywords;
	}
	
	virtual FLinearColor GetNodeTitleColor() const override
	{
		return FLinearColor(0.047f, 0.654f, 0.537f);
	}
	
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override
	{
		static const FSlateIcon ImageSequenceIcon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.RenderSequenceOutput");

		OutColor = FLinearColor::White;
		return ImageSequenceIcon;
	}
#endif
	
	//~ Begin IMovieGraphEvaluationNodeInjector interface
	virtual void InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes) override;
	//~ End IMovieGraphEvaluationNodeInjector interface
	
	//~ Begin UMovieGraphSettingNode interface
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	//~ End UMovieGraphSettingNode interface
	
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Compression : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_CompressionLevel : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_LosslessCompression : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableBurnIn : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInClass : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCompositeOntoFinalImage : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInFileNameFormat : 1;

	/**
	 * The type of compression the EXR file should be compressed with. This compression method will be applied to all layers unless the data in
	 * a layer has been flagged as needing lossless compression. See "Lossless Compression" for more details.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(EditCondition="bOverride_Compression"), Category = "EXR")
	EEXRCompressionFormat Compression;

	/**
	 * The DWA quality level to use when the compression type is DWAA or DWAB.
	 * 
	 * 0 - Visually lossless and maximum file size.
	 * 45 - Perceptually lossless.
	 * 100+ - Maximum compression and minimum file size.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(EditCondition="bOverride_CompressionLevel", ClampMin="0", ClampMax="500", UIMin="0", UIMax="500"), Category = "EXR")
	int32 CompressionLevel = 45;

	/**
	 * The type of lossless compression that should be used when a render pass is either forced to use lossless (eg, Object ID), or when the pass
	 * has been flagged to need lossless (eg, Post Process Material passes can opt-in to lossless).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(EditCondition="bOverride_LosslessCompression", ValidEnumValues="None,RLE,ZIPS,ZIP,PIZ"), Category = "EXR")
	EEXRCompressionFormat LosslessCompression;

	/** If true, this output node will also generate a burn-in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bEnableBurnIn"))
	bool bEnableBurnIn = false;

	/** The widget that the burn-in should use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Burn In", meta=(MetaClass="/Script/MovieRenderPipelineCore.MovieGraphBurnInWidget", EditCondition="bOverride_BurnInClass"))
	FSoftClassPath BurnInClass = FSoftClassPath(TEXT("/MovieRenderPipeline/Blueprints/Graph/DefaultGraphBurnIn.DefaultGraphBurnIn_C"));

	/** If true, the burn-in that's generated will be composited onto this output. Otherwise, the burn-in will be written to a different file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bCompositeOntoFinalImage"))
	bool bCompositeOntoFinalImage = true;

	/** The file name format used for writing out the burn-in image (if not composited). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_BurnInFileNameFormat", MrgTokenAutocomplete))
	FString BurnInFileNameFormat = TEXT("{sequence_name}.{layer_name}.BurnIn.{frame_number}");

protected:

#if WITH_UNREALEXR

	/** Convenience function to create a new EXR image write task, given a file name and compression format. */
	UE_API TUniquePtr<FEXRImageWriteTask> CreateImageWriteTask(
		FString InFileName,
		EEXRCompressionFormat InCompression,
		bool bMultiPart = false,
		int32 InCompressionLevel = 45
	) const;
	
	/** Convenience function to prepare the image write task's global file metadata. */
	UE_API void PrepareTaskGlobalMetadata(
		FEXRImageWriteTask& InOutImageTask,
		UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData,
		TMap<FString, FString>& InMetadata
	) const;

	/** Convenience function to update the image write task for layer data. */
	UE_API void UpdateTaskPerLayer(
		FEXRImageWriteTask& InOutImageTask,
		const UMovieGraphImageSequenceOutputNode* InParentNode,
		TUniquePtr<FImagePixelData> InImageData,
		int32 InLayerIndex,
		const FString& InLayerName = {},
		const TMap<FString, FString>& InResolvedOCIOContext = {}
	) const;

#endif // WITH_UNREALEXR
};


/**
 * Image sequence output node that can write multi-layer EXR files.
 */
UCLASS(MinimalAPI, meta=(PrioritizeCategories="EXR OCIO"))
class UMovieGraphImageSequenceOutputNode_MultiLayerEXR : public UMovieGraphImageSequenceOutputNode_EXR
{
	GENERATED_BODY()

public:
	UMovieGraphImageSequenceOutputNode_MultiLayerEXR()
		: UMovieGraphImageSequenceOutputNode_EXR()
	{
		// Multi-layer default excludes {layer_name}.
		FileNameFormat = TEXT("{sequence_name}.{frame_number}");
	}

	virtual EMovieGraphBranchRestriction GetBranchRestriction() const override
	{
		return EMovieGraphBranchRestriction::Globals;
	}

#if WITH_UNREALEXR
	UE_API virtual void OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask) override;
#endif // WITH_UNREALEXR

	virtual bool ShouldCropOverscanImpl() const override { return false; }
	
#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override
	{
		static const FText EXRSequenceNodeName = NSLOCTEXT("MovieGraph", "NodeName_EXRSequenceMultilayer", ".exr Sequence (Multilayer)");
		return EXRSequenceNodeName;
	}
	
	virtual FText GetBasicConfigShortDisplayName() const override
	{
		return NSLOCTEXT("MovieGraph", "BasicConfigShortName_EXR", "EXR");
	}

	virtual FText GetKeywords() const override
	{
		static const FText Keywords = NSLOCTEXT("MovieGraph", "ImageSequenceOutputNode_EXRMultilayer_Keywords", ".exr image multi layer (Multilayer)");
		return Keywords;
	}
		
	virtual FLinearColor GetNodeTitleColor() const override
	{
		return FLinearColor(0.047f, 0.654f, 0.537f);
	}

	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override
	{
		static const FSlateIcon ImageSequenceIcon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.RenderSequenceOutput");

		OutColor = FLinearColor::White;
		return ImageSequenceIcon;
	}

	virtual FText GetMenuCategory() const override
	{
		return NSLOCTEXT("MovieGraphNodes", "FileOutputGraphNode_Category", "Output Type");
	}
#endif
	
	//~ Begin IMovieGraphEvaluationNodeInjector interface
	virtual void InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes) override;
	//~ End IMovieGraphEvaluationNodeInjector interface

	//~ Begin UMovieGraphSettingNode interface
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	//~ End UMovieGraphSettingNode interface

private:
	/** Stores the EXR output config to use for a specific file name */
	struct FEXROutputConfigForFilename
	{
		TArray<FMovieGraphRenderDataIdentifier> RenderIDs;
		FMovieGraphResolveArgs ResolveArgs;
		FString FrameTemplatedFilename;
		FIntPoint MaximumResolution;
	};

	/**
	 * Generates an output config for each filename, which consists of a list of render IDs, the resolve args that were created
	 * when resolving the filename, the frame-templated filenames (ie, filenames that have placeholders in the place of frame numbers),
	 * and the maximum resolution of all the layers written to the file, which will be used to pad out lower resolution layers so that
	 * they can all be in the same EXR file (when multi-part is not used).
	 */
	UE_API void GetFilenameToEXROutputConfigMappings(
		const UMovieGraphImageSequenceOutputNode_MultiLayerEXR* InParentNode,
		UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData,
		TMap<FString, FEXROutputConfigForFilename>& OutFilenameToOutputConfigs) const;
	
	/**
	 * Generates the filename that the EXR will be written to, as well as the resulting resolve args via OutResolveArgs.
	 * Use GetFilenameToRenderIDMappings() to guarantee that the filename respects EXR limitations.
	 */
	UE_API FString ResolveOutputFilename(
		const UMovieGraphImageSequenceOutputNode_MultiLayerEXR* InParentNode,
		const UMovieGraphPipeline* InPipeline, const UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData,
		const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier, FMovieGraphResolveArgs& OutResolveArgs,
		FString& OutFrameTemplatedFilename) const;
	
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bMultiPart : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInLayerNameFormat : 1;
	
	/**
	 * Indicates whether the exr file should be written as a multi-part exr file, which supports having different image types and resolutions for each layer.
	 * Multi-part EXRs are a feature of EXR 2.0 and may not be supported by all software. If the console variable 'MoviePipeline.PadLayersForMultiPartEXR' is enabled,
	 * then all parts of the multi-part EXR will be padded to match the data window of the largest layer, as some software does not support different data window sizes.
	 *
	 * Note that multi-part will be forced on if layers are using more than one compression type (this can happen if rendering a beauty pass using lossy
	 * compression, and Object IDs using lossless compression, for example).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "EXR", meta=(EditCondition="bOverride_bMultiPart"))
	bool bMultipart = true;

	/** The layer name used for this burn-in inside of multi-layer EXRs. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (DisplayAfter = "BurnInClass", EditCondition = "bOverride_BurnInLayerNameFormat"))
	FString BurnInLayerNameFormat = TEXT("BurnIn");
};

/**
* Save the images generated by the Movie Graph Pipeline as an lossless 8 bit bmp format. This can
* be useful in rare occasions (bmp files are uncompressed but larger). sRGB is applied.
* No metadata is supported.
*/
UCLASS(MinimalAPI, meta=(PrioritizeCategories="OCIO"))
class UMovieGraphImageSequenceOutputNode_BMP : public UMovieGraphImageSequenceOutputNode, public IMovieGraphEvaluationNodeInjector
{
	GENERATED_BODY()

public:
	UMovieGraphImageSequenceOutputNode_BMP()
	{
		OutputFormat = EImageFormat::BMP;
		bQuantizeTo8Bit = true;
	}

#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override 
	{ 
		if(bGetDescriptive)
		{
			return NSLOCTEXT("MovieGraph", "ImgSequenceBMPSetting_NodeTitleFull", ".bmp Sequence\n[8bit]"); 
		}
		return NSLOCTEXT("MovieGraph", "ImgSequenceBMPSetting_NodeTitleShort", ".bmp Sequence");
	}

	virtual FText GetBasicConfigShortDisplayName() const override
	{
		return NSLOCTEXT("MovieGraph", "BasicConfigShortName_BMP", "BMP");
	}

	virtual FText GetKeywords() const override
	{
		static const FText Keywords = NSLOCTEXT("MovieGraph", "ImageSequenceOutputNode_BMP_Keywords", "bmp image");
		return Keywords;
	}
	
	virtual FLinearColor GetNodeTitleColor() const override
	{
		return FLinearColor(0.047f, 0.654f, 0.537f);
	}
	
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override
	{
		static const FSlateIcon ImageSequenceIcon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.RenderSequenceOutput");

		OutColor = FLinearColor::White;
		return ImageSequenceIcon;
	}
#endif
	
	//~ Begin IMovieGraphEvaluationNodeInjector interface
	virtual void InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes) override;
	//~ End IMovieGraphEvaluationNodeInjector interface

	//~ Begin UMovieGraphSettingNode interface
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	//~ End UMovieGraphSettingNode interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableBurnIn : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInClass : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCompositeOntoFinalImage : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInFileNameFormat : 1;

	/** If true, this output node will also generate a burn-in. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bEnableBurnIn"))
    bool bEnableBurnIn = false;

    /** The widget that the burn-in should use. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Burn In", meta=(MetaClass="/Script/MovieRenderPipelineCore.MovieGraphBurnInWidget", EditCondition="bOverride_BurnInClass"))
    FSoftClassPath BurnInClass = FSoftClassPath(TEXT("/MovieRenderPipeline/Blueprints/Graph/DefaultGraphBurnIn.DefaultGraphBurnIn_C"));

	/** If true, the burn-in that's generated will be composited onto this output. Otherwise, the burn-in will be written to a different file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bCompositeOntoFinalImage"))
	bool bCompositeOntoFinalImage = true;

	/** The file name format used for writing out the burn-in image (if not composited). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_BurnInFileNameFormat", MrgTokenAutocomplete))
	FString BurnInFileNameFormat = TEXT("{sequence_name}.{layer_name}.BurnIn.{frame_number}");
};


/**
* Save the images generated by the Movie Graph Pipeline as an 8 bit jpg format. JPEG image files
* are lossy, but a good balance between compression speed and final filesize. sRGB is applied.
* No metadata is supported.
*/
UCLASS(MinimalAPI, meta=(PrioritizeCategories="OCIO"))
class UMovieGraphImageSequenceOutputNode_JPG : public UMovieGraphImageSequenceOutputNode, public IMovieGraphEvaluationNodeInjector
{
	GENERATED_BODY()

public:
	UMovieGraphImageSequenceOutputNode_JPG()
	{
		OutputFormat = EImageFormat::JPEG;
		bQuantizeTo8Bit = true;
	}

#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override 
	{ 
		if(bGetDescriptive)
		{
			return NSLOCTEXT("MovieGraph", "ImgSequenceJPGSetting_NodeTitleFull", ".jpg Sequence\n[8bit]"); 
		}
		return NSLOCTEXT("MovieGraph", "ImgSequenceJPGSetting_NodeTitleShort", ".jpg Sequence");
	}

	virtual FText GetKeywords() const override
	{
		static const FText Keywords = NSLOCTEXT("MovieGraph", "ImageSequenceOutputNode_JPG_Keywords", "jpg jpeg image");
		return Keywords;
	}
	
	virtual FText GetBasicConfigShortDisplayName() const override
	{
		return NSLOCTEXT("MovieGraph", "BasicConfigShortName_JPG", "JPG");
	}

	virtual FLinearColor GetNodeTitleColor() const override
	{
		return FLinearColor(0.047f, 0.654f, 0.537f);
	}
	
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override
	{
		static const FSlateIcon ImageSequenceIcon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.RenderSequenceOutput");

		OutColor = FLinearColor::White;
		return ImageSequenceIcon;
	}
#endif

	//~ Begin IMovieGraphEvaluationNodeInjector interface
	virtual void InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes) override;
	//~ End IMovieGraphEvaluationNodeInjector interface

	//~ Begin UMovieGraphSettingNode interface
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	//~ End UMovieGraphSettingNode interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableBurnIn : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInClass : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCompositeOntoFinalImage : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInFileNameFormat : 1;

	/** If true, this output node will also generate a burn-in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bEnableBurnIn"))
	bool bEnableBurnIn = false;

	/** The widget that the burn-in should use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Burn In", meta=(MetaClass="/Script/MovieRenderPipelineCore.MovieGraphBurnInWidget", EditCondition="bOverride_BurnInClass"))
	FSoftClassPath BurnInClass = FSoftClassPath(TEXT("/MovieRenderPipeline/Blueprints/Graph/DefaultGraphBurnIn.DefaultGraphBurnIn_C"));

	/** If true, the burn-in that's generated will be composited onto this output. Otherwise, the burn-in will be written to a different file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bCompositeOntoFinalImage"))
	bool bCompositeOntoFinalImage = true;

	/** The file name format used for writing out the burn-in image (if not composited). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_BurnInFileNameFormat", MrgTokenAutocomplete))
	FString BurnInFileNameFormat = TEXT("{sequence_name}.{layer_name}.BurnIn.{frame_number}");
};

/**
* Save the images generated by the Movie Graph Pipeline as an 8 bit png format. PNG image files
* are lossless but slow to compress and have a larger final filesize than JPEG. sRGB is applied.
* No metadata is supported.
*/
UCLASS(MinimalAPI, meta=(PrioritizeCategories="OCIO"))
class UMovieGraphImageSequenceOutputNode_PNG : public UMovieGraphImageSequenceOutputNode, public IMovieGraphEvaluationNodeInjector
{
	GENERATED_BODY()

public:
	UMovieGraphImageSequenceOutputNode_PNG()
	{
		OutputFormat = EImageFormat::PNG;

		// Note: we could offer linear 16-bit pngs simply by letting users turn this to false.
		bQuantizeTo8Bit = true;
	}

#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override 
	{ 
		if(bGetDescriptive)
		{
			return NSLOCTEXT("MovieGraph", "ImgSequencePNGSetting_NodeTitleFull", ".png Sequence\n[8bit]"); 
		}
		return NSLOCTEXT("MovieGraph", "ImgSequencePNGSetting_NodeTitleShort", ".png Sequence");
	}

	virtual FText GetKeywords() const override
	{
		static const FText Keywords = NSLOCTEXT("MovieGraph", "ImageSequenceOutputNode_PNG_Keywords", "png image");
		return Keywords;
	}
	
	virtual FText GetBasicConfigShortDisplayName() const override
	{
		return NSLOCTEXT("MovieGraph", "BasicConfigShortName_PNG", "PNG");
	}

	virtual FLinearColor GetNodeTitleColor() const override
	{
		return FLinearColor(0.047f, 0.654f, 0.537f);
	}
	
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override
	{
		static const FSlateIcon ImageSequenceIcon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.RenderSequenceOutput");

		OutColor = FLinearColor::White;
		return ImageSequenceIcon;
	}
#endif
	
	//~ Begin IMovieGraphEvaluationNodeInjector interface
	virtual void InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes) override;
	//~ End IMovieGraphEvaluationNodeInjector interface

	//~ Begin UMovieGraphSettingNode interface
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	//~ End UMovieGraphSettingNode interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableBurnIn : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInClass : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCompositeOntoFinalImage : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInFileNameFormat : 1;

	/** If true, this output node will also generate a burn-in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bEnableBurnIn"))
	bool bEnableBurnIn = false;

	/** The widget that the burn-in should use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Burn In", meta=(MetaClass="/Script/MovieRenderPipelineCore.MovieGraphBurnInWidget", EditCondition="bOverride_BurnInClass"))
	FSoftClassPath BurnInClass = FSoftClassPath(TEXT("/MovieRenderPipeline/Blueprints/Graph/DefaultGraphBurnIn.DefaultGraphBurnIn_C"));

	/** If true, the burn-in that's generated will be composited onto this output. Otherwise, the burn-in will be written to a different file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bCompositeOntoFinalImage"))
	bool bCompositeOntoFinalImage = true;

	/** The file name format used for writing out the burn-in image (if not composited). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_BurnInFileNameFormat", MrgTokenAutocomplete))
	FString BurnInFileNameFormat = TEXT("{sequence_name}.{layer_name}.BurnIn.{frame_number}");
};

#undef UE_API
