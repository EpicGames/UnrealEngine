// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineImagePassBase.h"
#include "ActorLayerUtilities.h"
#include "OpenColorIODisplayExtension.h"
#include "MoviePipelineDeferredPasses.generated.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

class UTextureRenderTarget2D;
struct FImageOverlappedAccumulator;
class FSceneViewFamily;
class FSceneView;
struct FAccumulatorPool;
struct FSceneViewStateSystemMemoryMirror;

USTRUCT(BlueprintType)
struct FMoviePipelinePostProcessPass
{
	GENERATED_BODY()

public:
	/** Additional passes add a significant amount of render time. May produce multiple output files if using Screen Percentage. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bEnabled = false;

	/**
	 * An optional name which which will identify this material in the file name. For MRQ, the material name will be included in the {render_pass}
	 * token. For Movie Render Graph, the material name will be used in {renderer_sub_name}. If a name is not specified here, the full name of the material
	 * will be used in these tokens instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (MrgTokenAutocomplete))
	FString Name;

	/** 
	* Material should be set to Post Process domain, and Blendable Location = After Tonemapping. 
	* This will need bDisableMultisampleEffects enabled for pixels to line up(ie : no DoF, MotionBlur, TAA)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TSoftObjectPtr<UMaterialInterface> Material;

	/** Request output to be 32-bit, usually for data exports. Note that scene color precision is still defined by r.SceneColorFormat.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", DisplayName="Use High Precision (32-bit) Output")
	bool bHighPrecisionOutput = false;

	/**
	 * Whether this pass should use lossless compression in formats that support it. Should usually be turned on for passes that need a highly
	 * accurate representation of the data (eg, depth).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bUseLosslessCompression = false;
};

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineDeferredPassBase : public UMoviePipelineImagePassBase
{
	GENERATED_BODY()

public:
	UE_API UMoviePipelineDeferredPassBase();
	
	// UObject Interface
	UE_API virtual void PostLoad() override;
	// ~UObject Interface

protected:
	// UMoviePipelineRenderPass API
	UE_API virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;
	UE_API virtual void RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState) override;
	UE_API virtual void OnTileStartImpl(FIntPoint TileIndexes) override;
	UE_API virtual void OnTileEndImpl(FIntPoint TileIndexes) override;
	UE_API virtual void TeardownImpl() override;
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DeferredBasePassSetting_DisplayName_Lit", "Deferred Rendering"); }
#endif
	UE_API virtual void MoviePipelineRenderShowFlagOverride(FEngineShowFlags& OutShowFlag) override;
	UE_API virtual void GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses) override;
	virtual bool IsAntiAliasingSupported() const { return !bDisableMultisampleEffects; }
	virtual int32 GetOutputFileSortingOrder() const override { return 0; }
	UE_API virtual FSceneViewStateInterface* GetSceneViewStateInterface(IViewCalcPayload* OptPayload = nullptr) override;
	UE_API virtual FSceneViewStateInterface* GetExposureSceneViewStateInterface(IViewCalcPayload* OptPayload = nullptr, int32 CubeFaceIndex = INDEX_NONE) override;
	UE_API virtual void AddViewExtensions(FSceneViewFamilyContext& InContext, FMoviePipelineRenderPassMetrics& InOutSampleState) override;
	UE_API virtual bool IsAutoExposureAllowed(const FMoviePipelineRenderPassMetrics& InSampleState) const override;
	UE_API virtual void BlendPostProcessSettings(FSceneView* InView, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload = nullptr);
	UE_API virtual UE::MoviePipeline::FImagePassCameraViewData GetCameraInfo(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload = nullptr) const;
	// ~UMoviePipelineRenderPass

	// FGCObject Interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	// ~FGCObject Interface

	UE_API TFunction<void(TUniquePtr<FImagePixelData>&&)> MakeForwardingEndpoint(const FMoviePipelinePassIdentifier InPassIdentifier, const FMoviePipelineRenderPassMetrics& InSampleState);
	UE_API virtual void PostRendererSubmission(const FMoviePipelineRenderPassMetrics& InSampleState, const FMoviePipelinePassIdentifier InPassIdentifier, const int32 InSortingOrder, FCanvas& InCanvas);

	UE_API virtual int32 GetNumCamerasToRender() const;
	UE_API virtual int32 GetCameraIndexForRenderPass(const int32 InCameraIndex) const;
	UE_API virtual FString GetCameraName(const int32 InCameraIndex) const;
	UE_API virtual FString GetCameraNameOverride(const int32 InCameraIndex) const;

	UE_API virtual FMoviePipelineRenderPassMetrics GetRenderPassMetricsForCamera(const int32 InCameraIndex, const FMoviePipelineRenderPassMetrics& InSampleState) const;

	UE_DEPRECATED(5.6, "GetEffectiveOutputResolutionForCamera is deprecated. Use GetOverscannedResolutionForCamera instead")
	UE_API virtual FIntPoint GetEffectiveOutputResolutionForCamera(const int32 InCameraIndex) const;
	UE_API virtual FIntPoint GetOverscannedResolutionForCamera(const int32 InCameraIndex) const;
	UE_API bool GetManualExposureEnabledForCamera(const int32 InCameraIndex) const;

	UE_API bool IsUsingDataLayers() const;
	UE_API int32 GetNumStencilLayers() const;
	UE_API TArray<FString> GetStencilLayerNames() const;
	UE_API bool IsActorInLayer(AActor* InActor, int32 InLayerIndex) const;
	UE_API bool IsActorInAnyStencilLayer(AActor* InActor) const;
	UE_API FSoftObjectPath GetValidDataLayerByIndex(const int32 InIndex) const;

	UE_API bool CheckIfPathTracerIsSupported() const;
	UE_API void PathTracerValidationImpl();

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

private:
	/** Gets the name for a post-process material (either the custom-specified name, or the material name). */
	UE_API FString GetNameForPostProcessMaterial(const UMaterialInterface* InMaterial);

public:
	/**
	* Should multiple temporal/spatial samples accumulate the alpha channel? This adds ~30% cost to the
	* accumulation so you should not enable it unless necessary. You must delete both the sky and fog to ensure
	* that they do not make all pixels opaque.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bAccumulatorIncludesAlpha;

	/**
	* Certain passes don't support post-processing effects that blend pixels together. These include effects like
	* Depth of Field, Temporal Anti-Aliasing, Motion Blur and chromattic abberation. When these post processing
	* effects are used then each final output pixel is composed of the influence of many other pixels which is
	* undesirable when rendering out an object id pass (which does not support post processing). This checkbox lets
	* you disable them on a per-render basis instead of having to disable them in the editor as well.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Processing")
	bool bDisableMultisampleEffects;

#if WITH_EDITORONLY_DATA
	/**
	* Should the additional post-process materials write out to a 32-bit render target instead of 16-bit?
	*/
	UE_DEPRECATED(5.5, "bUse32BitPostProcessMaterials has been deprecated, please use the setting per material.")
	UPROPERTY()
	bool bUse32BitPostProcessMaterials_DEPRECATED;
#endif

	/**
	* An array of additional post-processing materials to run after the frame is rendered. Using this feature may add a notable amount of render time.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Deferred Renderer Data")
	TArray<FMoviePipelinePostProcessPass> AdditionalPostProcessMaterials;

	/**
	* This can be turned off if you're only doing a stencil-layer based render and don't need the main non-stencil approach.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stencil Clip Layers")
	bool bRenderMainPass;

	/**
	* If true, an additional stencil layer will be rendered which contains all objects which do not belong to layers
	* specified in the Stencil Layers. This is useful for wanting to isolate one or two layers but still have everything
	* else to composite them over without having to remember to add all objects to a default layer.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stencil Clip Layers")
	bool bAddDefaultLayer;

	/** 
	* For each layer in the array, the world will be rendered and then a stencil mask will clip all pixels not affected
	* by the objects on that layer. This is NOT a true layer system, as translucent objects will show opaque objects from
	* another layer behind them. Does not write out additional post-process materials per-layer as they will match the
	* base layer. Only works with materials that can write to custom depth.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stencil Clip Layers")
	TArray<FActorLayer> ActorLayers;
	
	/**
	* If the map you are working with is a World Partition map, you can specify Data layers instead of Actor Layers. If any
	* Data Layers are specified, this will take precedence over any ActorLayers in this config. Does not affect whether or
	* not the Data Layers are actually loaded, you must ensure layers are loaded for rendering.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (AllowedClasses = "/Script/Engine.DataLayerAsset"), Category = "Stencil Clip Layers")
	TArray<FSoftObjectPath> DataLayers;

protected:
	/** While rendering, store an array of the non-null valid materials loaded from AdditionalPostProcessMaterials. Cleared on teardown. */
	UPROPERTY(Transient, DuplicateTransient)
	TArray<TObjectPtr<UMaterialInterface>> ActivePostProcessMaterials;

	UPROPERTY(Transient, DuplicateTransient)
	TSet<TObjectPtr<UMaterialInterface>> ActiveHighPrecisionPostProcessMaterials;

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UMaterialInterface> StencilLayerMaterial;

	struct FMultiCameraViewStateData
	{
		struct FPerTile
		{
			TArray<FSceneViewStateReference> SceneViewStates;
		};

		TMap<FIntPoint, FPerTile> TileData;
		TPimplPtr<FSceneViewStateSystemMemoryMirror> SystemMemoryMirror;
	};

	TArray<FMultiCameraViewStateData> CameraViewStateData;

	// Cache the custom stencil value. Only has meaning if they have stencil layers.
	TOptional<int32> PreviousCustomDepthValue;
	UE_DEPRECATED(5.5, "PreviousDumpFramesValue has been deprecated.")
	TOptional<int32> PreviousDumpFramesValue;
	UE_DEPRECATED(5.5, "PreviousColorFormatValue has been deprecated.")
	TOptional<int32> PreviousColorFormatValue;

	TSharedPtr<FAccumulatorPool, ESPMode::ThreadSafe> AccumulatorPool;
	/** The lifetime of this SceneViewExtension is only during the rendering process. It is destroyed as part of TearDown. */
	TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe> OCIOSceneViewExtension;
public:
	static UE_API FString StencilLayerMaterialAsset;
	static UE_API FString DefaultDepthAsset;
	static UE_API FString DefaultMotionVectorsAsset;

private:
	/** Cached de-duplicated stencil layer names. */
	TArray<FString> UniqueStencilLayerNames;
};



UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineDeferredPass_Unlit : public UMoviePipelineDeferredPassBase
{
	GENERATED_BODY()

public:
	UMoviePipelineDeferredPass_Unlit() : UMoviePipelineDeferredPassBase()
	{
		PassIdentifier = FMoviePipelinePassIdentifier("Unlit");
	}
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DeferredBasePassSetting_DisplayName_Unlit", "Deferred Rendering (Unlit)"); }
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutViewModeIndex = EViewModeIndex::VMI_Unlit;
	}
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }

};

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineDeferredPass_DetailLighting : public UMoviePipelineDeferredPassBase
{
	GENERATED_BODY()

public:
	UMoviePipelineDeferredPass_DetailLighting() : UMoviePipelineDeferredPassBase()
	{
		PassIdentifier = FMoviePipelinePassIdentifier("DetailLightingOnly");
	}
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DeferredBasePassSetting_DisplayName_DetailLighting", "Deferred Rendering (Detail Lighting)"); }
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutShowFlag.SetLightingOnlyOverride(true);
		OutViewModeIndex = EViewModeIndex::VMI_Lit_DetailLighting;
	}
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }

};

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineDeferredPass_LightingOnly : public UMoviePipelineDeferredPassBase
{
	GENERATED_BODY()

public:
	UMoviePipelineDeferredPass_LightingOnly() : UMoviePipelineDeferredPassBase()
	{
		PassIdentifier = FMoviePipelinePassIdentifier("LightingOnly");
	}
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DeferredBasePassSetting_DisplayName_LightingOnly", "Deferred Rendering (Lighting Only)"); }
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutShowFlag.SetLightingOnlyOverride(true);
		OutViewModeIndex = EViewModeIndex::VMI_LightingOnly;
	}
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }

};

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineDeferredPass_ReflectionsOnly : public UMoviePipelineDeferredPassBase
{
	GENERATED_BODY()

public:
	UMoviePipelineDeferredPass_ReflectionsOnly() : UMoviePipelineDeferredPassBase()
	{
		PassIdentifier = FMoviePipelinePassIdentifier("ReflectionsOnly");
	}
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DeferredBasePassSetting_DisplayName_ReflectionsOnly", "Deferred Rendering (Reflections Only)"); }
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutShowFlag.SetReflectionOverride(true);
		OutViewModeIndex = EViewModeIndex::VMI_ReflectionOverride;
	}
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }
};


UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineDeferredPass_PathTracer : public UMoviePipelineDeferredPassBase
{
	GENERATED_BODY()

public:
	UMoviePipelineDeferredPass_PathTracer() : UMoviePipelineDeferredPassBase()
	{
		PassIdentifier = FMoviePipelinePassIdentifier("PathTracer");
	}
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DeferredBasePassSetting_DisplayName_PathTracer", "Path Tracer"); }
	UE_API virtual FText GetFooterText(UMoviePipelineExecutorJob* InJob) const override;
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutShowFlag.SetPathTracing(true);
		OutShowFlag.SetMotionBlur(!bReferenceMotionBlur);
		OutViewModeIndex = EViewModeIndex::VMI_PathTracing;
	}
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }

	virtual bool IsAntiAliasingSupported() const { return false; }
	UE_API virtual void ValidateStateImpl() override;
	UE_API virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;

	UE_API virtual TSharedPtr<FSceneViewFamilyContext> CalculateViewFamily(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload) override;

	virtual bool NeedsFrameThrottle() const override { return true; }
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

	/** When enabled, the path tracer will blend all spatial and temporal samples prior to the denoising and will disable post-processed motion blur.
	 *  In this mode it is possible to use higher temporal sample counts to improve the motion blur quality.
	 *  When this option is disabled, the path tracer will accumulate spatial samples, but denoise them prior to accumulation of temporal samples.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Reference Motion Blur")
	bool bReferenceMotionBlur;
};

#undef UE_API
