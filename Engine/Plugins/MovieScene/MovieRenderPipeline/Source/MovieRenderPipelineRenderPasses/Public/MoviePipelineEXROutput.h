// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ImageWriteTask.h"
#include "ImagePixelData.h"
#include "MoviePipelineImageSequenceOutput.h"
#include "Misc/StringFormatArg.h"

#if WITH_UNREALEXR
THIRD_PARTY_INCLUDES_START
#include "Imath/ImathBox.h"
#include "OpenEXR/ImfArray.h"
#include "OpenEXR/ImfChannelList.h"
#include "OpenEXR/ImfHeader.h"
#include "OpenEXR/ImfIO.h"
#include "OpenEXR/ImfInputFile.h"
#include "OpenEXR/ImfOutputFile.h"
#include "OpenEXR/ImfRgbaFile.h"
#include "OpenEXR/ImfStdIO.h"
THIRD_PARTY_INCLUDES_END
#endif // WITH_UNREALEXR

#include "MoviePipelineEXROutput.generated.h"

class UMoviePipelineColorSetting;
class FEXRImageWriteTask;

namespace UE
{
	namespace MoviePipeline
	{
		/** CVar indicating if the EXR file writer is allowed to inflate layers of a lower resolution */
		extern TAutoConsoleVariable<bool> CVarMoviePipelinePadLayersForMultiPartEXR;

		/** Collection of color space metadata for EXR. */
		struct FEXRColorSpaceMetadata
		{
			FString SourceName;
			FString DestinationName;
			TArray<FVector2d> Chromaticities;
		};

		/** Update the image write task with color space metadata from the OpenColorIO transform. */
		void UpdateColorSpaceMetadata(const FOpenColorIOColorConversionSettings& InConversionSettings, FEXRImageWriteTask& InOutImageTask);

		/** Update the image write task with color space metadata from the render capture source mode. */
		void UpdateColorSpaceMetadata(ESceneCaptureSource InSceneCaptureSource, FEXRImageWriteTask& InOutImageTask);
	}
}

// Exr compression format options. Exactly matches the exr library Imf::Compression enum.
UENUM(BlueprintType)
enum class EEXRCompressionFormat : uint8
{
	/** No compression is applied. */
	None = 0,
	/** This compression method is fast, and works well for images with large flat areas but yields worse results for grainy images. Lossless. */
	RLE = 1,
	/** This compression method is similar to ZIP but compresses only one image row at a time. Lossless. */
	ZIPS = 2 UMETA(DisplayName = "ZIP (1 scanline)"),
	/** Good compression quality for images with low amounts of noise. This compression method operates in in blocks of 16 scan lines. Lossless. */
	ZIP = 3  UMETA(DisplayName = "ZIP (16 scanlines)"),
	/** Good compression quality for grainy images. Lossless.*/
	PIZ = 4,
	/** This format only stores 24 bits of the 32 bit data and has subsequently a significant loss of precision. This method is only applied when saving in FLOAT color depth. HALF and UINT remain unchanged. Lossy. */
	PXR24 = 5,
	/** This compression method only applies to images stored in HALF color depth. Blocks of 4×4 pixels are stored with using only 14 byte each (instead of the 32 byte they would normally need). Each block is compressed to the exact same size. Different images with the same dimensions require the same storage space regardless of image content. Lossy. */
	B44 = 6,
	/** A modified version of B44. If all pixels in a 4*4 block have the same color it will use only 3 instead of 14 byte. */
	B44A = 7,
	/** Lossy DCT-based compression for RGB channels. Alpha and other channels are uncompressed. More efficient than DWAB for partial buffer access on read in 3rd party tools. */
	DWAA = 8,
	/** Similar to DWAA but goes in blocks of 256 scanlines instead of 32. More efficient disk space and faster to decode than DWAA. */
	DWAB = 9,
	/** High-Throughput JPEG2000 (HTJ2K), 256 scanlines */
	HTJ2K256 = 10,
	/** High-Throughput JPEG2000 (HTJ2K), 32 scanlines */
	HTJ2K32 = 11,

	Max UMETA(Hidden)
};

#if WITH_UNREALEXR
class FEXRImageWriteTask : public IImageWriteTaskBase
{
public:

	/** The filename to write to */
	FString Filename;

	/** True if this task is allowed to overwrite an existing file, false otherwise. */
	bool bOverwriteFile;
	
	/**
	 * Compression method used for the resulting EXR files. Applies to all layers, but per-layer compression can be specified in
	 * `CompressionByLayer` if needed.
	 */
	EEXRCompressionFormat Compression;

	/**
	 * Controls the compression per layer. Only used if `bMultiPart` is true. Takes precedence over `Compression`, but is ignored if this array is
	 * either empty, or its size does not match the number of layers given in `Layers`.
	 */
	TArray<EEXRCompressionFormat> CompressionByLayer;

	/** DWA compression quality level used for DWAA/DWAB compression. */
	int32 CompressionLevel;

	/** A function to invoke on the game thread when the task has completed */
	TFunction<void(bool)> OnCompleted;
	
	/** Width/Height of the image data. All samples should match this. */
	int32 Width;

	int32 Height;

	/** A set of key/value pairs to write into the exr file as metadata. */
	TMap<FString, FStringFormatArg> FileMetadata;

	/** The image data to write. Supports multiple layers of different bitdepths. */
	TArray<TUniquePtr<FImagePixelData>> Layers;

	/** Per-layer array of preprocessors to apply serially to the pixel data when this task is executed. */
	TSortedMap<int32, TArray<FPixelPreProcessor>> PixelPreprocessors;

	/** Optional. A mapping between the FImagePixelData and a name. The standard is that the default layer is nameless (at which point it would be omitted) and other layers are prefixed. */
	TMap<FImagePixelData*, FString> LayerNames;

	/** Overscan info used to create apropriate dataWindow for EXR output. Goes from 0.0 to 1.0. */
	float OverscanPercentage;

	/** Crop rectangle used to set the displayWindow for EXR output. Used to crop out overscan */
	FIntRect CropRectangle;
	
	/** Color space chromaticity metadata. */
	TArray<FVector2d> ColorSpaceChromaticities;

	/** Indicates if a multi-part file should be used, which allows each channel to have its own header and properties */
	bool bMultipart;
	
	/** Indicates that any layers that do not match the base resolution of the EXR file should be inflated with empty pixels to that resolution */
	bool bPadToDataWindowSize;
	
	FEXRImageWriteTask()
		: bOverwriteFile(true)
		, Compression(EEXRCompressionFormat::PIZ)
		, CompressionLevel(45)
		, OverscanPercentage(0.0f)
		, CropRectangle(FIntRect())
		, bMultipart(true)
		, bPadToDataWindowSize(true)
	{}

public:

	virtual bool RunTask() override final;
	virtual void OnAbandoned() override final;

private:
	/** EXR file parts and their metadata (scanline count, bytes written) */
	struct FEXRPart
	{
		Imf::Header Header;
		Imf::FrameBuffer FrameBuffer;
		int32 Scanlines = 0;
		int64 BytesWritten = 0;
	};

	/** Generates EXR file parts (header, framebuffer, etc) for the specified layers at the specified resolution */
	bool GenerateFilePartsForResolution(const FIntPoint& Resolution, const TArray<FImagePixelData*>& InLayers, const EEXRCompressionFormat CompressionType, const FString& InPartName, FEXRPart& OutPart, TArray<TUniquePtr<FImagePixelData>>& OutQuantizedLayers);
	
	/**
	 * Run the task, attempting to write out the raw data using the currently specified parameters
	 *
	 * @return true on success, false on any failure
	 */
	bool WriteToDisk();

	/**
	 * Ensures that the desired output filename is writable, deleting an existing file if bOverwriteFile is true
	 *
	 * @return True if the file is writable and the task can proceed, false otherwise
	 */
	bool EnsureWritableFile();

	/**
	* Adds arbitrary key/value pair metadata to the header of the file.
	*/
	void AddFileMetadata(Imf::Header& InHeader);

	/**
	 * Run over all the processors for the pixel data
	 */
	void PreProcess();

	template <Imf::PixelType OutputFormat>
	int64 CompressRaw(Imf::Header& InHeader, Imf::FrameBuffer& InFrameBuffer, FImagePixelData* InLayer);
};
#endif // WITH_UNREALEXR

UCLASS(MinimalAPI)
class UMoviePipelineImageSequenceOutput_EXR : public UMoviePipelineImageSequenceOutputBase
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "ImgSequenceEXRSettingDisplayName", ".exr Sequence [16bit]"); }
#endif
public:
	UMoviePipelineImageSequenceOutput_EXR()
	{
		OutputFormat = EImageFormat::EXR;
		Compression = EEXRCompressionFormat::PIZ;
		bMultilayer = true;
		bMultipart = false;
	}

#if WITH_UNREALEXR
	MOVIERENDERPIPELINERENDERPASSES_API virtual void OnReceiveImageDataImpl(FMoviePipelineMergerOutputFrame* InMergedOutputFrame) override;
#endif // WITH_UNREALEXR
	virtual bool IsAlphaAllowed() const override { return true; }
	virtual bool ShouldCropOverscanImpl() const override { return false; }

	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:
	/**
	* Which compression method should the resulting EXR file be compressed with
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "EXR")
	EEXRCompressionFormat Compression;

	/**
	* Should we write all render passes to the same exr file? Not all software supports multi-layer exr files.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "EXR")
	bool bMultilayer;

	/**
	 * Indicates whether the exr file should be written as a multi-part exr file, which supports having different image types and resolutions for each layer
	 * Multi-part exr files are a feature of EXR 2.0 and may not be supported by all software. If the console variable 'MoviePipeline.PadLayersForMultiPartEXR' is enabled,
	 * then all parts of the multi-part EXR will be padded to match the data window of the largest layer, as some software does not support different data window sizes.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "EXR")
	bool bMultipart;
};
