// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineRenderPass.h"
#include "MovieRenderPipelineDataTypes.h"
#include "SceneTypes.h"
#include "SceneView.h"
#include "MoviePipelineSurfaceReader.h"
#include "UObject/GCObject.h"
#include "Async/AsyncWork.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/ThreadSafeBool.h"
#include "Templates/Function.h"
#include "Stats/Stats.h"
#include "CanvasTypes.h"
#include "MovieRenderPipelineCoreModule.h"
#include "OpenColorIODisplayExtension.h"

#include "MoviePipelineImagePassBase.generated.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

class UTextureRenderTarget2D;
struct FImageOverlappedAccumulator;
class FSceneViewFamily;
class FSceneView;
struct FAccumulatorPool;

namespace UE
{
	namespace MoviePipeline
	{
		struct FImagePassCameraViewData
		{
			FImagePassCameraViewData()
				: ViewActor(nullptr)
				, bUseCustomProjectionMatrix(false)
			{}
			
			FMinimalViewInfo ViewInfo;
			TMap<FString, FString> FileMetadata;
			AActor* ViewActor;

			bool bUseCustomProjectionMatrix;
			FMatrix CustomProjectionMatrix;
		};
	}
}

class FMoviePipelineBackgroundAccumulateTask
{
public:
	FGraphEventRef LastCompletionEvent;

public:
	FGraphEventRef Execute(TUniqueFunction<void()> InFunctor)
	{
		if (LastCompletionEvent)
		{
			LastCompletionEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(InFunctor), GetStatId(), LastCompletionEvent);
		}
		else
		{
			LastCompletionEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(InFunctor), GetStatId());
		}
		return LastCompletionEvent;
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMoviePipelineBackgroundAccumulateTask, STATGROUP_ThreadPoolAsyncTasks);
	}
};

namespace MoviePipeline
{
	/** Describes the data required to clear the letterbox pixels outside the active pixel area */
	struct FLetterboxData
	{
		/** Active pixel area of the full frame within channel plane dimensions. i.e. Region not letterboxed */
		FIntRect FrameActiveArea;

		/** True if letterboxing is active and drawing the border was deferred */
		bool bDrawLetterboxBorder;

		/** Left sample pixels already cleared that should therefore not be sampled */
		int32 LeftSamplePixelsClearedBeforeAccumulation;

		/** Right sample pixels already cleared that should therefore not be sampled */
		int32 RightSamplePixelsClearedBeforeAccumulation;

		/** Top sample pixels already cleared that should therefore not be sampled */
		int32 TopSamplePixelsClearedBeforeAccumulation;

		/** Bottom sample pixels already cleared that should therefore not be sampled */
		int32 BottomSamplePixelsClearedBeforeAccumulation;

		FLetterboxData() :
			  bDrawLetterboxBorder(false)
			, LeftSamplePixelsClearedBeforeAccumulation(0)
			, RightSamplePixelsClearedBeforeAccumulation(0)
			, TopSamplePixelsClearedBeforeAccumulation(0)
			, BottomSamplePixelsClearedBeforeAccumulation(0)
		{};
	};

	struct FImageSampleAccumulationArgs
	{
	public:
		TWeakPtr<FImageOverlappedAccumulator, ESPMode::ThreadSafe> ImageAccumulator;
		TWeakPtr<IMoviePipelineOutputMerger, ESPMode::ThreadSafe> OutputMerger;
		bool bAccumulateAlpha;

		/** Contains the data needed for clearing the letterbox area */
		FLetterboxData LetterboxData;
	};

	void MOVIERENDERPIPELINERENDERPASSES_API AccumulateSample_TaskThread(TUniquePtr<FImagePixelData>&& InPixelData, const MoviePipeline::FImageSampleAccumulationArgs& InParams);
}


UCLASS(MinimalAPI, BlueprintType, Abstract)
class UMoviePipelineImagePassBase : public UMoviePipelineRenderPass
{
	GENERATED_BODY()

public:
	UMoviePipelineImagePassBase()
		: UMoviePipelineRenderPass()
		, bAllowCameraAspectRatio(true)
	{
		PassIdentifier = FMoviePipelinePassIdentifier("ImagePassBase");
	}

	/* Dummy interface to allow classes with overriden functiosn to pass their own data around. */
	struct IViewCalcPayload {};
protected:

	// UMoviePipelineRenderPass API
	UE_API virtual void OnFrameStartImpl() override;
	UE_API virtual void GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses) override;
	UE_API virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;
	UE_API virtual void RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState) override;
	UE_API virtual void WaitUntilTasksComplete() override;
	UE_API virtual void TeardownImpl() override;
	// ~UMovieRenderPassAPI

	// FGCObject Interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	// ~FGCObject Interface

	UE_API FVector4 CalculatePrinciplePointOffsetForTiling(const FMoviePipelineRenderPassMetrics& InSampleState) const;
	UE_API virtual void ModifyProjectionMatrixForTiling(const FMoviePipelineRenderPassMetrics& InSampleState, const bool bInOrthographic, FMatrix& InOutProjectionMatrix, float& OutDoFSensorScale) const;

	UE_API virtual FMatrix CalcCubeFaceTransform(ECubeFace Face) const final;

protected:
	UE_API virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const;
	UE_API virtual TSharedPtr<FSceneViewFamilyContext> CalculateViewFamily(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload = nullptr);
	UE_API virtual void BlendPostProcessSettings(FSceneView* InView, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload = nullptr);
	UE_API virtual void SetupViewForViewModeOverride(FSceneView* View);
	virtual void MoviePipelineRenderShowFlagOverride(FEngineShowFlags& OutShowFlag) {}
	virtual bool IsScreenPercentageSupported() const { return true; }	
	virtual bool IsAntiAliasingSupported() const { return true; }
	virtual int32 GetOutputFileSortingOrder() const { return -1; }
	virtual FSceneViewStateInterface* GetSceneViewStateInterface(IViewCalcPayload* OptPayload = nullptr) { return ViewState.GetReference(); }
	virtual FSceneViewStateInterface* GetExposureSceneViewStateInterface(IViewCalcPayload* OptPayload = nullptr, int32 CubeFaceIndex = INDEX_NONE) { return nullptr; }
	virtual void AddViewExtensions(FSceneViewFamilyContext& InContext, FMoviePipelineRenderPassMetrics& InOutSampleState) { }
	virtual bool IsAutoExposureAllowed(const FMoviePipelineRenderPassMetrics& InSampleState) const { return true; }
	UE_API virtual FSceneView* GetSceneViewForSampleState(FSceneViewFamily* ViewFamily, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload = nullptr);
	UE_API virtual UE::MoviePipeline::FImagePassCameraViewData GetCameraInfo(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload = nullptr) const;

	UE_API virtual TWeakObjectPtr<UTextureRenderTarget2D> GetOrCreateViewRenderTarget(const FIntPoint& InSize, IViewCalcPayload* OptPayload = nullptr);
	UE_API virtual TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> GetOrCreateSurfaceQueue(const FIntPoint& InSize, IViewCalcPayload* OptPayload = nullptr);

	UE_API virtual TWeakObjectPtr<UTextureRenderTarget2D> CreateViewRenderTargetImpl(const FIntPoint& InSize, IViewCalcPayload* OptPayload = nullptr) const;
	UE_API virtual TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> CreateSurfaceQueueImpl(const FIntPoint& InSize, IViewCalcPayload* OptPayload = nullptr) const;

	UE_DEPRECATED(5.1, "GetViewRenderTarget is deprecated, please use GetOrCreateViewRenderTarget")
	virtual UTextureRenderTarget2D* GetViewRenderTarget(IViewCalcPayload* OptPayload = nullptr) const { return nullptr; }

public:
	

protected:
	/** A temporary render target that we render the view to. */
	TMap<FIntPoint, TWeakObjectPtr<UTextureRenderTarget2D>> TileRenderTargets;

	/** The history for the view */
	FSceneViewStateReference ViewState;

	/** A queue of surfaces that the render targets can be copied to. If no surface is available the game thread should hold off on submitting more samples. */
	TMap<FIntPoint, TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe>> SurfaceQueues;

	/** Some render passes may ignore the aspect ratio of the camera. */
	bool bAllowCameraAspectRatio;

	FMoviePipelinePassIdentifier PassIdentifier;

	/** Accessed by the Render Thread when starting up a new task. */
	FGraphEventArray OutstandingTasks;
};

struct FAccumulatorPool : public TSharedFromThis<FAccumulatorPool>
{
	struct FAccumulatorInstance
	{
		FAccumulatorInstance(TSharedPtr<MoviePipeline::IMoviePipelineOverlappedAccumulator, ESPMode::ThreadSafe> InAccumulator)
		{
			Accumulator = InAccumulator;
			ActiveFrameNumber = INDEX_NONE;
			bIsActive = false;
		}


		bool IsActive() const;
		void SetIsActive(const bool bInIsActive);

		TSharedPtr<MoviePipeline::IMoviePipelineOverlappedAccumulator, ESPMode::ThreadSafe> Accumulator;
		int32 ActiveFrameNumber;
		FMoviePipelinePassIdentifier ActivePassIdentifier;
		FThreadSafeBool bIsActive;
		FGraphEventRef TaskPrereq;
	};

	virtual ~FAccumulatorPool() = default;
	
	TArray<TSharedPtr<FAccumulatorInstance, ESPMode::ThreadSafe>> Accumulators;
	FCriticalSection CriticalSection;


	UE_API TSharedPtr<FAccumulatorPool::FAccumulatorInstance, ESPMode::ThreadSafe> BlockAndGetAccumulator_GameThread(int32 InFrameNumber, const FMoviePipelinePassIdentifier& InPassIdentifier);

protected:
	virtual TSharedPtr<FAccumulatorInstance, ESPMode::ThreadSafe> CreateNewAccumulatorInstance() { return nullptr; }
	virtual FName GetPoolName() const { return NAME_None; }
};

template<typename AccumulatorType>
struct TAccumulatorPool : FAccumulatorPool
{
	TAccumulatorPool(int32 InNumAccumulators)
		: FAccumulatorPool()
	{
		for (int32 Index = 0; Index < InNumAccumulators; Index++)
		{
			// Create a new instance of the accumulator
			TSharedPtr<MoviePipeline::IMoviePipelineOverlappedAccumulator, ESPMode::ThreadSafe> Accumulator = MakeShared<AccumulatorType, ESPMode::ThreadSafe>();
			Accumulators.Add(MakeShared<FAccumulatorInstance, ESPMode::ThreadSafe>(Accumulator));
		}

	}

protected:
	virtual TSharedPtr<FAccumulatorInstance, ESPMode::ThreadSafe> CreateNewAccumulatorInstance() override
	{
		TSharedPtr<MoviePipeline::IMoviePipelineOverlappedAccumulator, ESPMode::ThreadSafe> Accumulator = MakeShared<AccumulatorType, ESPMode::ThreadSafe>();
		return MakeShared<FAccumulatorInstance, ESPMode::ThreadSafe>(Accumulator);
	}

	virtual FName GetPoolName() const override
	{
		return AccumulatorType::GetName();
	}
};


DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_WaitForAvailableAccumulator"), STAT_MoviePipeline_WaitForAvailableAccumulator, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_WaitForAvailableSurface"), STAT_MoviePipeline_WaitForAvailableSurface, STATGROUP_MoviePipeline);

#undef UE_API
