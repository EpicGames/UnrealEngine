// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineOutputBase.h"
#include "Async/Future.h"
#include "MoviePipelineImageSequenceOutput.generated.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

// Forward Declare
struct FImagePixelData;
class IImageWriteQueue;
class FImageWriteTask;

namespace UE
{
	namespace MoviePipeline
	{
		struct FAsyncImageQuantization
		{
			/** Constructor.*/
			FAsyncImageQuantization(FImageWriteTask* InWriteTask, const bool bInConvertToSRGB);

			/** Process/quantize pixel data */
			void operator()(FImagePixelData* PixelData);

			/** Parent image write task. */
			FImageWriteTask* ParentWriteTask;

			/** True if the quantization should also apply an sRGB encoding. */
			bool bConvertToSRGB;
		};
	}
}

UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMoviePipelineImageSequenceOutputBase : public UMoviePipelineOutputBase
{
	GENERATED_BODY()
public:
	UE_API UMoviePipelineImageSequenceOutputBase();

	UE_API virtual void OnReceiveImageDataImpl(FMoviePipelineMergerOutputFrame* InMergedOutputFrame) override;

protected:
	// UMovieRenderPipelineOutputContainer interface
	UE_API virtual void BeginFinalizeImpl() override;
	UE_API virtual bool HasFinishedProcessingImpl() override;
	UE_API virtual void OnShotFinishedImpl(const UMoviePipelineExecutorShot* InShot, const bool bFlushToDisk) override;
	// ~UMovieRenderPipelineOutputContainer interface

	UE_API virtual void GetFormatArguments(FMoviePipelineFormatArgs& InOutFormatArgs) const override;
	virtual bool IsAlphaAllowed() const { return false; }
protected:
	/** The format of the image to write out */
	EImageFormat OutputFormat;

	/** A pointer to the image write queue used for asynchronously writing images */
	IImageWriteQueue* ImageWriteQueue;
private:

	/** A fence to keep track of when the Image Write queue has fully flushed. */
	TFuture<void> FinalizeFence;
};

UCLASS(MinimalAPI)
class UMoviePipelineImageSequenceOutput_BMP : public UMoviePipelineImageSequenceOutputBase
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "ImgSequenceBMPSettingDisplayName", ".bmp Sequence [8bit]"); }
#endif
public:
	UMoviePipelineImageSequenceOutput_BMP()
	{
		OutputFormat = EImageFormat::BMP;
	}

	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
};

UCLASS(MinimalAPI)
class UMoviePipelineImageSequenceOutput_PNG : public UMoviePipelineImageSequenceOutputBase
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "ImgSequencePNGSettingDisplayName", ".png Sequence [8bit]"); }
#endif
	virtual bool IsAlphaAllowed() const override { return bWriteAlpha; }

public:
	UMoviePipelineImageSequenceOutput_PNG()
	{
		OutputFormat = EImageFormat::PNG;
		bWriteAlpha = true;
	}

	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "PNG")
	bool bWriteAlpha;
};

UCLASS(MinimalAPI)
class UMoviePipelineImageSequenceOutput_JPG : public UMoviePipelineImageSequenceOutputBase
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "ImgSequenceJPGSettingDisplayName", ".jpg Sequence [8bit]"); }
#endif
public:
	UMoviePipelineImageSequenceOutput_JPG()
	{
		OutputFormat = EImageFormat::JPEG;
	}

	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
};

#undef UE_API
