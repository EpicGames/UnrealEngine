// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineOutputBase.h"
#include "Sound/SampleBufferIO.h"
#include "MoviePipelineWaveOutput.generated.h"

#define UE_API MOVIERENDERPIPELINERENDERPASSES_API

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineWaveOutput : public UMoviePipelineOutputBase
{
	GENERATED_BODY()

	UMoviePipelineWaveOutput()
	{
	}

public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "AudioSettingDisplayName", ".wav Audio"); }
#endif
	UE_API virtual void OnShotFinishedImpl(const UMoviePipelineExecutorShot* InShot, const bool bFlushToDisk) override;

protected:
	UE_API virtual void BeginFinalizeImpl() override;
	UE_API virtual bool HasFinishedProcessingImpl() override;
	UE_API virtual void ValidateStateImpl() override;
	UE_API virtual void BuildNewProcessCommandLineArgsImpl(TArray<FString>& InOutUnrealURLParams, TArray<FString>& InOutCommandLineArgs, TArray<FString>& InOutDeviceProfileCvars, TArray<FString>& InOutExecCmds) const override;
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:
	/* File name format string override. If specified it will override the FileNameFormat from the Output setting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output")
	FString FileNameFormatOverride;

private:
	/** Kept alive during finalization because the writer writes async to disk but doesn't expect to fall out of scope */
	TArray<TUniquePtr<Audio::FSoundWavePCMWriter>> ActiveWriters;

	/** Keep track of segments that we've already written to disk to avoid re-writing them (and generating new Output Futures) */
	TSet<FGuid> AlreadyWrittenSegments;
};

#undef UE_API
