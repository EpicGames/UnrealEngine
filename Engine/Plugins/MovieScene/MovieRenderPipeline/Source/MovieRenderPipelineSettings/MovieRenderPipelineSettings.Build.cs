// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MovieRenderPipelineSettings : ModuleRules
{
	public MovieRenderPipelineSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"NonRealtimeAudioRenderer",
				"AudioMixer",
				"UMG",
				"Slate",
				"RenderCore",
				"RHI",
				"ApplicationCore"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"SlateCore",
                "MovieScene",
                "MovieSceneTracks",
				"LevelSequence",
				"Engine",
				"MovieRenderPipelineCore",
				"OpenColorIO",
			}
		);
		
		if (Target.bBuildEditor)
		{
			PublicDependencyModuleNames.Add("ConsoleVariablesEditor");
		}
	}
}
