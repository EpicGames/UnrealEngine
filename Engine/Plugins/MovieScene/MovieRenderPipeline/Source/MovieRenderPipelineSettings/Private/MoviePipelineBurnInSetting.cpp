// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineBurnInSetting.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/WidgetRenderer.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineBurnInWidget.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineCameraSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipeline.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MoviePipelineOutputBuilder.h"
#include "ImagePixelData.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineQueue.h"
#include "TextureResource.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineBurnInSetting)

FString UMoviePipelineBurnInSetting::DefaultBurnInWidgetAsset = TEXT("/MovieRenderPipeline/Blueprints/DefaultBurnIn.DefaultBurnIn_C");

void UMoviePipelineBurnInSetting::GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses)
{
	if (BurnInClass.IsValid() && WidgetRenderer != nullptr)
	{
		UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
		UMoviePipelineCameraSetting* CameraSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineCameraSetting>(CurrentShot);
		int32 NumCameras = CameraSettings->bRenderAllCameras ? CurrentShot->SidecarCameras.Num() : 1;

		for (int32 CameraIndex = 0; CameraIndex < NumCameras; CameraIndex++)
		{
			FMoviePipelinePassIdentifier PassIdentifierForCurrentCamera;
			PassIdentifierForCurrentCamera.Name = TEXT("BurnInOverlay");

			// If we're not rendering all cameras, we need to pass -1 so we pick up the real camera name.
			int32 LocalCameraIndex = CameraSettings->bRenderAllCameras ? CameraIndex : -1;
			PassIdentifierForCurrentCamera.CameraName = CurrentShot->GetCameraName(LocalCameraIndex);

			ExpectedRenderPasses.Add(PassIdentifierForCurrentCamera);
		}
	}
}

void UMoviePipelineBurnInSetting::RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState)
{
	if (InSampleState.bDiscardResult)
	{
		return;
	}

	if (!WidgetRenderer)
	{
		return;
	}

	const bool bFirstTile = InSampleState.GetTileIndex() == 0;
	const bool bFirstSpatial = InSampleState.SpatialSampleIndex == (InSampleState.SpatialSampleCount - 1);
	const bool bFirstTemporal = InSampleState.TemporalSampleIndex == (InSampleState.TemporalSampleCount - 1);

	if (bFirstTile && bFirstSpatial && bFirstTemporal)
	{
		UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
		UMoviePipelineCameraSetting* CameraSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineCameraSetting>(CurrentShot);
		int32 NumCameras = CameraSettings->bRenderAllCameras ? CurrentShot->SidecarCameras.Num() : 1;
		for (int32 CameraIndex = 0; CameraIndex < NumCameras; CameraIndex++)
		{
			// If we're not rendering all cameras, we need to pass -1 so we pick up the real camera name.
			int32 LocalCameraIndex = CameraSettings->bRenderAllCameras ? CameraIndex : -1;

			FMoviePipelinePassIdentifier PassIdentifierForCurrentCamera;
			PassIdentifierForCurrentCamera.Name = TEXT("BurnInOverlay");
			PassIdentifierForCurrentCamera.CameraName = CurrentShot->GetCameraName(LocalCameraIndex);

			UMoviePipelineBurnInWidget* CurrentWidget = BurnInWidgetInstances[FMath::Clamp(LocalCameraIndex, 0, LocalCameraIndex)];
			// Put the widget in our window
			VirtualWindow->SetContent(CurrentWidget->TakeWidget());

			// Update the Widget with the latest frame information
			CurrentWidget->OnOutputFrameStarted(GetPipeline());

			// Draw the widget to the render target. This leaves the texture in SRV state so no transition is needed.
			// Note: If this draw call is updated, be sure to also update the one below.
			WidgetRenderer->DrawWindow(RenderTarget, VirtualWindow->GetHittestGrid(), VirtualWindow.ToSharedRef(),
				1.f, OutputResolution, InSampleState.OutputState.TimeData.FrameDeltaTime);

			// If this is the first frame being rendered, then re-render. This will ensure that all layout computations are correct (for example, large
			// text blocks may not have their layouts computed by the time the first render above occurs, especially if the text is driven by the
			// UMG blueprint).
			if (InSampleState.OutputState.ShotOutputFrameNumber == 0)
			{
				WidgetRenderer->DrawWindow(RenderTarget, VirtualWindow->GetHittestGrid(), VirtualWindow.ToSharedRef(),
					1.f, OutputResolution, InSampleState.OutputState.TimeData.FrameDeltaTime);
			}

			FRenderTarget* BackbufferRenderTarget = RenderTarget->GameThread_GetRenderTargetResource();
			TSharedPtr<FMoviePipelineOutputMerger, ESPMode::ThreadSafe> OutputBuilder = GetPipeline()->OutputBuilder;

			ENQUEUE_RENDER_COMMAND(BurnInRenderTargetResolveCommand)(
				[InSampleState, PassIdentifierForCurrentCamera, bComposite = bCompositeOntoFinalImage, BackbufferRenderTarget, OutputBuilder](FRHICommandListImmediate& RHICmdList)
				{
					FIntRect SourceRect = FIntRect(0, 0, BackbufferRenderTarget->GetSizeXY().X, BackbufferRenderTarget->GetSizeXY().Y);

					// Read the data back to the CPU
					TArray<FColor> RawPixels;
					RawPixels.SetNum(SourceRect.Width() * SourceRect.Height());

					FReadSurfaceDataFlags ReadDataFlags(ERangeCompressionMode::RCM_MinMax);
					ReadDataFlags.SetLinearToGamma(false);

					RHICmdList.ReadSurfaceData(BackbufferRenderTarget->GetRenderTargetTexture(), SourceRect, RawPixels, ReadDataFlags);

					TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> FrameData = MakeShared<FImagePixelDataPayload, ESPMode::ThreadSafe>();
					FrameData->PassIdentifier = PassIdentifierForCurrentCamera;
					FrameData->SampleState = InSampleState;
					FrameData->bRequireTransparentOutput = true;
					FrameData->SortingOrder = 4;
					FrameData->bCompositeToFinalImage = bComposite;

					TUniquePtr<FImagePixelData> PixelData = MakeUnique<TImagePixelData<FColor>>(SourceRect.Size(), TArray64<FColor>(MoveTemp(RawPixels)), FrameData);

					OutputBuilder->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(PixelData));
				});
		}
	}
}

void UMoviePipelineBurnInSetting::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	if (!BurnInClass.IsValid())
	{
		return;
	}

	// TODO: Add multi-camera support? For now, simply use the default player controller camera overscan
	const float CameraOverscan = GetPipeline()->GetCachedCameraOverscan(INDEX_NONE);
	const FIntRect CropRect = UMoviePipelineBlueprintLibrary::GetOverscanCropRectangle(GetPipeline()->GetPipelinePrimaryConfig(), GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()], CameraOverscan);
	
	// Composited elements should be sized to the original frustum size, as the final image is either cropped to that size, or the composite will be offset
	// to match the original frustum
	OutputResolution = CropRect.Size();
	int32 MaxResolution = GetMax2DTextureDimension();
	if (OutputResolution.X > MaxResolution || OutputResolution.Y > MaxResolution)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Resolution %dx%d exceeds maximum allowed by GPU. Burn-ins do not support high-resolution tiling and thus can't exceed %dx%d.", OutputResolution.X, OutputResolution.Y, MaxResolution, MaxResolution);
		GetPipeline()->Shutdown(true);
		return;
	}

	UClass* BurnIn = BurnInClass.TryLoadClass<UMoviePipelineBurnInWidget>();
	if (!ensureAlwaysMsgf(BurnIn, TEXT("Failed to load burnin class: '%s'."), *BurnInClass.GetAssetPathString()))
	{
		return;
	}

	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	UMoviePipelineCameraSetting* CameraSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineCameraSetting>(CurrentShot);
	int32 NumCameras = CameraSettings->bRenderAllCameras ? CurrentShot->SidecarCameras.Num() : 1;
	for (int32 CameraIndex = 0; CameraIndex < NumCameras; CameraIndex++)
	{
		BurnInWidgetInstances.Add(CreateWidget<UMoviePipelineBurnInWidget>(GetWorld(), BurnIn));
	}

	VirtualWindow = SNew(SVirtualWindow).Size(FVector2D(OutputResolution.X, OutputResolution.Y));

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().RegisterVirtualWindow(VirtualWindow.ToSharedRef());
	}

	RenderTarget = NewObject<UTextureRenderTarget2D>();
	RenderTarget->ClearColor = FLinearColor::Transparent;

	bool bInForceLinearGamma = false;
	RenderTarget->InitCustomFormat(OutputResolution.X, OutputResolution.Y, EPixelFormat::PF_B8G8R8A8, bInForceLinearGamma);

	bool bApplyGammaCorrection = false;
	WidgetRenderer = MakeShared<FWidgetRenderer>(bApplyGammaCorrection);
}

void UMoviePipelineBurnInSetting::TeardownImpl() 
{
	FlushRenderingCommands();

	if (FSlateApplication::IsInitialized() && VirtualWindow.IsValid())
	{
		FSlateApplication::Get().UnregisterVirtualWindow(VirtualWindow.ToSharedRef());
	}
	
	VirtualWindow = nullptr;
	
	WidgetRenderer = nullptr;
	RenderTarget = nullptr;
	BurnInWidgetInstances.Reset();
}
