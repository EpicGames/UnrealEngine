// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineConsoleVariableSetting.h"

#include "MoviePipelineQueue.h"
#include "MoviePipelineTelemetry.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MovieScene.h"
#include "LevelSequence.h"
#include "MovieSceneCommonHelpers.h"
#include "HAL/IConsoleManager.h"
#include "Engine/World.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/DefaultValueHelper.h"
#include "Sections/MovieSceneConsoleVariableTrackInterface.h"
#include "Sections/MovieSceneCVarSection.h"
#include "Tracks/MovieSceneCVarTrack.h"
#include "Tracks/MovieSceneSubTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineConsoleVariableSetting)

#define LOCTEXT_NAMESPACE "MoviePipelineConsoleVariableSetting"

namespace UE
{
	namespace MoviePipeline
	{
		static void SetValue(IConsoleVariable* InCVar, float InValue)
		{
			check(InCVar);

			// When Set is called on a cvar the value is turned into a string. With very large
			// floats this is turned into scientific notation. If the cvar is later retrieved as
			// an integer, the scientific notation doesn't parse into integer correctly. We'll
			// cast to integer first (to avoid scientific notation) if we know the cvar is an integer.
			if (InCVar->IsVariableInt())
			{
				InCVar->SetWithCurrentPriority(static_cast<int32>(InValue), NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
			}
			else if (InCVar->IsVariableBool())
			{
				InCVar->SetWithCurrentPriority(InValue != 0.f ? true : false, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
			}
			else
			{
				InCVar->SetWithCurrentPriority(InValue, NAME_None, ECVF_SetByConsole, ECVF_SetByScalability);
			}
		}

		/**
		 * Determine if the given UMovieScene contains any CVar tracks that are not muted, and also have an
		 * active section with CVars that are set. Sub-sequences will be searched for CVar tracks as well.
		 */
		static bool IsCVarTrackPresent(const UMovieScene* InMovieScene)
		{
			if (!InMovieScene)
			{
				return false;
			}

			for (UMovieSceneTrack* Track : InMovieScene->GetTracks())
			{
				// Process CVar tracks. Return immediately if any of the CVar tracks contain CVars that are set.
				// If this is the case, sub tracks don't need to be searched.
				if (Track->IsA<UMovieSceneCVarTrack>())
				{
					const UMovieSceneCVarTrack* CVarTrack = Cast<UMovieSceneCVarTrack>(Track);
					for (const UMovieSceneSection* Section : CVarTrack->GetAllSections())
					{
						const UMovieSceneCVarSection* CVarSection = Cast<UMovieSceneCVarSection>(Section);
						if (!CVarSection || !MovieSceneHelpers::IsSectionKeyable(CVarSection))
						{
							continue;
						}
						
						// Does this CVar track have anything in it?
						if (!CVarSection->ConsoleVariableCollections.IsEmpty() || !CVarSection->ConsoleVariables.ValuesByCVar.IsEmpty())
						{
							return true;
						}
					}
				}
				
				// Process sub tracks (which could potentially contain other sequences with CVar tracks)
				if (Track->IsA<UMovieSceneSubTrack>())
				{
					const UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track);
					for (const UMovieSceneSection* Section : SubTrack->GetAllSections())
					{
						const UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
						if (!SubSection || !MovieSceneHelpers::IsSectionKeyable(SubSection))
						{
							continue;
						}

						// Recurse into sub-sequences
						if (const UMovieSceneSequence* SubSequence = SubSection->GetSequence())
						{
							if (IsCVarTrackPresent(SubSequence->GetMovieScene()))
							{
								return true;
							}
						}
					}
				}
			}
			
			return false;
		}
	}
}

#if WITH_EDITOR

FText UMoviePipelineConsoleVariableSetting::GetFooterText(UMoviePipelineExecutorJob* InJob) const
{
	if (!InJob)
	{
		return FText();
	}
	
	const ULevelSequence* LoadedSequence = Cast<ULevelSequence>(InJob->Sequence.TryLoad());
	if (!LoadedSequence)
	{
		return FText();
	}
	
	if (!UE::MoviePipeline::IsCVarTrackPresent(LoadedSequence->MovieScene))
	{
		return FText();
	}
	
	return FText(LOCTEXT(
		"SequencerCvarWarning",
		"The current job contains a Level Sequence with a Console Variables Track, additional settings are configured in Sequencer."));
}

FString UMoviePipelineConsoleVariableSetting::ResolvePresetValue(const FString& InCVarName) const
{
	FString ResolvedValue;

	// Iterate the presets in reverse; cvars in presets at the end of the array take precedence.
	for (int32 Index = ConsoleVariablePresets.Num() - 1; Index >= 0; --Index)
	{
		const TScriptInterface<IMovieSceneConsoleVariableTrackInterface>& Preset = ConsoleVariablePresets[Index];
		if (!Preset)
		{
			continue;
		}
		
		const bool bOnlyIncludeChecked = true;
		TArray<TPair<FString, FString>> PresetCVars;
		Preset->GetConsoleVariablesForTrack(bOnlyIncludeChecked, PresetCVars);
		
		for (const TPair<FString, FString>& CVar : PresetCVars)
		{
			if (CVar.Key.Equals(InCVarName))
			{
				// Found a matching cvar in the preset
				return CVar.Value;
			}
		}
	}

	return FString();
}

FString UMoviePipelineConsoleVariableSetting::ResolveDisabledValue(const FMoviePipelineConsoleVariableEntry& InEntry) const
{
	// There may be multiple cvar overrides w/ the same name, so even if the provided entry is disabled, we still need
	// to look through the other entries for cvars with the same name. Iterate in reverse, since cvars at the end of the
	// array take precedence.
	for (int32 Index = CVars.Num() - 1; Index >= 0; --Index)
	{
		const FMoviePipelineConsoleVariableEntry& CVarEntry = CVars[Index];
		if (CVarEntry.Name.Equals(InEntry.Name) && CVarEntry.bIsEnabled)
		{
			return FString::SanitizeFloat(CVarEntry.Value);
		}
	}

	// If no override value was found, look for a value from the presets
	const FString PresetValue = ResolvePresetValue(InEntry.Name);
	if (!PresetValue.IsEmpty())
	{
		return PresetValue;
	}

	FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();
	const TWeakPtr<FConsoleVariablesEditorCommandInfo> WeakCommandInfo = ConsoleVariablesEditorModule.FindCommandInfoByName(InEntry.Name);

	// Fall back to the startup value of the cvar
	const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = WeakCommandInfo.Pin();
	if (CommandInfo.IsValid())
	{
		return CommandInfo->StartupValueAsString;
	}

	return FString();
}

FMoviePipelineConsoleVariableEntry* UMoviePipelineConsoleVariableSetting::GetCVarAtIndex(const int32 InIndex)
{
	if (!CVars.IsValidIndex(InIndex))
	{
		return nullptr;
	}

	return &CVars[InIndex];
}

#endif // WITH_EDITOR

void UMoviePipelineConsoleVariableSetting::SetupForPipelineImpl(UMoviePipeline* InPipeline)
{
	ApplyCVarSettings(true);
}

void UMoviePipelineConsoleVariableSetting::TeardownForPipelineImpl(UMoviePipeline* InPipeline)
{
	ApplyCVarSettings(false);
}

void UMoviePipelineConsoleVariableSetting::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bSetsConsoleVariables |= CVars.ContainsByPredicate([](const FMoviePipelineConsoleVariableEntry& Entry)
	{
		return Entry.bIsEnabled;
	});
	InTelemetry->bUsesConsoleVariablePreset |= !ConsoleVariablePresets.IsEmpty();
}
	
void UMoviePipelineConsoleVariableSetting::ApplyCVarSettings(const bool bOverrideValues)
{
	if (bOverrideValues)
	{
		MergeInPresetConsoleVariables();
		PreviousConsoleVariableValues.Reset();
		PreviousConsoleVariableValues.SetNumZeroed(MergedConsoleVariables.Num());
	}

	int32 Index = 0;
	for(const FMoviePipelineConsoleVariableEntry& CVarEntry : MergedConsoleVariables)
	{
		// We don't use the shared macro here because we want to soft-warn the user instead of tripping an ensure over missing cvar values.
		const FString TrimmedCvar = CVarEntry.Name.TrimStartAndEnd();
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*TrimmedCvar); 
		if (CVar)
		{
			if (bOverrideValues)
			{
				PreviousConsoleVariableValues[Index] = CVar->GetFloat();
				UE::MoviePipeline::SetValue(CVar, CVarEntry.Value);
				UE_LOGF(LogMovieRenderPipeline, Log, "Applying CVar \"%ls\" PreviousValue: %f NewValue: %f", *CVarEntry.Name, PreviousConsoleVariableValues[Index], CVarEntry.Value);
			}
			else
			{
				UE::MoviePipeline::SetValue(CVar, PreviousConsoleVariableValues[Index]);
				UE_LOGF(LogMovieRenderPipeline, Log, "Restoring CVar \"%ls\" PreviousValue: %f NewValue: %f", *CVarEntry.Name, CVarEntry.Value, PreviousConsoleVariableValues[Index]);
			}
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to apply CVar \"%ls\" due to no cvar by that name. Ignoring.", *CVarEntry.Name);
		}

		Index++;
	}

	if (bOverrideValues)
	{
		for (const FString& Command : StartConsoleCommands)
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "Executing Console Command \"%ls\" before shot starts.", *Command);
			UKismetSystemLibrary::ExecuteConsoleCommand(GetWorld(), Command, nullptr);
		}
	}
	else
	{
		for (const FString& Command : EndConsoleCommands)
		{
			UE_LOGF(LogMovieRenderPipeline, Log, "Executing Console Command \"%ls\" after shot ends.", *Command);
			UKismetSystemLibrary::ExecuteConsoleCommand(GetWorld(), Command, nullptr);
		}
	}
}

TArray<FMoviePipelineConsoleVariableEntry> UMoviePipelineConsoleVariableSetting::GetConsoleVariables() const
{
	return CVars;
}

bool UMoviePipelineConsoleVariableSetting::RemoveConsoleVariable(const FString& Name, const bool bRemoveAllInstances)
{
	if (bRemoveAllInstances)
	{
		const int32 NumRemoved = CVars.RemoveAll([&Name](const FMoviePipelineConsoleVariableEntry& Entry)
		{
			return Entry.Name.Equals(Name);
		});

		return NumRemoved != 0;
	}

	const int32 LastMatch = CVars.FindLastByPredicate([&Name](const FMoviePipelineConsoleVariableEntry& Entry)
	{
		return Entry.Name.Equals(Name);
	});

	if (LastMatch != INDEX_NONE)
	{
		CVars.RemoveAt(LastMatch);
		return true;
	}

	return false;
}

bool UMoviePipelineConsoleVariableSetting::AddOrUpdateConsoleVariable(const FString& Name, const float Value)
{
	const int32 LastMatch = CVars.FindLastByPredicate([&Name](const FMoviePipelineConsoleVariableEntry& Entry)
	{
		return Entry.Name.Equals(Name);
	});

	if (LastMatch != INDEX_NONE)
	{
		CVars[LastMatch].Value = Value;
		return true;
	}

	CVars.Add(FMoviePipelineConsoleVariableEntry(Name, Value));

	return true;
}

bool UMoviePipelineConsoleVariableSetting::AddConsoleVariable(const FString& Name, const float Value)
{
	CVars.Add(FMoviePipelineConsoleVariableEntry(Name, Value));

	return true;
}

bool UMoviePipelineConsoleVariableSetting::UpdateConsoleVariableEnableState(const FString& Name, const bool bIsEnabled)
{
	const int32 LastMatch = CVars.FindLastByPredicate([&Name](const FMoviePipelineConsoleVariableEntry& Entry)
	{
		return Entry.Name.Equals(Name);
	});

	if (LastMatch != INDEX_NONE)
	{
		CVars[LastMatch].bIsEnabled = bIsEnabled;
		return true;
	}

	return false;
}

void UMoviePipelineConsoleVariableSetting::MergeInPresetConsoleVariables()
{
	MergedConsoleVariables.Reset();
	
	// Merge in the presets
	for (const TScriptInterface<IMovieSceneConsoleVariableTrackInterface>& Preset : ConsoleVariablePresets)
	{
		if (!Preset)
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Invalid CVar preset specified. Ignoring.");
			continue;
		}
		
		const bool bOnlyIncludeChecked = true;
		TArray<TTuple<FString, FString>> PresetCVars;
		Preset->GetConsoleVariablesForTrack(bOnlyIncludeChecked, PresetCVars);
		
		for (const TTuple<FString, FString>& CVarPair : PresetCVars)
		{
			float CVarFloatValue = 0.0f;
			bool bCvarBoolValue = false;
			
			if (FDefaultValueHelper::ParseFloat(CVarPair.Value, CVarFloatValue))
			{
				MergedConsoleVariables.Add(FMoviePipelineConsoleVariableEntry(CVarPair.Key, CVarFloatValue));
			}
			else if (FDefaultValueHelper::ParseBool(CVarPair.Value, bCvarBoolValue))
			{
				MergedConsoleVariables.Add(FMoviePipelineConsoleVariableEntry(CVarPair.Key, bCvarBoolValue ? 1.0f : 0.0f));
			}
			else
			{
				UE_LOGF(LogMovieRenderPipeline, Warning, "Failed to apply CVar \"%ls\" (from preset \"%ls\") because value could not be parsed into a float. Ignoring.",
					*CVarPair.Key, *Preset.GetObject()->GetName());
			}
		}
	}
	
	// Merge in the overrides
	for (const FMoviePipelineConsoleVariableEntry& Entry : CVars)
	{
		if (!Entry.bIsEnabled)
		{
			continue;
		}
		
		MergedConsoleVariables.Add(Entry);
	}
}

#undef LOCTEXT_NAMESPACE