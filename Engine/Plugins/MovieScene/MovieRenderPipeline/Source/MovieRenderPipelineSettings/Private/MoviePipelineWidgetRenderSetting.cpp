// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineWidgetRenderSetting.h"
#include "Engine/GameViewportClient.h"
#include "Slate/WidgetRenderer.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineBurnInWidget.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelineCameraSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineBlueprintLibrary.h"
#include "MoviePipeline.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MoviePipelineOutputBuilder.h"
#include "ImagePixelData.h"
#include "Widgets/SViewport.h"
#include "Slate/SGameLayerManager.h"
#include "MovieRenderPipelineCoreModule.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineTelemetry.h"
#include "TextureResource.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineWidgetRenderSetting)

void UMoviePipelineWidgetRenderer::GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses)
{
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	UMoviePipelineCameraSetting* CameraSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineCameraSetting>(CurrentShot);
	int32 NumCameras = CameraSettings->bRenderAllCameras ? CurrentShot->SidecarCameras.Num() : 1;

	for (int32 CameraIndex = 0; CameraIndex < NumCameras; CameraIndex++)
	{
		FMoviePipelinePassIdentifier PassIdentifierForCurrentCamera;
		PassIdentifierForCurrentCamera.Name = TEXT("ViewportUI");
		// If we're not rendering all cameras, we need to pass -1 so we pick up the real camera name.
		int32 LocalCameraIndex = CameraSettings->bRenderAllCameras ? CameraIndex : -1;
		PassIdentifierForCurrentCamera.CameraName = CurrentShot->GetCameraName(LocalCameraIndex);

		ExpectedRenderPasses.Add(PassIdentifierForCurrentCamera);
	}
}

void UMoviePipelineWidgetRenderer::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesUIRenderer = true;
}

void UMoviePipelineWidgetRenderer::RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState)
{
	if (InSampleState.bDiscardResult)
	{
		return;
	}

	const bool bFirstTile = InSampleState.GetTileIndex() == 0;
	const bool bFirstSpatial = InSampleState.SpatialSampleIndex == (InSampleState.SpatialSampleCount - 1);
	const bool bFirstTemporal = InSampleState.TemporalSampleIndex == (InSampleState.TemporalSampleCount - 1);

	if (bFirstTile && bFirstSpatial && bFirstTemporal)
	{
		UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
		UMoviePipelineCameraSetting* CameraSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineCameraSetting>(CurrentShot);
		int32 NumCameras = CameraSettings->bRenderAllCameras ? CurrentShot->SidecarCameras.Num() : 1;
		for (int32 CameraIndex = 0; CameraIndex < NumCameras; CameraIndex++)
		{
			FMoviePipelinePassIdentifier PassIdentifierForCurrentCamera;
			PassIdentifierForCurrentCamera.Name = TEXT("ViewportUI");

			// If we're not rendering all cameras, we need to pass -1 so we pick up the real camera name.
			int32 LocalCameraIndex = CameraSettings->bRenderAllCameras ? CameraIndex : -1;
			PassIdentifierForCurrentCamera.CameraName = CurrentShot->GetCameraName(LocalCameraIndex);

			// Draw the widget to the render target
			FRenderTarget* BackbufferRenderTarget = RenderTarget->GameThread_GetRenderTargetResource();

			ULocalPlayer* LocalPlayer = GetWorld()->GetFirstLocalPlayerFromController();

			// Cast the interface to a widget is a little yucky but the implementation is unlikely to change.
			TSharedPtr<SGameLayerManager> GameLayerManager = StaticCastSharedPtr<SGameLayerManager>(LocalPlayer->ViewportClient->GetGameLayerManager());

			//  This leaves the texture in SRV state so no transition is needed.
			WidgetRenderer->DrawWidget(BackbufferRenderTarget, GameLayerManager.ToSharedRef(), 1.f, FVector2D(RenderTarget->SizeX, RenderTarget->SizeY), (float)InSampleState.OutputState.TimeData.FrameDeltaTime);

			TSharedPtr<FMoviePipelineOutputMerger, ESPMode::ThreadSafe> OutputBuilder = GetPipeline()->OutputBuilder;

			ENQUEUE_RENDER_COMMAND(BurnInRenderTargetResolveCommand)(
				[InSampleState, PassIdentifierForCurrentCamera, bComposite = bCompositeOntoFinalImage, BackbufferRenderTarget, OutputBuilder](FRHICommandListImmediate& RHICmdList)
				{
					FIntRect SourceRect = FIntRect(0, 0, BackbufferRenderTarget->GetSizeXY().X, BackbufferRenderTarget->GetSizeXY().Y);

					// Read the data back to the CPU
					TArray<FColor> RawPixels;
					RawPixels.SetNum(SourceRect.Width() * SourceRect.Height());

					FReadSurfaceDataFlags ReadDataFlags(ERangeCompressionMode::RCM_MinMax);
					ReadDataFlags.SetLinearToGamma(false);

					RHICmdList.ReadSurfaceData(BackbufferRenderTarget->GetRenderTargetTexture(), SourceRect, RawPixels, ReadDataFlags);

					TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> FrameData = MakeShared<FImagePixelDataPayload, ESPMode::ThreadSafe>();
					FrameData->PassIdentifier = PassIdentifierForCurrentCamera;
					FrameData->SampleState = InSampleState;
					FrameData->bRequireTransparentOutput = true;
					FrameData->SortingOrder = 3;
					FrameData->bCompositeToFinalImage = bComposite;

					TUniquePtr<FImagePixelData> PixelData = MakeUnique<TImagePixelData<FColor>>(SourceRect.Size(), TArray64<FColor>(MoveTemp(RawPixels)), FrameData);

					OutputBuilder->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(PixelData));
				});
		}
	}
}

void UMoviePipelineWidgetRenderer::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	RenderTarget = NewObject<UTextureRenderTarget2D>();
	RenderTarget->ClearColor = FLinearColor::Transparent;

	bool bInForceLinearGamma = false;

	// TODO: Add multi-camera support? For now, simply use the default player controller camera overscan
	const float CameraOverscan = GetPipeline()->GetCachedCameraOverscan(INDEX_NONE);
	const FIntRect CropRect = UMoviePipelineBlueprintLibrary::GetOverscanCropRectangle(GetPipeline()->GetPipelinePrimaryConfig(), GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()], CameraOverscan);

	// Composited elements should be sized to the original frustum size, as the final image is either cropped to that size, or the composite will be offset
	// to match the original frustum
	FIntPoint OutputResolution = CropRect.Size();
	
	int32 MaxResolution = GetMax2DTextureDimension();
	if (OutputResolution.X > MaxResolution || OutputResolution.Y > MaxResolution)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Resolution %dx%d exceeds maximum allowed by GPU. Widget Renderer does not support high-resolution tiling and thus can't exceed %dx%d.", OutputResolution.X, OutputResolution.Y, MaxResolution, MaxResolution);
		GetPipeline()->Shutdown(true);
		return;
	}

	RenderTarget->InitCustomFormat(OutputResolution.X, OutputResolution.Y, EPixelFormat::PF_B8G8R8A8, bInForceLinearGamma);

	bool bApplyGammaCorrection = false;
	WidgetRenderer = MakeShared<FWidgetRenderer>(bApplyGammaCorrection);
}

void UMoviePipelineWidgetRenderer::TeardownImpl() 
{
	FlushRenderingCommands();

	WidgetRenderer = nullptr;
	RenderTarget = nullptr;
}
