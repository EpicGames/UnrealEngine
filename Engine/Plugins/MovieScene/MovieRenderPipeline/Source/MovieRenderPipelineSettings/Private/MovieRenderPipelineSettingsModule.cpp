// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "MoviePipelineBurnInSetting.h"
#include "UObject/ICookInfo.h"

class FMovieRenderPipelineSettingsModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
#if WITH_EDITOR
		if (IsRunningCookCommandlet())
		{
			UE::Cook::FDelegates::ModifyCook.AddLambda(
				[](UE::Cook::ICookInfo& CookInfo, TArray<UE::Cook::FPackageCookRule>& InOutPackageCookRules)
				{
					// Ensure these assets (which are referenced only by code) get packaged
					const FString* Assets[] =
					{
						&UMoviePipelineBurnInSetting::DefaultBurnInWidgetAsset
					};

					for (FString const* Asset : Assets)
					{
						InOutPackageCookRules.Add(
							UE::Cook::FPackageCookRule{
								.PackageName = FName(FSoftObjectPath(*Asset).GetLongPackageName()),
								.InstigatorName = FName("FMovieRenderPipelineSettingsModule"),
								.CookRule = UE::Cook::EPackageCookRule::AddToCook
							}
						);
					}
				}
			);
		}
#endif
	}

	virtual void ShutdownModule() override
	{

	}
};

IMPLEMENT_MODULE(FMovieRenderPipelineSettingsModule, MovieRenderPipelineSettings);
