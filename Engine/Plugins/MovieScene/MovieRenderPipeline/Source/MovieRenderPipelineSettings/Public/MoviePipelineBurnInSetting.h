// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineRenderPass.h"
#include "MovieRenderPipelineDataTypes.h"
#include "UObject/SoftObjectPath.h"
#include "MoviePipelineBurnInSetting.generated.h"

#define UE_API MOVIERENDERPIPELINESETTINGS_API

class FWidgetRenderer;
class SVirtualWindow;
class UTextureRenderTarget2D;
class UMoviePipelineBurnInWidget;

namespace MoviePipeline { struct FMoviePipelineRenderPassInitSettings; }

UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineBurnInSetting : public UMoviePipelineRenderPass
{
	GENERATED_BODY()

	UMoviePipelineBurnInSetting()
		: BurnInClass(DefaultBurnInWidgetAsset)
		, bCompositeOntoFinalImage(true)
	{
	}

protected:
	// UMoviePipelineRenderPass Interface
	UE_API virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;
	UE_API virtual void TeardownImpl() override;
	UE_API virtual void GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses) override;
	UE_API virtual void RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState) override;
	// ~UMoviePipelineRenderPass Interface


public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "BurnInSettingDisplayName", "Burn In"); }
	virtual FText GetCategoryText() const { return NSLOCTEXT("MovieRenderPipeline", "DefaultCategoryName_Text", "Settings"); }
#endif
	virtual bool IsValidOnShots() const override { return false; }
	virtual bool IsValidOnPrimary() const override { return true; }
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(MetaClass="/Script/MovieRenderPipelineSettings.MoviePipelineBurnInWidget"), Category = "Widget Settings")
	FSoftClassPath BurnInClass;

	/** If true, the Burn In image will be composited into the Final Image pass. Doesn't apply to multi-layer EXR files. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (MetaClass = "/Script/MovieRenderPipelineSettings.MoviePipelineBurnInWidget"), Category = "Widget Settings")
	bool bCompositeOntoFinalImage;

private:
	FIntPoint OutputResolution;
	TSharedPtr<FWidgetRenderer> WidgetRenderer;
	TSharedPtr<SVirtualWindow> VirtualWindow;

	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMoviePipelineBurnInWidget>> BurnInWidgetInstances;
public:
	static UE_API FString DefaultBurnInWidgetAsset;
};

#undef UE_API
