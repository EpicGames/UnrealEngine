// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "MoviePipelineBurnInWidget.generated.h"

#define UE_API MOVIERENDERPIPELINESETTINGS_API

/**
 * Base class for level sequence burn ins
 */
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UMoviePipelineBurnInWidget : public UUserWidget
{
public:
	GENERATED_BODY()

	/** 
	* Called on the first temporal and first spatial sample of each output frame with the information about the frame being produced.
	* @param	ForPipeline		The pipeline the burn in is for. This will be consistent throughout a burn in widget's life.
	*/
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnOutputFrameStarted(UMoviePipeline* ForPipeline);
};

#undef UE_API
