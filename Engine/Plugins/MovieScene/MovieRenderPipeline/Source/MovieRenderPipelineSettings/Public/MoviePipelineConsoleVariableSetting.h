// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "ConsoleVariablesEditorCommandInfo.h"
#include "ConsoleVariablesEditorModule.h"
#endif

#include "MoviePipelineSetting.h"
#include "MovieRenderPipelineDataTypes.h"
#include "HAL/IConsoleManager.h"

#include "MoviePipelineConsoleVariableSetting.generated.h"

class IMovieSceneConsoleVariableTrackInterface;

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineConsoleVariableSetting : public UMoviePipelineSetting
{
	GENERATED_BODY()
public:

public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "ConsoleVariableSettingDisplayName", "Console Variables"); }
	MOVIERENDERPIPELINESETTINGS_API virtual FText GetFooterText(UMoviePipelineExecutorJob* InJob) const override;

	/**
	 * Returns the value of the given console variable within the presets for this setting. If none of the presets contain
	 * this cvar, an empty string is returned.
	 */
    MOVIERENDERPIPELINESETTINGS_API FString ResolvePresetValue(const FString& InCVarName) const;

	/**
	 * Returns the value of the console variable if the given entry were to be disabled in this setting. Searches other
	 * override cvars first, then presets, and lastly the startup value.
	 */
	MOVIERENDERPIPELINESETTINGS_API FString ResolveDisabledValue(const FMoviePipelineConsoleVariableEntry& InEntry) const;

	/* Returns the console variable entry at the specified index, else nullptr if the index is invalid. */
	MOVIERENDERPIPELINESETTINGS_API FMoviePipelineConsoleVariableEntry* GetCVarAtIndex(const int32 InIndex);
#endif
	virtual bool IsValidOnShots() const override { return true; }
	virtual bool IsValidOnPrimary() const override { return true; }
	MOVIERENDERPIPELINESETTINGS_API virtual void SetupForPipelineImpl(UMoviePipeline* InPipeline) override;
	MOVIERENDERPIPELINESETTINGS_API virtual void TeardownForPipelineImpl(UMoviePipeline* InPipeline) override;
	MOVIERENDERPIPELINESETTINGS_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

	// This needs to be higher priority than the Game Override setting so that the values the user specifies for cvars here are the ones actually applied during renders
	// otherwise the Scalability Settings of the Game Override setting can change values away from what the user expects.
	virtual int32 GetPriority() const override { return 1; }
protected:
	MOVIERENDERPIPELINESETTINGS_API void ApplyCVarSettings(const bool bOverrideValues);

public:
	// Note that the interface is used here instead of directly using UConsoleVariablesAsset in order to not
	// depend on the Console Variables Editor.
	/**
	 * An array of presets from the Console Variables Editor. The preset cvars will be applied (in the order they are
	 * specified) before any of the cvars in "Console Variables". In other words, cvars in "Console Variables" will
	 * take precedence over the cvars coming from these presets.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<TScriptInterface<IMovieSceneConsoleVariableTrackInterface>> ConsoleVariablePresets;
	
	/**
	* An array of console commands to execute when this shot is started. If you need to restore the value 
	* after the shot, add a matching entry in the EndConsoleCommands array. Because they are commands
	* and not values we cannot save the preivous value automatically.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<FString> StartConsoleCommands;

	/**
	* An array of console commands to execute when this shot is finished. Used to restore changes made by
	* StartConsoleCommands.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<FString> EndConsoleCommands;
	
	/**
	 * Gets a copy of all console variable overrides. These are not meant to be changed; use the mutator methods if
	 * console variables need to be updated.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINESETTINGS_API TArray<FMoviePipelineConsoleVariableEntry> GetConsoleVariables() const;
	
	/**
	 * Removes the console variable override with the specified name. If more than one with the same name exists, the
	 * last one will be removed. Returns true if at least one console variable was removed, else false.
	 * @param bRemoveAllInstances Remove all console variables overrides with the given name (not just the last one)
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINESETTINGS_API bool RemoveConsoleVariable(const FString& Name, const bool bRemoveAllInstances = false);

	/**
	 * Adds a console variable override with the given name and value if one does not already exist. If the console
	 * variable with the given name already exists, its value will be updated (the last one will be updated if there are
	 * duplicates with the same name). Returns true if the operation was successful, else false.
	 * @see AddConsoleVariable()
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINESETTINGS_API bool AddOrUpdateConsoleVariable(const FString& Name, const float Value);

	/**
	 * Adds a console variable override with the given name and value, and will add a duplicate if one with the provided
	 * name already exists. Returns true if the operation was successful, else false.
	 * @see AddOrUpdateConsoleVariable()
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINESETTINGS_API bool AddConsoleVariable(const FString& Name, const float Value);

	/**
	 * Updates the enable state of the console variable override with the provided name. If there are duplicate cvars
	 * with the same name, the last one with the provided name will be updated. Returns true if the operation was
	 * successful, else false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings")
	MOVIERENDERPIPELINESETTINGS_API bool UpdateConsoleVariableEnableState(const FString& Name, const bool bIsEnabled);

private:
	/** Merge together preset and override cvars into MergedConsoleVariables. Discards result of a prior merge (if any). */
	MOVIERENDERPIPELINESETTINGS_API void MergeInPresetConsoleVariables();

private:
	/**
	 * An array of console variable overrides which are applied during render and reverted after the render completes.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayName = "Console Variables", DisplayAfter = "ConsoleVariablePresets", ScriptNoExport))
	TArray<FMoviePipelineConsoleVariableEntry> CVars;
	
	TArray<float> PreviousConsoleVariableValues;

	/** Merged result of preset cvars and override cvars. */
	TArray<FMoviePipelineConsoleVariableEntry> MergedConsoleVariables;
};
