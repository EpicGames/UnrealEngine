// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineRenderPass.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelineWidgetRenderSetting.generated.h"

#define UE_API MOVIERENDERPIPELINESETTINGS_API

class FWidgetRenderer;
class UTextureRenderTarget2D;
class UMoviePipelineBurnInWidget;

namespace MoviePipeline { struct FMoviePipelineRenderPassInitSettings; }

UCLASS(MinimalAPI, Blueprintable)
class UMoviePipelineWidgetRenderer : public UMoviePipelineRenderPass
{
	GENERATED_BODY()

protected:
	UMoviePipelineWidgetRenderer()
		: UMoviePipelineRenderPass()
		, bCompositeOntoFinalImage(true)
	{
	}

	// UMoviePipelineRenderPass Interface
	UE_API virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;
	UE_API virtual void TeardownImpl() override;
	UE_API virtual void GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses) override;
	UE_API virtual void RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState) override;
	// ~UMoviePipelineRenderPass Interface

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:
	/** If true, the widget renderer image will be composited into the Final Image pass. Doesn't apply to multi-layer EXR files. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (MetaClass = "/Script/MovieRenderPipelineSettings.MoviePipelineBurnInWidget"), Category = "Widget Settings")
	bool bCompositeOntoFinalImage;

public:
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "WidgetRendererSettingDisplayName", "UI Renderer"); }
	virtual FText GetCategoryText() const { return NSLOCTEXT("MovieRenderPipeline", "WidgetRendererSettingCategoryName", "Rendering"); }
#endif
	virtual bool IsValidOnShots() const override { return true; }
	virtual bool IsValidOnPrimary() const override { return true; }
private:
	TSharedPtr<FWidgetRenderer> WidgetRenderer;

	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;
};

#undef UE_API
