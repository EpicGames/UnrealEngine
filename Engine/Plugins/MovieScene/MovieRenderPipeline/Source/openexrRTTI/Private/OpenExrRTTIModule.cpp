// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOpenExrRTTIModule.h"
#include "Modules/ModuleManager.h"

THIRD_PARTY_INCLUDES_START
#include "OpenEXR/ImfFloatAttribute.h"
#include "OpenEXR/ImfIntAttribute.h"
#include "OpenEXR/ImfStringAttribute.h"
THIRD_PARTY_INCLUDES_END


class FOpenExrRTTIModule : public IOpenExrRTTIModule
{
public:
	virtual void AddFileMetadata(const TMap<FString, FStringFormatArg>& InMetadata, Imf::Header& InHeader) override
	{
		for (const TPair<FString, FStringFormatArg>& KVP : InMetadata)
		{
			OPENEXR_IMF_INTERNAL_NAMESPACE::Attribute* Attribute = nullptr;
			switch (KVP.Value.Type)
			{
			case FStringFormatArg::EType::Int:
				Attribute = new OPENEXR_IMF_INTERNAL_NAMESPACE::IntAttribute(KVP.Value.IntValue);
				break;
			case FStringFormatArg::EType::Double:
				// Not all EXR readers support the double attribute, and we can't tell double from float in the FStringFormatArgs,
				// so unfortunately we have to downgrade them here. 
				Attribute = new OPENEXR_IMF_INTERNAL_NAMESPACE::FloatAttribute(KVP.Value.DoubleValue);
				break;
			case FStringFormatArg::EType::String:
				Attribute = new OPENEXR_IMF_INTERNAL_NAMESPACE::StringAttribute(std::string(TCHAR_TO_ANSI(*KVP.Value.StringValue)));
				break;
			case FStringFormatArg::EType::StringLiteralANSI:
				Attribute = new OPENEXR_IMF_INTERNAL_NAMESPACE::StringAttribute(std::string(KVP.Value.StringLiteralANSIValue));
				break;
			case FStringFormatArg::EType::StringLiteralWIDE:
				Attribute = new OPENEXR_IMF_INTERNAL_NAMESPACE::StringAttribute(std::string(StringCast<ANSICHAR>(KVP.Value.StringLiteralWIDEValue).Get()));
				break;
			case FStringFormatArg::EType::StringLiteralUCS2:
				Attribute = new OPENEXR_IMF_INTERNAL_NAMESPACE::StringAttribute(std::string(StringCast<ANSICHAR>(KVP.Value.StringLiteralUCS2Value).Get()));
				break;
			case FStringFormatArg::EType::StringLiteralUTF8:
				Attribute = new OPENEXR_IMF_INTERNAL_NAMESPACE::StringAttribute(std::string(StringCast<ANSICHAR>(KVP.Value.StringLiteralUTF8Value).Get()));
				break;

			case FStringFormatArg::EType::UInt:
			default:
				ensureMsgf(false, TEXT("Failed to add Metadata to EXR file, unsupported type."));
			}

			if (Attribute)
			{
				InHeader.insert(std::string(TCHAR_TO_ANSI(*KVP.Key)), *Attribute);
				delete Attribute;
			}
		}
	}
};

IMPLEMENT_MODULE(FOpenExrRTTIModule, UEOpenExrRTTI);
