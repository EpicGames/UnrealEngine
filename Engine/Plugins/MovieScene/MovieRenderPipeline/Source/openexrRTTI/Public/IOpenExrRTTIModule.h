// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Modules/ModuleInterface.h"
#include "Containers/Map.h"
#include "Misc/StringFormatArg.h"
#include "Containers/UnrealString.h"

THIRD_PARTY_INCLUDES_START
#include "OpenEXR/ImfHeader.h"
THIRD_PARTY_INCLUDES_END


class IOpenExrRTTIModule : public IModuleInterface
{
public:
	virtual void AddFileMetadata(const TMap<FString, FStringFormatArg>& InMetadata, Imf::Header& InHeader) = 0;
};
