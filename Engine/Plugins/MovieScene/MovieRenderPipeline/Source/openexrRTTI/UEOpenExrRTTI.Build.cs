// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UEOpenExrRTTI : ModuleRules
{
    public UEOpenExrRTTI(ReadOnlyTargetRules Target) : base(Target)
    {
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) || Target.Platform == UnrealTargetPlatform.Mac || Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
        {
			bEnableExceptions = true;
			bDisableAutoRTFMInstrumentation = true; // AutoRTFM cannot be used with exceptions

			// To write metadata into EXR files, we need to enable RTTI. To limit the spread of RTTI modules
			// this module can selectively expose the functionality that needs RTTI.
			bUseRTTI = true;
		
			PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core"
			}
			);
			
            PublicDependencyModuleNames.AddRange(
			new string[] {
				"Imath",
				"UEOpenExr"
			}
			);
			
			// Required for UEOpenExr
			AddEngineThirdPartyPrivateStaticDependencies(Target, "zlib");
        }
    }
}

